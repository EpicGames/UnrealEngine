// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MovieScene : ModuleRules
{
	public MovieScene(ReadOnlyTargetRules Target) : base(Target)
	{
        PrivateIncludePathModuleNames.AddRange(
            new string[] {
				"AutoRTFM",
				"MovieSceneTracks",
				"TargetPlatform",
				"UniversalObjectLocator"
			}
        );

        PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
                "InputCore",
                "Engine",
				"SlateCore",
				"TimeManagement",
				"UniversalObjectLocator"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AutoRTFM",
			}
		);

		SetupIrisSupport(Target);
	}
}
