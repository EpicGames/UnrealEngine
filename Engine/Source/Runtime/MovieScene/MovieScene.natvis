<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">

  <Intrinsic 
    Name="GetPlaybackCapabilityIDRegistry" 
    SideEffect="false" 
    Expression="*((UE::MovieScene::FPlaybackCapabilityIDRegistry**)GDebuggingState->Ptrs[(strstr(GDebuggingState->GuidString, &quot;f9062bb9d77c4bc7861e5d87d5afb22d&quot;) - GDebuggingState->GuidString)/32])">
  </Intrinsic>

  <Type Name="UE::MovieScene::FPlaybackCapabilitiesImpl">

    <DisplayString>{{Capabilities = {Num}}}</DisplayString>

    <Expand>
      
      <Synthetic Name="IDs">
        <Expand>
          <CustomListItems>
            <Variable Name="CapBit" InitialValue="0" />
            <Variable Name="MaxCapBit" InitialValue="64" />
            <Variable Name="CapIndex" InitialValue="0" />

            <Size>Num</Size>
            <Loop>
              <!-- Look for the next set bit -->
              <Loop>
                <Break Condition="CapBit &gt;= MaxCapBit || ((1 &lt;&lt; CapBit) &amp; AllCapabilities) != 0" />
                <Exec>CapBit = CapBit + 1</Exec>
              </Loop>
              
              <Break Condition="CapBit &gt;= MaxCapBit"/>
              
              <Item Name="[{CapIndex}] {CapBit}">((UE::MovieScene::FPlaybackCapabilityIDInfo*)GetPlaybackCapabilityIDRegistry()-&gt;Infos.AllocatorInstance.Data + CapBit)-&gt;DebugName,sub</Item>
              <Exec>CapIndex = CapIndex + 1</Exec>
              <Exec>CapBit = CapBit + 1</Exec>
            </Loop>
          </CustomListItems>
        </Expand>
      </Synthetic>

      <Synthetic Name="Data">
        <Expand>
          <CustomListItems>
            <Variable Name="CapIndex" InitialValue="0" />
            <Variable Name="HeaderPtr" InitialValue="(UE::MovieScene::FPlaybackCapabilityHeader*)0" />

            <Size>Num</Size>
            <Loop>
              <Exec>HeaderPtr = &amp;((UE::MovieScene::FPlaybackCapabilityHeader*)Memory)[CapIndex]</Exec>
              <Exec>CapIndex = CapIndex + 1</Exec>

              <Item Name="[{CapIndex}]" Condition="HeaderPtr->DebugPtr == 0">Memory + HeaderPtr->Capability.Offset + HeaderPtr->PointerOffset</Item>
              <Item Name="[{CapIndex}]" Condition="HeaderPtr->DebugPtr != 0">HeaderPtr->DebugPtr</Item>
            </Loop>
          </CustomListItems>
        </Expand>
      </Synthetic>

      <Synthetic Name="Headers">
        <Expand>
          <ArrayItems>
            <Size>Num</Size>
            <ValuePointer>(UE::MovieScene::FPlaybackCapabilityHeader*)Memory</ValuePointer>
          </ArrayItems>
        </Expand>
      </Synthetic>
      
    </Expand>

  </Type>

  <Type Name="UE::MovieScene::TPlaybackCapabilityDebuggingTypedPtr&lt;*&gt;">
    <Expand>
      <ExpandedItem>*($T1*)Ptr</ExpandedItem>
    </Expand>
  </Type>

  <Intrinsic Name="GetEntityManager" SideEffect="false" Expression="*((UE::MovieScene::FEntityManager**)GDebuggingState->Ptrs[(strstr(GDebuggingState->GuidString, &quot;af15add3d76b4d2ca79d574bb4cd9706&quot;) - GDebuggingState->GuidString)/32])">
  </Intrinsic>

  <Type Name="FMovieSceneNumericVariant">
    <DisplayString Condition="((*(uint64*)Data) &amp; 0xFFF8000000000000) != 0xFFF8000000000000">{ *(double*)Data }</DisplayString>
    <DisplayString Condition="((*(uint64*)Data) &amp; 0xFFF8000000000000) == 0xFFF8000000000000">{ (UMovieSceneNumericVariantGetter*)((*(uint64*)Data) &amp; ~0xFFFF000000000000u) }</DisplayString>
  <Expand>
    <Item Name="Ptr" Condition="((*(uint64*)Data) &amp; 0xFFF8000000000000) == 0xFFF8000000000000">(UMovieSceneNumericVariantGetter*)((*(uint64*)Data) &amp; ~0xFFFF000000000000u)</Item>
  </Expand>
  </Type>

  <Type Name="FMovieSceneTimeWarpVariant">
    <Intrinsic Name="GetType" Expression="((*(uint64*)Variant.Data) &amp; 0xFFF8000000000000) == 0xFFF8000000000000 ? ( ( ((*(uint64*)Variant.Data) &amp; 0x0007000000000000) >> 48) + 1 &amp; 0x7 ) : 0"></Intrinsic>
    <Intrinsic Name="GetData" Expression="(*(uint64*)Variant.Data) &amp; 0x0000FFFFFFFFFFFFu"></Intrinsic>
    <Intrinsic Name="NumeratorBits" Expression="&amp;((FMovieSceneTimeWarpFrameRate*)Variant.Data)->FrameRateNumerator[0]"></Intrinsic>
    <Intrinsic Name="DenominatorBits" Expression="&amp;((FMovieSceneTimeWarpFrameRate*)Variant.Data)->FrameRateDenominator[0]"></Intrinsic>
    <Intrinsic Name="FrameRateNumerator" Expression="int32(NumeratorBits()[0] | NumeratorBits()[1]&lt;&lt;8 | NumeratorBits()[2] &lt;&lt; 16) &amp; ~0x00800000 | ((NumeratorBits()[2] &amp; 0x8) &lt;&lt; 31)"></Intrinsic>
    <Intrinsic Name="FrameRateDenominator" Expression="int32(DenominatorBits()[0] | DenominatorBits()[1]&lt;&lt;8 | DenominatorBits()[2] &lt;&lt; 16) &amp; ~0x00800000 | ((DenominatorBits()[2] &amp; 0x8) &lt;&lt; 31)"></Intrinsic>
    <DisplayString Condition="GetType() == 0">{ *(double*)Variant.Data }</DisplayString>
    <DisplayString Condition="GetType() == 1">{ (UMovieSceneNumericVariantGetter*)((*(uint64*)Variant.Data) &amp; ~0xFFFF000000000000u) }</DisplayString>
    <DisplayString Condition="GetType() == 2">Fixed@{ ((FMovieSceneTimeWarpFixedFrame*)Variant.Data)->FrameNumber }</DisplayString>
    <DisplayString Condition="GetType() == 3 &amp;&amp; FrameRateDenominator() == 1">FrameRate={ FrameRateNumerator() }fps</DisplayString>
    <DisplayString Condition="GetType() == 3 &amp;&amp; FrameRateDenominator() != 1">FrameRate={ FrameRateNumerator() }/{ FrameRateDenominator() }</DisplayString>
    <DisplayString Condition="GetType() == 4">Loop [0:{((FMovieSceneTimeWarpLoop*)Variant.Data)->Duration })</DisplayString>
    <DisplayString Condition="GetType() == 5">Clamp [0:{((FMovieSceneTimeWarpClamp*)Variant.Data)->Max }]</DisplayString>
    <DisplayString Condition="GetType() == 6">Loop [0:{((FMovieSceneTimeWarpLoopFloat*)Variant.Data)->Duration }]</DisplayString>
    <DisplayString Condition="GetType() == 7">Clamp [0:{((FMovieSceneTimeWarpClampFloat*)Variant.Data)->Max }]</DisplayString>
    <Expand>

      <Item Name="Value" Condition="GetType() == 0"> *(double*)Variant.Data </Item>
      <Item Name="Value" Condition="GetType() == 1"> (UMovieSceneNumericVariantGetter*)((*(uint64*)Variant.Data) &amp; ~0xFFFF000000000000u) </Item>
      <Item Name="Value" Condition="GetType() == 2"> ((FMovieSceneTimeWarpFixedFrame*)Variant.Data)->FrameNumber </Item>
      <Synthetic Name="Value" Condition="GetType() == 3" >
        <DisplayString>{{ Numerator={FrameRateNumerator()}, Denominator={FrameRateDenominator()} }}</DisplayString>
        <Expand>
          <Item Name="Numerator">FrameRateNumerator()</Item>
          <Item Name="Denominator">FrameRateDenominator()</Item>
        </Expand>
      </Synthetic>
      <Item Name="Value" Condition="GetType() == 4"> ((FMovieSceneTimeWarpLoop*)Variant.Data)->Duration </Item>
      <Item Name="Value" Condition="GetType() == 5"> ((FMovieSceneTimeWarpClamp*)Variant.Data)->Max </Item>
      <Item Name="Value" Condition="GetType() == 6"> ((FMovieSceneTimeWarpLoopFloat*)Variant.Data)->Duration </Item>
      <Item Name="Value" Condition="GetType() == 7"> ((FMovieSceneTimeWarpClampFloat*)Variant.Data)->Max </Item>

    </Expand>
  </Type>

  <Type Name="UE::MovieScene::FComponentTypeID">

    <Intrinsic Name="GetDebugName" Expression="(*(UE::MovieScene::FComponentTypeInfo*)((TSparseArray&lt;UE::MovieScene::FComponentTypeInfo,FDefaultSparseArrayAllocator&gt;::FElementOrFreeListLink*)GetEntityManager()-&gt;ComponentRegistry-&gt;ComponentTypes.Data.AllocatorInstance.Data + Value)).DebugInfo.Ptr->DebugName" />
    <Intrinsic Name="GetDebugTypeName" Expression="(*(UE::MovieScene::FComponentTypeInfo*)((TSparseArray&lt;UE::MovieScene::FComponentTypeInfo,FDefaultSparseArrayAllocator&gt;::FElementOrFreeListLink*)GetEntityManager()-&gt;ComponentRegistry-&gt;ComponentTypes.Data.AllocatorInstance.Data + Value)).DebugInfo.Ptr->DebugTypeName" />

    <DisplayString Condition="Value == 65535">Invalid</DisplayString>
    <DisplayString Condition="Value != 65535 &amp;&amp; GetEntityManager() == nullptr">{ Value }</DisplayString>
    <DisplayString Condition="Value != 65535 &amp;&amp; GetEntityManager() != nullptr">[{ (*(UE::MovieScene::FComponentTypeInfo*)((TSparseArray&lt;UE::MovieScene::FComponentTypeInfo,FDefaultSparseArrayAllocator&gt;::FElementOrFreeListLink*)GetEntityManager()-&gt;ComponentRegistry-&gt;ComponentTypes.Data.AllocatorInstance.Data + Value)).DebugInfo.Ptr->DebugTypeName,sub }] { (*(UE::MovieScene::FComponentTypeInfo*)((TSparseArray&lt;UE::MovieScene::FComponentTypeInfo,FDefaultSparseArrayAllocator&gt;::FElementOrFreeListLink*)GetEntityManager()-&gt;ComponentRegistry-&gt;ComponentTypes.Data.AllocatorInstance.Data + Value)).DebugInfo.Ptr->DebugName,sub }</DisplayString>
    
    <Expand>

      <Item Name="BitIndex">Value</Item>
      <Item Name="DebugName" Condition="GetEntityManager() != nullptr">(*(UE::MovieScene::FComponentTypeInfo*)((TSparseArray&lt;UE::MovieScene::FComponentTypeInfo,FDefaultSparseArrayAllocator&gt;::FElementOrFreeListLink*)GetEntityManager()-&gt;ComponentRegistry-&gt;ComponentTypes.Data.AllocatorInstance.Data + Value)).DebugInfo.Ptr->DebugName,sub</Item>
      <Item Name="DebugTypeName" Condition="GetEntityManager() != nullptr">(*(UE::MovieScene::FComponentTypeInfo*)((TSparseArray&lt;UE::MovieScene::FComponentTypeInfo,FDefaultSparseArrayAllocator&gt;::FElementOrFreeListLink*)GetEntityManager()-&gt;ComponentRegistry-&gt;ComponentTypes.Data.AllocatorInstance.Data + Value)).DebugInfo.Ptr->DebugTypeName,sub</Item>

    </Expand>
  </Type>


  <Type Name="UE::MovieScene::TComponentDebuggingTypedPtr&lt;*&gt;">
    <Expand>
      <ExpandedItem>*($T1*)Ptr</ExpandedItem>
    </Expand>
  </Type>

  <Type Name="UE::MovieScene::TComponentHeader&lt;*&gt;">
    <Expand>
      <ArrayItems>
        <Size>*Size</Size>
        <ValuePointer>($T1*)Components</ValuePointer>
      </ArrayItems>
    </Expand>
  </Type>

  <Type Name="UE::MovieScene::FMovieSceneEntityID">
    <Intrinsic Name="GetLocation" Expression="(UE::MovieScene::FEntityManager::FEntityLocation*)((TSparseArray&lt;UE::MovieScene::FEntityManager::FEntityLocation,FDefaultSparseArrayAllocator&gt;::FElementOrFreeListLink*)GetEntityManager()->EntityLocations.Data.AllocatorInstance.Data + Value)"></Intrinsic>
    <Intrinsic Name="GetAllocation" Expression="(*(UE::MovieScene::FEntityAllocation**)( (TSparseArray&lt;UE::MovieScene::FEntityAllocation *,FDefaultSparseArrayAllocator&gt;::FElementOrFreeListLink*) GetEntityManager()-&gt;EntityAllocations.Data.AllocatorInstance.Data + GetLocation()->AllocationIndex))"></Intrinsic>

    <DisplayString>ID={ Value }</DisplayString>
    <Expand>
      <Synthetic Name="Components">
        <Expand>
          <CustomListItems>
            <Variable Name="pEntity"     InitialValue="GetLocation()" />
            <Variable Name="pAllocation" InitialValue="GetAllocation()" />

            <Variable Name="ComponentIndex" InitialValue="0" />
            <Variable Name="Count" InitialValue="0" />
            <Variable Name="pComponentHeader" InitialValue="(UE::MovieScene::FComponentHeader*)0" />

            <Loop>
              <Break Condition="ComponentIndex == pAllocation->NumComponents" />
              <Exec>pComponentHeader = &amp;pAllocation->ComponentHeaders[ComponentIndex]</Exec>

              <If Condition="pComponentHeader->Sizeof != 0">
                <Exec>Count = Count + 1</Exec>
                <Item Condition="*GetEntityManager()->RichComponentDebuggingPtr" Name="{ pComponentHeader->ComponentType.GetDebugName(),sub }">pComponentHeader->DebugComponents[pEntity->EntryIndexWithinAllocation],nanr</Item>
                <Item Condition="!*GetEntityManager()->RichComponentDebuggingPtr" Name="{ pComponentHeader->ComponentType.GetDebugName(),sub }">"Enable Sequencer.RichComponentDebugging and re-run to view",sb</Item>
              </If>
              <Exec>ComponentIndex = ComponentIndex + 1</Exec>
            </Loop>
            <Item Name="Num">Count</Item>
          </CustomListItems>
        </Expand>
      </Synthetic>

      <Synthetic Name="Tags">
        <Expand>
          <CustomListItems>
            <Variable Name="pEntity"     InitialValue="(UE::MovieScene::FEntityManager::FEntityLocation*)((TSparseArray&lt;UE::MovieScene::FEntityManager::FEntityLocation,FDefaultSparseArrayAllocator&gt;::FElementOrFreeListLink*)GetEntityManager()->EntityLocations.Data.AllocatorInstance.Data + Value)" />
            <Variable Name="pAllocation" InitialValue="(*(UE::MovieScene::FEntityAllocation**)( (TSparseArray&lt;UE::MovieScene::FEntityAllocation *,FDefaultSparseArrayAllocator&gt;::FElementOrFreeListLink*) GetEntityManager()-&gt;EntityAllocations.Data.AllocatorInstance.Data + pEntity->AllocationIndex))" />

            <Variable Name="ComponentIndex" InitialValue="0" />
            <Variable Name="Count" InitialValue="0" />
            <Variable Name="ComponentPtr" InitialValue="(void*)0" />
            <Variable Name="pComponentHeader" InitialValue="(UE::MovieScene::FComponentHeader*)0" />

            <Loop>
              <Exec>pComponentHeader = &amp;pAllocation->ComponentHeaders[ComponentIndex]</Exec>

              <Break Condition="ComponentIndex == pAllocation->NumComponents" />

              <If Condition="pComponentHeader->Sizeof == 0">
                <Item Name="[{Count}]">pComponentHeader->ComponentType</Item>
                <Exec>Count = Count + 1</Exec>
              </If>
              <Exec>ComponentIndex = ComponentIndex + 1</Exec>
            </Loop>
            <Item Name="Num">Count</Item>
          </CustomListItems>
        </Expand>
      </Synthetic>

      <Synthetic Name="[Allocation Details]">
        <Expand>
          <Synthetic Name="AllocationIndex">
            <DisplayString>{ (*(UE::MovieScene::FEntityManager::FEntityLocation*)((TSparseArray&lt;UE::MovieScene::FEntityManager::FEntityLocation,FDefaultSparseArrayAllocator&gt;::FElementOrFreeListLink*)GetEntityManager()->EntityLocations.Data.AllocatorInstance.Data + Value)).AllocationIndex }</DisplayString>
          </Synthetic>
          <Synthetic Name="ComponentOffset">
            <DisplayString>{ (*(UE::MovieScene::FEntityManager::FEntityLocation*)((TSparseArray&lt;UE::MovieScene::FEntityManager::FEntityLocation,FDefaultSparseArrayAllocator&gt;::FElementOrFreeListLink*)GetEntityManager()->EntityLocations.Data.AllocatorInstance.Data + Value)).EntryIndexWithinAllocation }</DisplayString>
          </Synthetic>
          <Item Name="Owning Allocation">GetAllocation()</Item>

        </Expand>
      </Synthetic>
    </Expand>
  </Type>

  <Type Name="UE::MovieScene::FComponentMask">
    <Expand>

      <CustomListItems MaxItemsPerView="256" Condition="GetEntityManager()">
        <Variable Name="BitIndex" InitialValue="0" />
    <Variable Name="NumSetBits" InitialValue="0" />
        <Loop>
          <Break Condition="BitIndex == Bits.NumBits" />
          <Exec Condition="((reinterpret_cast&lt;uint32*&gt;(Bits.AllocatorInstance.InlineData)[BitIndex/32]&gt;&gt;(BitIndex%32)) &amp; 1) != 0">NumSetBits = NumSetBits + 1</Exec>
          <Exec>BitIndex = BitIndex + 1</Exec>
        </Loop>
    <Item Name="Num">NumSetBits</Item>
      </CustomListItems>
    
      <CustomListItems MaxItemsPerView="256" Condition="GetEntityManager()">
        <Variable Name="BitIndex" InitialValue="0" />

        <Loop>
          <Break Condition="BitIndex == Bits.NumBits" />


          <Item Name="[{ (*(UE::MovieScene::FComponentTypeInfo*)((TSparseArray&lt;UE::MovieScene::FComponentTypeInfo,FDefaultSparseArrayAllocator&gt;::FElementOrFreeListLink*)GetEntityManager()-&gt;ComponentRegistry-&gt;ComponentTypes.Data.AllocatorInstance.Data + BitIndex)).DebugInfo.Ptr->DebugTypeName,sub }] { (*(UE::MovieScene::FComponentTypeInfo*)((TSparseArray&lt;UE::MovieScene::FComponentTypeInfo,FDefaultSparseArrayAllocator&gt;::FElementOrFreeListLink*)GetEntityManager()-&gt;ComponentRegistry-&gt;ComponentTypes.Data.AllocatorInstance.Data + BitIndex)).DebugInfo.Ptr->DebugName,sub }" Condition="((reinterpret_cast&lt;uint32*&gt;(Bits.AllocatorInstance.InlineData)[BitIndex/32]&gt;&gt;(BitIndex%32)) &amp; 1) != 0">BitIndex</Item>
          <Exec>BitIndex = BitIndex + 1</Exec>
        </Loop>
      </CustomListItems>

      <Synthetic Condition="Bits.NumBits != 0" Name="[All Bits]">
        <Expand>
          <IndexListItems Condition="Bits.NumBits &gt; 0">
            <Size>Bits.NumBits</Size>
            <ValueNode Condition="((reinterpret_cast&lt;uint32*&gt;(Bits.AllocatorInstance.InlineData)[$i/32]&gt;&gt;($i%32)) &amp; 1) != 0">1</ValueNode>
            <ValueNode Condition="((reinterpret_cast&lt;uint32*&gt;(Bits.AllocatorInstance.InlineData)[$i/32]&gt;&gt;($i%32)) &amp; 1) == 0">0</ValueNode>
          </IndexListItems>
        </Expand>
      </Synthetic>

    </Expand>

  </Type>

  <Type Name="UE::MovieScene::FEntityAllocation">
    <DisplayString>Num={ Size }, Num Components={ NumComponents }</DisplayString>

    <Expand>

      <Synthetic Name="Components">
        <Expand>
          <CustomListItems>

            <Variable Name="HeaderIndex" InitialValue="0" />
            <Variable Name="Count" InitialValue="0" />
            <Variable Name="pComponentHeader" InitialValue="(UE::MovieScene::FComponentHeader*)0" />
            <Loop>
              <Break Condition="HeaderIndex == NumComponents" />
              <Exec>pComponentHeader = &amp;ComponentHeaders[HeaderIndex]</Exec>
              <Item Condition="*GetEntityManager()->RichComponentDebuggingPtr  &amp;&amp; pComponentHeader->Sizeof != 0" Name="{ pComponentHeader->ComponentType.GetDebugName(),sub }">pComponentHeader,nanr</Item>
              <Item Condition="!*GetEntityManager()->RichComponentDebuggingPtr &amp;&amp; pComponentHeader->Sizeof != 0" Name="{ pComponentHeader->ComponentType.GetDebugName(),sub }">"Enable Sequencer.RichComponentDebugging and re-run to view",sb</Item>
              <Exec Condition="pComponentHeader->Sizeof != 0">Count = Count + 1</Exec>
              <Exec>HeaderIndex = HeaderIndex + 1</Exec>
            </Loop>
            <Item Name="Num">Count</Item>
          </CustomListItems>
        </Expand>
      </Synthetic>

      <Synthetic Name="Tags">
        <Expand>
          <CustomListItems>
            <Variable Name="HeaderIndex" InitialValue="0" />
            <Variable Name="Count" InitialValue="0" />
            <Variable Name="pComponentHeader" InitialValue="(UE::MovieScene::FComponentHeader*)0" />
            <Loop>
              <Break Condition="HeaderIndex == NumComponents" />
              <Exec>pComponentHeader = &amp;ComponentHeaders[HeaderIndex]</Exec>

              <Item Name="[{Count}]" Condition="pComponentHeader->Sizeof == 0">pComponentHeader->ComponentType</Item>

              <Exec Condition="pComponentHeader->Sizeof == 0">Count = Count + 1</Exec>
              <Exec>HeaderIndex = HeaderIndex + 1</Exec>
            </Loop>

            <Item Name="Num">Count</Item>
          </CustomListItems>
        </Expand>
        
      </Synthetic>

      <Item Name="SerialNumber">SerialNumber</Item>
      <Item Name="Capacity">Capacity</Item>
      <Item Name="MaxCapacity">MaxCapacity</Item>

      <Synthetic Name="Entities">
        <DisplayString>Num={ Size }</DisplayString>
        <Expand>
          <ArrayItems>
            <Size>Size</Size>
            <ValuePointer>EntityIDs</ValuePointer>
          </ArrayItems>
        </Expand>
      </Synthetic>
    </Expand>
  </Type>

  <Type Name="UE::MovieScene::FComponentHeader::TDebugTypes&lt;*&gt;">
    <Expand>
      <ArrayItems>
        <Size>Size</Size>
        <ValuePointer>TypedComponents</ValuePointer>
      </ArrayItems>
    </Expand>
  </Type>

  <!--
  <Type Name="UE::MovieScene::TMovieSceneEntityTaskComponents&lt;*&gt;">

    <Expand>
      <ExpandedItem>Accessors</ExpandedItem>
    </Expand>

  </Type>-->

  <Type Name="UE::MovieScene::TAdd&lt;*&gt;" Inheritable="false">
    <DisplayString>Add Component={"$T1"}, Value={ Payload }, ComponentType={ ComponentType }</DisplayString>
  </Type>

  <Type Name="UE::MovieScene::FAddUntyped" Inheritable="false">
    <DisplayString>Add Component={ComponentType}, Value=Defaulted</DisplayString>
  </Type>

  <Type Name="UE::MovieScene::TReadOptional&lt;*&gt;" Inheritable="false">
    <DisplayString>(Optional) Read Component={"$T1"}, ComponentType={ ComponentType }</DisplayString>
  </Type>
  
  <Type Name="UE::MovieScene::TRead&lt;*&gt;" Inheritable="false">
    <DisplayString>Read Component={"$T1"}, ComponentType={ ComponentType }</DisplayString>
  </Type>

  <Type Name="UE::MovieScene::TReadProjected&lt;*&gt;" Inheritable="false">
    <DisplayString>Read Component Projection={"$T1"}, ComponentType={ ComponentType }, Projection={ Projection }</DisplayString>
  </Type>

  <Type Name="UE::MovieScene::TWrite&lt;*&gt;" Inheritable="false">
    <DisplayString>Write To Component={"$T1"}, ComponentType={ ComponentType }</DisplayString>
  </Type>

  <Type Name="UE::MovieScene::TWriteOptional&lt;*&gt;" Inheritable="false">
    <DisplayString>(Optional) Write To Component={"$T1"}, ComponentType={ ComponentType }</DisplayString>
  </Type>

  <Type Name="UE::MovieScene::TAddConditional&lt;*&gt;" Inheritable="false">
    <DisplayString Condition="bCondition == false">noop</DisplayString>
    <DisplayString Condition="bCondition == true">Add Type={"$T1"}, Value={ Payload }</DisplayString>
  </Type>

  <Type Name="UE::MovieScene::FAddTag" Inheritable="false">
    <DisplayString>Add Tag {ComponentType}</DisplayString>
  </Type>

  <Type Name="UE::MovieScene::FAddTagConditional" Inheritable="false">
    <DisplayString Condition="bCondition == false">noop</DisplayString>
    <DisplayString Condition="bCondition == true">Add Tag {ComponentType}</DisplayString>
  </Type>

  <Type Name="UE::MovieScene::FReadEntityIDs" Inheritable="false">
    <DisplayString>Read Entity ID</DisplayString>
  </Type>

  <Type Name="UE::MovieScene::TPartialProjection&lt;*&gt;" Inheritable="false">
    <DisplayString Condition="Projection == nullptr">ComponentType={ ComponentTypeID } (no projection)"</DisplayString>
    <DisplayString Condition="Projection != nullptr">ComponentType={ ComponentTypeID} (with projection)</DisplayString>
  </Type>

  <Type Name="UE::MovieScene::FHierarchicalBlendTarget">
    <Expand>
  <ArrayItems>
    <Size>Capacity</Size>
    <ValuePointer Condition="Capacity==7">(int16*)Data</ValuePointer>
    <ValuePointer Condition="Capacity&gt;7">*(int16**)Data</ValuePointer>
  </ArrayItems>
    </Expand>
  </Type>

  <Type Name="UE::MovieScene::TFixedSparseBitSet&lt;*,UE::MovieScene::TDynamicSparseBitSetBucketStorage&lt;*&gt;&gt;" Inheritable="false">
    <Expand>
      <!-- First of count how many set bits there are -->
      <CustomListItems>
        <Variable Name="Bucket" InitialValue="Buckets.Storage.AllocatorInstance.SecondaryData.Data == 0 ? ($T2*)Buckets.Storage.AllocatorInstance.InlineData : ($T2*)Buckets.Storage.AllocatorInstance.SecondaryData.Data" />

        <Variable Name="SetBitCount" InitialValue="0" />
        <Variable Name="HashIndex" InitialValue="0" />
        <Variable Name="BucketBitIndex" InitialValue="0" />
        <Variable Name="BucketIndex" InitialValue="0" />
        <Variable Name="BitIndex" InitialValue="0" />
        <Variable Name="MaxHashIndex" InitialValue="sizeof($T1)*8" />
        <Variable Name="MaxBucketBitIndex" InitialValue="sizeof($T2)*8" />

        <Loop>
          <Loop>
            <Break Condition="(BucketHash &amp; ((($T1)1) &lt;&lt; HashIndex)) || HashIndex == MaxHashIndex"/>
            <Exec>HashIndex = HashIndex + 1</Exec>
            <Exec>BitIndex = BitIndex + MaxBucketBitIndex</Exec>
          </Loop>

          <Break Condition="HashIndex == MaxHashIndex"/>

          <Exec>BucketBitIndex = 0</Exec>
          <Loop>
            <Exec Condition="((Bucket[BucketIndex]) &amp; ((($T2)1) &lt;&lt; BucketBitIndex))">SetBitCount = SetBitCount + 1</Exec>
            <Break Condition="BucketBitIndex == MaxBucketBitIndex"/>
            <Exec>BucketBitIndex = BucketBitIndex + 1</Exec>
            <Exec>BitIndex = BitIndex + 1</Exec>
          </Loop>

          <Exec>HashIndex = HashIndex + 1</Exec>
          <Exec>BucketIndex = BucketIndex + 1</Exec>
        </Loop>
        <Item Name="Num">SetBitCount</Item>
      </CustomListItems>

      <CustomListItems Condition="BucketHash != 0">
        <Variable Name="Bucket" InitialValue="Buckets.Storage.AllocatorInstance.SecondaryData.Data == 0 ? ($T2*)Buckets.Storage.AllocatorInstance.InlineData : ($T2*)Buckets.Storage.AllocatorInstance.SecondaryData.Data" />

        <Variable Name="HashIndex" InitialValue="0" />
        <Variable Name="BucketBitIndex" InitialValue="0" />
        <Variable Name="BucketIndex" InitialValue="0" />
        <Variable Name="BitIndex" InitialValue="0" />
        <Variable Name="MaxHashIndex" InitialValue="sizeof($T1)*8" />
        <Variable Name="MaxBucketBitIndex" InitialValue="sizeof($T2)*8" />

        <!-- Reset everything and show the elements -->
        <Exec>HashIndex = 0</Exec>
        <Exec>BucketIndex = 0</Exec>
        <Exec>BucketBitIndex = 0</Exec>

        <Loop>
          <Loop>
            <Break Condition="(BucketHash &amp; ((($T1)1) &lt;&lt; HashIndex)) || HashIndex == MaxHashIndex"/>
            <Exec>HashIndex = HashIndex + 1</Exec>
            <Exec>BitIndex = BitIndex + MaxBucketBitIndex</Exec>
          </Loop>

          <Break Condition="HashIndex == MaxHashIndex"/>

          <Exec>BucketBitIndex = 0</Exec>
          <Loop>
            <Item Condition="((Bucket[BucketIndex]) &amp; ((($T2)1) &lt;&lt; BucketBitIndex))">BitIndex</Item>

            <Break Condition="BucketBitIndex == MaxBucketBitIndex"/>
            <Exec>BucketBitIndex = BucketBitIndex + 1</Exec>
            <Exec>BitIndex = BitIndex + 1</Exec>
          </Loop>

          <Exec>HashIndex = HashIndex + 1</Exec>
          <Exec>BucketIndex = BucketIndex + 1</Exec>
        </Loop>
      </CustomListItems>

      <Synthetic Name="Bitfield">
        <Expand>
          <CustomListItems>
            <Variable Name="Bucket" InitialValue="Buckets.Storage.AllocatorInstance.SecondaryData.Data == 0 ? ($T2*)Buckets.Storage.AllocatorInstance.InlineData : ($T2*)Buckets.Storage.AllocatorInstance.SecondaryData.Data" />

            <Variable Name="HashIndex" InitialValue="0" />
            <Variable Name="MaxHashIndex" InitialValue="sizeof($T1)*8" />
            <Loop>
              <Item Name="{ HashIndex }" Condition="(BucketHash &amp; (1 &lt;&lt; HashIndex))">(*Bucket),b</Item>
              <Item Name="{ HashIndex }" Condition="(BucketHash &amp; (1 &lt;&lt; HashIndex)) == 0">($T2)0,b</Item>

              <Exec Condition="(BucketHash &amp; (1 &lt;&lt; HashIndex))">Bucket = Bucket + 1</Exec>
              <Exec>HashIndex = HashIndex + 1</Exec>
              <Break Condition="HashIndex == MaxHashIndex" />
            </Loop>
          </CustomListItems>
        </Expand>
      </Synthetic>
    </Expand>
  </Type>

</AutoVisualizer>
