// Copyright Epic Games, Inc. All Rights Reserved.

#include "Bindings/MovieSceneCustomBinding.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneCustomBinding)

const int32 UMovieSceneCustomBinding::BaseEnginePriority = 0;
const int32 UMovieSceneCustomBinding::BaseCustomPriority = 100;
