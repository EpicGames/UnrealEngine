// Copyright Epic Games, Inc. All Rights Reserved.

#include "Bindings/MovieSceneReplaceableBinding.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "MovieScene.h"
#include "Engine/World.h"
#include "Engine/Level.h"
#include "MovieSceneBindingReferences.h"
#include "Tracks/MovieSceneBindingLifetimeTrack.h"
#include "Sections/MovieSceneBindingLifetimeSection.h"
#include "Bindings/MovieSceneSpawnableBinding.h"
#include "MovieSceneCommonHelpers.h"
#include "Styling/SlateBrush.h"
#include "Styling/AppStyle.h"
#include "Internationalization/Internationalization.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneReplaceableBinding)

#define LOCTEXT_NAMESPACE "FPossessableModel"

namespace UE::MovieScene::ReplaceableBinding
{
	static const FName SequencerPreviewActorTag("SequencerPreviewActor");
} // namespace UE::MovieScene::ReplaceableBinding

#if WITH_EDITOR
void UMovieSceneReplaceableBindingBase::SetupDefaults(UObject* SpawnedObject, FGuid ObjectBindingId, UMovieScene& OwnerMovieScene)
{
	Super::SetupDefaults(SpawnedObject, ObjectBindingId, OwnerMovieScene);
	// Ensure it has a binding lifetime track which it will need in editor
	UMovieSceneBindingLifetimeTrack* BindingLifetimeTrack = Cast<UMovieSceneBindingLifetimeTrack>(OwnerMovieScene.FindTrack(UMovieSceneBindingLifetimeTrack::StaticClass(), ObjectBindingId, NAME_None));
	if (!BindingLifetimeTrack)
	{
		BindingLifetimeTrack = Cast<UMovieSceneBindingLifetimeTrack>(OwnerMovieScene.AddTrack(UMovieSceneBindingLifetimeTrack::StaticClass(), ObjectBindingId));
	}

	if (BindingLifetimeTrack && BindingLifetimeTrack->GetAllSections().IsEmpty())
	{
		UMovieSceneBindingLifetimeSection* BindingLifetimeSection = Cast<UMovieSceneBindingLifetimeSection>(BindingLifetimeTrack->CreateNewSection());
		BindingLifetimeSection->SetRange(TRange<FFrameNumber>::All());
		BindingLifetimeTrack->AddSection(*BindingLifetimeSection);
	}
}

FSlateIcon UMovieSceneReplaceableBindingBase::GetBindingTrackCustomIconOverlay() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.ReplaceableIconOverlay");
}

FText UMovieSceneReplaceableBindingBase::GetBindingTrackIconTooltip() const
{
	return LOCTEXT("CustomReplaceableTooltip", "This item is dynamically bound at runtime, and may spawn a preview object in Editor within Sequencer");
}


bool UMovieSceneReplaceableBindingBase::SupportsConversionFromBinding(const FMovieSceneBindingReference& BindingReference, const UObject* SourceObject) const
{
	return SupportsBindingCreationFromObject(SourceObject);
}

UMovieSceneCustomBinding* UMovieSceneReplaceableBindingBase::CreateCustomBindingFromBinding(const FMovieSceneBindingReference& BindingReference, UObject* SourceObject, UMovieScene& OwnerMovieScene)
{
	return CreateNewCustomBinding(SourceObject, OwnerMovieScene);
}

#endif

UClass* UMovieSceneReplaceableBindingBase::GetBoundObjectClass() const
{
	// We use the bound object class of the preview spawnable by default
	if (TSubclassOf<UMovieSceneSpawnableBindingBase> SpawnableBindingClass = GetInnerSpawnableClass())
	{
		return SpawnableBindingClass->GetDefaultObject<UMovieSceneSpawnableBindingBase>()->GetBoundObjectClass();
	}
	return AActor::StaticClass();
}

bool UMovieSceneReplaceableBindingBase::SupportsBindingCreationFromObject(const UObject* SourceObject) const
{
	// We can create this binding if our chosen inner spawnable can be created from it.
	if (TSubclassOf<UMovieSceneSpawnableBindingBase> SpawnableBindingClass = GetInnerSpawnableClass())
	{
		return SpawnableBindingClass->GetDefaultObject<UMovieSceneSpawnableBindingBase>()->SupportsBindingCreationFromObject(SourceObject);
	}
	return false;
}

bool UMovieSceneReplaceableBindingBase::WillSpawnObject(TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const
{
#if WITH_EDITOR
	if (UObject* WorldContext = SharedPlaybackState->GetPlaybackContext())
	{
		if (UWorld* World = WorldContext->GetWorld())
		{
			if (World->WorldType == EWorldType::Editor)
			{
				return true;
			}
		}
	}
#endif
	return false;
}

FMovieSceneBindingResolveResult UMovieSceneReplaceableBindingBase::ResolveBinding(const FMovieSceneBindingResolveParams& ResolveParams, int32 BindingIndex, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const
{
#if WITH_EDITOR
	if (UObject* WorldContext = SharedPlaybackState->GetPlaybackContext())
	{
		if (UWorld* World = WorldContext->GetWorld())
		{
			if (World->WorldType == EWorldType::Editor && PreviewSpawnable)
			{
				FMovieSceneBindingResolveResult Result = PreviewSpawnable->ResolveBinding(ResolveParams, BindingIndex, SharedPlaybackState);
				for (TObjectPtr<UObject> Object : Result.Objects)
				{
					if (AActor* Actor = Cast<AActor>(Object.Get()))
					{
						// In addition to the spawnable tag (which the spawnable will have added), we add a replaceable tag
						Actor->Tags.AddUnique(UE::MovieScene::ReplaceableBinding::SequencerPreviewActorTag);
					}
				}
				return Result;
			}
		}
	}
#endif

	return ResolveRuntimeBindingInternal(ResolveParams, BindingIndex, SharedPlaybackState);
}


const UMovieSceneSpawnableBindingBase* UMovieSceneReplaceableBindingBase::AsSpawnable(TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const
{
#if WITH_EDITOR
	if (UObject* WorldContext = SharedPlaybackState->GetPlaybackContext())
	{
		if (UWorld* World = WorldContext->GetWorld())
		{
			if (World->WorldType == EWorldType::Editor)
			{
				return PreviewSpawnable;
			}
		}
	}
#endif

	return nullptr;
}

UMovieSceneSpawnableBindingBase* UMovieSceneReplaceableBindingBase::CreateInnerSpawnable(UObject* SourceObject, UMovieScene& OwnerMovieScene)
{
	if (TSubclassOf<UMovieSceneSpawnableBindingBase> SpawnableClass = GetInnerSpawnableClass())
	{
		ensure(!SpawnableClass->HasAnyClassFlags(EClassFlags::CLASS_Abstract));
		
		return Cast<UMovieSceneSpawnableBindingBase>(SpawnableClass->GetDefaultObject<UMovieSceneSpawnableBindingBase>()->CreateNewCustomBinding(SourceObject, OwnerMovieScene));
	}
	return nullptr;
}

UMovieSceneCustomBinding* UMovieSceneReplaceableBindingBase::CreateNewCustomBinding(UObject* SourceObject, UMovieScene& OwnerMovieScene)
{
	UMovieSceneReplaceableBindingBase* NewCustomBinding = nullptr;

	const FName TemplateName = MakeUniqueObjectName(&OwnerMovieScene, UObject::StaticClass(), SourceObject ? SourceObject->GetFName() : GetClass()->GetFName());
	const FName InstancedBindingName = MakeUniqueObjectName(&OwnerMovieScene, UObject::StaticClass(), *FString(TemplateName.ToString() + TEXT("_CustomBinding")));

	NewCustomBinding = NewObject<UMovieSceneReplaceableBindingBase>(&OwnerMovieScene, GetClass(), InstancedBindingName, RF_Transactional);
#if WITH_EDITORONLY_DATA
	NewCustomBinding->PreviewSpawnable = NewCustomBinding->CreateInnerSpawnable(SourceObject, OwnerMovieScene);
#endif
	NewCustomBinding->InitReplaceableBinding(SourceObject, OwnerMovieScene);
	return NewCustomBinding;
}

void UMovieSceneReplaceableBindingBase::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);

#if WITH_EDITOR
	if (PreviewSpawnable != nullptr)
	{
		// Only duplicate the inner PreviewSpawnable if this binding has been bound into a different valid UMovieScene.
		if (UMovieScene* OwnerMovieScene = GetTypedOuter<UMovieScene>(); 
			OwnerMovieScene != nullptr && PreviewSpawnable->GetTypedOuter<UMovieScene>() != OwnerMovieScene)
		{
			// Duplicate the inner spawnable into the new owning movie scene
			PreviewSpawnable = Cast<UMovieSceneSpawnableBindingBase>(StaticDuplicateObject(PreviewSpawnable, OwnerMovieScene));
		}
	}
#endif // WITH_EDITOR
}

#undef LOCTEXT_NAMESPACE
