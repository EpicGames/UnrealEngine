// Copyright Epic Games, Inc. All Rights Reserved.

#include "Bindings/MovieSceneSpawnableBinding.h"
#include "MovieSceneSpawnRegister.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "MovieScene.h"
#include "Engine/World.h"
#include "Engine/Level.h"
#include "MovieSceneBindingReferences.h"
#include "Tracks/MovieSceneBindingLifetimeTrack.h"
#include "Sections/MovieSceneBindingLifetimeSection.h"
#include "Styling/SlateBrush.h"
#include "Styling/AppStyle.h"
#include "Internationalization/Internationalization.h"
#include "Tracks/MovieSceneSpawnTrack.h"
#include "Sections/MovieSceneBoolSection.h"
#include "MovieSceneSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSpawnableBinding)

#define LOCTEXT_NAMESPACE "FPossessableModel"

static TAutoConsoleVariable<int32> CVarEnableReadableActorLabelsForSpawnables(
	TEXT("LevelSequence.EnableReadableActorLabelsForSpawnables"),
	1,
	TEXT("If true, in the editor during PIE, Sequencer will set the DisplayName of spawned actors to match their Spawnable name in Sequencer, mimicking edit-time behavior. This helps with identifying spawnables more reliably, but isn't available in packaged builds. Try disabling this if you see async loads being flushed during actor spawning in PIE.\n")
	TEXT("0: off, 1: on"),
	ECVF_Default);

UObject* UMovieSceneSpawnableBindingBase::SpawnObject(const FGuid& BindingId, int32 BindingIndex, UMovieScene& MovieScene, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState)
{
	UWorld* WorldContext = GetWorldContext(SharedPlaybackState);

	if (WorldContext == nullptr)
	{
		UE_LOGF(LogMovieScene, Warning, "Can't find world to spawn '%ls' into, defaulting to Persistent level", *MovieScene.GetName());

		WorldContext = GWorld;
	}

	FName SpawnName = GetSpawnName(BindingId, MovieScene, TemplateID, SharedPlaybackState);

	// If there's an object that already exists with the requested name, it needs to be renamed (it's probably pending kill)
	if (!SpawnName.IsNone())
	{
		UObject* ExistingObject = StaticFindObjectFast(nullptr, WorldContext->PersistentLevel.Get(), SpawnName);
		if (ExistingObject)
		{
			FName DefunctName = MakeUniqueObjectName(WorldContext->PersistentLevel.Get(), ExistingObject->GetClass());
			ExistingObject->Rename(*DefunctName.ToString(), nullptr);
		}
	}

	// Spawn Object Internal

	UObject* SpawnedObject = SpawnObjectInternal(WorldContext, SpawnName, BindingId, BindingIndex, MovieScene, TemplateID, SharedPlaybackState);

	if (!SpawnedObject)
	{
		return nullptr;
	}

#if WITH_EDITOR
	if (GIsEditor)
	{
		// Explicitly set RF_Transactional on spawned objects so we can undo/redo properties on them.
		SpawnedObject->SetFlags(RF_Transactional);
	}
#endif

	// If have spawned an actor, do some actor-specific setup
	if (AActor* SpawnedActor = Cast<AActor>(SpawnedObject))
	{
		// Ensure this spawnable is not a preview actor. Preview actors will not have BeginPlay() called on them.
#if WITH_EDITOR
		SpawnedActor->bIsEditorPreviewActor = false;
#endif

		static const FName SequencerActorTag("SequencerActor");
		// tag this actor so we know it was spawned by sequencer
		SpawnedActor->Tags.AddUnique(SequencerActorTag);

#if WITH_EDITOR
		if (GIsEditor)
		{
			// Explicitly set RF_Transactional on spawned actors so we can undo/redo properties on them.
			// This particular UObject will be marked RF_Transactional by the caller, but we need to set it on the components.

			for (UActorComponent* Component : SpawnedActor->GetComponents())
			{
				if (Component)
				{
					Component->SetFlags(RF_Transactional);
				}
			}
		}
#endif
	}

	// Allows derived classes to perform post-spawn logic such as mesh setup on actors.
	PostSpawnObject(SpawnedObject, WorldContext, BindingId, BindingIndex, MovieScene, TemplateID, SharedPlaybackState);

	return SpawnedObject;
}

void UMovieSceneSpawnableBindingBase::DestroySpawnedObject(UObject* Object)
{
	if (!Object)
	{
		return;
	}
#if WITH_EDITOR
	if (GIsEditor)
	{
		// Explicitly remove RF_Transactional on spawned objects since we don't want to transact spawn/destroy events
		Object->ClearFlags(RF_Transactional);
	}
	if (AActor* Actor = Cast<AActor>(Object))
	{
		// Explicitly remove RF_Transactional on spawned actors since we don't want to trasact spawn/destroy events
		// This particular UObject will have RF_Transactional cleared by the caller, but we need to cleared it on the components.
		for (UActorComponent* Component : Actor->GetComponents())
		{
			if (Component)
			{
				Component->ClearFlags(RF_Transactional);
			}
		}
	}

#endif

	DestroySpawnedObjectInternal(Object);
}

#if WITH_EDITOR
void UMovieSceneSpawnableBindingBase::SetupDefaults(UObject* SpawnedObject, FGuid ObjectBindingId, UMovieScene& OwnerMovieScene)
{
	Super::SetupDefaults(SpawnedObject, ObjectBindingId, OwnerMovieScene);

	// TODO: For now we are not using binding lifetime track for this, though it will support it. We continue to use spawn track until we improve UX of splitting sections
	// // 
	// 
	//// Ensure it has a binding lifetime track
	//UMovieSceneBindingLifetimeTrack* BindingLifetimeTrack = Cast<UMovieSceneBindingLifetimeTrack>(OwnerMovieScene.FindTrack(UMovieSceneBindingLifetimeTrack::StaticClass(), ObjectBindingId, NAME_None));
	//if (!BindingLifetimeTrack)
	//{
	//	BindingLifetimeTrack = Cast<UMovieSceneBindingLifetimeTrack>(OwnerMovieScene.AddTrack(UMovieSceneBindingLifetimeTrack::StaticClass(), ObjectBindingId));
	//}

	//if (BindingLifetimeTrack && BindingLifetimeTrack->GetAllSections().IsEmpty())
	//{
	//	UMovieSceneBindingLifetimeSection* BindingLifetimeSection = Cast<UMovieSceneBindingLifetimeSection>(BindingLifetimeTrack->CreateNewSection());
	//	BindingLifetimeSection->SetRange(TRange<FFrameNumber>::All());
	//	BindingLifetimeTrack->AddSection(*BindingLifetimeSection);
	//}

	UMovieSceneSpawnTrack* SpawnTrack = Cast<UMovieSceneSpawnTrack>(OwnerMovieScene.FindTrack(UMovieSceneSpawnTrack::StaticClass(), ObjectBindingId, NAME_None));
	if (!SpawnTrack)
	{
		SpawnTrack = Cast<UMovieSceneSpawnTrack>(OwnerMovieScene.AddTrack(UMovieSceneSpawnTrack::StaticClass(), ObjectBindingId));
	}

	if (SpawnTrack && SpawnTrack->GetAllSections().Num() == 0)
	{
		SpawnTrack->Modify();

		UMovieSceneBoolSection* SpawnSection = Cast<UMovieSceneBoolSection>(SpawnTrack->CreateNewSection());
		SpawnSection->GetChannel().SetDefault(true);
		SpawnSection->SetRange(TRange<FFrameNumber>::All());
		SpawnTrack->AddSection(*SpawnSection);
		SpawnTrack->SetObjectId(ObjectBindingId);
	}
}

FSlateIcon UMovieSceneSpawnableBindingBase::GetBindingTrackCustomIconOverlay() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.SpawnableIconOverlay");
}

FText UMovieSceneSpawnableBindingBase::GetBindingTrackIconTooltip() const
{
	return LOCTEXT("CustomSpawnableTooltip", "This item is spawned by sequencer by a custom spawnable binding according to this object's spawn track.");
}

#endif

UWorld* UMovieSceneSpawnableBindingBase::GetWorldContext(TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const
{
	UObject* PlaybackContext = SharedPlaybackState->GetPlaybackContext();
	return PlaybackContext ? PlaybackContext->GetWorld() : nullptr;
}

FMovieSceneBindingResolveResult UMovieSceneSpawnableBindingBase::ResolveBinding(const FMovieSceneBindingResolveParams& ResolveParams, int32 BindingIndex, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const
{
	FMovieSceneBindingResolveResult Result;
	const FMovieSceneSpawnRegister* SpawnRegister = SharedPlaybackState->FindCapability<FMovieSceneSpawnRegister>();
	UObject* SpawnedObject = SpawnRegister ? SpawnRegister->FindSpawnedObject(ResolveParams.ObjectBindingID, ResolveParams.SequenceID, BindingIndex).Get() : nullptr;
	if (SpawnedObject)
	{
		Result.Objects.Add(SpawnedObject);
		Result.Object = SpawnedObject; // Object deprecated 5.7
	}
	return Result;
}

const UMovieSceneSpawnableBindingBase* UMovieSceneSpawnableBindingBase::AsSpawnable(TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const
{
	return Cast<UMovieSceneSpawnableBindingBase>(this);
}

#undef LOCTEXT_NAMESPACE
