// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/MovieSceneAudioTriggerChannel.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Curves/IntegralCurve.h"
#include "Misc/FrameRate.h"
#include "MovieSceneFwd.h"
#include "MovieSceneFrameMigration.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneAudioTriggerChannel)

bool FMovieSceneAudioTriggerChannel::EvaluatePossibleTriggers(const FMovieSceneContext& InContext, FMoveSceneAudioTriggerState& InState, bool& OutTriggered) const
{
	FFrameTime CurrentTime = InContext.GetTime();

	if (Times.Num())
	{
		bool bHasGoneBackwards = InState.PreviousUpdateTime.IsSet() && *InState.PreviousUpdateTime > CurrentTime;

		// If we've somehow gone backwards, reset out state.
		if (bHasGoneBackwards || InContext.GetDirection() == EPlayDirection::Backwards)
		{
			InState.PreviousIndex.Reset();
			InState.PreviousUpdateTime.Reset();
		}

		if (InContext.GetDirection() == EPlayDirection::Forwards)
		{
			const int32 Index = FMath::Max(0, Algo::UpperBound(Times, CurrentTime.FrameNumber) - 1);
			if (Times.IsValidIndex(Index) && CurrentTime.FrameNumber > Times[Index] )
			{
				bool bNewIndex = InState.PreviousIndex.IsSet() ? Index > *InState.PreviousIndex : true;
				OutTriggered = bNewIndex;
				InState.PreviousIndex = Index;
				InState.PreviousUpdateTime = CurrentTime;
				return true;
			}
		}	
	}
	return false;
}

void FMovieSceneAudioTriggerChannel::GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles)
{
	GetData().GetKeys(WithinRange, OutKeyTimes, OutKeyHandles);
}

void FMovieSceneAudioTriggerChannel::GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes)
{
	GetData().GetKeyTimes(InHandles, OutKeyTimes);
}

void FMovieSceneAudioTriggerChannel::SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes)
{
	GetData().SetKeyTimes(InHandles, InKeyTimes);
}

void FMovieSceneAudioTriggerChannel::DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles)
{
	GetData().DuplicateKeys(InHandles, OutNewHandles);
}

void FMovieSceneAudioTriggerChannel::DeleteKeys(TArrayView<const FKeyHandle> InHandles)
{
	GetData().DeleteKeys(InHandles);
}

void FMovieSceneAudioTriggerChannel::DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore)
{
	GetData().DeleteKeysFrom(InTime, bDeleteKeysBefore);
}

void FMovieSceneAudioTriggerChannel::RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer)
{
	GetData().RemapTimes(Retimer);
}

TRange<FFrameNumber> FMovieSceneAudioTriggerChannel::ComputeEffectiveRange() const
{
	return GetData().GetTotalRange();
}

int32 FMovieSceneAudioTriggerChannel::GetNumKeys() const
{
	return Times.Num();
}

void FMovieSceneAudioTriggerChannel::Reset()
{
	Times.Reset();
	Values.Reset();
	KeyHandles.Reset();
}

void FMovieSceneAudioTriggerChannel::Optimize(const FKeyDataOptimizationParams& InParameters)
{
	UE::MovieScene::Optimize(this, InParameters);
}

FKeyHandle FMovieSceneAudioTriggerChannel::GetHandle(int32 Index)
{
	return GetData().GetHandle(Index);
}

int32 FMovieSceneAudioTriggerChannel::GetIndex(FKeyHandle Handle)
{
	return GetData().GetIndex(Handle);
}

void FMovieSceneAudioTriggerChannel::Offset(FFrameNumber DeltaPosition)
{
	GetData().Offset(DeltaPosition);
}
