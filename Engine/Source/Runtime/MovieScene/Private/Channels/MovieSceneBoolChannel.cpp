// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/MovieSceneBoolChannel.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Curves/IntegralCurve.h"
#include "Misc/FrameRate.h"
#include "MovieSceneFwd.h"
#include "MovieSceneFrameMigration.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneBoolChannel)

bool FMovieSceneBoolChannel::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	static const FName IntegralCurveName("IntegralCurve");
	if (Tag.GetType().IsStruct(IntegralCurveName))
	{
		FIntegralCurve IntegralCurve;
		FIntegralCurve::StaticStruct()->SerializeItem(Slot, &IntegralCurve, nullptr);

		if (IntegralCurve.GetDefaultValue() != MAX_int32)
		{
			bHasDefaultValue = true;
			DefaultValue = IntegralCurve.GetDefaultValue() != 0;
		}

		Times.Reserve(IntegralCurve.GetNumKeys());
		Values.Reserve(IntegralCurve.GetNumKeys());

		FFrameRate LegacyFrameRate = GetLegacyConversionFrameRate();

		int32 Index = 0;
		for (auto It = IntegralCurve.GetKeyIterator(); It; ++It)
		{
			FFrameNumber KeyTime = UpgradeLegacyMovieSceneTime(nullptr, LegacyFrameRate, It->Time);

			bool Val = It->Value != 0;
			ConvertInsertAndSort<bool>(Index++, KeyTime, Val, Times, Values);
		}
		return true;
	}

	return false;
}

bool FMovieSceneBoolChannel::Evaluate(FFrameTime InTime, bool& OutValue) const
{
	if (Times.Num())
	{
		const FFrameNumber MinFrame = Times[0];
		const FFrameNumber MaxFrame = Times.Last();
		//we do None, Constant, and Linear first there is no cycling and so we just exit
		if (InTime < FFrameTime(MinFrame))
		{
			if (PreInfinityExtrap == RCCE_None)
			{
				return false;
			}

			if (PreInfinityExtrap == RCCE_Constant || PreInfinityExtrap == RCCE_Linear)
			{
				OutValue = Values[0];
				return true;
			}
		}
		else if (InTime > FFrameTime(MaxFrame))
		{
			if (PostInfinityExtrap == RCCE_None)
			{
				return false;
			}

			if (PostInfinityExtrap == RCCE_Constant || PostInfinityExtrap == RCCE_Linear)
			{
				OutValue = Values.Last();
				return true;
			}
		}

		// Compute the cycled time based on extrapolation
		UE::MovieScene::FCycleParams Params = UE::MovieScene::CycleTime(MinFrame, MaxFrame, InTime);

		// Deal with cycles and oscillation
		if (InTime < FFrameTime(MinFrame))
		{
			if (PreInfinityExtrap == RCCE_Oscillate)
			{
				Params.Oscillate(MinFrame.Value, MaxFrame.Value);
			}
		}
		else if (InTime > FFrameTime(MaxFrame))
		{
			if (PostInfinityExtrap == RCCE_Oscillate)
			{
				Params.Oscillate(MinFrame.Value, MaxFrame.Value);
			}
		}
		
		const int32 Index = FMath::Max(0, Algo::UpperBound(Times, Params.Time)-1);
		OutValue = Values[Index];
		return true;
	}
	else if (bHasDefaultValue)
	{
		OutValue = DefaultValue;
		return true;
	}

	return false;
}

void FMovieSceneBoolChannel::GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles)
{
	GetData().GetKeys(WithinRange, OutKeyTimes, OutKeyHandles);
}

void FMovieSceneBoolChannel::GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes)
{
	GetData().GetKeyTimes(InHandles, OutKeyTimes);
}

void FMovieSceneBoolChannel::SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes)
{
	GetData().SetKeyTimes(InHandles, InKeyTimes);
}

void FMovieSceneBoolChannel::DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles)
{
	GetData().DuplicateKeys(InHandles, OutNewHandles);
}

void FMovieSceneBoolChannel::DeleteKeys(TArrayView<const FKeyHandle> InHandles)
{
	GetData().DeleteKeys(InHandles);
}

void FMovieSceneBoolChannel::DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore)
{
	// Insert a key at the current time to maintain evaluation
	if (GetData().GetTimes().Num() > 0)
	{
		bool Value = false;
		if (Evaluate(InTime, Value))
		{
			GetData().UpdateOrAddKey(InTime, Value);
		}
	}

	GetData().DeleteKeysFrom(InTime, bDeleteKeysBefore);
}

void FMovieSceneBoolChannel::RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer)
{
	GetData().RemapTimes(Retimer);
}


TRange<FFrameNumber> FMovieSceneBoolChannel::ComputeEffectiveRange() const
{
	return GetData().GetTotalRange();
}

int32 FMovieSceneBoolChannel::GetNumKeys() const
{
	return Times.Num();
}

void FMovieSceneBoolChannel::Reset()
{
	Times.Reset();
	Values.Reset();
	KeyHandles.Reset();
	bHasDefaultValue = false;
}

void FMovieSceneBoolChannel::Optimize(const FKeyDataOptimizationParams& InParameters)
{
	UE::MovieScene::Optimize(this, InParameters);
}

void FMovieSceneBoolChannel::Offset(FFrameNumber DeltaPosition)
{
	GetData().Offset(DeltaPosition);
}

FKeyHandle FMovieSceneBoolChannel::GetHandle(int32 Index)
{
	return GetData().GetHandle(Index);
}

int32 FMovieSceneBoolChannel::GetIndex(FKeyHandle Handle)
{
	return GetData().GetIndex(Handle);
}

void FMovieSceneBoolChannel::ClearDefault()
{
	bHasDefaultValue = false;
}

