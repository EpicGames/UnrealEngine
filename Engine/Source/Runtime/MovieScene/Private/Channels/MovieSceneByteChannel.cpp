// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/MovieSceneByteChannel.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Curves/IntegralCurve.h"
#include "MovieSceneFwd.h"
#include "Misc/FrameRate.h"
#include "MovieSceneFrameMigration.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneByteChannel)

bool FMovieSceneByteChannel::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	static const FName IntegralCurveName("IntegralCurve");
	if (Tag.GetType().IsStruct(IntegralCurveName))
	{
		FIntegralCurve IntegralCurve;
		FIntegralCurve::StaticStruct()->SerializeItem(Slot, &IntegralCurve, nullptr);

		if (IntegralCurve.GetDefaultValue() != MAX_int32)
		{
			bHasDefaultValue = true;
			// We cast rather than clamp here as the old integer curve used to wrap around
			DefaultValue = static_cast<uint8>(IntegralCurve.GetDefaultValue());
		}

		Times.Reserve(IntegralCurve.GetNumKeys());
		Values.Reserve(IntegralCurve.GetNumKeys());

		FFrameRate LegacyFrameRate = GetLegacyConversionFrameRate();

		int32 Index = 0;
		for (auto It = IntegralCurve.GetKeyIterator(); It; ++It)
		{
			FFrameNumber KeyTime = UpgradeLegacyMovieSceneTime(nullptr, LegacyFrameRate, It->Time);

			// We cast rather than clamp here as the old integer curve used to wrap around
			uint8 Val = static_cast<uint8>(It->Value);
			ConvertInsertAndSort<uint8>(Index++, KeyTime, Val, Times, Values);
		}
		return true;
	}

	return false;
}

bool FMovieSceneByteChannel::Evaluate(FFrameTime InTime, uint8& OutValue) const
{
	if (Times.Num())
	{
		const FFrameNumber MinFrame = Times[0];
		const FFrameNumber MaxFrame = Times.Last();
		//we do None, Constant, and Linear first there is no cycling and so we just exit
		if (InTime < FFrameTime(MinFrame))
		{
			if (PreInfinityExtrap == RCCE_None)
			{
				return false;
			}

			if (PreInfinityExtrap == RCCE_Constant || PreInfinityExtrap == RCCE_Linear)
			{
				OutValue = Values[0];
				return true;
			}
		}
		else if (InTime > FFrameTime(MaxFrame))
		{
			if (PostInfinityExtrap == RCCE_None)
			{
				return false;
			}

			if (PostInfinityExtrap == RCCE_Constant || PostInfinityExtrap == RCCE_Linear)
			{
				OutValue = Values.Last();
				return true;
			}
		}

		// Compute the cycled time based on extrapolation
		UE::MovieScene::FCycleParams Params = UE::MovieScene::CycleTime(MinFrame, MaxFrame, InTime);

		// Deal with offset cycles and oscillation
		// Move int over to double then we will convert back to int if doing an offset
		const double FirstValue = double(Values[0]);
		const double LastValue = double(Values.Last());
		if (InTime < FFrameTime(MinFrame))
		{
			switch (PreInfinityExtrap)
			{
			case RCCE_CycleWithOffset: Params.ComputePreValueOffset(FirstValue, LastValue); break;
			case RCCE_Oscillate:       Params.Oscillate(MinFrame.Value, MaxFrame.Value);                       break;
			}
		}
		else if (InTime > FFrameTime(MaxFrame))
		{
			switch (PostInfinityExtrap)
			{
			case RCCE_CycleWithOffset: Params.ComputePostValueOffset(FirstValue, LastValue); break;
			case RCCE_Oscillate:       Params.Oscillate(MinFrame.Value, MaxFrame.Value);    break;
			}
		}

		const int32 Index = FMath::Max(0, Algo::UpperBound(Times, Params.Time)-1);
		OutValue = Values[Index] + (uint8)(Params.ValueOffset + 0.5);
		return true;
	}
	else if (bHasDefaultValue)
	{
		OutValue = DefaultValue;
		return true;
	}

	return false;
}

void FMovieSceneByteChannel::GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles)
{
	GetData().GetKeys(WithinRange, OutKeyTimes, OutKeyHandles);
}

void FMovieSceneByteChannel::GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes)
{
	GetData().GetKeyTimes(InHandles, OutKeyTimes);
}

void FMovieSceneByteChannel::SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes)
{
	GetData().SetKeyTimes(InHandles, InKeyTimes);
}

void FMovieSceneByteChannel::DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles)
{
	GetData().DuplicateKeys(InHandles, OutNewHandles);
}

void FMovieSceneByteChannel::DeleteKeys(TArrayView<const FKeyHandle> InHandles)
{
	GetData().DeleteKeys(InHandles);
}

void FMovieSceneByteChannel::DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore)
{
	// Insert a key at the current time to maintain evaluation
	if (GetData().GetTimes().Num() > 0)
	{
		uint8 Value = 0;
		if (Evaluate(InTime, Value))
		{
			GetData().UpdateOrAddKey(InTime, Value);
		}
	}

	GetData().DeleteKeysFrom(InTime, bDeleteKeysBefore);
}

void FMovieSceneByteChannel::RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer)
{
	GetData().RemapTimes(Retimer);
}


TRange<FFrameNumber> FMovieSceneByteChannel::ComputeEffectiveRange() const
{
	return GetData().GetTotalRange();
}

int32 FMovieSceneByteChannel::GetNumKeys() const
{
	return Times.Num();
}

void FMovieSceneByteChannel::Reset()
{
	Times.Reset();
	Values.Reset();
	KeyHandles.Reset();
	bHasDefaultValue = false;
}

void FMovieSceneByteChannel::Optimize(const FKeyDataOptimizationParams& InParameters)
{
	UE::MovieScene::Optimize(this, InParameters);
}

void FMovieSceneByteChannel::Offset(FFrameNumber DeltaPosition)
{
	GetData().Offset(DeltaPosition);
}

void FMovieSceneByteChannel::ClearDefault()
{
	bHasDefaultValue = false;
}
