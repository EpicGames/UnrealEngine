// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/MovieSceneChannel.h"
#include "Curves/KeyHandle.h"
#include "IMovieSceneRetimingInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneChannel)

void FMovieSceneChannel::GetKeyTime(const FKeyHandle InHandle, FFrameNumber& OutKeyTime)
{
	GetKeyTimes(MakeArrayView(&InHandle, 1), MakeArrayView(&OutKeyTime, 1));
}

void FMovieSceneChannel::SetKeyTime(const FKeyHandle InHandle, const FFrameNumber InKeyTime)
{
	SetKeyTimes(MakeArrayView(&InHandle, 1), MakeArrayView(&InKeyTime, 1));
}

void FMovieSceneChannel::ChangeFrameResolution(FFrameRate SourceRate, FFrameRate DestinationRate)
{
	RemapTimes(UE::MovieScene::FFrameRateRetiming{ SourceRate, DestinationRate });
}