// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/MovieSceneChannelData.h"
#include "Misc/FrameRate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneChannelData)

namespace UE
{
namespace MovieScene
{

	void EvaluateTime(TArrayView<const FFrameNumber> InTimes, FFrameTime InTime, int32& OutIndex1, int32& OutIndex2)
	{
		const int32 Index2 = Algo::UpperBound(InTimes, InTime.FrameNumber);
		const int32 Index1 = Index2 - 1;

		OutIndex1 = Index1 >= 0            ? Index1 : INDEX_NONE;
		OutIndex2 = Index2 < InTimes.Num() ? Index2 : INDEX_NONE;
	}

	void EvaluateTime(TArrayView<const FFrameNumber> InTimes, FFrameTime InTime, int32& OutIndex1, int32& OutIndex2, double& OutInterp)
	{
		const int32 Index2 = Algo::UpperBound(InTimes, InTime.FrameNumber);
		const int32 Index1 = Index2 - 1;

		OutIndex1 = Index1 >= 0            ? Index1 : INDEX_NONE;
		OutIndex2 = Index2 < InTimes.Num() ? Index2 : INDEX_NONE;

		if (Index1 >= 0 && Index2 < InTimes.Num())
		{
			// Stay in integer space as long as possible
			const int32 Time1 = InTimes[Index1].Value, Time2 = InTimes[Index2].Value;
			const int32 Difference = Time2 - Time1;

			OutInterp = (InTime.AsDecimal() - Time1) / double(Difference);
		}
		else
		{
			OutInterp = 0.f;
		}
	}

	void FindRange(TArrayView<const FFrameNumber> InTimes, FFrameNumber PredicateTime, FFrameNumber InTolerance, int32 MaxNum, int32& OutMin, int32& OutMax)
	{
		const int32 LowerBound = Algo::LowerBound(InTimes, PredicateTime);

		int32 MinIndex = LowerBound, MaxIndex = LowerBound;
		int32 FwdIndex = LowerBound, BwdIndex = LowerBound-1;

		for (;MaxIndex - MinIndex < MaxNum;)
		{
			const bool bConsiderFwdIndex = FwdIndex < InTimes.Num() && FMath::Abs(InTimes[FwdIndex] - PredicateTime) <= InTolerance;
			const bool bConsiderBwdIndex = BwdIndex >= 0            && FMath::Abs(InTimes[BwdIndex] - PredicateTime) <= InTolerance;

			if (bConsiderFwdIndex && bConsiderBwdIndex)
			{
				const FFrameNumber FwdDifference = FMath::Abs(PredicateTime - InTimes[FwdIndex]);
				const FFrameNumber BwdDifference = FMath::Abs(PredicateTime - InTimes[BwdIndex]);

				if (FwdDifference < BwdDifference)
				{
					MaxIndex = ++FwdIndex;
				}
				else
				{
					MinIndex = BwdIndex--;
				}
			}
			else if (bConsiderFwdIndex)
			{
				MaxIndex = ++FwdIndex;
				// Stop considering backwards
				BwdIndex = INDEX_NONE;
			}
			else if (bConsiderBwdIndex)
			{
				MinIndex = BwdIndex--;
				// Stop considering forwards
				FwdIndex = InTimes.Num();
			}
			else
			{
				break;
			}
		}

		OutMin = MinIndex;
		OutMax = MaxIndex;
	}

} // namespace MovieScene
} // namespace UE

FMovieSceneChannelData::FMovieSceneChannelData(FMovieSceneChannel* InChannel, TArray<FFrameNumber>* InTimes, FKeyHandleLookupTable* InKeyHandles)
	: Times(InTimes), KeyHandles(InKeyHandles), OwningChannel(InChannel)
{}

FMovieSceneChannelData::FMovieSceneChannelData(TArray<FFrameNumber>* InTimes, FKeyHandleLookupTable* InKeyHandles, FMovieSceneChannel* InChannel)
	: Times(InTimes), KeyHandles(InKeyHandles), OwningChannel(InChannel)
{}

FKeyHandle FMovieSceneChannelData::GetHandle(int32 Index)
{
	check(Times->IsValidIndex(Index));
	ensureMsgf(KeyHandles, TEXT("This channel does not contain key handles"));
	return KeyHandles ? KeyHandles->FindOrAddKeyHandle(Index) : FKeyHandle::Invalid();
}

int32 FMovieSceneChannelData::GetIndex(FKeyHandle Handle)
{
	ensureMsgf(KeyHandles, TEXT("This channel does not contain key handles"));
	return KeyHandles ? KeyHandles->GetIndex(Handle) : INDEX_NONE;
}

int32 FMovieSceneChannelData::FindKey(FFrameNumber InTime, FFrameNumber InTolerance)
{
	int32 MinIndex = 0, MaxIndex = 0;
	UE::MovieScene::FindRange(*Times, InTime, InTolerance, 1, MinIndex, MaxIndex);
	if (Times->IsValidIndex(MinIndex) && FMath::Abs((*Times)[MinIndex] - InTime) <= InTolerance)
	{
		return MinIndex;
	}

	return INDEX_NONE;
}

void FMovieSceneChannelData::FindKeys(FFrameNumber InTime, int32 MaxNum, int32& OutMinIndex, int32& OutMaxIndex, int32 InTolerance)
{
	UE::MovieScene::FindRange(*Times, InTime, InTolerance, MaxNum, OutMinIndex, OutMaxIndex);
}

int32 FMovieSceneChannelData::AddKeyInternal(FFrameNumber InTime)
{
	const int32 InsertIndex = Algo::UpperBound(*Times, InTime);

	Times->Insert(InTime, InsertIndex);
	if (KeyHandles)
	{
		KeyHandles->AllocateHandle(InsertIndex);
	}
	return InsertIndex;
}

void FMovieSceneChannelData::ReplaceKeyHandlesInternal(TConstArrayView<int32> KeyIndices, TConstArrayView<FKeyHandle> NewHandles)
{
	if (ensureMsgf(KeyHandles, TEXT("This channel does not contain key handles"))
		&& ensure(KeyIndices.Num() == NewHandles.Num()))
	{
		for (int32 Index = 0; Index < KeyIndices.Num(); ++Index)
		{
			KeyHandles->ReplaceHandle(KeyIndices[Index], NewHandles[Index]);
		}
	}
}

int32 FMovieSceneChannelData::MoveKeyInternal(int32 KeyIndex, FFrameNumber InNewTime)
{
	check(Times->IsValidIndex(KeyIndex));
	FFrameNumber OldTime = (*Times)[KeyIndex];
	int32 NewIndex = Algo::LowerBound(*Times, InNewTime);
	if (NewIndex < KeyIndex || NewIndex > KeyIndex+1)
	{
		// If we're inserting after this key, decrement the new index since we will remove this key
		if (NewIndex > KeyIndex)
		{
			--NewIndex;
		}

		// We have to remove the key and re-add it in the right place
		// This could probably be done better by just shuffling up/down the items that need to move, without ever changing the size of the array
		Times->RemoveAt(KeyIndex, EAllowShrinking::No);
		Times->Insert(InNewTime, NewIndex);

		if (KeyHandles)
		{
			KeyHandles->MoveHandle(KeyIndex, NewIndex);
		}
		if (OwningChannel && OwningChannel->OnKeyMovedEvent().IsBound())
		{
			TArray<FKeyMoveEventItem> Items;
			Items.Add(FKeyMoveEventItem(KeyIndex, OldTime, NewIndex,InNewTime));
			OwningChannel->OnKeyMovedEvent().Broadcast(OwningChannel, Items);
		}
		return NewIndex;
	}
	else
	{
		if (OwningChannel && OwningChannel->OnKeyMovedEvent().IsBound())
		{
			TArray<FKeyMoveEventItem> Items;
			Items.Add(FKeyMoveEventItem(KeyIndex, OldTime, KeyIndex, InNewTime));
			OwningChannel->OnKeyMovedEvent().Broadcast(OwningChannel, Items);
		}
		(*Times)[KeyIndex] = InNewTime;
		return KeyIndex;
	}
}


TRange<FFrameNumber> FMovieSceneChannelData::GetTotalRange() const
{
	return Times->Num() ? TRange<FFrameNumber>((*Times)[0], TRangeBound<FFrameNumber>::Inclusive((*Times)[Times->Num()-1])) : TRange<FFrameNumber>::Empty();
}

void FMovieSceneChannelData::ChangeFrameResolution(FFrameRate SourceRate, FFrameRate DestinationRate)
{
	for (int32 Index = 0; Index < Times->Num(); ++Index)
	{
		(*Times)[Index] = ConvertFrameTime((*Times)[Index], SourceRate, DestinationRate).RoundToFrame();
	}
}

void FMovieSceneChannelData::GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles)
{
	if (!Times->Num())
	{
		return;
	}

	const int32 FirstIndex = WithinRange.GetLowerBound().IsClosed() ? Algo::LowerBound(*Times, WithinRange.GetLowerBoundValue()) : 0;
	const int32 LastIndex  = WithinRange.GetUpperBound().IsClosed() ? Algo::UpperBound(*Times, WithinRange.GetUpperBoundValue()) : Times->Num();

	const int32 NumInRange = LastIndex - FirstIndex;
	if (NumInRange > 0)
	{
		if (OutKeyTimes)
		{
			OutKeyTimes->Reserve(OutKeyTimes->Num() + NumInRange);
			OutKeyTimes->Append(&(*Times)[FirstIndex], NumInRange);
		}

		if (OutKeyHandles)
		{
			OutKeyHandles->Reserve(OutKeyHandles->Num() + NumInRange);

			for (int32 Index = FirstIndex; Index < LastIndex; ++Index)
			{
				OutKeyHandles->Add(GetHandle(Index));
			}
		}
	}
}

void FMovieSceneChannelData::GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes)
{
	check(InHandles.Num() == OutKeyTimes.Num());

	for (int32 Index = 0; Index < InHandles.Num(); ++Index)
	{
		const int32 KeyIndex = GetIndex(InHandles[Index]);
		if (Times->IsValidIndex(KeyIndex))
		{
			OutKeyTimes[Index] = (*Times)[KeyIndex];
		}
	}
}

void FMovieSceneChannelData::Offset(FFrameNumber DeltaTime)
{
	if (OwningChannel == nullptr || OwningChannel->OnKeyMovedEvent().IsBound() == false)
	{
		for (FFrameNumber& Time : *Times)
		{
			Time += DeltaTime;
		}
	}
	else
	{
		TArray<FKeyMoveEventItem> Items;
		for (int32 Index = 0; Index < Times->Num(); ++Index)
		{
			const FFrameNumber OldTime = (*Times)[Index];
			FFrameNumber NewTime = OldTime + DeltaTime;
			Items.Add(FKeyMoveEventItem(Index, OldTime, Index, NewTime));
			(*Times)[Index] = NewTime;
		}
		OwningChannel->OnKeyMovedEvent().Broadcast(OwningChannel, Items);

	}
	

}

