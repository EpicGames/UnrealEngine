// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/MovieSceneChannelEditorData.h"
#include "MovieSceneSection.h"

#if WITH_EDITOR

const FText FCommonChannelData::ChannelX = NSLOCTEXT("MovieSceneChannels", "ChannelX", "X");
const FText FCommonChannelData::ChannelY = NSLOCTEXT("MovieSceneChannels", "ChannelY", "Y");
const FText FCommonChannelData::ChannelZ = NSLOCTEXT("MovieSceneChannels", "ChannelZ", "Z");
const FText FCommonChannelData::ChannelW = NSLOCTEXT("MovieSceneChannels", "ChannelW", "W");

const FText FCommonChannelData::ChannelR = NSLOCTEXT("MovieSceneChannels", "ChannelR", "R");
const FText FCommonChannelData::ChannelG = NSLOCTEXT("MovieSceneChannels", "ChannelG", "G");
const FText FCommonChannelData::ChannelB = NSLOCTEXT("MovieSceneChannels", "ChannelB", "B");
const FText FCommonChannelData::ChannelA = NSLOCTEXT("MovieSceneChannels", "ChannelA", "A");

const FLinearColor FCommonChannelData::RedChannelColor(1.0f, 0.05f, 0.05f, 0.9f);
const FLinearColor FCommonChannelData::GreenChannelColor(0.05f, 1.0f, 0.05f, 0.9f);
const FLinearColor FCommonChannelData::BlueChannelColor(0.1f, 0.2f, 1.0f, 0.9f);

const FName FCommonChannelData::GroupDisplayName = TEXT("GroupDisplayName");


FMovieSceneChannelMetaData::FMovieSceneChannelMetaData()
	: bEnabled(true)
	, bCanCollapseToTrack(true)
	, bRelativeToSection(false)
	, SortOrder(0)
	, bCanBeKeyed(true)
	, bSortEmptyGroupsLast(true)
	, bInvertValue(false)
	, bCanPreserveRatio(false)
	, bPreserveRatio(false)
	, Name(NAME_None)
	, KeyOffset(0)
{}

FMovieSceneChannelMetaData::FMovieSceneChannelMetaData(FName InName, FText InDisplayText, FText InGroup, bool bInEnabled)
	: bEnabled(bInEnabled)
	, bCanCollapseToTrack(true)
	, bRelativeToSection(false)
	, SortOrder(0)
	, bCanBeKeyed(true)
	, bSortEmptyGroupsLast(true)
	, bInvertValue(false)
	, bCanPreserveRatio(false)
	, bPreserveRatio(false)
	, Name(InName)
	, DisplayText(InDisplayText)
	, Group(InGroup)
	, KeyOffset(0)
{}

void FMovieSceneChannelMetaData::SetIdentifiers(FName InName, FText InDisplayText, FText InGroup)
{
	Group = InGroup;
	Name = InName;
	DisplayText = InDisplayText;
}

FString FMovieSceneChannelMetaData::GetPropertyMetaData(const FName& InKey) const
{
	return PropertyMetaData.FindRef(InKey);
}

FFrameNumber FMovieSceneChannelMetaData::GetOffsetTime(const UMovieSceneSection* InSection) const
{
	FFrameNumber OffsetTime = (bRelativeToSection && InSection->HasStartFrame())
		? InSection->GetInclusiveStartFrame()
		: 0;
	return OffsetTime + KeyOffset.Get();
}

#endif	// WITH_EDITOR
