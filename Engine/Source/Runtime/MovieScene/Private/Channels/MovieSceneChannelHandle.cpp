// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/MovieSceneChannelHandle.h"
#include "Channels/MovieSceneChannelProxy.h"

namespace UE::MovieScene
{


#if DO_CHECK
	TMap<FName, TWeakObjectPtr<UStruct>> GChannelTypeNamesToClassTypes;
#endif // DO_CHECK

} // namespace UE::MovieScene

FMovieSceneChannelHandle::FMovieSceneChannelHandle()
	: ChannelTypeName(NAME_None)
	, ChannelIndex(INDEX_NONE)
{}

FMovieSceneChannelHandle::FMovieSceneChannelHandle(TWeakPtr<FMovieSceneChannelProxy> InWeakChannelProxy, FName InChannelTypeName, int32 InChannelIndex)
	: WeakChannelProxy(InWeakChannelProxy)
	, ChannelTypeName(InChannelTypeName)
	, ChannelIndex(InChannelIndex)
{}

FMovieSceneChannel* FMovieSceneChannelHandle::Get() const
{
	TSharedPtr<FMovieSceneChannelProxy> PinnedProxy = WeakChannelProxy.Pin();
	if (PinnedProxy.IsValid())
	{
		const FMovieSceneChannelEntry* Entry = PinnedProxy->FindEntry(ChannelTypeName);
		if (Entry)
		{
			TArrayView<FMovieSceneChannel* const> Channels = Entry->GetChannels();
			if (ensureMsgf(Channels.IsValidIndex(ChannelIndex), TEXT("Channel handle created with an invalid index.")))
			{
				return Channels[ChannelIndex];
			}
		}
	}

	return nullptr;
}

FName FMovieSceneChannelHandle::GetChannelTypeName() const
{
	return ChannelTypeName;
}

int32 FMovieSceneChannelHandle::GetChannelIndex() const
{
	return ChannelIndex;
}

#if WITH_EDITOR

const FMovieSceneChannelMetaData* FMovieSceneChannelHandle::GetMetaData() const
{
	TSharedPtr<FMovieSceneChannelProxy> PinnedProxy = WeakChannelProxy.Pin();
	if (PinnedProxy.IsValid())
	{
		const FMovieSceneChannelEntry* Entry = PinnedProxy->FindEntry(ChannelTypeName);
		if (Entry)
		{
			TArrayView<const FMovieSceneChannelMetaData> MetaData = Entry->GetMetaData();
			if (ensureMsgf(MetaData.IsValidIndex(ChannelIndex), TEXT("Channel handle created with an invalid index.")))
			{
				return &MetaData[ChannelIndex];
			}
		}
	}

	return nullptr;
}

const void* FMovieSceneChannelHandle::GetExtendedEditorData() const
{
	TSharedPtr<FMovieSceneChannelProxy> PinnedProxy = WeakChannelProxy.Pin();
	if (PinnedProxy.IsValid())
	{
		const FMovieSceneChannelEntry* Entry = PinnedProxy->FindEntry(ChannelTypeName);
		if (Entry)
		{
			return Entry->GetExtendedEditorData(ChannelIndex);
		}
	}

	return nullptr;
}

#endif // WITH_EDITOR



#if DO_CHECK

void FMovieSceneChannelHandle::TrackChannelTypeNameInternal(UStruct* ChannelType)
{
	FName ChannelName = ChannelType->GetFName();
	if (!UE::MovieScene::GChannelTypeNamesToClassTypes.Contains(ChannelName))
	{
		UE::MovieScene::GChannelTypeNamesToClassTypes.Add(ChannelName, ChannelType);
	}
}
UStruct* FMovieSceneChannelHandle::GetChannelTypeByName(FName ChannelName)
{
	return UE::MovieScene::GChannelTypeNamesToClassTypes.FindRef(ChannelName).Get();
}

bool FMovieSceneChannelHandle::IsCastValidInternal(UStruct* OtherType) const
{
	if (OtherType->GetFName() != ChannelTypeName)
	{
		UStruct* ThisType = GetChannelTypeByName(ChannelTypeName);
		if (!ThisType || !(ThisType->IsChildOf(OtherType) || OtherType->IsChildOf(ThisType)))
		{
			return false;
		}
	}
	return true;
}

#endif // DO_CHECK