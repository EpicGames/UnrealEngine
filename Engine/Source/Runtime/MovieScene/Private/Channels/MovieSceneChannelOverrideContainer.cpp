// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/MovieSceneChannelOverrideContainer.h"
#include "Channels/MovieSceneChannel.h"
#include "Templates/SubclassOf.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneChannelOverrideContainer)

void UMovieSceneChannelOverrideContainer::GetOverrideCandidates(FName InDefaultChannelTypeName, FOverrideCandidates& OutCandidates)
{
	for (TObjectIterator<UClass> ClassIterator; ClassIterator; ++ClassIterator)
	{
		if (ClassIterator->IsChildOf(UMovieSceneChannelOverrideContainer::StaticClass()) && 
				!ClassIterator->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
		{
			const UMovieSceneChannelOverrideContainer* OverrideContainer = ClassIterator->GetDefaultObject<UMovieSceneChannelOverrideContainer>();
			if (OverrideContainer && OverrideContainer->SupportsOverride(InDefaultChannelTypeName))
			{
				OutCandidates.Add(TSubclassOf<UMovieSceneChannelOverrideContainer>(*ClassIterator));
			}
		}
	}
}
