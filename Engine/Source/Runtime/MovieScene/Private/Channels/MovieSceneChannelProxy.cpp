// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/MovieSceneChannelProxy.h"
#include "Algo/BinarySearch.h"


int32 FMovieSceneChannelProxy::NumChannels() const
{
	int32 Num = 0;
	for (const FMovieSceneChannelEntry& Entry : Entries)
	{
		Num += Entry.GetChannels().Num();
	}
	return Num;
}

FMovieSceneChannelHandle FMovieSceneChannelProxy::MakeHandle(FName ChannelTypeName, int32 Index)
{
	TWeakPtr<FMovieSceneChannelProxy> WeakProxy = AsShared();
	return FMovieSceneChannelHandle(WeakProxy, ChannelTypeName, Index);
}

const FMovieSceneChannelEntry* FMovieSceneChannelProxy::FindEntry(FName ChannelTypeName) const
{
	const int32 ChannelTypeIndex = Algo::BinarySearchBy(Entries, ChannelTypeName, &FMovieSceneChannelEntry::ChannelTypeName, FNameLexicalLess());

	if (ChannelTypeIndex != INDEX_NONE)
	{
		return &Entries[ChannelTypeIndex];
	}

	return nullptr;
}

int32 FMovieSceneChannelProxy::FindIndex(FName ChannelTypeName, const FMovieSceneChannel* ChannelPtr) const
{
	const FMovieSceneChannelEntry* FoundEntry = FindEntry(ChannelTypeName);
	if (FoundEntry)
	{
		return FoundEntry->GetChannels().IndexOfByKey(ChannelPtr);
	}

	return INDEX_NONE;
}

FMovieSceneChannel* FMovieSceneChannelProxy::GetChannel(FName ChannelTypeName, int32 ChannelIndex) const
{
	if (const FMovieSceneChannelEntry* Entry = FindEntry(ChannelTypeName))
	{
		TArrayView<FMovieSceneChannel* const> Channels = Entry->GetChannels();
		return Channels.IsValidIndex(ChannelIndex) ? Channels[ChannelIndex] : nullptr;
	}
	return nullptr;
}

#if WITH_EDITOR

void FMovieSceneChannelProxy::EnsureHandlesByNamePopulated() const
{
	if (bHandlesByNamePopulated)
	{
		return;
	}

	bHandlesByNamePopulated = true;
	HandlesByName.Empty();

	TWeakPtr<FMovieSceneChannelProxy> WeakNonConstThis = const_cast<FMovieSceneChannelProxy*>(this)->AsShared();

	for (const FMovieSceneChannelEntry& Entry : Entries)
	{
		const FName ChannelTypeName = Entry.GetChannelTypeName();
		TArrayView<const FMovieSceneChannelMetaData> ChannelMetaDatas = Entry.GetMetaData();
		for (int32 Index = 0; Index < ChannelMetaDatas.Num(); ++Index)
		{
			const FMovieSceneChannelMetaData& ChannelMetaData = ChannelMetaDatas[Index];
			HandlesByName.Add(ChannelMetaData.Name, FMovieSceneChannelHandle(WeakNonConstThis, ChannelTypeName, Index));
		}
	}
}

FMovieSceneChannelHandle FMovieSceneChannelProxy::GetChannelByName(FName ChannelName) const
{
	EnsureHandlesByNamePopulated();
	if (const FMovieSceneChannelHandle* Handle = HandlesByName.Find(ChannelName))
	{
		return *Handle;
	}
	return FMovieSceneChannelHandle();
}

#endif // WITH_EDITOR
