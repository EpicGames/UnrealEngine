// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/MovieSceneCurveChannelCommon.h"
#include "MovieSceneFrameMigration.h"
#include "UObject/SequencerObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneCurveChannelCommon)

bool FMovieSceneTangentData::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FSequencerObjectVersion::GUID);
	if (Ar.CustomVer(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::SerializeFloatChannel)
	{
		return false;
	}

	if (Ar.CustomVer(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::SerializeFloatChannelCompletely)
	{
		Ar << ArriveTangent;
		Ar << LeaveTangent;
		Ar << TangentWeightMode;
		Ar << ArriveTangentWeight;
		Ar << LeaveTangentWeight;
	}
	else
	{
		Ar << ArriveTangent;
		Ar << LeaveTangent;
		Ar << ArriveTangentWeight;
		Ar << LeaveTangentWeight;
		Ar << TangentWeightMode;
	}

	return true;
}

bool FMovieSceneTangentData::operator==(const FMovieSceneTangentData& TangentData) const
{
	return (ArriveTangent == TangentData.ArriveTangent) && (LeaveTangent == TangentData.LeaveTangent) && (TangentWeightMode == TangentData.TangentWeightMode) && (ArriveTangentWeight == TangentData.ArriveTangentWeight) && (LeaveTangentWeight == TangentData.LeaveTangentWeight);
}

bool FMovieSceneTangentData::operator!=(const FMovieSceneTangentData& Other) const
{
	return !(*this == Other);
}


