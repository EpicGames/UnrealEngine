// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/MovieSceneCurveChannelImpl.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneInterpolation.h"
#include "HAL/ConsoleManager.h"
#include "MovieSceneDrawBezierCurve.h"
#include "MovieSceneFrameMigration.h"
#include "Curves/CurveEvaluation.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/SequencerObjectVersion.h"

int32 GSequencerLinearCubicInterpolation = 1;
static FAutoConsoleVariableRef CVarSequencerLinearCubicInterpolation(
	TEXT("Sequencer.LinearCubicInterpolation"),
	GSequencerLinearCubicInterpolation,
	TEXT("If 1 Linear Keys Act As Cubic Interpolation with Linear Tangents, if 0 Linear Key Forces Linear Interpolation to Next Key."),
	ECVF_Default);

int32 GCachedSequencerAutoTangentInterpolation = 2;
static FAutoConsoleVariableRef CVarSequencerAutoTangentInterpolation(
	TEXT("Sequencer.AutoTangentNew"),
	GCachedSequencerAutoTangentInterpolation,
	TEXT("If 2 Sequencer will flatten tangents with no overshoot, if 1 Auto Tangent will use new algorithm to gradually flatten maximum/minimum keys, if 0 Auto Tangent will average all keys (pre 4.23 behavior)."),
	ECVF_Default);

static TAutoConsoleVariable<float> CVarSequencerSmartAutoBlendLocationPercentage(
	TEXT("Sequencer.SmartAutoBlendLocationPercentage"),
	0.8f,
	TEXT("Percentage near the next value that the tangent will blend to the adjacent tangent, if over 1.0 we won't blend. Default to 0.8"),
	ECVF_Default);


namespace UE
{
namespace MovieScene
{

template<typename ChannelType>
static typename ChannelType::CurveValueType
	EvalForTwoKeys(
		const typename ChannelType::ChannelValueType& Key1, FFrameNumber Key1Time,
		const typename ChannelType::ChannelValueType& Key2, FFrameNumber Key2Time,
		FFrameNumber InTime,
		FFrameRate DisplayRate)
{
	using CurveValueType = typename ChannelType::CurveValueType;

	double DecimalRate = DisplayRate.AsDecimal();

	double Diff = (double)(Key2Time - Key1Time).Value;
	Diff /= DecimalRate;
	const int CheckBothLinear = GSequencerLinearCubicInterpolation;

	if (Diff > 0)
	{
		if (Key1.InterpMode != RCIM_Constant)
		{
			const double Alpha = ((double)(InTime - Key1Time).Value / DecimalRate) / Diff;
			const CurveValueType P0 = Key1.Value;
			const CurveValueType P3 = Key2.Value;

			if (Key1.InterpMode == RCIM_Linear && (!CheckBothLinear || Key2.InterpMode != RCIM_Cubic))
			{
				return FMath::Lerp(P0, P3, Alpha);
			}
			else
			{
				double LeaveTangent = Key1.Tangent.LeaveTangent * DecimalRate;
				double ArriveTangent = Key2.Tangent.ArriveTangent * DecimalRate;

				const double OneThird = 1.0 / 3.0;
				const CurveValueType P1 = P0 + (LeaveTangent * Diff * OneThird);
				const CurveValueType P2 = P3 - (ArriveTangent * Diff * OneThird);

				return UE::Curves::BezierInterp(P0, P1, P2, P3, Alpha);
			}
		}
		else
		{
			return InTime < Key2Time ? Key1.Value : Key2.Value;
		}
	}
	else
	{
		return Key1.Value;
	}
}

FCycleParams CycleTime(FFrameNumber MinFrame, FFrameNumber MaxFrame, FFrameTime InTime)
{
	int64 Duration = int64(MaxFrame.Value) - int64(MinFrame.Value);
	if (Duration > MAX_int32)
	{
		return FCycleParams(InTime, 0);
	}

	FCycleParams Params(InTime, static_cast<int32>(Duration));
	if (Params.Duration == 0)
	{
		Params.Time = MaxFrame;
		Params.CycleCount = 0;
	}
	else if (InTime < MinFrame)
	{
		const int64 CycleCount = (int64(MaxFrame.Value) - int64(InTime.FrameNumber.Value)) / Params.Duration;

		Params.CycleCount = static_cast<int32>(-CycleCount);
		Params.Time = InTime;
		Params.Time.FrameNumber.Value = static_cast<int32>(int64(Params.Time.FrameNumber.Value) + int64(Params.Duration)*CycleCount);
	}
	else if (InTime > MaxFrame)
	{
		const int64 CycleCount = (int64(InTime.FrameNumber.Value) - int64(MinFrame.Value)) / Params.Duration;

		Params.CycleCount = static_cast<int32>(CycleCount);
		Params.Time = InTime;
		Params.Time.FrameNumber.Value = static_cast<int32>(int64(Params.Time.FrameNumber.Value) - int64(Params.Duration) * CycleCount);
	}

	return Params;
}

}
}


template<typename ChannelType>
void TMovieSceneCurveChannelImpl<ChannelType>::Set(ChannelType* InChannel, TArray<FFrameNumber> InTimes, TArray<ChannelValueType> InValues)
{
	check(InTimes.Num() == InValues.Num());

	InChannel->Times = MoveTemp(InTimes);
	InChannel->Values = MoveTemp(InValues);

	InChannel->KeyHandles.Reset();
	for (int32 Index = 0; Index < InChannel->Times.Num(); ++Index)
	{
		InChannel->KeyHandles.AllocateHandle(Index);
	}
}

template<typename ChannelType>
int32 TMovieSceneCurveChannelImpl<ChannelType>::InsertKeyInternal(ChannelType* InChannel, FFrameNumber InTime)
{
	const int32 InsertIndex = Algo::UpperBound(InChannel->Times, InTime);

	InChannel->Times.Insert(InTime, InsertIndex);
	InChannel->Values.Insert(ChannelValueType(), InsertIndex);

	InChannel->KeyHandles.AllocateHandle(InsertIndex);

	return InsertIndex;
}

template<typename ChannelType>
int32 TMovieSceneCurveChannelImpl<ChannelType>::AddConstantKey(ChannelType* InChannel, FFrameNumber InTime, CurveValueType InValue)
{
	const int32 Index = InsertKeyInternal(InChannel, InTime);

	ChannelValueType& Value = InChannel->Values[Index];
	Value.Value = InValue;
	Value.InterpMode = RCIM_Constant;

	AutoSetTangents(InChannel);

	return Index;
}

template<typename ChannelType>
int32 TMovieSceneCurveChannelImpl<ChannelType>::AddLinearKey(ChannelType* InChannel, FFrameNumber InTime, CurveValueType InValue)
{
	const int32 Index = InsertKeyInternal(InChannel, InTime);

	ChannelValueType& Value = InChannel->Values[Index];
	Value.Value = InValue;
	Value.InterpMode = RCIM_Linear;

	AutoSetTangents(InChannel);

	return Index;
}

template<typename ChannelType>
int32 TMovieSceneCurveChannelImpl<ChannelType>::AddCubicKey(ChannelType* InChannel, FFrameNumber InTime, CurveValueType InValue, ERichCurveTangentMode TangentMode, const FMovieSceneTangentData& Tangent)
{
	const int32 Index = InsertKeyInternal(InChannel, InTime);

	ChannelValueType& Value = InChannel->Values[Index];
	Value.Value = InValue;
	Value.InterpMode = RCIM_Cubic;
	Value.TangentMode = TangentMode;
	Value.Tangent = Tangent;

	AutoSetTangents(InChannel);

	return Index;
}

template<typename ChannelType>
bool TMovieSceneCurveChannelImpl<ChannelType>::CacheExtrapolation(const ChannelType* InChannel, FFrameTime InTime, UE::MovieScene::Interpolation::FCachedInterpolation& OutValue)
{
	using namespace UE::MovieScene::Interpolation;

	// If the time is outside of the curve, deal with extrapolation
	if (InTime < InChannel->Times[0])
	{
		if (InChannel->PreInfinityExtrap == RCCE_None)
		{
			return false;
		}

		FCachedInterpolationRange Range = FCachedInterpolationRange::Until(InChannel->Times[0]);

		if (InChannel->PreInfinityExtrap == RCCE_Constant)
		{
			OutValue = FCachedInterpolation(Range, FConstantValue(InChannel->Times[0], InChannel->Values[0].Value));
			return true;
		}

		if (InChannel->PreInfinityExtrap == RCCE_Linear)
		{
			const ChannelValueType FirstValue = InChannel->Values[0];

			if (FirstValue.InterpMode == RCIM_Constant)
			{
				OutValue = FCachedInterpolation(Range, FConstantValue(InChannel->Times[0], FirstValue.Value));
			}
			else if(FirstValue.InterpMode == RCIM_Cubic)
			{
				OutValue = FCachedInterpolation(Range, FLinearInterpolation(InChannel->Times[0], FirstValue.Tangent.ArriveTangent, FirstValue.Value));
			}
			else if(FirstValue.InterpMode == RCIM_Linear)
			{
				const int32 InterpStartFrame = InChannel->Times[1].Value;

				const double DY = InChannel->Values[1].Value - FirstValue.Value;
				const double DX = InChannel->Times[1].Value - InChannel->Times[0].Value;
				const double Coefficient = DX == 0.0 ? 0.0 : DY/DX;

				OutValue = FCachedInterpolation(Range, FLinearInterpolation(InChannel->Times[0], Coefficient, FirstValue.Value));
			}
			return true;
		}
	}
	else if (InTime > InChannel->Times.Last())
	{
		if (InChannel->PostInfinityExtrap == RCCE_None)
		{
			return false;
		}

		FCachedInterpolationRange Range = FCachedInterpolationRange::From(InChannel->Times.Last());

		if (InChannel->PostInfinityExtrap == RCCE_Constant)
		{
			OutValue = FCachedInterpolation(Range, FConstantValue(InChannel->Times.Last(), InChannel->Values.Last().Value));
			return true;
		}

		if (InChannel->PostInfinityExtrap == RCCE_Linear)
		{
			const ChannelValueType LastValue = InChannel->Values.Last();

			if (LastValue.InterpMode == RCIM_Constant)
			{
				OutValue = FCachedInterpolation(Range, FConstantValue(InChannel->Times.Last(), LastValue.Value));
			}
			else if(LastValue.InterpMode == RCIM_Cubic)
			{
				OutValue = FCachedInterpolation(Range, FLinearInterpolation(InChannel->Times.Last(), LastValue.Tangent.LeaveTangent, LastValue.Value));
			}
			else if(LastValue.InterpMode == RCIM_Linear)
			{
				const int32 NumKeys          = InChannel->Times.Num();
				const int32 InterpStartFrame = InChannel->Times[NumKeys-2].Value;
				const int32 DeltaFrame       = InChannel->Times.Last().Value - InterpStartFrame;

				const double DY = LastValue.Value - InChannel->Values[NumKeys-2].Value;
				const double DX = InChannel->Times.Last().Value - InChannel->Times[NumKeys-2].Value;
				const double Coefficient = DX == 0.0 ? 0.0 : DY/DX;

				OutValue = FCachedInterpolation(Range, FLinearInterpolation(InChannel->Times.Last(), Coefficient, LastValue.Value));
			}
			return true;
		}
	}

	return false;
}

template<typename ChannelType>
UE::MovieScene::Interpolation::FCachedInterpolation TMovieSceneCurveChannelImpl<ChannelType>::GetInterpolationForTime(const ChannelType* InChannel, FFrameTime InTime)
{
	return GetInterpolationForTime(InChannel, nullptr, InTime);
}

template<typename ChannelType>
UE::MovieScene::Interpolation::FCachedInterpolation TMovieSceneCurveChannelImpl<ChannelType>::GetInterpolationForTime(const ChannelType* InChannel, FTimeEvaluationCache* InOutEvaluationCache, FFrameTime InTime)
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Interpolation;

	const int32 NumKeys = InChannel->Times.Num();

	// No keys means default value, or nothing
	if (NumKeys == 0)
	{
		if (InChannel->bHasDefaultValue)
		{
			return FCachedInterpolation(FCachedInterpolationRange::Infinite(), FConstantValue(0, InChannel->DefaultValue));
		}

		return FCachedInterpolation();
	}

	// For single keys, we can only ever return that value
	if (NumKeys == 1)
	{
		return FCachedInterpolation(FCachedInterpolationRange::Infinite(), FConstantValue(InChannel->Times[0], InChannel->Values[0].Value));
	}

	// Cache extrapolation if we're outside the bounds of the curve
	// for any non-piecewise extrapolation algorithms (constant or linear)
	{
		FCachedInterpolation ExtrapolatedCache;
		if (CacheExtrapolation(InChannel, InTime, ExtrapolatedCache))
		{
			return ExtrapolatedCache;
		}
	}

	// Piecewise spline evaluation from this point
	const FFrameNumber MinFrame = InChannel->Times[0];
	const FFrameNumber MaxFrame = InChannel->Times.Last();

	// Compute the cycled time based on extrapolation
	FCycleParams Params = CycleTime(MinFrame, MaxFrame, InTime);

	// Deal with offset cycles and oscillation
	if (InTime < FFrameTime(MinFrame))
	{
		switch (InChannel->PreInfinityExtrap)
		{
		case RCCE_CycleWithOffset: Params.ComputePreValueOffset(InChannel->Values[0].Value, InChannel->Values[NumKeys-1].Value); break;
		case RCCE_Oscillate:       Params.Oscillate(MinFrame.Value, MaxFrame.Value);                       break;
		}
	}
	else if (InTime > FFrameTime(MaxFrame))
	{
		switch (InChannel->PostInfinityExtrap)
		{
		case RCCE_CycleWithOffset: Params.ComputePostValueOffset(InChannel->Values[0].Value, InChannel->Values[NumKeys-1].Value); break;
		case RCCE_Oscillate:       Params.Oscillate(MinFrame.Value, MaxFrame.Value);                        break;
		}
	}

	if (!ensureMsgf(Params.Time.FrameNumber >= MinFrame && Params.Time.FrameNumber <= MaxFrame, TEXT("Invalid time computed for float channel evaluation")))
	{
		return FCachedInterpolation();
	}

	// Find the pair of keys that we need to evaluate
	double Interp = 0.0;
	int32 Index1 = INDEX_NONE, Index2 = INDEX_NONE;

	// Initialize cache if not yet performed
	if(InOutEvaluationCache && InOutEvaluationCache->CachedNumFrames == INDEX_NONE)
	{
		UE::MovieScene::EvaluateTime(InChannel->Times, Params.Time, InOutEvaluationCache->Index1, InOutEvaluationCache->Index2, InOutEvaluationCache->InterpValue);
		InOutEvaluationCache->CachedNumFrames = InChannel->Times.Num();
		InOutEvaluationCache->CacheFrameTime = Params.Time;
	}

	// If cache matches contained number of frames, copy data out rather than evaluating time data again
	if (InOutEvaluationCache && InOutEvaluationCache->CachedNumFrames == InChannel->Times.Num() && InOutEvaluationCache->CacheFrameTime == Params.Time)
	{
		Interp = InOutEvaluationCache->InterpValue;
		Index1 = InOutEvaluationCache->Index1;
		Index2 = InOutEvaluationCache->Index2;
	}
	else
	{
		UE::MovieScene::EvaluateTime(InChannel->Times, Params.Time, Index1, Index2, Interp);
	}

	if (Index1 == INDEX_NONE)
	{
		// No starting key - we are probably evaluating directly on the first or last key
		//   we explicitly only cache this for the current time to ensure that subsequent caches can cache the correct pair
		FCachedInterpolationRange Range = FCachedInterpolationRange::Only(Params.Time.GetFrame());
		return FCachedInterpolation(Range, FConstantValue(InChannel->Times[Index2], Params.ValueOffset + InChannel->Values[Index2].Value));
	}
	else if (Index2 == INDEX_NONE)
	{
		// No ending key - we are probably evaluating directly on the first or last key
		//   we explicitly only cache this for the current time to ensure that subsequent caches can cache the correct pair
		FCachedInterpolationRange Range = FCachedInterpolationRange::Only(Params.Time.GetFrame());
		return FCachedInterpolation(Range, FConstantValue(InChannel->Times[Index1], Params.ValueOffset + InChannel->Values[Index1].Value));
	}
	else
	{
		return GetInterpolationForKey(InChannel, Index1, Index2, &Params);
	}
}

template<typename ChannelType>
UE::MovieScene::Interpolation::FCachedInterpolation TMovieSceneCurveChannelImpl<ChannelType>::GetInterpolationForKey(const ChannelType* InChannel, int32 Index, const UE::MovieScene::FCycleParams* Params)
{
	using namespace UE::MovieScene::Interpolation;
	check(InChannel->Times.IsValidIndex(Index));

	if (Index == InChannel->Times.Num() - 1)
	{
		FCachedInterpolationRange Range = FCachedInterpolationRange::Only(InChannel->Times.Last());
		return FCachedInterpolation(Range, FConstantValue(Range.Start, InChannel->Values.Last().Value));
	}

	return GetInterpolationForKey(InChannel, Index, Index+1, Params);
}

template<typename ChannelType>
UE::MovieScene::Interpolation::FCachedInterpolation TMovieSceneCurveChannelImpl<ChannelType>::GetInterpolationForKey(const ChannelType* InChannel, int32 Index1, int32 Index2, const UE::MovieScene::FCycleParams* Params)
{
	using namespace UE::MovieScene::Interpolation;

	check(InChannel->Times.IsValidIndex(Index1) && InChannel->Times.IsValidIndex(Index2));

	// We have a valid pair of keys to cache on the spline.
	const double DX = InChannel->Times[Index2].Value - InChannel->Times[Index1].Value;

	// Cache the pair of keys from the spline at the correct time
	ChannelValueType Key1 = InChannel->Values[Index1];
	ChannelValueType Key2 = InChannel->Values[Index2];

	// Manipulate they keys by translating them by the cycle count in the time-domain.
	// By caching the control points in this translated state we can avoid having to 
	// do any manipulation on the input time
	const FFrameNumber CycleOffset = Params ? Params->CycleCount*Params->Duration : 0;
	const double       ValueOffset = Params ? Params->ValueOffset : 0.0;

	const FFrameNumber MinFrame = InChannel->Times[0];
	const FFrameNumber MaxFrame = InChannel->Times.Last();

	// Compute the start and end values and tangents.
	FFrameNumber Time1, Time2;

	// Control point 1
	double V1 = ValueOffset + Key1.Value;
	double T1 = Key1.Tangent.LeaveTangent;
	double W1 = Key1.Tangent.LeaveTangentWeight;
	bool bIsWeighted1 = (Key1.Tangent.TangentWeightMode == RCTWM_WeightedBoth || Key1.Tangent.TangentWeightMode == RCTWM_WeightedLeave);

	// Control point 2
	double V2 = ValueOffset + Key2.Value;
	double T2 = Key2.Tangent.ArriveTangent;
	double W2 = Key2.Tangent.ArriveTangentWeight;
	bool bIsWeighted2 = (Key2.Tangent.TangentWeightMode == RCTWM_WeightedBoth || Key2.Tangent.TangentWeightMode == RCTWM_WeightedArrive);

	// Mirror the curve for oscillation by swapping the control points
	// and mirroring the times based on the width.
	if (Params && Params->ShouldMirrorCurve())
	{
		Swap(V1, V2);
		Swap(T1, T2);
		Swap(W1, W2);
		Swap(bIsWeighted1, bIsWeighted2);

		T1 = -T1;
		T2 = -T2;

		// Mirror the times of the control points such that
		// MinFrame                              MaxFrame
		//  | x1                    x2   x3       |
		//  |               becomes               |
		//  |      x3   x2                     x1 |
		// and offset them by the cycle offset
		Time1 = MinFrame + (MaxFrame - InChannel->Times[Index2]) + CycleOffset;
		Time2 = MinFrame + (MaxFrame - InChannel->Times[Index1]) + CycleOffset;
	}
	else
	{
		// No oscillation so just offset the control points to be in their final positions
		Time1 = InChannel->Times[Index1] + CycleOffset;
		Time2 = InChannel->Times[Index2] + CycleOffset;
	}

	// Cache this interpolation for any time between the two control points
	FCachedInterpolationRange Range = FCachedInterpolationRange::Finite(Time1, Time2);

	// Careful: We use the original Key1 rather than the oscillated interpmode to ensure that
	//          we use the correct key to produce a mirror image
	TEnumAsByte<ERichCurveInterpMode> InterpMode = Key1.InterpMode;
	const int CheckBothLinear = GSequencerLinearCubicInterpolation;
 	if(InterpMode == RCIM_Linear && (CheckBothLinear  && Key2.InterpMode == RCIM_Cubic))
	{
		InterpMode = RCIM_Cubic;
	}

	switch (InterpMode)
	{
	case RCIM_Cubic:
		{
			if (V1 == V2 && T1 == 0 && T2 == 0 && W1 == 0 && W2 == 0)
			{
				return FCachedInterpolation(Range, FConstantValue(Time1, V1));
			}
			// Weighted if either control point has weight on their tangent
			else if (bIsWeighted1 || bIsWeighted2)
			{
				return FCachedInterpolation(Range, FWeightedCubicInterpolation(
					InChannel->TickResolution, Time1,
					Time1, V1, T1, W1, bIsWeighted1,
					Time2, V2, T2, W2, bIsWeighted2
				));
			}
			else
			{
				return FCachedInterpolation(Range, FCubicBezierInterpolation(Time1, DX, V1, V2, T1, T2));
			}
		}
		break;

	case RCIM_Linear:
		{
			const double DY = V2 - V1;
			if (DY == 0.0)
			{
				return FCachedInterpolation(Range, FConstantValue(Time1, V1));
			}
			else
			{
				return FCachedInterpolation(Range, FLinearInterpolation(Time1, DY/DX, V1));
			}
		}

	default:
		return FCachedInterpolation(Range, FConstantValue(Time1, V1));
	}
}

template<typename ChannelType>
bool TMovieSceneCurveChannelImpl<ChannelType>::Evaluate(const ChannelType* InChannel, FFrameTime InTime, CurveValueType& OutValue) 
{
	return EvaluateCached(InChannel, nullptr, InTime, OutValue);
}

template<typename ChannelType>
bool TMovieSceneCurveChannelImpl<ChannelType>::EvaluateWithCache(const ChannelType* InChannel, FTimeEvaluationCache* InOutEvaluationCache, FFrameTime InTime, CurveValueType& OutValue) 
{
	using namespace UE::MovieScene;
	return EvaluateCached(InChannel, InOutEvaluationCache, InTime, OutValue);
}

template<typename ChannelType>
bool TMovieSceneCurveChannelImpl<ChannelType>::EvaluateCached(const ChannelType* InChannel, FTimeEvaluationCache* InOutEvaluationCache, FFrameTime InTime, CurveValueType& OutValue)
{
	double ResultValue = 0.0;
	if (GetInterpolationForTime(InChannel, InOutEvaluationCache, InTime).Evaluate(InTime, ResultValue))
	{
		OutValue = static_cast<CurveValueType>(ResultValue);
		return true;
	}
	return false;
}

template <typename ChannelValueType>
int signNoZero(ChannelValueType val)
{
	return (ChannelValueType(0) < val) ? -1 : 1;
};

template <typename ChannelValueType>
ChannelValueType ClampTangent(ChannelValueType NewTangent, ChannelValueType PreviousSlope, ChannelValueType NextSlope)
{
	if (signNoZero(PreviousSlope) != signNoZero(NextSlope) ||
		signNoZero(NewTangent) != signNoZero(NextSlope))
	{
		NewTangent = 0.0;
	}
	else if (NextSlope >= 0)
	{
		NewTangent = FMath::Min(FMath::Min(NewTangent, NextSlope), PreviousSlope);
	}
	else
	{
		NewTangent = FMath::Max(FMath::Max(NewTangent, NextSlope), PreviousSlope);
	}
	return NewTangent;
};
template<typename ChannelType>
float TMovieSceneCurveChannelImpl<ChannelType>::CalcSmartTangent(ChannelType* InChannel, int32 Index)
{
	ChannelValueType& ThisKey = InChannel->Values[Index];
	int32 PrevIndex = Index - 1;
	ChannelValueType  PrevKey = InChannel->Values[PrevIndex];

	int32 NextIndex = Index + 1;
	ChannelValueType NextKey = InChannel->Values[NextIndex];

	float NewTangent = 0.0f;
	// if key doesn't lie between we keep it flat(0.0), except if using auto tangent option 2 since that handles it automatically
    // and let's us do blending to tangents that have overshoot
	if ((GCachedSequencerAutoTangentInterpolation == 2)|| (ThisKey.Value > PrevKey.Value && ThisKey.Value < NextKey.Value) ||
		(ThisKey.Value < PrevKey.Value && ThisKey.Value > NextKey.Value))
	{
		if (GCachedSequencerAutoTangentInterpolation != 2) //for older versions we try to match over a longer period of tie, not needed with improved flattening
		{
			while (NextIndex <= (InChannel->Values.Num() - 2)
				&& FMath::IsNearlyZero(NextKey.Tangent.ArriveTangent)
				&& NextKey.InterpMode == RCIM_Cubic && (NextKey.TangentMode == RCTM_Auto || NextKey.TangentMode == RCTM_SmartAuto)
				&& ((NextKey.Value > ThisKey.Value && NextKey.Value < InChannel->Values[NextIndex + 1].Value) ||
					(NextKey.Value < ThisKey.Value && NextKey.Value > InChannel->Values[NextIndex + 1].Value)))
			{
				++NextIndex;
				NextKey = InChannel->Values[NextIndex];
			}
		}

		const CurveValueType OneThird = static_cast<CurveValueType>(1.0 / 3.0);
		const CurveValueType TwoThird = static_cast<CurveValueType>(2.0 / 3.0);
		//we calculate the tangent one third from the previous and next tangents
		const double TimeToPrevious = FMath::Max<double>(KINDA_SMALL_NUMBER, InChannel->Times[Index].Value - InChannel->Times[PrevIndex].Value);
		const double OneThirdTimeToPrevious = OneThird * TimeToPrevious;
		CurveValueType PrevY = PrevKey.Value + (PrevKey.Tangent.LeaveTangent * OneThirdTimeToPrevious);

		const double TimeToNext = FMath::Max<double>(KINDA_SMALL_NUMBER, InChannel->Times[NextIndex].Value - InChannel->Times[Index].Value);
		const double OneThirdTimeToNext = OneThird * TimeToNext;
		CurveValueType NextY = NextKey.Value - (NextKey.Tangent.ArriveTangent * OneThirdTimeToNext);
		
		//leaving ThisKey.Value - ThisKey.Value there since it is needed if we decide to add any weighting
		NewTangent = ((ThisKey.Value - PrevY) + (NextY - ThisKey.Value))
			/ (TwoThird * TimeToPrevious + TwoThird * TimeToNext);

		
		if (GCachedSequencerAutoTangentInterpolation == 2) //use flattening, no overshoot
		{
			//if two keys are equivalent in value and both auto tangent is zero
			if (FMath::IsNearlyEqual(ThisKey.Value, NextKey.Value) && NextKey.InterpMode == RCIM_Cubic && (NextKey.TangentMode == RCTM_Auto || NextKey.TangentMode == RCTM_SmartAuto))
			{
				NewTangent = 0.0;
			}
			else
			{
				const double PreviousSlope = (ThisKey.Value - PrevY) / (TwoThird * TimeToPrevious);
				const double NextSlope = (NextY - ThisKey.Value) / (TwoThird * TimeToNext);
				NewTangent = ClampTangent<double>(NewTangent, PreviousSlope, NextSlope);
			}
		}
		else
		{
			const float BlendToNextRange = CVarSequencerSmartAutoBlendLocationPercentage->GetFloat();
			const double ValDiff = FMath::Abs<double>(NextKey.Value - PrevKey.Value);
			const double OurDiff = FMath::Abs<double>(ThisKey.Value - PrevKey.Value);
			//ValDiff won't be zero ever due to previous check
			double PercDiff = OurDiff / ValDiff;
			float NextTangent = (NextKey.InterpMode == RCIM_Cubic && (NextKey.TangentMode == RCTM_Auto || NextKey.TangentMode == RCTM_SmartAuto)) ?
				0.0 : NextKey.Tangent.ArriveTangent;
			float PrevTangent = (PrevKey.InterpMode == RCIM_Cubic && (PrevKey.TangentMode == RCTM_Auto || PrevKey.TangentMode == RCTM_SmartAuto)) ?
				0.0 : PrevKey.Tangent.LeaveTangent;
			if (BlendToNextRange >= 0.0f && BlendToNextRange <= 1.0f)
			{
				NextTangent = PrevTangent = 0.0;
				if (PercDiff > BlendToNextRange)
				{
					PercDiff = (PercDiff - BlendToNextRange) / (1.0 - BlendToNextRange);
					NewTangent = NewTangent * (1.0 - PercDiff) + (PercDiff * NextTangent);
				}
				else if (PercDiff < (1.0 - BlendToNextRange))
				{
					PercDiff = PercDiff / (1.0 - BlendToNextRange);
					NewTangent = NewTangent * PercDiff + (1.0 - PercDiff) * PrevTangent;
				}
			}
		}
	}
	return NewTangent;
}

template<typename ChannelType>
void TMovieSceneCurveChannelImpl<ChannelType>::AutoSetTangents(ChannelType* InChannel, float Tension)
{
	if (InChannel->Values.Num() < 2)
	{
		return;
	}

	{
		ChannelValueType& FirstValue = InChannel->Values[0];
		if (FirstValue.InterpMode == RCIM_Linear)
		{
			FirstValue.Tangent.TangentWeightMode = RCTWM_WeightedNone;
			ChannelValueType& NextKey = InChannel->Values[1];
			const double NextTimeDiff = FMath::Max<double>(KINDA_SMALL_NUMBER, InChannel->Times[1].Value - InChannel->Times[0].Value);
			const double NewTangent = (NextKey.Value - FirstValue.Value) / NextTimeDiff;
			FirstValue.Tangent.LeaveTangent = NewTangent;
		}
		else if (FirstValue.InterpMode == RCIM_Cubic && (FirstValue.TangentMode == RCTM_Auto || FirstValue.TangentMode == RCTM_SmartAuto))
		{
			FirstValue.Tangent.LeaveTangent = FirstValue.Tangent.ArriveTangent = 0.0f;
			FirstValue.Tangent.TangentWeightMode = RCTWM_WeightedNone; 
		}
	}

	{
		ChannelValueType& LastValue = InChannel->Values.Last();
		if (LastValue.InterpMode == RCIM_Linear)
		{
			LastValue.Tangent.TangentWeightMode = RCTWM_WeightedNone;
			int32 Index = InChannel->Values.Num() - 1;
			ChannelValueType& PrevKey = InChannel->Values[Index-1];
			const double PrevTimeDiff = FMath::Max<double>(KINDA_SMALL_NUMBER, InChannel->Times[Index].Value - InChannel->Times[Index - 1].Value);
			const double NewTangent = (LastValue.Value - PrevKey.Value) / PrevTimeDiff;
			LastValue.Tangent.ArriveTangent = NewTangent;
		}
		else if (LastValue.InterpMode == RCIM_Cubic && (LastValue.TangentMode == RCTM_Auto || LastValue.TangentMode == RCTM_SmartAuto))
		{
			LastValue.Tangent.LeaveTangent = LastValue.Tangent.ArriveTangent = 0.0f;
			LastValue.Tangent.TangentWeightMode = RCTWM_WeightedNone;
		}
	}

	const int32 MaxSmartAutoIterations = 10; // with smart auto we may need to iterate tangent calculations since it depends 
											 // upon next tangent value which may be 0 since it is new.
	int32 SmartAutoCount = 0;
	for (int32 Index = 1; Index < InChannel->Values.Num() - 1; ++Index)
	{
		ChannelValueType  PrevKey = InChannel->Values[Index-1];
		ChannelValueType& ThisKey = InChannel->Values[Index  ];

		if (ThisKey.InterpMode == RCIM_Cubic && (ThisKey.TangentMode == RCTM_Auto || ThisKey.TangentMode == RCTM_SmartAuto))
		{
			ChannelValueType NextKey = InChannel->Values[Index + 1];
			float NewTangent = 0.0f; //flat tangent by default
			double PrevToNextTimeDiff = FMath::Max<double>(KINDA_SMALL_NUMBER, InChannel->Times[Index + 1].Value - InChannel->Times[Index - 1].Value);
			if (ThisKey.TangentMode == RCTM_SmartAuto)
			{
				NewTangent = CalcSmartTangent(InChannel, Index);
				//okay we changed our tangent so we need to recalc the previous tangent
				//but only if it's non zero
				if (FMath::IsNearlyZero(ThisKey.Tangent.ArriveTangent,1e-3f) == false && FMath::IsNearlyEqual(ThisKey.Tangent.ArriveTangent, NewTangent,1e-3f) == false)
				{
					if (SmartAutoCount < MaxSmartAutoIterations && Index >= 2)
					{
						++SmartAutoCount;
						Index -= 2; //go back 2 since we will increment by one
					}
					else
					{
						SmartAutoCount = 0;
					}
				}
				else
				{
					SmartAutoCount = 0;
				}
			}
			else if (GCachedSequencerAutoTangentInterpolation == 0)
			{
				//need to pass in the curve value type since though unfortunately tangents are just always floats (for doubles or floats),
				//the AutoCalcTangent works with either.
				CurveValueType ValueNewTangent = 0.0;
				AutoCalcTangent(PrevKey.Value, ThisKey.Value, NextKey.Value, Tension, ValueNewTangent);
				NewTangent = ValueNewTangent;
				NewTangent /= PrevToNextTimeDiff;
			}
			else
			{
				if (GCachedSequencerAutoTangentInterpolation >= 1)
				{
					// if key doesn't lie between we keep it flat(0.0).
					if ((ThisKey.Value > PrevKey.Value && ThisKey.Value < NextKey.Value) ||
						(ThisKey.Value < PrevKey.Value && ThisKey.Value > NextKey.Value))
					{
						CurveValueType ValueNewTangent = 0.0;
						AutoCalcTangent(PrevKey.Value, ThisKey.Value, NextKey.Value, Tension, ValueNewTangent);
						NewTangent = ValueNewTangent;
						NewTangent /= PrevToNextTimeDiff;
						if (GCachedSequencerAutoTangentInterpolation < 2)
						{
							//if within 0 to 15% or 85% to 100% range we gradually weight tangent to zero
							const double AverageToZeroRange = 0.85;
							const double ValDiff = FMath::Abs<double>(NextKey.Value - PrevKey.Value);
							const double OurDiff = FMath::Abs<double>(ThisKey.Value - PrevKey.Value);
							//ValDiff won't be zero ever due to previous check
							double PercDiff = OurDiff / ValDiff;
							if (PercDiff > AverageToZeroRange)
							{
								PercDiff = (PercDiff - AverageToZeroRange) / (1.0 - AverageToZeroRange);
								NewTangent = NewTangent * (1.0 - PercDiff);
							}
							else if (PercDiff < (1.0 - AverageToZeroRange))
							{
								PercDiff = PercDiff / (1.0 - AverageToZeroRange);
								NewTangent = NewTangent * PercDiff;
							}

						}				
						else if (GCachedSequencerAutoTangentInterpolation == 2) //use flattening, no overshoot
						{
							const double TwoThird = 2.0 / 3.0;
							const double TimeToPrevious = FMath::Max<double>(KINDA_SMALL_NUMBER, InChannel->Times[Index].Value - InChannel->Times[Index - 1].Value);
							const double TimeToNext = FMath::Max<double>(KINDA_SMALL_NUMBER, InChannel->Times[Index + 1].Value - InChannel->Times[Index].Value);
							const double PreviousSlope = (ThisKey.Value - PrevKey.Value) / (TwoThird * TimeToPrevious);
							const double NextSlope = (NextKey.Value - ThisKey.Value) / (TwoThird * TimeToNext);
							NewTangent = ClampTangent<double>(NewTangent, PreviousSlope, NextSlope);

						}
						
					}
				}
			}

			// In 'auto' mode, arrive and leave tangents are always the same
			ThisKey.Tangent.LeaveTangent = ThisKey.Tangent.ArriveTangent = (double)NewTangent;
			ThisKey.Tangent.TangentWeightMode = RCTWM_WeightedNone;
		}
		else if (ThisKey.InterpMode == RCIM_Linear)
		{
			ThisKey.Tangent.TangentWeightMode = RCTWM_WeightedNone; 
			ChannelValueType& NextKey = InChannel->Values[Index + 1];

			const double PrevTimeDiff = FMath::Max<double>(KINDA_SMALL_NUMBER, InChannel->Times[Index].Value - InChannel->Times[Index - 1].Value);
			double NewTangent  = (ThisKey.Value - PrevKey.Value) / PrevTimeDiff;
			ThisKey.Tangent.ArriveTangent = NewTangent;
			
			const double NextTimeDiff = FMath::Max<double>(KINDA_SMALL_NUMBER, InChannel->Times[Index + 1].Value - InChannel->Times[Index].Value);
			NewTangent = (NextKey.Value - ThisKey.Value) / NextTimeDiff;
			ThisKey.Tangent.LeaveTangent = NewTangent;
		}
	}
}

template<typename ChannelType>
void TMovieSceneCurveChannelImpl<ChannelType>::DeleteKeysFrom(ChannelType* InChannel, FFrameNumber InTime, bool bDeleteKeysBefore)
{
	// Insert a key at the current time to maintain evaluation
	TMovieSceneChannelData<ChannelValueType> ChannelData(InChannel->GetData());
	if (ChannelData.GetTimes().Num() > 0)
	{
		int32 KeyHandleIndex = ChannelData.FindKey(InTime);
		if (KeyHandleIndex == INDEX_NONE)
		{
			CurveValueType Value = 0;
			if (Evaluate(InChannel, InTime, Value))
			{
				EMovieSceneKeyInterpolation Interpolation = GetInterpolationMode(InChannel, InTime, EMovieSceneKeyInterpolation::SmartAuto);
				AddKeyToChannel(InChannel, InTime, Value, Interpolation);
			}
		}
	}

	ChannelData.DeleteKeysFrom(InTime, bDeleteKeysBefore);
}

template<typename ChannelType>
void TMovieSceneCurveChannelImpl<ChannelType>::ChangeFrameResolution(ChannelType* InChannel, FFrameRate SourceRate, FFrameRate DestinationRate)
{
	RemapTimes(InChannel, UE::MovieScene::FFrameRateRetiming(SourceRate, DestinationRate));
}

template<typename ChannelType>
void TMovieSceneCurveChannelImpl<ChannelType>::RemapTimes(ChannelType* InChannel, const UE::MovieScene::IRetimingInterface& Retimer)
{
	check(InChannel->Times.Num() == InChannel->Values.Num());

	double IntervalFactor = Retimer.GetScale();
	InChannel->GetData().RemapTimes(Retimer);

	for (int32 Index = 0; Index < InChannel->Times.Num(); ++Index)
	{
		ChannelValueType& Value = InChannel->Values[Index];
		Value.Tangent.ArriveTangent *= IntervalFactor;
		Value.Tangent.LeaveTangent  *= IntervalFactor;
	}
}

template<typename ChannelType>
void TMovieSceneCurveChannelImpl<ChannelType>::Optimize(ChannelType* InChannel, const FKeyDataOptimizationParams& InParameters)
{
	TMovieSceneChannelData<ChannelValueType> ChannelData = InChannel->GetData();
	TArray<FFrameNumber> OutKeyTimes;
	TArray<FKeyHandle> OutKeyHandles;

	InChannel->GetKeys(InParameters.Range, &OutKeyTimes, &OutKeyHandles);

	if (OutKeyHandles.Num() > 2)
	{
		int32 MostRecentKeepKeyIndex = 0;
		TArray<FKeyHandle> KeysToRemove;

		for (int32 TestIndex = 1; TestIndex < OutKeyHandles.Num() - 1; ++TestIndex)
		{
			int32 Index = ChannelData.GetIndex(OutKeyHandles[TestIndex]);
			int32 NextIndex = ChannelData.GetIndex(OutKeyHandles[TestIndex+1]);

			const CurveValueType KeyValue = ChannelData.GetValues()[Index].Value;
			const CurveValueType ValueWithoutKey = UE::MovieScene::EvalForTwoKeys<ChannelType>(
				ChannelData.GetValues()[MostRecentKeepKeyIndex], ChannelData.GetTimes()[MostRecentKeepKeyIndex].Value,
				ChannelData.GetValues()[NextIndex], ChannelData.GetTimes()[NextIndex].Value,
				ChannelData.GetTimes()[Index].Value,
				InParameters.DisplayRate);
				
			if (FMath::Abs(ValueWithoutKey - KeyValue) > InParameters.Tolerance) // Is this key needed
			{
				MostRecentKeepKeyIndex = Index;
			}
			else
			{
				KeysToRemove.Add(OutKeyHandles[TestIndex]);
			}
		}

		ChannelData.DeleteKeys(KeysToRemove);

		if (InParameters.bAutoSetInterpolation)
		{
			AutoSetTangents(InChannel);
		}
	}
}


template<typename ChannelType>
EMovieSceneKeyInterpolation TMovieSceneCurveChannelImpl<ChannelType>::GetInterpolationMode(ChannelType* InChannel, const FFrameNumber& InTime, EMovieSceneKeyInterpolation DefaultInterpolationMode)
{
	auto ChannelData = InChannel->GetData();
	const TArrayView<const ChannelValueType> Values = ChannelData.GetValues();
	const TArrayView<FFrameNumber> Times = ChannelData.GetTimes();
	if (Times.Num() > 0)
	{
		//get previous key or first key if first
		int32 Index = Algo::LowerBound(Times, InTime) - 1;
		if (Index < 0)
		{
			Index = 0;
		}
		const ChannelValueType& Value = Values[Index];
		switch (Value.InterpMode.GetValue())
		{
		case RCIM_Linear:
			return EMovieSceneKeyInterpolation::Linear;
			break;
		case RCIM_Constant:
			return EMovieSceneKeyInterpolation::Constant;
			break;

		case RCIM_Cubic:

			switch (Value.TangentMode.GetValue())
			{
			case RCTM_Auto:
				return EMovieSceneKeyInterpolation::Auto;
				break;
			case RCTM_SmartAuto:
				return EMovieSceneKeyInterpolation::SmartAuto;
				break;
			case RCTM_Break:
				return EMovieSceneKeyInterpolation::Break;
				break;
			case RCTM_User:
				return EMovieSceneKeyInterpolation::User;
				break;
			}
			break;
		}

	}
	return DefaultInterpolationMode;
}

template<typename ChannelType>
double TMovieSceneCurveChannelImpl<ChannelType>::GetTangentValue(ChannelType* InChannel, const FFrameNumber InFrameTime, const float InFloatValue, double DeltaTime)
{
	//if zero set to .1 default
	if (FMath::IsNearlyZero(DeltaTime))
	{
		DeltaTime = 0.1;
	}
	// Time as seconds
	double InValue = InFloatValue;
	FFrameRate TickResolution = InChannel->GetTickResolution();
	const double InTime = TickResolution.AsSeconds(InFrameTime);

	double TargetTime = InTime + DeltaTime;				// The time to get tangent value. Could be left or right depending on is DeltaTime is negative or positive
	CurveValueType CurveTargetValue = 0;	// The helper value to get Tangent value
	CurveValueType CurveValue = 0;
	Evaluate(InChannel, InFrameTime, CurveValue);
	Evaluate(InChannel, TargetTime * TickResolution, CurveTargetValue);	// Initialize TargetValue by TargetTime
	double TargetValue = CurveTargetValue;
	double Value = CurveValue;
	double TangentValue = (TargetValue - InValue) / FMath::Abs(DeltaTime);	// The tangent value to return
	
	double PrevTangent = DBL_MAX;						// Used for determine whether the tangent is close to the limit
	int32 Count = 10;									// Preventing we stuck in this function for too long

	// Logic
	// While the tangents not close enough and we haven't reach the max iteration time
	while (!FMath::IsNearlyEqual(FMath::Abs(TangentValue), FMath::Abs(PrevTangent)) && Count > 0)
	{
		// Update previous tangent value and make delta time smaller
		PrevTangent = TangentValue;
		DeltaTime /= 2.0;
		TargetTime = InTime + DeltaTime;

		// Calculate a more precise tangent value
		Evaluate(InChannel, TargetTime * TickResolution, CurveTargetValue);
		TargetValue = CurveTargetValue;
		TangentValue = (TargetValue - InValue) / FMath::Abs(DeltaTime);

		--Count;
	}
	return TangentValue * TickResolution.AsInterval();
}

template<typename ChannelType>
FKeyHandle TMovieSceneCurveChannelImpl<ChannelType>::AddKeyToChannel(ChannelType* InChannel, FFrameNumber InFrameNumber, float InValue, EMovieSceneKeyInterpolation Interpolation)
{
	TMovieSceneChannelData<ChannelValueType> ChannelData = InChannel->GetData();
	int32 ExistingIndex = ChannelData.FindKey(InFrameNumber);
	if (ExistingIndex != INDEX_NONE)
	{
		ChannelValueType& Value = ChannelData.GetValues()[ExistingIndex];
		Value.Value = InValue;
		AutoSetTangents(InChannel);
	}
	else
	{
		FMovieSceneTangentData TangentData;
		if ((Interpolation == EMovieSceneKeyInterpolation::User || Interpolation == EMovieSceneKeyInterpolation::Break)
			&& ChannelData.GetTimes().Num() >= 2)
		{
			//if we are within the range of existing keys set the tangent to be that of the slope of the curve, otherwise
			//just set it as flat(leave as default)
			TRange<FFrameNumber> Range = ChannelData.GetTotalRange();
			if (Range.Contains(InFrameNumber))
			{
				const double DeltaTime = 0.1;
				// Left
				const float ArriveTangent = -GetTangentValue(InChannel, InFrameNumber, InValue, -DeltaTime);

				// Right
				const float LeaveTangent = GetTangentValue(InChannel, InFrameNumber, InValue, DeltaTime);

				TangentData.ArriveTangent = TangentData.LeaveTangent = (ArriveTangent + LeaveTangent) * 0.5f;
			}
			

		}
		switch (Interpolation)
		{
			case EMovieSceneKeyInterpolation::SmartAuto:     ExistingIndex = InChannel->AddCubicKey(InFrameNumber, InValue, RCTM_SmartAuto);  break;
			case EMovieSceneKeyInterpolation::Auto:     ExistingIndex = InChannel->AddCubicKey(InFrameNumber, InValue, RCTM_Auto);  break;
			case EMovieSceneKeyInterpolation::User:     ExistingIndex = InChannel->AddCubicKey(InFrameNumber, InValue, RCTM_User,TangentData);  break;
			case EMovieSceneKeyInterpolation::Break:    ExistingIndex = InChannel->AddCubicKey(InFrameNumber, InValue, RCTM_Break,TangentData); break;
			case EMovieSceneKeyInterpolation::Linear:   ExistingIndex = InChannel->AddLinearKey(InFrameNumber, InValue);            break;
			case EMovieSceneKeyInterpolation::Constant: ExistingIndex = InChannel->AddConstantKey(InFrameNumber, InValue);          break;
		}
		
	}

	return InChannel->GetData().GetHandle(ExistingIndex);
}

template<typename ChannelType>
void TMovieSceneCurveChannelImpl<ChannelType>::Dilate(ChannelType* InChannel, FFrameNumber Origin, float DilationFactor)
{
	TArrayView<FFrameNumber> Times = InChannel->GetData().GetTimes();
	for (FFrameNumber& Time : Times)
	{
		Time = Origin + FFrameNumber(FMath::FloorToInt((Time - Origin).Value * DilationFactor));
	}
	AutoSetTangents(InChannel);
}

template<typename ChannelType>
void TMovieSceneCurveChannelImpl<ChannelType>::AssignValue(ChannelType* InChannel, FKeyHandle InKeyHandle, typename ChannelType::CurveValueType InValue)
{
	TMovieSceneChannelData<ChannelValueType> ChannelData = InChannel->GetData();
	int32 ValueIndex = ChannelData.GetIndex(InKeyHandle);

	if (ValueIndex != INDEX_NONE)
	{
		ChannelData.GetValues()[ValueIndex].Value = InValue;
	}
}

template<typename ChannelType>
void TMovieSceneCurveChannelImpl<ChannelType>::PopulateCurvePoints(const ChannelType* InChannel, double StartTimeSeconds, double EndTimeSeconds, double TimeThreshold, CurveValueType ValueThreshold, FFrameRate InTickResolution, TArray<TTuple<double, double>>& OutPoints)
{
	if (InChannel)
	{
		UE::MovieScene::DrawBezierCurve(
			*InChannel, 
			InTickResolution, 
			TimeThreshold, 
			ValueThreshold, 
			StartTimeSeconds, 
			EndTimeSeconds, 
			OutPoints);
	}
}

template<typename ChannelType>
bool TMovieSceneCurveChannelImpl<ChannelType>::ValueExistsAtTime(const ChannelType* Channel, FFrameNumber InFrameNumber, typename ChannelType::CurveValueType Value)
{
	const FFrameTime FrameTime(InFrameNumber);

	CurveValueType ExistingValue = 0.0;
	return Channel->Evaluate(FrameTime, ExistingValue) && FMath::IsNearlyEqual(ExistingValue, Value, (CurveValueType)KINDA_SMALL_NUMBER);
}

template<typename ChannelType>
bool TMovieSceneCurveChannelImpl<ChannelType>::ValueExistsAtTime(const ChannelType* Channel, FFrameNumber InFrameNumber, const typename ChannelType::ChannelValueType& InValue)
{
	return ValueExistsAtTime(Channel, InFrameNumber, InValue.Value);
}

template<typename ChannelType>
bool TMovieSceneCurveChannelImpl<ChannelType>::Serialize(ChannelType* InChannel, FArchive& Ar)
{
	Ar.UsingCustomVersion(FSequencerObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (Ar.CustomVer(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::SerializeFloatChannelCompletely &&
		Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::SerializeFloatChannelShowCurve)
	{
		return false;
	}

	const bool bSerializeShowCurve = (
			Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::SerializeFloatChannelShowCurve);

	Ar << InChannel->PreInfinityExtrap;
	Ar << InChannel->PostInfinityExtrap;

	// Save FFrameNumber(int32) and channel value arrays.
	// We try to save and load the full array data, unless we are
	// ByteSwapping or the Size has a mismatch on load, then we do normal save/load
	if (Ar.IsLoading())
	{
		int32 CurrentSerializedElementSize = sizeof(FFrameNumber);
		int32 SerializedElementSize = 0;
		Ar << SerializedElementSize;
		if (SerializedElementSize != CurrentSerializedElementSize || Ar.IsByteSwapping())
		{
			Ar << InChannel->Times;
		}
		else
		{
			InChannel->Times.CountBytes(Ar);
			int32 NewArrayNum = 0;
			Ar << NewArrayNum;
			InChannel->Times.Empty(NewArrayNum);
			if (NewArrayNum > 0)
			{
				InChannel->Times.AddUninitialized(NewArrayNum);
				Ar.Serialize(InChannel->Times.GetData(), NewArrayNum * SerializedElementSize);
			}
		}
		CurrentSerializedElementSize = sizeof(ChannelValueType);
		Ar << SerializedElementSize;

		if (SerializedElementSize != CurrentSerializedElementSize || Ar.IsByteSwapping())
		{
			Ar << InChannel->Values;
		}
		else
		{
			InChannel->Values.CountBytes(Ar);
			int32 NewArrayNum = 0;
			Ar << NewArrayNum;
			InChannel->Values.Empty(NewArrayNum);
			if (NewArrayNum > 0)
			{
				InChannel->Values.AddUninitialized(NewArrayNum);
				Ar.Serialize(InChannel->Values.GetData(), NewArrayNum * SerializedElementSize);
			}
		}
	}
	else if (Ar.IsSaving())
	{
		int32 SerializedElementSize = sizeof(FFrameNumber);
		Ar << SerializedElementSize;
		InChannel->Times.CountBytes(Ar);
		int32 ArrayCount = InChannel->Times.Num();
		Ar << ArrayCount;
		if (ArrayCount > 0)
		{
			Ar.Serialize(InChannel->Times.GetData(), ArrayCount * SerializedElementSize);
		}
		InChannel->Values.CountBytes(Ar);
		SerializedElementSize = sizeof(ChannelValueType);
		Ar << SerializedElementSize;
		ArrayCount = InChannel->Values.Num();
		Ar << ArrayCount;
		if (ArrayCount > 0)
		{
			Ar.Serialize(InChannel->Values.GetData(), ArrayCount * SerializedElementSize);
		}
	}

	Ar << InChannel->DefaultValue;
	Ar << InChannel->bHasDefaultValue;
	Ar << InChannel->TickResolution.Numerator;
	Ar << InChannel->TickResolution.Denominator;
	if (Ar.IsTransacting())
	{
		Ar << InChannel->KeyHandles;
	}

	if (bSerializeShowCurve)
	{
#if WITH_EDITOR
		Ar << InChannel->bShowCurve;
#else
		bool bUnused = false;
		Ar << bUnused;
#endif
	}
	return true;
}

template<typename ChannelType>
bool TMovieSceneCurveChannelImpl<ChannelType>::SerializeFromRichCurve(ChannelType* InChannel, const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	static const FName RichCurveName("RichCurve");

	check(InChannel);

	if (Tag.GetType().IsStruct(RichCurveName))
	{
		FRichCurve RichCurve;
		FRichCurve::StaticStruct()->SerializeItem(Slot, &RichCurve, nullptr);

		if (RichCurve.GetDefaultValue() != MAX_flt)
		{
			InChannel->bHasDefaultValue = true;
			InChannel->DefaultValue = RichCurve.GetDefaultValue();
		}

		InChannel->PreInfinityExtrap = RichCurve.PreInfinityExtrap;
		InChannel->PostInfinityExtrap = RichCurve.PostInfinityExtrap;

		InChannel->Times.Reserve(RichCurve.GetNumKeys());
		InChannel->Values.Reserve(RichCurve.GetNumKeys());

		const FFrameRate LegacyFrameRate = GetLegacyConversionFrameRate();
		const float      Interval        = LegacyFrameRate.AsInterval();

		int32 Index = 0;
		for (auto It = RichCurve.GetKeyIterator(); It; ++It)
		{
			const FRichCurveKey& Key = *It;

			FFrameNumber KeyTime = UpgradeLegacyMovieSceneTime(nullptr, LegacyFrameRate, It->Time);

			ChannelValueType NewValue;
			NewValue.Value = Key.Value;
			NewValue.InterpMode  = Key.InterpMode;
			NewValue.TangentMode = Key.TangentMode;
			NewValue.Tangent.ArriveTangent = Key.ArriveTangent * Interval;
			NewValue.Tangent.LeaveTangent  = Key.LeaveTangent  * Interval;
			ConvertInsertAndSort<ChannelValueType>(Index++, KeyTime, NewValue, InChannel->Times, InChannel->Values);
		}

		return true;
	}

	return false;
}

template<typename ChannelType>
bool TMovieSceneCurveChannelImpl<ChannelType>::SerializeChannelValue(ChannelValueType& InValue, FArchive& Ar)
{
	Ar.UsingCustomVersion(FSequencerObjectVersion::GUID);
	if (Ar.CustomVer(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::SerializeFloatChannel)
	{
		return false;
	}

	if constexpr(std::is_same_v<CurveValueType, double>)
	{
		if(Ar.UEVer() >= EUnrealEngineObjectUE5Version::LARGE_WORLD_COORDINATES)
		{
			Ar << InValue.Value;
		}
		else
		{
			// Serialize as float and convert to doubles.
			checkf(Ar.IsLoading(), TEXT("float -> double conversion applied outside of load!"));
			float TempValue = (float)InValue.Value;
			Ar << TempValue;
			InValue.Value = (double)TempValue;
		}
	}
	else
	{
		Ar << InValue.Value;
	}

	if (Ar.CustomVer(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::SerializeFloatChannelCompletely)
	{
		// Serialization is handled manually to avoid the extra size overhead of FProperty tagging.
		// Otherwise with many keys in a FMovieSceneFloatValue the size can become quite large.
		Ar << InValue.InterpMode;
		Ar << InValue.TangentMode;
		Ar << InValue.Tangent;
	}
	else
	{
		Ar << InValue.Tangent.ArriveTangent;
		Ar << InValue.Tangent.LeaveTangent;
		Ar << InValue.Tangent.ArriveTangentWeight;
		Ar << InValue.Tangent.LeaveTangentWeight;
		Ar << InValue.Tangent.TangentWeightMode;
		Ar << InValue.InterpMode;
		Ar << InValue.TangentMode;
		Ar << InValue.PaddingByte;
	}

	return true;
}

template struct MOVIESCENE_API TMovieSceneCurveChannelImpl<FMovieSceneFloatChannel>;
template struct MOVIESCENE_API TMovieSceneCurveChannelImpl<FMovieSceneDoubleChannel>;

