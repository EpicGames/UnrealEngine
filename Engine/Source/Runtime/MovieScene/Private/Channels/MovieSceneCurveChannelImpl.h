// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannel.h"
#include "Channels/MovieSceneChannelData.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneChannelTraits.h"
#include "Channels/MovieSceneCurveChannelCommon.h"
#include "Curves/RealCurve.h"
#include "Curves/RichCurve.h"
#include "KeyParams.h"
#include "MovieSceneFwd.h"


namespace UE::MovieScene::Interpolation
{
	struct FCachedInterpolation;
}


/** Utility class for curve channels */
template<typename ChannelType>
struct TMovieSceneCurveChannelImpl
{
	/** The type of channel value structs */
	using ChannelValueType = typename ChannelType::ChannelValueType;
	/** The type of curve values (float or double) */
	using CurveValueType = typename ChannelType::CurveValueType;

	/** Structure used to store the result of UE::MovieScene::EvaluateTime for a given channel/key distribution*/
	struct FTimeEvaluationCache
	{
		double InterpValue = 0.0;
		int32 Index1 = INDEX_NONE, Index2 = INDEX_NONE;
		int32 CachedNumFrames = INDEX_NONE;
		FFrameTime CacheFrameTime;
	};

	/** Read-only methods */

	/** Evaluate this channel with the frame resolution */
	static bool Evaluate(const ChannelType* InChannel, FFrameTime InTime, CurveValueType& OutValue);

	/**
	 * Evaluate this channel by returning a cachable interpolation structure
	 */
	static UE::MovieScene::Interpolation::FCachedInterpolation GetInterpolationForTime(const ChannelType* InChannel, FFrameTime InTime);
	static UE::MovieScene::Interpolation::FCachedInterpolation GetInterpolationForTime(const ChannelType* InChannel, FTimeEvaluationCache* InOutEvaluationCache, FFrameTime InTime);

	static UE::MovieScene::Interpolation::FCachedInterpolation GetInterpolationForKey(const ChannelType* InChannel, int32 KeyIndex, const UE::MovieScene::FCycleParams* Params = nullptr);
	static UE::MovieScene::Interpolation::FCachedInterpolation GetInterpolationForKey(const ChannelType* InChannel, int32 KeyIndex1, int32 KeyIndex2, const UE::MovieScene::FCycleParams* Params = nullptr);

	/** Evaluate this channel at provided FrameTime, using or populating cached time to frame-number(s) calculation */
	static bool EvaluateWithCache(const ChannelType* InChannel, FTimeEvaluationCache* InOutEvaluationCache, FFrameTime InTime, CurveValueType& OutValue);
	
	 /*
	  * Populate the specified array with times and values that represent the smooth interpolation of 
	  * the given channel across the specified range
	  */
	static void PopulateCurvePoints(const ChannelType* InChannel, double StartTimeSeconds, double EndTimeSeconds, double TimeThreshold, CurveValueType ValueThreshold, FFrameRate TickResolution, TArray<TTuple<double, double>>& OutPoints);

	/** Checks if the given value is the curve channel's value at the given time */
	static bool ValueExistsAtTime(const ChannelType* Channel, FFrameNumber InFrameNumber, typename ChannelType::CurveValueType Value);

	/** Checks if the given value is the curve channel's value at the given time */
	static bool ValueExistsAtTime(const ChannelType* Channel, FFrameNumber InFrameNumber, const typename ChannelType::ChannelValueType& InValue);



	/** Editing methods */

	/** Set the channel's times and values to the requested values */
	static void Set(ChannelType* InChannel, TArray<FFrameNumber> InTimes, TArray<ChannelValueType> InValues);

	/** Adds a constant key */
	static int32 AddConstantKey(ChannelType* InChannel, FFrameNumber InTime, CurveValueType InValue);

	/** Adds a linear interpolation key */
	static int32 AddLinearKey(ChannelType* InChannel, FFrameNumber InTime, CurveValueType InValue);

	/** Adds a cubic interpolation key */
	static int32 AddCubicKey(ChannelType* InChannel, FFrameNumber InTime, CurveValueType InValue, ERichCurveTangentMode TangentMode = RCTM_Auto, const FMovieSceneTangentData& Tangent = FMovieSceneTangentData());

	/** Calculated smart tangent at that index*/
	static float CalcSmartTangent(ChannelType* InChannel, int32 Index);

	/** Auto-sets all tangents in the channel's curve */
	static void AutoSetTangents(ChannelType* InChannel, float Tension = 0.f);

	/** Delete keys before or after the given time */
	static void DeleteKeysFrom(ChannelType* InChannel, FFrameNumber InTime, bool bDeleteKeysBefore);

	/** Change the frame resolution of the channel */
	UE_DEPRECATED(5.6, "Please use RemapTimes")
	static void ChangeFrameResolution(ChannelType* InChannel, FFrameRate SourceRate, FFrameRate DestinationRate);

	/** Remap key positions using a custom remap interface */
	static void RemapTimes(ChannelType* InChannel, const UE::MovieScene::IRetimingInterface& Retimer);

	/** Optimize the channel's curve */
	static void Optimize(ChannelType* InChannel, const FKeyDataOptimizationParams& InParameters);

	/** Add a new key to a channel at a given time */
	static FKeyHandle AddKeyToChannel(ChannelType* InChannel, FFrameNumber InFrameNumber, float InValue, EMovieSceneKeyInterpolation Interpolation);

	/** Get Tangent Value at the specified time with specified delta in seconds*/
	static double GetTangentValue(ChannelType* InChannel, const FFrameNumber InFrameTime, const float InValue, double InDeltaTime);

	/**
	 * Get the interpolation mode to use at a specified time
	 *
	 * @param InChannel          The channel to find the interpolation mode
	 * @param InTime             The time we are looking for an interpolation mode
	 * @param DefaultInterpolationMode Current default interpolation mode, may be returned as the current mode
	 * @return Interpolation mode to use at that frame
	 */
	static EMovieSceneKeyInterpolation GetInterpolationMode(ChannelType* InChannel, const FFrameNumber& InTime, EMovieSceneKeyInterpolation DefaultInterpolationMode);

	/**
	 * Evaluate this channel's extrapolation by populating a cachable structure. Assumes more than 1 key is present.
	 *
	 * @param InTime     The time to evaluate at
	 * @param OutValue   A value to receive the result
	 * @return true if the time was evaluated with extrapolation, false otherwise
	 */
	static bool CacheExtrapolation(const ChannelType* InChannel, FFrameTime InTime, UE::MovieScene::Interpolation::FCachedInterpolation& OutValue);
	
	/** Dilate channel data.*/
	static void Dilate(ChannelType* InChannel, FFrameNumber Origin, float DilationFactor);

	/** Assigns a given value on a key */
	static void AssignValue(ChannelType* InChannel, FKeyHandle InKeyHandle, typename ChannelType::CurveValueType InValue);

	/** Serialization methods */

	/** Serialize the channel */
	static bool Serialize(ChannelType* InChannel, FArchive& Ar);

	/** Serialization helper */
	static bool SerializeFromRichCurve(ChannelType* InChannel, const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	/** Serialize a channel value */
	static bool SerializeChannelValue(ChannelValueType& InValue, FArchive& Ar);


	/** Miscellaneous methods */

	/** Copy a channel of another type into a channel of our type, using a simple cast of values. */
	template<typename OtherChannelType>
	static void CopyChannel(const OtherChannelType* InSourceChannel, ChannelType* OutDestinationChannel);

private:

	/**
	 * Insert a key with a default value at the given time.
	 *
	 * @param InTime  The time to insert the key at
	 * @return The index of the new key
	 */
	static int32 InsertKeyInternal(ChannelType* InChannel, FFrameNumber InTime);

	static bool EvaluateCached(const ChannelType* InChannel, FTimeEvaluationCache* InOutEvaluationCache, FFrameTime InTime, CurveValueType& OutValue);
};

template<typename ChannelType>
template<typename OtherChannelType>
void TMovieSceneCurveChannelImpl<ChannelType>::CopyChannel(const OtherChannelType* InSourceChannel, ChannelType* OutDestinationChannel)
{
	using OtherChannelValueType = typename OtherChannelType::ChannelValueType;
	using OtherCurveValueType = typename OtherChannelType::CurveValueType;

	OutDestinationChannel->PreInfinityExtrap = InSourceChannel->PreInfinityExtrap;
	OutDestinationChannel->PostInfinityExtrap = InSourceChannel->PostInfinityExtrap;

	TOptional<OtherCurveValueType> OtherChannelDefaultValue = InSourceChannel->GetDefault();
	OutDestinationChannel->DefaultValue = (CurveValueType)OtherChannelDefaultValue.Get(0.f);
	OutDestinationChannel->bHasDefaultValue = OtherChannelDefaultValue.IsSet();

	TArrayView<const FFrameNumber> OtherChannelTimes = InSourceChannel->GetTimes();
	TArrayView<const OtherChannelValueType> OtherChannelValues = InSourceChannel->GetValues();
	OutDestinationChannel->Times.Empty(OtherChannelTimes.Num());
	OutDestinationChannel->Values.Empty(OtherChannelValues.Num());
	for (int32 Index = 0; Index < OtherChannelTimes.Num(); ++Index)
	{
		OutDestinationChannel->Times.Add(OtherChannelTimes[Index]);

		const OtherChannelValueType& OtherChannelValue = OtherChannelValues[Index];
		ChannelValueType NewDoubleValue((CurveValueType)OtherChannelValue.Value);
		NewDoubleValue.Tangent = OtherChannelValue.Tangent;
		NewDoubleValue.InterpMode = OtherChannelValue.InterpMode;
		NewDoubleValue.TangentMode = OtherChannelValue.TangentMode;
		OutDestinationChannel->Values.Add(NewDoubleValue);
	}

	OutDestinationChannel->TickResolution = InSourceChannel->GetTickResolution();

#if WITH_EDITORONLY_DATA
	OutDestinationChannel->bShowCurve = InSourceChannel->GetShowCurve();
#endif
}


