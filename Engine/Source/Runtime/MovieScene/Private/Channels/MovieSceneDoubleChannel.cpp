// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/MovieSceneDoubleChannel.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneCurveChannelImpl.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneInterpolation.h"
#include "Channels/MovieScenePiecewiseCurve.h"
#include "Channels/MovieScenePiecewiseCurveUtils.inl"
#include "HAL/Platform.h"
#include "MovieSceneFrameMigration.h"
#include "MovieSceneFwd.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/SequencerObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneDoubleChannel)

static_assert(
		sizeof(FMovieSceneDoubleValue) == 32,
		"The size of the float channel value has changed. You need to update the padding byte at the end of the structure. "
		"You also need to update the layout in FMovieSceneDoubleValue so that they match!");


namespace UE::MovieScene
{
	void OnRemapChannelKeyTime(const FMovieSceneChannel* Channel, const IRetimingInterface& Retimer, FFrameNumber PreviousTime, FFrameNumber CurrentTime, FMovieSceneDoubleValue& InOutValue)
	{
		if (InOutValue.InterpMode == ERichCurveInterpMode::RCIM_Cubic)
		{
			// This is a bit of a hack, but we scale tangents if the remapper has stretched the time around the key that was remapped
			//    We figure out this stretch factor by retiming a time slightly ahead (1/4 of a second) of the key, and seeing how it differs from the new key time
			FFrameTime Diff = 0.25 * static_cast<const FMovieSceneDoubleChannel*>(Channel)->GetTickResolution();

			const double StretchFactor = (Retimer.RemapTime(FFrameTime(PreviousTime + Diff)) - CurrentTime).AsDecimal() / Diff.AsDecimal();
			if (!FMath::IsNearlyEqual(StretchFactor, 1.0) && !FMath::IsNearlyEqual(StretchFactor, 0.0))
			{
				if (InOutValue.Tangent.ArriveTangent != 0.0)
				{
					InOutValue.Tangent.ArriveTangent = static_cast<float>(InOutValue.Tangent.ArriveTangent / StretchFactor);
				}
				else
				{
					InOutValue.Tangent.ArriveTangentWeight = static_cast<float>(InOutValue.Tangent.ArriveTangentWeight * StretchFactor);
				}

				if (InOutValue.Tangent.LeaveTangent != 0.0)
				{
					InOutValue.Tangent.LeaveTangent  = static_cast<float>(InOutValue.Tangent.LeaveTangent  / StretchFactor);
				}
				else
				{
					InOutValue.Tangent.LeaveTangentWeight = static_cast<float>(InOutValue.Tangent.LeaveTangentWeight * StretchFactor);
				}
			}
		}
	}
} // namespace UE::MovieScene

namespace UE::MovieScene::Interpolation
{
	struct FDoubleChannelPiecewiseData
	{
		const FMovieSceneDoubleChannel* Channel;

		bool HasDefaultValue() const
		{
			return Channel->GetDefault().IsSet();
		}
		double GetDefaultValue() const
		{
			return Channel->GetDefault().Get(0.0);
		}
		double PreExtrapolate(const FFrameTime& InTime) const
		{
			double Result = 0.0;
			TMovieSceneCurveChannelImpl<FMovieSceneDoubleChannel>::Evaluate(Channel, InTime, Result);
			return Result;
		}
		double PostExtrapolate(const FFrameTime& InTime) const
		{
			double Result = 0.0;
			TMovieSceneCurveChannelImpl<FMovieSceneDoubleChannel>::Evaluate(Channel, InTime, Result);
			return Result;
		}
		int32 NumPieces() const
		{
			return FMath::Max(0, Channel->GetData().GetValues().Num() - 1);
		}
		int32 GetIndexOfPieceByTime(const FFrameTime& Time) const
		{
			TArrayView<const FFrameNumber> Times = Channel->GetData().GetTimes();
			return FMath::Max(Algo::UpperBound(Times, Time)-1, 0);
		}
		Interpolation::FCachedInterpolation GetPieceByIndex(int32 Index) const
		{
			return TMovieSceneCurveChannelImpl<FMovieSceneDoubleChannel>::GetInterpolationForKey(Channel, Index);
		}
		Interpolation::FCachedInterpolation GetPieceByTime(const FFrameTime& Time) const
		{
			return TMovieSceneCurveChannelImpl<FMovieSceneDoubleChannel>::GetInterpolationForTime(Channel, Time);
		}
		FFrameNumber GetFiniteStart() const
		{
			return Channel->GetData().GetTimes()[0];
		}
		FFrameNumber GetFiniteEnd() const
		{
			return Channel->GetData().GetTimes().Last();
		}
		ERichCurveExtrapolation GetPreExtrapolation() const
		{
			return Channel->PreInfinityExtrap;
		}
		ERichCurveExtrapolation GetPostExtrapolation() const
		{
			return Channel->PostInfinityExtrap;
		}
		double GetStartingValue() const
		{
			return Channel->GetData().GetValues()[0].Value;
		}
		double GetEndingValue() const
		{
			return Channel->GetData().GetValues().Last().Value;
		}
	};

} // namespace UE::MovieScene::Interpolation



bool FMovieSceneDoubleValue::Serialize(FArchive& Ar)
{
	return TMovieSceneCurveChannelImpl<FMovieSceneDoubleChannel>::SerializeChannelValue(*this, Ar);
}

bool FMovieSceneDoubleValue::operator==(const FMovieSceneDoubleValue& DoubleValue) const
{
	return (Value == DoubleValue.Value) && (InterpMode == DoubleValue.InterpMode) && (TangentMode == DoubleValue.TangentMode) && (Tangent == DoubleValue.Tangent);
}

bool FMovieSceneDoubleValue::operator!=(const FMovieSceneDoubleValue& Other) const
{
	return !(*this == Other);
}

int32 FMovieSceneDoubleChannel::AddConstantKey(FFrameNumber InTime, double InValue)
{
	return FMovieSceneDoubleChannelImpl::AddConstantKey(this, InTime, InValue);
}

int32 FMovieSceneDoubleChannel::AddLinearKey(FFrameNumber InTime, double InValue)
{
	return FMovieSceneDoubleChannelImpl::AddLinearKey(this, InTime, InValue);
}

int32 FMovieSceneDoubleChannel::AddCubicKey(FFrameNumber InTime, double InValue, ERichCurveTangentMode TangentMode, const FMovieSceneTangentData& Tangent)
{
	return FMovieSceneDoubleChannelImpl::AddCubicKey(this, InTime, InValue, TangentMode, Tangent);
}

bool FMovieSceneDoubleChannel::Evaluate(FFrameTime InTime,  double& OutValue) const
{
	return FMovieSceneDoubleChannelImpl::Evaluate(this, InTime, OutValue);
}

bool FMovieSceneDoubleChannel::Evaluate(FFrameTime InTime, float& OutValue) const
{
	double Temp;
	const bool bResult = Evaluate(InTime, Temp);
	OutValue = (float)Temp;
	return bResult;
}

UE::MovieScene::Interpolation::FCachedInterpolation FMovieSceneDoubleChannel::GetInterpolationForTime(FFrameTime InTime) const
{
	return FMovieSceneDoubleChannelImpl::GetInterpolationForTime(this, InTime);
}

void FMovieSceneDoubleChannel::Set(TArray<FFrameNumber> InTimes, TArray<FMovieSceneDoubleValue> InValues)
{
	FMovieSceneDoubleChannelImpl::Set(this, InTimes, InValues);
}

void FMovieSceneDoubleChannel::SetKeysOnly(TArrayView<FFrameNumber> InTimes, TArrayView<FMovieSceneDoubleValue> InValues)
{
	check(InTimes.Num() == InValues.Num());

	Times = MoveTemp(InTimes);
	Values = MoveTemp(InValues);

	KeyHandles.Reset();
}

void FMovieSceneDoubleChannel::AutoSetTangents(float Tension)
{
	FMovieSceneDoubleChannelImpl::AutoSetTangents(this, Tension);
}

void FMovieSceneDoubleChannel::PopulateCurvePoints(double StartTimeSeconds, double EndTimeSeconds, double TimeThreshold, double ValueThreshold, FFrameRate InTickResolution, TArray<TTuple<double, double>>& InOutPoints) const
{
	FMovieSceneDoubleChannelImpl::PopulateCurvePoints(this, StartTimeSeconds, EndTimeSeconds, TimeThreshold, ValueThreshold, InTickResolution, InOutPoints);
}

void FMovieSceneDoubleChannel::GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles)
{
	GetData().GetKeys(WithinRange, OutKeyTimes, OutKeyHandles);
}

void FMovieSceneDoubleChannel::GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes)
{
	GetData().GetKeyTimes(InHandles, OutKeyTimes);
}

void FMovieSceneDoubleChannel::SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes)
{
	GetData().SetKeyTimes(InHandles, InKeyTimes);
	AutoSetTangents();
}

void FMovieSceneDoubleChannel::DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles)
{
	GetData().DuplicateKeys(InHandles, OutNewHandles);
}

void FMovieSceneDoubleChannel::DeleteKeys(TArrayView<const FKeyHandle> InHandles)
{
	GetData().DeleteKeys(InHandles);
	AutoSetTangents();
}

void FMovieSceneDoubleChannel::DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore)
{
	FMovieSceneDoubleChannelImpl::DeleteKeysFrom(this, InTime, bDeleteKeysBefore);
}

void FMovieSceneDoubleChannel::RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer)
{
	FMovieSceneDoubleChannelImpl::RemapTimes(this, Retimer);
}

TRange<FFrameNumber> FMovieSceneDoubleChannel::ComputeEffectiveRange() const
{
	return GetData().GetTotalRange();
}

int32 FMovieSceneDoubleChannel::GetNumKeys() const
{
	return Times.Num();
}

void FMovieSceneDoubleChannel::Reset()
{
	Times.Reset();
	Values.Reset();
	KeyHandles.Reset();
	bHasDefaultValue = false;
}

void FMovieSceneDoubleChannel::PostEditChange()
{
	AutoSetTangents();
}

void FMovieSceneDoubleChannel::Offset(FFrameNumber DeltaPosition)
{
	GetData().Offset(DeltaPosition);
	AutoSetTangents();
}

FKeyHandle FMovieSceneDoubleChannel::GetHandle(int32 Index)
{
	return GetData().GetHandle(Index);
}

int32 FMovieSceneDoubleChannel::GetIndex(FKeyHandle Handle)
{
	return GetData().GetIndex(Handle);
}

void FMovieSceneDoubleChannel::Optimize(const FKeyDataOptimizationParams& Params)
{
	FMovieSceneDoubleChannelImpl::Optimize(this, Params);
}

void FMovieSceneDoubleChannel::ClearDefault()
{
	bHasDefaultValue = false;
}

int32 FMovieSceneDoubleChannel::GetCycleCount(FFrameTime InTime) const
{
	if (Times.Num() == 0)
	{
		return 0;
	}

	if (InTime < Times[0])
	{
		switch (PreInfinityExtrap)
		{
		case RCCE_None:     return -1;
		case RCCE_Constant: return -1;
		case RCCE_Linear:   return -1;
		default:            break;
		}
	}
	else if (InTime > Times.Last())
	{
		switch (PostInfinityExtrap)
		{
		case RCCE_None:     return 1;
		case RCCE_Constant: return 1;
		case RCCE_Linear:   return 1;
		default:            break;
		}
	}

	return UE::MovieScene::CycleTime(Times[0], Times.Last(), InTime).CycleCount;
}

TRange<FFrameNumber> FMovieSceneDoubleChannel::GetCycleRange(int32 InCycleCount) const
{
	if (Times.Num() <= 0)
	{
		return TRange<FFrameNumber>::All();
	}

	const bool bCyclePre  = PreInfinityExtrap == RCCE_Cycle  || PreInfinityExtrap == RCCE_CycleWithOffset  || PreInfinityExtrap == RCCE_Oscillate;
	const bool bCyclePost = PostInfinityExtrap == RCCE_Cycle || PostInfinityExtrap == RCCE_CycleWithOffset || PostInfinityExtrap == RCCE_Oscillate;

	if (InCycleCount == 0 || (InCycleCount < 0 && bCyclePre) || (InCycleCount > 0 && bCyclePost))
	{
		const FFrameNumber MinFrame = Times[0];
		const FFrameNumber MaxFrame = Times.Last();
		const FFrameNumber Offset   = (MaxFrame - MinFrame) * InCycleCount;

		return TRange<FFrameNumber>::Inclusive(MinFrame+Offset, MaxFrame+Offset);
	}
	else if (InCycleCount < 0 && (PreInfinityExtrap == RCCE_Linear || PreInfinityExtrap == RCCE_Constant))
	{
		return TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Open(), Times[0]);
	}
	else if (InCycleCount > 0 && (PostInfinityExtrap == RCCE_Linear || PostInfinityExtrap == RCCE_Constant))
	{
		return TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Exclusive(Times.Last()), TRangeBound<FFrameNumber>::Open());
	}

	return TRange<FFrameNumber>::Empty();
}

bool FMovieSceneDoubleChannel::InverseEvaluateBetween(double InValue, FFrameTime StartTime, FFrameTime EndTime, const TFunctionRef<bool(FFrameTime)>& VisitorCallback) const
{
	using namespace UE::MovieScene;

	if (Values.Num() == 0)
	{
		if (bHasDefaultValue && InValue == DefaultValue)
		{
			// Infinite number of solutions - just pick one
			return VisitorCallback(0);
		}

		// No solution
		return true;
	}

	if (Values.Num() == 1)
	{
		if (InValue == Values[0].Value)
		{
			// Infinite number of solutions - just pick one
			return VisitorCallback(0);
		}

		// No solution
		return true;
	}

	FFrameTime TmpSolutions[4];

	Interpolation::FCachedInterpolation Interp = FMovieSceneDoubleChannelImpl::GetInterpolationForTime(this, StartTime);
	while (Interp.IsValid())
	{
		const int32 LocalNumSolutions = Interp.InverseEvaluate(InValue, TmpSolutions);

		for (int32 SolutionIndex = 0; SolutionIndex < LocalNumSolutions; ++SolutionIndex)
		{
			if (!VisitorCallback(TmpSolutions[SolutionIndex]))
			{
				return false;
			}
		}

		// Move on to the next one
		FFrameNumber ThisInterpEnd = Interp.GetRange().End;
		if (ThisInterpEnd != TNumericLimits<FFrameNumber>::Max() && ThisInterpEnd < EndTime)
		{
			Interp = FMovieSceneDoubleChannelImpl::GetInterpolationForTime(this, ThisInterpEnd+1);
		}
		else
		{
			Interp = Interpolation::FCachedInterpolation();
		}
	}

	return true;
}

TOptional<FFrameTime> FMovieSceneDoubleChannel::InverseEvaluate(double InValue, FFrameTime InTimeHint, UE::MovieScene::EInverseEvaluateFlags Flags) const
{
	using namespace UE::MovieScene;

	if (Values.Num() == 0)
	{
		if (bHasDefaultValue && InValue == DefaultValue)
		{
			// Infinite number of solutions - just pick one
			return FFrameTime(0);
		}

		// No solution
		return TOptional<FFrameTime>();
	}

	if (Values.Num() == 1)
	{
		if (InValue == Values[0].Value)
		{
			// Infinite number of solutions - just pick one
			return FFrameTime(0);
		}

		// No solution
		return TOptional<FFrameTime>();
	}

	// Never walk more than this number of iterations away from the time hint unless we have a cycle with offset mode
	int32 MaxIterations = Times.Num()*2;
	if (EnumHasAnyFlags(Flags, EInverseEvaluateFlags::Cycle) && (PreInfinityExtrap == RCCE_CycleWithOffset || PostInfinityExtrap == RCCE_CycleWithOffset))
	{
		const double ValueOffset = Values.Last().Value - Values[0].Value;
		if (!FMath::IsNearlyZero(ValueOffset))
		{
			MaxIterations = 1000;
		}
	}

	const FFrameNumber MinFrame = Times[0];
	const FFrameNumber MaxFrame = Times.Last();
	const int32 TimeHintCycle = CycleTime(MinFrame, MaxFrame, InTimeHint).CycleCount;

	// Use the hint to find our first interpolation
	Interpolation::FCachedInterpolation NextInterp = FMovieSceneDoubleChannelImpl::GetInterpolationForTime(this, InTimeHint);
	if (!NextInterp.IsValid())
	{
		return TOptional<FFrameTime>();
	}

	// Compute the preceeding interpolation
	Interpolation::FCachedInterpolation PrevInterp = EnumHasAnyFlags(Flags, EInverseEvaluateFlags::Backwards)
		? FMovieSceneDoubleChannelImpl::GetInterpolationForTime(this, NextInterp.GetRange().Start-1)
		: Interpolation::FCachedInterpolation();
	

	// Choose the nearest of all the solutions
	FFrameTime TmpSolutions[4];
	TOptional<FFrameTime> Result;
	TOptional<int32> ResultCycleDiff;
	
	double Difference = std::numeric_limits<double>::max();

	int32 IterationCount = 0;


	auto ReportSolution = [Flags, InTimeHint, TimeHintCycle, MinFrame, MaxFrame, &Result, &Difference, &ResultCycleDiff](FFrameTime InResult)
	{
		// Reject solutions that occur at the same time hint if we're not searching with the equal flag
		if (!EnumHasAnyFlags(Flags, EInverseEvaluateFlags::Equal) && InResult == InTimeHint)
		{
			return false;
		}

		// Reject solutions that occur before the time hint if we're not searching backwards
		if (!EnumHasAnyFlags(Flags, EInverseEvaluateFlags::Backwards) && InResult < InTimeHint)
		{
			return false;
		}

		// Reject solutions that occur after the time hint if we're not searching forwards
		if (!EnumHasAnyFlags(Flags, EInverseEvaluateFlags::Forwards) && InResult > InTimeHint)
		{
			return false;
		}

		const int32 SolutionCycle = CycleTime(MinFrame, MaxFrame, InResult).CycleCount;

		// Reject solutions that occur in a different cycle if we don't allow cycling
		if(!EnumHasAnyFlags(Flags, EInverseEvaluateFlags::Cycle) && SolutionCycle != TimeHintCycle)
		{
			return false;
		}

		const double ThisDiff  = FMath::Abs((InResult - InTimeHint).AsDecimal());
		const int32  CycleDiff = FMath::Abs(SolutionCycle - TimeHintCycle);
		
		if (Result.IsSet())
		{
			if (CycleDiff > ResultCycleDiff.GetValue())
			{
				return false;
			}
			if (ThisDiff > Difference)
			{
				return false;
			}
		}

		Result = InResult;
		Difference = ThisDiff;
		ResultCycleDiff = CycleDiff;
		return true;
	};


	// Walk forwards
	while (NextInterp.IsValid() && IterationCount < MaxIterations)
	{
		++IterationCount;

		const int32 LocalNumSolutions = NextInterp.InverseEvaluate(InValue, TmpSolutions);

		for (int32 SolutionIndex = 0; SolutionIndex < LocalNumSolutions; ++SolutionIndex)
		{
			ReportSolution(TmpSolutions[SolutionIndex]);
		}

		if (!Result.IsSet() && EnumHasAnyFlags(Flags, EInverseEvaluateFlags::Forwards))
		{
			// Move on to the next one if possible
			FFrameNumber ThisInterpEnd = NextInterp.GetRange().End;
			if (ThisInterpEnd < TNumericLimits<FFrameNumber>::Max())
			{
				NextInterp = FMovieSceneDoubleChannelImpl::GetInterpolationForTime(this, ThisInterpEnd+1);
				continue;
			}
		}

		// Should only get here if there were solutions, or we're at the end of the range
		break;
	}

	// Walk backwards
	while (PrevInterp.IsValid() && IterationCount < MaxIterations)
	{
		++IterationCount;

		const int32 LocalNumSolutions = PrevInterp.InverseEvaluate(InValue, TmpSolutions);
		for (int32 SolutionIndex = 0; SolutionIndex < LocalNumSolutions; ++SolutionIndex)
		{
			ReportSolution(TmpSolutions[SolutionIndex]);
		}

		if (!Result.IsSet())
		{
			// Move on to the previous one if possible
			FFrameNumber ThisInterpStart = PrevInterp.GetRange().Start;
			if (ThisInterpStart > TNumericLimits<FFrameNumber>::Lowest())
			{
				PrevInterp = FMovieSceneDoubleChannelImpl::GetInterpolationForTime(this, ThisInterpStart-1);
				continue;
			}
		}

		// Should only get here if there were solutions, or we're at the start of the range
		break;
	}

	return Result;
}

UE::MovieScene::Interpolation::FInterpolationExtents FMovieSceneDoubleChannel::ComputeExtents(FFrameTime StartTime, FFrameTime EndTime) const
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Interpolation;

	return ComputePiecewiseExtents(FDoubleChannelPiecewiseData{ this }, StartTime, EndTime);
}

UE::MovieScene::FPiecewiseCurve FMovieSceneDoubleChannel::AsPiecewiseCurve(bool bWithPreAndPostInfinityExtrap) const
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Interpolation;

	FPiecewiseCurve Curve;

	if (Times.Num() == 0)
	{
		if (bHasDefaultValue)
		{
			Curve.Values.Add(FCachedInterpolation(FCachedInterpolationRange::Infinite(), FConstantValue(0, DefaultValue)));
		}
		return Curve;
	}

	if (bWithPreAndPostInfinityExtrap && PreInfinityExtrap != RCCE_None)
	{
		Interpolation::FCachedInterpolation PreExtrap;
		if (FMovieSceneDoubleChannelImpl::CacheExtrapolation(this, Times[0] - 1, PreExtrap))
		{
			Curve.Values.Emplace(PreExtrap);
		}
		else
		{
			ensureMsgf(false, TEXT("Unrepresentable extrapolation mode encountered for piecewise curve"));
		}
	}

	Curve.Values.Reserve(Times.Num());
	for (int32 Index = 0; Index < Times.Num() - 1; ++Index)
	{
		using namespace UE::MovieScene::Interpolation;

		// Add a constant interp if this is the index of the last key, or if the next key sits on the same frame
		const bool bNoRange =
			Index == Times.Num() - 1 ||
			Times[Index] == Times[Index + 1];

		if (bNoRange)
		{
			FCachedInterpolationRange Range = FCachedInterpolationRange::Only(Times[Index]);
			Curve.Values.Emplace(Range, FConstantValue(Range.Start, Values[Index].Value));
		}
		else
		{
			Interpolation::FCachedInterpolation Interp = FMovieSceneDoubleChannelImpl::GetInterpolationForKey(this, Index);
			if (ensure(Interp.IsValid()))
			{
				Curve.Values.Add(MoveTemp(Interp));
			}
		}
	}

	if (bWithPreAndPostInfinityExtrap && PostInfinityExtrap != RCCE_None)
	{
		Interpolation::FCachedInterpolation PostExtrap;
		if (FMovieSceneDoubleChannelImpl::CacheExtrapolation(this, Times.Last() + 1, PostExtrap))
		{
			Curve.Values.Emplace(PostExtrap);
		}
		else
		{
			ensureMsgf(false, TEXT("Unrepresentable extrapolation mode encountered for piecewise curve"));
		}
	}

	return Curve;
}

EMovieSceneKeyInterpolation GetInterpolationMode(FMovieSceneDoubleChannel* InChannel, const FFrameNumber& InTime, EMovieSceneKeyInterpolation DefaultInterpolationMode)
{
	return TMovieSceneCurveChannelImpl<FMovieSceneDoubleChannel>::GetInterpolationMode(InChannel, InTime, DefaultInterpolationMode);
}

FKeyHandle AddKeyToChannel(FMovieSceneDoubleChannel* Channel, FFrameNumber InFrameNumber, double InValue, EMovieSceneKeyInterpolation Interpolation)
{
	return TMovieSceneCurveChannelImpl<FMovieSceneDoubleChannel>::AddKeyToChannel(Channel, InFrameNumber, InValue, Interpolation);
}

void Dilate(FMovieSceneDoubleChannel* InChannel, FFrameNumber Origin, double DilationFactor)
{
	return TMovieSceneCurveChannelImpl<FMovieSceneDoubleChannel>::Dilate(InChannel, Origin, DilationFactor);
}

bool ValueExistsAtTime(const FMovieSceneDoubleChannel* InChannel, FFrameNumber InFrameNumber, const FMovieSceneDoubleValue& InValue)
{
	return TMovieSceneCurveChannelImpl<FMovieSceneDoubleChannel>::ValueExistsAtTime(InChannel, InFrameNumber, InValue);
}

bool ValueExistsAtTime(const FMovieSceneDoubleChannel* InChannel, FFrameNumber InFrameNumber, double InValue)
{
	return TMovieSceneCurveChannelImpl<FMovieSceneDoubleChannel>::ValueExistsAtTime(InChannel, InFrameNumber, InValue);
}

bool ValueExistsAtTime(const FMovieSceneDoubleChannel* InChannel, FFrameNumber InFrameNumber, float InValue)
{
	return TMovieSceneCurveChannelImpl<FMovieSceneDoubleChannel>::ValueExistsAtTime(InChannel, InFrameNumber, (double)InValue);
}

void AssignValue(FMovieSceneDoubleChannel* InChannel, FKeyHandle InKeyHandle, double InValue)
{
	return TMovieSceneCurveChannelImpl<FMovieSceneDoubleChannel>::AssignValue(InChannel, InKeyHandle, InValue);
}

void AssignValue(FMovieSceneDoubleChannel* InChannel, FKeyHandle InKeyHandle, float InValue)
{
	return TMovieSceneCurveChannelImpl<FMovieSceneDoubleChannel>::AssignValue(InChannel, InKeyHandle, (double)InValue);
}

void InvertValue(double& InOutValue)
{
	InOutValue = -InOutValue;
}

void ReciprocalValue(double& InOutValue)
{
	double OriginalValue = InOutValue;

	if (InOutValue != 0)
	{
		InOutValue = 1.0 / InOutValue;
	}
}

void TransformValue(double& InOutValue, const FMovieSceneChannelTraitsTransform<double>& Transform)
{
	double OriginalValue = InOutValue;

	if (Transform.Scale != 0.0)
	{
		InOutValue = (InOutValue - Transform.Pivot) * Transform.Scale + Transform.Pivot;
	}
	InOutValue += Transform.Offset;
}

void FMovieSceneDoubleChannel::AddKeys(const TArray<FFrameNumber>& InTimes, const TArray<FMovieSceneDoubleValue>& InValues)
{
	check(InTimes.Num() == InValues.Num());
	int32 Index = Times.Num();
	Times.Append(InTimes);
	Values.Append(InValues);
	for (; Index < Times.Num(); ++Index)
	{
		KeyHandles.AllocateHandle(Index);
	}
	AutoSetTangents();
}

void FMovieSceneDoubleChannel::UpdateOrAddKeys(const TArrayView<const FFrameNumber> InTimes, const TArrayView<FMovieSceneDoubleValue> InValues)
{
	GetData().UpdateOrAddKeys(InTimes, InValues);

	AutoSetTangents();
}

#if WITH_EDITORONLY_DATA

bool FMovieSceneDoubleChannel::GetShowCurve() const
{
	return bShowCurve;
}

void FMovieSceneDoubleChannel::SetShowCurve(bool bInShowCurve)
{
	bShowCurve = bInShowCurve;
}

#endif

bool FMovieSceneDoubleChannel::Serialize(FArchive& Ar)
{
	return FMovieSceneDoubleChannelImpl::Serialize(this, Ar);
}

bool FMovieSceneDoubleChannel::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	// Load old content that was saved with rich curves.
	const bool bSeralizedFromRichCurve = FMovieSceneDoubleChannelImpl::SerializeFromRichCurve(this, Tag, Slot);
	if (bSeralizedFromRichCurve)
	{
		return true;
	}

	// Load pre-LWC content that was saved with a float channel.
	static const FName FloatChannelName("MovieSceneFloatChannel");
	if (Tag.GetType().IsStruct(FloatChannelName))
	{
		// We have to load the whole structure into a float channel, and then convert it into our data.
		// It's not ideal but it's the safest way to make it work.

		FMovieSceneFloatChannel TempChannel;

		// We also need to setup the temp channel object so that it matches the current channel. This is
		// because, for instance, the Translation/Rotation/Scale channels of the 3d transform section are
		// initialize with a default value of 0. But the default constructor of a channel leaves the
		// default value unset. So if we don't correctly initialize our temp object, it will have its
		// default value left unset unless the saved channel had a non-default value. So the bHasDefaultValue
		// would be left to "false" unless it was set to non-true in the channel... which mean it would
		// always be "false"!
		TMovieSceneCurveChannelImpl<FMovieSceneFloatChannel>::CopyChannel(this, &TempChannel);

		// Serialize the temp channel.
		FMovieSceneFloatChannel::StaticStruct()->SerializeItem(Slot, &TempChannel, nullptr);

		// Now copy the temp channel back into us.
		FMovieSceneDoubleChannelImpl::CopyChannel(&TempChannel, this);

		return true;
	}

	return false;
}


