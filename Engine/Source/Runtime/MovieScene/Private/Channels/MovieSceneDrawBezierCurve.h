// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/MovieSceneDoubleChannel.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Curves/RealCurve.h"
#include "Misc/FrameRate.h"
#include "Templates/UnrealTemplate.h"
#include <type_traits>

namespace UE::MovieScene
{
	/** Currently, conceptually double and float channels support bezier curve painting */
	template <typename ChannelType>
	concept CSupportsBezierCurvePainting =
		std::is_same_v<ChannelType, FMovieSceneDoubleChannel> ||
		std::is_same_v<ChannelType, FMovieSceneFloatChannel>;

	// Fwd
	namespace Private
	{
		template <typename ChannelType, typename ChannelValueType, typename CurveValueType> requires CSupportsBezierCurvePainting<ChannelType>
		struct FMovieSceneDrawInfiniteBezierCurve;

		template <typename ChannelType, typename ChannelValueType, typename CurveValueType> requires CSupportsBezierCurvePainting<ChannelType>
		static void PopulateCurvePoints(const ChannelType& InChannel, const FFrameRate& InTickResolution, const double InTimeThreshold, const CurveValueType InValueThreshold, const double InStartTimeSeconds, const double InEndTimeSeconds, TArray<TTuple<double, double>>& OutPoints);

		template <typename ChannelType, typename CurveValueType> requires CSupportsBezierCurvePainting<ChannelType>
		static void RefineCurvePoints(const ChannelType& InChannel, FFrameRate InTickResolution, double TimeThreshold, CurveValueType ValueThreshold, TArray<TTuple<double, double>>& OutPoints);
	}

	/**
	 * Draws bezier curves, correctly handling and fast painting complex pre-/post-infinity extrapolation.
	 *
	 * @param InChannel							The Movie Scene Channel for which the curve should be painted.
	 * @param InTickResolution					The Movie Scene's current tick resolution
	 * @param InTimeThreshold					The time that is visible as pixel delta on screen
	 * @param InValueThreshold					The value that is visible as pixel delta on screen
	 * @param InStartTimeSeconds				The desired start time in seconds
	 * @param InEndTimeSeconds					The desired end time in seconds
	 * @param OutInterpolatingPoints			The resulting interpolating points of the curve
	 */
	template <typename ChannelType> requires CSupportsBezierCurvePainting<ChannelType>
	static void DrawBezierCurve(
		const ChannelType& InChannel,
		const FFrameRate& InTickResolution,
		const double InTimeThreshold,
		const double InValueThreshold,
		const double InStartTimeSeconds,
		const double InEndTimeSeconds,
		TArray<TTuple<double, double>>& OutInterpolatingPoints)
	{
		/** The type of channel value structs */
		using ChannelValueType = typename ChannelType::ChannelValueType;

		/** The type of curve values (float or double) */
		using CurveValueType = typename ChannelType::CurveValueType;

		const ERichCurveExtrapolation PreInfinityExtrapolation = InChannel.PreInfinityExtrap;
		const ERichCurveExtrapolation PostInfinityExtrapolation = InChannel.PostInfinityExtrap;

		const TArrayView<const FFrameNumber> Times = InChannel.GetTimes();
		const TArrayView<const ChannelValueType> Values = InChannel.GetValues();

		const bool bComplexPreInfinityExtrapolation =
			PreInfinityExtrapolation == ERichCurveExtrapolation::RCCE_Cycle ||
			PreInfinityExtrapolation == ERichCurveExtrapolation::RCCE_CycleWithOffset ||
			PreInfinityExtrapolation == ERichCurveExtrapolation::RCCE_Oscillate;

		const bool bComplexPostInfinityExtrapolation =
			PostInfinityExtrapolation == ERichCurveExtrapolation::RCCE_Cycle ||
			PostInfinityExtrapolation == ERichCurveExtrapolation::RCCE_CycleWithOffset ||
			PostInfinityExtrapolation == ERichCurveExtrapolation::RCCE_Oscillate;

		const bool bDrawComplexPrePostInfinityExtrapolation =
			(bComplexPreInfinityExtrapolation || bComplexPostInfinityExtrapolation) &&
			(Times.Num() > 1 && Values.Num() > 1);

		if (bDrawComplexPrePostInfinityExtrapolation)
		{
			UE::MovieScene::Private::FMovieSceneDrawInfiniteBezierCurve<ChannelType, ChannelValueType, CurveValueType>(
				InChannel,
				InTickResolution,
				InTimeThreshold,
				InValueThreshold,
				InStartTimeSeconds,
				InEndTimeSeconds,
				OutInterpolatingPoints);
		}
		else
		{
			Private::PopulateCurvePoints<ChannelType, ChannelValueType, CurveValueType>(
				InChannel, 
				InTickResolution,
				InTimeThreshold,
				InValueThreshold,
				InStartTimeSeconds, 
				InEndTimeSeconds, 
				OutInterpolatingPoints);
		}
	}

	namespace Private
	{
		/*
		 * Populate the specified array with times and values that represent the smooth interpolation of
		 * the given channel across the specified range.
		 *
		 * Mind this algo only works correctly within in curve range, from the first to the last key.
		 * Mind when pre-/post-infinity is complex, this may not draw the first and the last point correctly when time is near bounds.
		 * 
		 * @param InChannel							The Movie Scene Channel for which the curve should be painted.
		 * @param InTickResolution					The Movie Scene's current tick resolution
		 * @param InTimeThreshold					The time that is visible as pixel delta on screen
		 * @param InValueThreshold					The value that is visible as pixel delta on screen
		 * @param InStartTimeSeconds				The desired start time in seconds
		 * @param InEndTimeSeconds					The desired end time in seconds
		 * @param OutInterpolatingPoints			The resulting interpolating points of the curve
		 */
		template <typename ChannelType, typename ChannelValueType, typename CurveValueType> requires CSupportsBezierCurvePainting<ChannelType>
		static void PopulateCurvePoints(
			const ChannelType& InChannel, 
			const FFrameRate& InTickResolution,
			const double InTimeThreshold,
			const CurveValueType InValueThreshold,
			const double InStartTimeSeconds, 
			const double InEndTimeSeconds, 
			TArray<TTuple<double, double>>& OutPoints)
		{			
			const FFrameNumber StartFrame = (InStartTimeSeconds * InTickResolution).FloorToFrame();
			const FFrameNumber EndFrame = (InEndTimeSeconds * InTickResolution).CeilToFrame();

			const TArrayView<const FFrameNumber> Times = InChannel.GetTimes();
			const TArrayView<const ChannelValueType> Values = InChannel.GetValues();

			const int32 StartingIndex = Algo::UpperBound(Times, StartFrame);
			const int32 EndingIndex = Algo::LowerBound(Times, EndFrame);

			// Add the lower bound of the visible space
			CurveValueType EvaluatedValue;
			if (InChannel.Evaluate(StartFrame, EvaluatedValue))
			{
				OutPoints.Emplace(StartFrame / InTickResolution, static_cast<double>(EvaluatedValue));
			}

			// Add all keys in-between
			for (int32 KeyIndex = StartingIndex; KeyIndex < EndingIndex; ++KeyIndex)
			{
				OutPoints.Emplace(Times[KeyIndex] / InTickResolution, static_cast<double>(Values[KeyIndex].Value));
			}

			// Add the upper bound of the visible space
			if (InChannel.Evaluate(EndFrame, EvaluatedValue))
			{
				OutPoints.Emplace(EndFrame / InTickResolution, static_cast<double>(EvaluatedValue));
			}

			int32 OldSize = OutPoints.Num();
			do
			{
				OldSize = OutPoints.Num();
				Private::RefineCurvePoints<ChannelType, CurveValueType>(InChannel, InTickResolution, InTimeThreshold, InValueThreshold, OutPoints);
			} while (OldSize != OutPoints.Num());
		}

		/**
		 * Adds median points between each of the supplied points if their evaluated value is 
		 * significantly different than the linear interpolation of those points.
		 *
		 * Mind this algo only works in curve range, from the first to the last key.
		 * 
		 * @param TickResolution        The tick resolution with which to interpret this channel's times
		 * @param TimeThreshold         A small time threshold in seconds below which we should stop adding new points
		 * @param ValueThreshold        A small value threshold below which we should stop adding new points where the linear interpolation would suffice
		 * @param InOutPoints           An array to populate with the evaluated points
		 */
		template <typename ChannelType, typename CurveValueType> requires CSupportsBezierCurvePainting<ChannelType>
		static void RefineCurvePoints(
			const ChannelType& InChannel, 
			FFrameRate InTickResolution,
			double InTimeThreshold, 
			CurveValueType InValueThreshold, 
			TArray<TTuple<double, double>>& OutPoints)
		{
			const float InterpTimes[] = { 0.25f, 0.5f, 0.6f };

			for (int32 Index = 0; Index < OutPoints.Num() - 1; ++Index)
			{
				TTuple<double, double> Lower = OutPoints[Index];
				TTuple<double, double> Upper = OutPoints[Index + 1];

				if ((Upper.Get<0>() - Lower.Get<0>()) >= InTimeThreshold)
				{
					bool bSegmentIsLinear = true;

					TTuple<double, double> Evaluated[UE_ARRAY_COUNT(InterpTimes)] = { TTuple<double, double>(0, 0) };

					for (int32 InterpIndex = 0; InterpIndex < UE_ARRAY_COUNT(InterpTimes); ++InterpIndex)
					{
						double& EvalTime = Evaluated[InterpIndex].Get<0>();

						EvalTime = FMath::Lerp(Lower.Get<0>(), Upper.Get<0>(), InterpTimes[InterpIndex]);

						CurveValueType Value = 0.0;
						InChannel.Evaluate(EvalTime * InTickResolution, Value);

						const CurveValueType LinearValue = FMath::Lerp(Lower.Get<1>(), Upper.Get<1>(), InterpTimes[InterpIndex]);
						if (bSegmentIsLinear)
						{
							bSegmentIsLinear = FMath::IsNearlyEqual(Value, LinearValue, InValueThreshold);
						}

						Evaluated[InterpIndex].Get<1>() = Value;
					}

					if (!bSegmentIsLinear)
					{
						// Add the point
						OutPoints.Insert(Evaluated, UE_ARRAY_COUNT(Evaluated), Index + 1);
						--Index;
					}
				}
			}
		}

		/**
		 * Draws bezier curves, correctly handling and fast painting complex pre-/post-infinity extrapolation.
		 * Should only be used when curve data result in complex pre-/post-infinity extrapolation (ensured).
		 */
		template <typename ChannelType, typename ChannelValueType, typename CurveValueType> requires CSupportsBezierCurvePainting<ChannelType>
		struct FMovieSceneDrawInfiniteBezierCurve
			: FNoncopyable
		{
		public:
			/**
			 * Draws complex pre-/post-infinity extrapolation.
			 * Should only be used when pre- or post-infinity extrapolation is complex (cycle, cycle with offset or oscillate).
			 *
			 * @param InChannel							The Movie Scene Channel for which the curve should be painted.
			 * @param InTickResolution					The Movie Scene's current tick resolution
			 * @param InStartTimeSeconds				The desired start time in seconds
			 * @param InEndTimeSeconds					The desired end time in seconds
			 * @param OutInterpolatingPoints			The resulting interpolating points of the curve
			 */
			FMovieSceneDrawInfiniteBezierCurve(
				const ChannelType& InChannel,
				const FFrameRate& InTickResolution,
				const double& InTimeThreshold,
				const double& InValueThreshold,
				const double& InStartTimeSeconds,
				const double& InEndTimeSeconds,
				TArray<TTuple<double, double>>& OutInterpolatingPoints)
				: Channel(InChannel)
				, TickResolution(InTickResolution)
				, TimeThreshold(InTimeThreshold)
				, ValueThreshold(InValueThreshold)
				, StartTimeSeconds(InStartTimeSeconds)
				, EndTimeSeconds(InEndTimeSeconds)
			{
				PreInfinityExtrapolation = InChannel.PreInfinityExtrap;
				PostInfinityExtrapolation = InChannel.PostInfinityExtrap;

				const TArrayView<const FFrameNumber> Times = InChannel.GetTimes();
				const TArrayView<const ChannelValueType> Values = InChannel.GetValues();

				bComplexPreInfinityExtrapolation =
					PreInfinityExtrapolation == ERichCurveExtrapolation::RCCE_Cycle ||
					PreInfinityExtrapolation == ERichCurveExtrapolation::RCCE_CycleWithOffset ||
					PreInfinityExtrapolation == ERichCurveExtrapolation::RCCE_Oscillate;

				bComplexPostInfinityExtrapolation =
					PostInfinityExtrapolation == ERichCurveExtrapolation::RCCE_Cycle ||
					PostInfinityExtrapolation == ERichCurveExtrapolation::RCCE_CycleWithOffset ||
					PostInfinityExtrapolation == ERichCurveExtrapolation::RCCE_Oscillate;

				const bool bDrawComplexPrePostInfinityExtrapolation =
					(bComplexPreInfinityExtrapolation || bComplexPostInfinityExtrapolation) &&
					(Times.Num() > 1 && Values.Num() > 1);

				if (!ensure(bDrawComplexPrePostInfinityExtrapolation))
				{
					PopulateCurvePoints<ChannelType, ChannelValueType, CurveValueType>(
						InChannel,
						InTickResolution,
						InTimeThreshold,
						InValueThreshold,
						InStartTimeSeconds,
						InEndTimeSeconds,
						OutInterpolatingPoints);

					return;
				}

				CurveStartSeconds = Times[0] / TickResolution;
				CurveEndSeconds = Times.Last() / TickResolution;
				CurveDurationSeconds = CurveEndSeconds - CurveStartSeconds;

				VisibleCurveStartSeconds = FMath::Max(CurveStartSeconds, StartTimeSeconds);
				VisibleCurveEndSeconds = FMath::Min(CurveEndSeconds, EndTimeSeconds);

				PreInfinityDuration = CurveStartSeconds - StartTimeSeconds;
				PostInfinityDuration = EndTimeSeconds - CurveEndSeconds;

				bPreInfinityVisible = PreInfinityDuration > 0.0;
				bPostInfinityVisible = PostInfinityDuration > 0.0;

				bPreInfinityOscillates = PreInfinityExtrapolation == ERichCurveExtrapolation::RCCE_Oscillate;
				bPostInfinityOscillates = PostInfinityExtrapolation == ERichCurveExtrapolation::RCCE_Oscillate;

				// Find the interpolation range that is required to draw the curve, but also pre- and post-infinity extrapolation
				double InterpolationStartSeconds;
				double InterpolationEndSeconds;
				GetInterpolationRangeSeconds(InterpolationStartSeconds, InterpolationEndSeconds);

				TArray<TTuple<double, double>> CurveInterpolatingPoints;
				PopulateCurvePoints<ChannelType, ChannelValueType, CurveValueType>(
					InChannel,
					TickResolution,
					TimeThreshold,
					ValueThreshold,
					InterpolationStartSeconds,
					InterpolationEndSeconds,
					CurveInterpolatingPoints);

				// Handle the case where PopulateCurvePoints overdraws to frame time. This would be problematic as it can hold 
				// evaluated values from pre-/post-infinity, e.g. a cycled value, instead of the last value of the curve.
				if (!CurveInterpolatingPoints.IsEmpty())
				{
					if (CurveInterpolatingPoints[0].Get<0>() < CurveStartSeconds)
					{
						CurveInterpolatingPoints[0] = MakeTuple(CurveStartSeconds, static_cast<double>(Values[0].Value));
					}

					if (CurveInterpolatingPoints.Last().Get<0>() > CurveEndSeconds)
					{
						CurveInterpolatingPoints.Last() = MakeTuple(CurveEndSeconds, static_cast<double>(Values.Last().Value));
					}
				}

				// Draw the curve
				if (bPreInfinityVisible)
				{
					if (bComplexPreInfinityExtrapolation)
					{
						DrawPreInfinityExtrapolationComplex(CurveInterpolatingPoints, OutInterpolatingPoints);
					}
					else
					{
						DrawPreInfinityExtrapolationLine(CurveInterpolatingPoints, OutInterpolatingPoints);
					}
				}

				OutInterpolatingPoints.Append(CurveInterpolatingPoints);

				if (bPostInfinityVisible)
				{
					if (bComplexPostInfinityExtrapolation)
					{
						DrawPostInfinityExtrapolationComplex(CurveInterpolatingPoints, OutInterpolatingPoints);
					}
					else
					{
						DrawPostInfinityExtrapolationLine(CurveInterpolatingPoints, OutInterpolatingPoints);
					}
				}
			}

		private:
			/** The channel which is painted */
			const ChannelType& Channel;

			/** The current tick resolution */
			const FFrameRate TickResolution;

			/** The time threshold that is visible on screen as pixel delta */
			const double TimeThreshold = 0.0;

			/** The value threshold that is visible on screen as pixel delta */
			const double ValueThreshold = 0.0;

			/** The start time of the painted range */
			const double StartTimeSeconds = 0.0;

			/** The end time of the painted range */
			const double EndTimeSeconds = 0.0;

			/** The pre-infinity extrapolation type */
			ERichCurveExtrapolation PreInfinityExtrapolation = ERichCurveExtrapolation::RCCE_None;

			/** The post-infinity extrapolation type */
			ERichCurveExtrapolation PostInfinityExtrapolation = ERichCurveExtrapolation::RCCE_None;

			/** If true the pre-infinity extrapolation is non-linear (cycling, cycling with offset or oscillating) */
			bool bComplexPreInfinityExtrapolation = false;

			/** If true the post-infinity extrapolation is non-linear (cycling, cycling with offset or oscillating) */
			bool bComplexPostInfinityExtrapolation = false;

			/** The start of the curve (the time of the first key) in seconds */
			double CurveStartSeconds = 0.0;

			/** The end of the curve (the time of the last key) in seconds */
			double CurveEndSeconds = 0.0;

			/** The duration of the curve, in seconds */
			double CurveDurationSeconds = 0.0;

			/** The lower visible bound of the curve, in seconds */
			double VisibleCurveStartSeconds = 0.0;

			/** The upper visible bound of the curve, in seconds */
			double VisibleCurveEndSeconds = 0.0;

			/** Duration of the pre-infinity extrapolation (the time before the first key) */
			double PreInfinityDuration = 0.0;

			/** Duration of the post-infinity extrapolation (the time after the last key) */
			double PostInfinityDuration = 0.0;

			/** True if pre-infinity is visible */
			bool bPreInfinityVisible = false;

			/** True if post-infinity is visible */
			bool bPostInfinityVisible = false;

			/** True if pre-infinity oscillates */
			bool bPreInfinityOscillates = false;

			/** True if post-infinity oscillates */
			bool bPostInfinityOscillates = false;

			/**
			 * Returns the interpolation range to draw the curve as well as pre- and post-infinity extrapolation.
			 * Mind a larger part of the curve might be visible in pre- or post-infinity extrapolation, while only a small part of the curve is visilbe.
			 */
			void GetInterpolationRangeSeconds(double& OutInterpolationStartSeconds, double& OutInterpolationEndSeconds)
			{
				// Translate pre- and post-infinity ranges to curve range
				const double TranslatedPreInfinityStartSeconds = [this]()
					{
						if (bPreInfinityVisible && bComplexPreInfinityExtrapolation)
						{
							return bPreInfinityOscillates ?
								CurveStartSeconds :
								FMath::Max(CurveStartSeconds, CurveEndSeconds - PreInfinityDuration);
						}
						else
						{
							return VisibleCurveStartSeconds;
						}
					}();

				const double TranslatedPreInfinityEndSeconds = [this]()
					{
						if (bPreInfinityVisible && bComplexPreInfinityExtrapolation)
						{
							return bPreInfinityOscillates ?
								FMath::Min(CurveEndSeconds, CurveStartSeconds + PreInfinityDuration) :
								CurveEndSeconds;
						}
						else
						{
							return VisibleCurveEndSeconds;
						}
					}();


				const double TranslatedPostInfinityStartSeconds = [this]()
					{
						if (bPostInfinityVisible && bComplexPostInfinityExtrapolation)
						{
							return bPostInfinityOscillates ?
								FMath::Max(CurveStartSeconds, CurveEndSeconds - PostInfinityDuration) :
								CurveStartSeconds;
						}
						else
						{
							return VisibleCurveStartSeconds;
						}
					}();


				const double TranslatedPostInfinityEndSeconds = [this]()
					{
						if (bPostInfinityVisible && bComplexPostInfinityExtrapolation)
						{
							return bPostInfinityOscillates ?
								CurveEndSeconds :
								FMath::Min(CurveEndSeconds, CurveStartSeconds + PostInfinityDuration);
						}
						else
						{
							return VisibleCurveEndSeconds;
						}
					}();

				OutInterpolationStartSeconds = FMath::Min3(VisibleCurveStartSeconds, TranslatedPreInfinityStartSeconds, TranslatedPostInfinityStartSeconds);
				OutInterpolationEndSeconds = FMath::Max3(VisibleCurveEndSeconds, TranslatedPreInfinityEndSeconds, TranslatedPostInfinityEndSeconds);
			}

			/** Draws pre-infinity extrapolation as a straight line */
			void DrawPreInfinityExtrapolationLine(const TArray<TTuple<double, double>>& InCurveInterpolatingPoints, TArray<TTuple<double, double>>& OutInterpolatingPoints)
			{
				const FVector2D FirstKey(InCurveInterpolatingPoints[0].Get<0>(), InCurveInterpolatingPoints[0].Get<1>());

				if (PreInfinityExtrapolation == ERichCurveExtrapolation::RCCE_None ||
					PreInfinityExtrapolation == ERichCurveExtrapolation::RCCE_Constant)
				{
					OutInterpolatingPoints.Emplace(StartTimeSeconds, FirstKey.Y);
				}
				else if (PreInfinityExtrapolation == ERichCurveExtrapolation::RCCE_Linear)
				{
					const FFrameNumber StartFrame = (StartTimeSeconds * TickResolution).FloorToFrame();

					CurveValueType EvaluatedValue;
					if (Channel.Evaluate(StartFrame, EvaluatedValue))
					{
						OutInterpolatingPoints.Emplace(StartTimeSeconds, EvaluatedValue);
					}
					else
					{
						OutInterpolatingPoints.Emplace(StartTimeSeconds, FirstKey.Y);
					}
				}
				else
				{
					ensureMsgf(0, TEXT("Unhandled enum value"));
				}
			}

			/** Draws complex pre-infinity extrapolation */
			void DrawPreInfinityExtrapolationComplex(const TArray<TTuple<double, double>>& InCurveInterpolatingPoints, TArray<TTuple<double, double>>& OutInterpolatingPoints)
			{
				const FVector2D FirstKey(InCurveInterpolatingPoints[0].Get<0>(), InCurveInterpolatingPoints[0].Get<1>());
				const FVector2D LastKey(InCurveInterpolatingPoints.Last().Get<0>(), InCurveInterpolatingPoints.Last().Get<1>());

				const double InfinityOffset = FirstKey.X - StartTimeSeconds;

				if (FMath::IsNearlyZero(CurveDurationSeconds))
				{
					// Draw a single line in the odd case where there are many keys with a nearly zero duration.
					OutInterpolatingPoints.Emplace(
						StartTimeSeconds,
						FirstKey.Y);

					OutInterpolatingPoints.Emplace(
						FirstKey.X,
						FirstKey.Y);
				}
				else
				{
					// Cycle or oscillate
					const double StartTime = FirstKey.X;
					const double ValueOffset = PreInfinityExtrapolation == ERichCurveExtrapolation::RCCE_CycleWithOffset ?
						LastKey.Y - FirstKey.Y :
						0.0;

					const int32 NumIterations = static_cast<int32>(InfinityOffset / CurveDurationSeconds) + 1;
					for (int32 Iteration = NumIterations; Iteration > 0; Iteration--)
					{
						const bool bReverse = PreInfinityExtrapolation == ERichCurveExtrapolation::RCCE_Oscillate && Iteration % 2 != 0;

						if (bReverse)
						{
							for (int32 PointIndex = InCurveInterpolatingPoints.Num() - 1; PointIndex >= 0; PointIndex--)
							{
								// Mirror around start time
								const double Time = 2 * StartTime + CurveDurationSeconds - InCurveInterpolatingPoints[PointIndex].Get<0>() - CurveDurationSeconds * Iteration;
								const double Value = InCurveInterpolatingPoints[PointIndex].Get<1>() - ValueOffset * Iteration;

								OutInterpolatingPoints.Emplace(Time, Value);
							}
						}
						else
						{
							for (const TTuple<double, double>& Point : InCurveInterpolatingPoints)
							{
								const double Time = Point.Get<0>() - CurveDurationSeconds * Iteration;
								const double Value = Point.Get<1>() - ValueOffset * Iteration;

								OutInterpolatingPoints.Emplace(Time, Value);
							}
						}
					}
				}
			}

			/** Draws post-infinity extrapolation as a straight line */
			void DrawPostInfinityExtrapolationLine(const TArray<TTuple<double, double>>& InCurveInterpolatingPoints, TArray<TTuple<double, double>>& OutInterpolatingPoints)
			{
				const FVector2D LastKey(InCurveInterpolatingPoints.Last().Get<0>(), InCurveInterpolatingPoints.Last().Get<1>());

				if (PostInfinityExtrapolation == ERichCurveExtrapolation::RCCE_None ||
					PostInfinityExtrapolation == ERichCurveExtrapolation::RCCE_Constant)
				{
					OutInterpolatingPoints.Emplace(EndTimeSeconds, LastKey.Y);
				}
				else if (PostInfinityExtrapolation == ERichCurveExtrapolation::RCCE_Linear)
				{
					const FFrameNumber EndFrame = (EndTimeSeconds * TickResolution).CeilToFrame();

					CurveValueType EvaluatedValue;
					if (Channel.Evaluate(EndFrame, EvaluatedValue))
					{
						OutInterpolatingPoints.Emplace(EndTimeSeconds, EvaluatedValue);
					}
					else
					{
						OutInterpolatingPoints.Emplace(EndTimeSeconds, LastKey.Y);
					}
				}
				else
				{
					ensureMsgf(0, TEXT("Unhandled enum value"));
				}
			}

			/** Draws complex post-infinity extrapolation */
			void DrawPostInfinityExtrapolationComplex(const TArray<TTuple<double, double>>& InCurveInterpolatingPoints, TArray<TTuple<double, double>>& OutInterpolatingPoints)
			{
				const FVector2D FirstKey(InCurveInterpolatingPoints[0].Get<0>(), InCurveInterpolatingPoints[0].Get<1>());
				const FVector2D LastKey(InCurveInterpolatingPoints.Last().Get<0>(), InCurveInterpolatingPoints.Last().Get<1>());

				const double InfinityOffset = EndTimeSeconds - LastKey.X;

				if (FMath::IsNearlyZero(CurveDurationSeconds))
				{
					// Draw a single line in the odd case where there are many keys with a nearly zero duration.
					OutInterpolatingPoints.Emplace(
						LastKey.X,
						LastKey.Y);

					OutInterpolatingPoints.Emplace(
						EndTimeSeconds,
						LastKey.Y);
				}
				else
				{
					// Cycle or oscillate
					const double StartTime = FirstKey.X;
					const double ValueOffset = PostInfinityExtrapolation == ERichCurveExtrapolation::RCCE_CycleWithOffset ?
						LastKey.Y - FirstKey.Y :
						0.0;

					const int32 NumIterations = static_cast<int32>(InfinityOffset / CurveDurationSeconds) + 1;
					for (int32 Iteration = 1; Iteration <= NumIterations; Iteration++)
					{
						const bool bReverse = PostInfinityExtrapolation == ERichCurveExtrapolation::RCCE_Oscillate && Iteration % 2 != 0;

						if (bReverse)
						{
							for (int32 PointIndex = InCurveInterpolatingPoints.Num() - 1; PointIndex >= 0; PointIndex--)
							{
								// Mirror around start time
								const double Time = 2 * StartTime + CurveDurationSeconds - InCurveInterpolatingPoints[PointIndex].Get<0>() + CurveDurationSeconds * Iteration;
								const double Value = InCurveInterpolatingPoints[PointIndex].Get<1>() - ValueOffset * Iteration;

								OutInterpolatingPoints.Emplace(Time, Value);
							}
						}
						else
						{
							for (const TTuple<double, double>& Point : InCurveInterpolatingPoints)
							{
								const double Time = Point.Get<0>() + CurveDurationSeconds * Iteration;
								const double Value = Point.Get<1>() + ValueOffset * Iteration;

								OutInterpolatingPoints.Emplace(Time, Value);
							}
						}
					}
				}
			}
		};
	}
}
