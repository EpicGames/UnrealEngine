// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneCurveChannelImpl.h"
#include "Channels/MovieSceneInterpolation.h"
#include "Channels/MovieScenePiecewiseCurveUtils.inl"
#include "MovieSceneFrameMigration.h"
#include "MovieSceneFwd.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/SequencerObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneFloatChannel)

static_assert(
		sizeof(FMovieSceneFloatValue) == 28,
		"The size of the float channel value has changed. You need to update the padding byte at the end of the structure. "
		"You also need to update the layout in FMovieSceneDoubleValue so that they match!");


namespace UE::MovieScene
{
	void OnRemapChannelKeyTime(const FMovieSceneChannel* Channel, const IRetimingInterface& Retimer, FFrameNumber PreviousTime, FFrameNumber CurrentTime, FMovieSceneFloatValue& InOutValue)
	{
		if (InOutValue.InterpMode == ERichCurveInterpMode::RCIM_Cubic)
		{
			// This is a bit of a hack, but we scale tangents if the remapper has stretched the time around the key that was remapped
			//    We figure out this stretch factor by retiming a time slightly ahead (1/4 of a second) of the key, and seeing how it differs from the new key time
			FFrameTime Diff = 0.25 * static_cast<const FMovieSceneFloatChannel*>(Channel)->GetTickResolution();

			const double StretchFactor = (Retimer.RemapTime(FFrameTime(PreviousTime + Diff)) - CurrentTime).AsDecimal() / Diff.AsDecimal();
			if (!FMath::IsNearlyEqual(StretchFactor, 1.0) && !FMath::IsNearlyEqual(StretchFactor, 0.0))
			{
				if (InOutValue.Tangent.ArriveTangent != 0.0)
				{
					InOutValue.Tangent.ArriveTangent = static_cast<float>(InOutValue.Tangent.ArriveTangent / StretchFactor);
				}
				else
				{
					InOutValue.Tangent.ArriveTangentWeight = static_cast<float>(InOutValue.Tangent.ArriveTangentWeight * StretchFactor);
				}

				if (InOutValue.Tangent.LeaveTangent != 0.0)
				{
					InOutValue.Tangent.LeaveTangent  = static_cast<float>(InOutValue.Tangent.LeaveTangent  / StretchFactor);
				}
				else
				{
					InOutValue.Tangent.LeaveTangentWeight = static_cast<float>(InOutValue.Tangent.LeaveTangentWeight * StretchFactor);
				}
			}
		}
	}
} // namespace UE::MovieScene

namespace UE::MovieScene::Interpolation
{
	struct FFloatChannelPiecewiseData
	{
		const FMovieSceneFloatChannel* Channel;

		bool HasDefaultValue() const
		{
			return Channel->GetDefault().IsSet();
		}
		double GetDefaultValue() const
		{
			return Channel->GetDefault().Get(0.0);
		}
		double PreExtrapolate(const FFrameTime& InTime) const
		{
			float Result = 0.0;
			TMovieSceneCurveChannelImpl<FMovieSceneFloatChannel>::Evaluate(Channel, InTime, Result);
			return Result;
		}
		double PostExtrapolate(const FFrameTime& InTime) const
		{
			float Result = 0.0;
			TMovieSceneCurveChannelImpl<FMovieSceneFloatChannel>::Evaluate(Channel, InTime, Result);
			return Result;
		}
		int32 NumPieces() const
		{
			return FMath::Max(0, Channel->GetData().GetValues().Num() - 1);
		}
		int32 GetIndexOfPieceByTime(const FFrameTime& Time) const
		{
			TArrayView<const FFrameNumber> Times = Channel->GetData().GetTimes();
			return FMath::Max(Algo::UpperBound(Times, Time)-1, 0);
		}
		Interpolation::FCachedInterpolation GetPieceByIndex(int32 Index) const
		{
			return TMovieSceneCurveChannelImpl<FMovieSceneFloatChannel>::GetInterpolationForKey(Channel, Index);
		}
		Interpolation::FCachedInterpolation GetPieceByTime(const FFrameTime& Time) const
		{
			return TMovieSceneCurveChannelImpl<FMovieSceneFloatChannel>::GetInterpolationForTime(Channel, Time);
		}
		FFrameNumber GetFiniteStart() const
		{
			return Channel->GetData().GetTimes()[0];
		}
		FFrameNumber GetFiniteEnd() const
		{
			return Channel->GetData().GetTimes().Last();
		}
		ERichCurveExtrapolation GetPreExtrapolation() const
		{
			return Channel->PreInfinityExtrap;
		}
		ERichCurveExtrapolation GetPostExtrapolation() const
		{
			return Channel->PostInfinityExtrap;
		}
		double GetStartingValue() const
		{
			return Channel->GetData().GetValues()[0].Value;
		}
		double GetEndingValue() const
		{
			return Channel->GetData().GetValues().Last().Value;
		}
	};

} // namespace UE::MovieScene::Interpolation

bool FMovieSceneFloatValue::Serialize(FArchive& Ar)
{
	return TMovieSceneCurveChannelImpl<FMovieSceneFloatChannel>::SerializeChannelValue(*this, Ar);
}

bool FMovieSceneFloatValue::operator==(const FMovieSceneFloatValue& FloatValue) const
{
	return (Value == FloatValue.Value) && (InterpMode == FloatValue.InterpMode) && (TangentMode == FloatValue.TangentMode) && (Tangent == FloatValue.Tangent);
}

bool FMovieSceneFloatValue::operator!=(const FMovieSceneFloatValue& Other) const
{
	return !(*this == Other);
}

FMovieSceneFloatChannel::FMovieSceneFloatChannel()
	: PreInfinityExtrap(RCCE_Constant)
	, PostInfinityExtrap(RCCE_Constant)
	, DefaultValue(0.f)
	, bHasDefaultValue(false)
#if WITH_EDITORONLY_DATA
	, bShowCurve(false)
#endif
{
}

FMovieSceneFloatChannel::~FMovieSceneFloatChannel() = default;

UE::MovieScene::Interpolation::FInterpolationExtents FMovieSceneFloatChannel::ComputeExtents(FFrameTime StartTime, FFrameTime EndTime) const
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Interpolation;

	return ComputePiecewiseExtents(FFloatChannelPiecewiseData{ this }, StartTime, EndTime);
}

UE::MovieScene::FPiecewiseCurve FMovieSceneFloatChannel::AsPiecewiseCurve(bool bWithPreAndPostInfinityExtrap) const
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Interpolation;

	FPiecewiseCurve Curve;

	if (Times.IsEmpty())
	{
		if (bHasDefaultValue)
		{
			Curve.Values.Add(FCachedInterpolation(FCachedInterpolationRange::Infinite(), FConstantValue(0, DefaultValue)));
		}
		return Curve;
	}

	if (bWithPreAndPostInfinityExtrap && PreInfinityExtrap != RCCE_None)
	{
		Interpolation::FCachedInterpolation PreExtrap;
		if (FMovieSceneFloatChannelImpl::CacheExtrapolation(this, Times[0] - 1, PreExtrap))
		{
			Curve.Values.Emplace(PreExtrap);
		}
		else
		{
			ensureMsgf(false, TEXT("Unrepresentable extrapolation mode encountered for piecewise curve"));
		}
	}

	Curve.Values.Reserve(Times.Num());
	for (int32 Index = 0; Index < Times.Num() - 1; ++Index)
	{
		using namespace UE::MovieScene::Interpolation;

		// Add a constant interp if this is the index of the last key, or if the next key sits on the same frame
		const bool bNoRange =
			Index == Times.Num() - 1 ||
			Times[Index] == Times[Index + 1];

		if (bNoRange)
		{
			FCachedInterpolationRange Range = FCachedInterpolationRange::Only(Times[Index]);
			Curve.Values.Emplace(Range, FConstantValue(Range.Start, Values[Index].Value));
		}
		else
		{
			Interpolation::FCachedInterpolation Interp = FMovieSceneFloatChannelImpl::GetInterpolationForKey(this, Index);
			if (ensure(Interp.IsValid()))
			{
				Curve.Values.Add(MoveTemp(Interp));
			}
		}
	}
	
	if (bWithPreAndPostInfinityExtrap && PostInfinityExtrap != RCCE_None)
	{
		Interpolation::FCachedInterpolation PostExtrap;
		if (FMovieSceneFloatChannelImpl::CacheExtrapolation(this, Times.Last() + 1, PostExtrap))
		{
			Curve.Values.Emplace(PostExtrap);
		}
		else
		{
			ensureMsgf(false, TEXT("Unrepresentable extrapolation mode encountered for piecewise curve"));
		}
	}

	return Curve;
}

int32 FMovieSceneFloatChannel::AddConstantKey(FFrameNumber InTime, float InValue)
{
	return FMovieSceneFloatChannelImpl::AddConstantKey(this, InTime, InValue);
}

int32 FMovieSceneFloatChannel::AddLinearKey(FFrameNumber InTime, float InValue)
{
	return FMovieSceneFloatChannelImpl::AddLinearKey(this, InTime, InValue);
}

int32 FMovieSceneFloatChannel::AddCubicKey(FFrameNumber InTime, float InValue, ERichCurveTangentMode TangentMode, const FMovieSceneTangentData& Tangent)
{
	return FMovieSceneFloatChannelImpl::AddCubicKey(this, InTime, InValue, TangentMode, Tangent);
}

bool FMovieSceneFloatChannel::Evaluate(FFrameTime InTime,  float& OutValue) const
{
	return FMovieSceneFloatChannelImpl::Evaluate(this, InTime, OutValue);
}

UE::MovieScene::Interpolation::FCachedInterpolation FMovieSceneFloatChannel::GetInterpolationForTime(FFrameTime InTime) const
{
	return FMovieSceneFloatChannelImpl::GetInterpolationForTime(this, InTime);
}

void FMovieSceneFloatChannel::Set(TArray<FFrameNumber> InTimes, TArray<FMovieSceneFloatValue> InValues)
{
	FMovieSceneFloatChannelImpl::Set(this, InTimes, InValues);
}

void FMovieSceneFloatChannel::SetKeysOnly(TArrayView<FFrameNumber> InTimes, TArrayView<FMovieSceneFloatValue> InValues)
{
	check(InTimes.Num() == InValues.Num());

	Times = MoveTemp(InTimes);
	Values = MoveTemp(InValues);

	KeyHandles.Reset();
}

void FMovieSceneFloatChannel::AutoSetTangents(float Tension)
{
	FMovieSceneFloatChannelImpl::AutoSetTangents(this, Tension);
}

void FMovieSceneFloatChannel::PopulateCurvePoints(double StartTimeSeconds, double EndTimeSeconds, double TimeThreshold, float ValueThreshold, FFrameRate InTickResolution, TArray<TTuple<double, double>>& InOutPoints) const
{
	FMovieSceneFloatChannelImpl::PopulateCurvePoints(this, StartTimeSeconds, EndTimeSeconds, TimeThreshold, ValueThreshold, InTickResolution, InOutPoints);
}

void FMovieSceneFloatChannel::GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles)
{
	GetData().GetKeys(WithinRange, OutKeyTimes, OutKeyHandles);
}

void FMovieSceneFloatChannel::GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes)
{
	GetData().GetKeyTimes(InHandles, OutKeyTimes);
}

void FMovieSceneFloatChannel::SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes)
{
	GetData().SetKeyTimes(InHandles, InKeyTimes);
	AutoSetTangents();
}

void FMovieSceneFloatChannel::DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles)
{
	GetData().DuplicateKeys(InHandles, OutNewHandles);
}

void FMovieSceneFloatChannel::DeleteKeys(TArrayView<const FKeyHandle> InHandles)
{
	GetData().DeleteKeys(InHandles);
	AutoSetTangents();
}

void FMovieSceneFloatChannel::DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore)
{
	FMovieSceneFloatChannelImpl::DeleteKeysFrom(this, InTime, bDeleteKeysBefore);
	AutoSetTangents();
}

void FMovieSceneFloatChannel::RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer)
{
	FMovieSceneFloatChannelImpl::RemapTimes(this, Retimer);
}

TRange<FFrameNumber> FMovieSceneFloatChannel::ComputeEffectiveRange() const
{
	return GetData().GetTotalRange();
}

int32 FMovieSceneFloatChannel::GetNumKeys() const
{
	return Times.Num();
}

void FMovieSceneFloatChannel::Reset()
{
	Times.Reset();
	Values.Reset();
	KeyHandles.Reset();
	bHasDefaultValue = false;
}

void FMovieSceneFloatChannel::PostEditChange()
{
	AutoSetTangents();
}

void FMovieSceneFloatChannel::Offset(FFrameNumber DeltaPosition)
{
	GetData().Offset(DeltaPosition);
	AutoSetTangents();
}

FKeyHandle FMovieSceneFloatChannel::GetHandle(int32 Index)
{
	return GetData().GetHandle(Index);
}

int32 FMovieSceneFloatChannel::GetIndex(FKeyHandle Handle)
{
	return GetData().GetIndex(Handle);
}

void FMovieSceneFloatChannel::Optimize(const FKeyDataOptimizationParams& Params)
{
	FMovieSceneFloatChannelImpl::Optimize(this, Params);
}

void FMovieSceneFloatChannel::ClearDefault()
{
	bHasDefaultValue = false;
}

EMovieSceneKeyInterpolation GetInterpolationMode(FMovieSceneFloatChannel* InChannel, const FFrameNumber& InTime, EMovieSceneKeyInterpolation DefaultInterpolationMode)
{
	return TMovieSceneCurveChannelImpl<FMovieSceneFloatChannel>::GetInterpolationMode(InChannel, InTime, DefaultInterpolationMode);
}

FKeyHandle AddKeyToChannel(FMovieSceneFloatChannel* Channel, FFrameNumber InFrameNumber, float InValue, EMovieSceneKeyInterpolation Interpolation)
{
	return TMovieSceneCurveChannelImpl<FMovieSceneFloatChannel>::AddKeyToChannel(Channel, InFrameNumber, InValue, Interpolation);
}

void Dilate(FMovieSceneFloatChannel* InChannel, FFrameNumber Origin, float DilationFactor)
{
	return TMovieSceneCurveChannelImpl<FMovieSceneFloatChannel>::Dilate(InChannel, Origin, DilationFactor);
}

bool ValueExistsAtTime(const FMovieSceneFloatChannel* InChannel, FFrameNumber InFrameNumber, float InValue)
{
	return TMovieSceneCurveChannelImpl<FMovieSceneFloatChannel>::ValueExistsAtTime(InChannel, InFrameNumber, InValue);
}

bool ValueExistsAtTime(const FMovieSceneFloatChannel* InChannel, FFrameNumber InFrameNumber, const FMovieSceneFloatValue& InValue)
{
	return TMovieSceneCurveChannelImpl<FMovieSceneFloatChannel>::ValueExistsAtTime(InChannel, InFrameNumber, InValue);
}

void AssignValue(FMovieSceneFloatChannel* InChannel, FKeyHandle InKeyHandle, float InValue)
{
	return TMovieSceneCurveChannelImpl<FMovieSceneFloatChannel>::AssignValue(InChannel, InKeyHandle, InValue);
}

void InvertValue(float& InOutValue)
{
	InOutValue = -InOutValue;
}

void ReciprocalValue(float& InOutValue)
{
	if (InOutValue != 0.f)
	{
		InOutValue = 1.0f / InOutValue;
	}
}

void TransformValue(float& InOutValue, const FMovieSceneChannelTraitsTransform<float>& Transform)
{
	if (Transform.Scale != 0.f)
	{
		InOutValue = (InOutValue - Transform.Pivot) * Transform.Scale + Transform.Pivot;
	}
	InOutValue += Transform.Offset;
}

void FMovieSceneFloatChannel::AddKeys(const TArray<FFrameNumber>& InTimes, const TArray<FMovieSceneFloatValue>& InValues)
{
	check(InTimes.Num() == InValues.Num());
	int32 Index = Times.Num();
	Times.Append(InTimes);
	Values.Append(InValues);
	for (; Index < Times.Num(); ++Index)
	{
		KeyHandles.AllocateHandle(Index);
	}
	AutoSetTangents();
}

void FMovieSceneFloatChannel::UpdateOrAddKeys(const TArrayView<const FFrameNumber> InTimes, const TArrayView<FMovieSceneFloatValue> InValues)
{
	GetData().UpdateOrAddKeys(InTimes, InValues);

	AutoSetTangents();
}

#if WITH_EDITORONLY_DATA

bool FMovieSceneFloatChannel::GetShowCurve() const
{
	return bShowCurve;
}

void FMovieSceneFloatChannel::SetShowCurve(bool bInShowCurve)
{
	bShowCurve = bInShowCurve;
}

#endif

bool FMovieSceneFloatChannel::Serialize(FArchive& Ar)
{
	return FMovieSceneFloatChannelImpl::Serialize(this, Ar);
}

#if WITH_EDITORONLY_DATA
void FMovieSceneFloatChannel::PostSerialize(const FArchive& Ar)
{
	if (Ar.CustomVer(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::ModifyLinearKeysForOldInterp)
	{
		bool bNeedAutoSetAlso = false;
		//we need to possibly modify cuvic tangents if we get a set of linear..cubic tangents so it works like it used to
		if (Values.Num() >= 2)
		{
			for (int32 Index = 1; Index < Values.Num(); ++Index)
			{
				FMovieSceneFloatValue  PrevKey = Values[Index - 1];
				FMovieSceneFloatValue& ThisKey = Values[Index];

				if (ThisKey.InterpMode == RCIM_Cubic && PrevKey.InterpMode == RCIM_Linear)
				{
					ThisKey.Tangent.TangentWeightMode = RCTWM_WeightedNone;
					ThisKey.TangentMode = RCTM_Break;
					//leave next tangent will be set up if auto or user, just need to modify prev.
					const float PrevTimeDiff = FMath::Max<double>(KINDA_SMALL_NUMBER, Times[Index].Value - Times[Index - 1].Value);
					float NewTangent = (ThisKey.Value - PrevKey.Value) / PrevTimeDiff;
					ThisKey.Tangent.ArriveTangent = NewTangent;
					bNeedAutoSetAlso = true;
				}
			}
		}
		if (bNeedAutoSetAlso)
		{
			AutoSetTangents();
		}
	}
}
#endif

bool FMovieSceneFloatChannel::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	return FMovieSceneFloatChannelImpl::SerializeFromRichCurve(this, Tag, Slot);
}


