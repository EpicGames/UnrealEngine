// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/MovieSceneIntegerChannel.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Curves/IntegralCurve.h"
#include "Misc/FrameRate.h"
#include "MovieSceneFwd.h"
#include "MovieSceneFrameMigration.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneIntegerChannel)


bool FMovieSceneIntegerChannel::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	static const FName IntegralCurveName("IntegralCurve");
	if (Tag.GetType().IsStruct(IntegralCurveName))
	{
		FIntegralCurve IntegralCurve;
		FIntegralCurve::StaticStruct()->SerializeItem(Slot, &IntegralCurve, nullptr);

		if (IntegralCurve.GetDefaultValue() != MAX_int32)
		{
			bHasDefaultValue = true;
			DefaultValue = IntegralCurve.GetDefaultValue();
		}

		Times.Reserve(IntegralCurve.GetNumKeys());
		Values.Reserve(IntegralCurve.GetNumKeys());

		FFrameRate LegacyFrameRate = GetLegacyConversionFrameRate();

		int32 Index = 0;
		for (auto It = IntegralCurve.GetKeyIterator(); It; ++It)
		{
			FFrameNumber KeyTime = UpgradeLegacyMovieSceneTime(nullptr, LegacyFrameRate, It->Time);

			int32 Val = It->Value;
			ConvertInsertAndSort<int32>(Index++, KeyTime, Val, Times, Values);
		}
		return true;
	}

	return false;
}

bool FMovieSceneIntegerChannel::Evaluate(FFrameTime InTime, int64& OutValue) const
{
	double InterpValue = 0.;
	bool bReturnValue = EvaluateInterp(InTime, InterpValue);
	OutValue = FMath::FloorToInt(InterpValue);
	return bReturnValue;
}

bool FMovieSceneIntegerChannel::EvaluateInterp(FFrameTime InTime, double& OutValue) const
{
	if (Times.Num())
	{
		const FFrameNumber MinFrame = Times[0];
		const FFrameNumber MaxFrame = Times.Last();
		//we do None,Constant, and Linear first ,there is no cycling and so we just exit
		if (InTime < FFrameTime(MinFrame))
		{
			if (PreInfinityExtrap == RCCE_None)
			{
				return false;
			}

			if (PreInfinityExtrap == RCCE_Constant || PreInfinityExtrap == RCCE_Linear)
			{
				OutValue = Values[0];
				return true;
			}
		}
		else if (InTime > FFrameTime(MaxFrame))
		{
			if (PostInfinityExtrap == RCCE_None)
			{
				return false;
			}

			if (PostInfinityExtrap == RCCE_Constant || PostInfinityExtrap == RCCE_Linear)
			{
				OutValue = Values.Last();
				return true;
			}
		}

		// Compute the cycled time based on extrapolation
		UE::MovieScene::FCycleParams Params = UE::MovieScene::CycleTime(MinFrame, MaxFrame, InTime);

		// Deal with offset cycles and oscillation
		// Move int over to double then we will convert back to int if doing an offset
		const double FirstValue = double(Values[0]);
		const double LastValue = double(Values.Last());
		if (InTime < FFrameTime(MinFrame))
		{
			switch (PreInfinityExtrap)
			{
			case RCCE_CycleWithOffset: Params.ComputePreValueOffset(FirstValue, LastValue); break;
			case RCCE_Oscillate:       Params.Oscillate(MinFrame.Value, MaxFrame.Value);                       break;
			}
		}
		else if (InTime > FFrameTime(MaxFrame))
		{
			switch (PostInfinityExtrap)
			{
			case RCCE_CycleWithOffset: Params.ComputePostValueOffset(FirstValue, LastValue); break;
			case RCCE_Oscillate:       Params.Oscillate(MinFrame.Value, MaxFrame.Value);    break;
			}
		}

		const int32 Index = FMath::Max(0, Algo::UpperBound(Times, Params.Time) - 1);

		if (bInterpolateLinearKeys && Index < Times.Num() - 1)
		{
			const double LowerValue = Values[Index];
			const double UpperValue = Values[Index+1];
			const double LowerTime = FFrameTime(Times[Index]).AsDecimal();
			const double UpperTime = FFrameTime(Times[Index+1]).AsDecimal();

			const double Interp = (Params.Time.AsDecimal() - LowerTime) / (UpperTime - LowerTime);

			const double Value = FMath::Lerp(LowerValue, UpperValue, Interp);
			OutValue = Value + Params.ValueOffset + 0.5;
		}
		else
		{
			OutValue = Values[Index] + (int32)(Params.ValueOffset + 0.5);
		}
		return true;
	}
	else if (bHasDefaultValue)
	{
		OutValue = DefaultValue;
		return true;
	}

	return false;
}

void FMovieSceneIntegerChannel::GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles)
{
	GetData().GetKeys(WithinRange, OutKeyTimes, OutKeyHandles);
}

void FMovieSceneIntegerChannel::GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes)
{
	GetData().GetKeyTimes(InHandles, OutKeyTimes);
}

void FMovieSceneIntegerChannel::SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes)
{
	GetData().SetKeyTimes(InHandles, InKeyTimes);
}

void FMovieSceneIntegerChannel::DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles)
{
	GetData().DuplicateKeys(InHandles, OutNewHandles);
}

void FMovieSceneIntegerChannel::DeleteKeys(TArrayView<const FKeyHandle> InHandles)
{
	GetData().DeleteKeys(InHandles);
}

void FMovieSceneIntegerChannel::DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore)
{
	// Insert a key at the current time to maintain evaluation
	if (GetData().GetTimes().Num() > 0)
	{
		int32 Value = 0;
		if (Evaluate(InTime, Value))
		{
			GetData().UpdateOrAddKey(InTime, Value);
		}
	}

	GetData().DeleteKeysFrom(InTime, bDeleteKeysBefore);
}

void FMovieSceneIntegerChannel::RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer)
{
	GetData().RemapTimes(Retimer);
}

TRange<FFrameNumber> FMovieSceneIntegerChannel::ComputeEffectiveRange() const
{
	return GetData().GetTotalRange();
}

int32 FMovieSceneIntegerChannel::GetNumKeys() const
{
	return Times.Num();
}

void FMovieSceneIntegerChannel::Reset()
{
	Times.Reset();
	Values.Reset();
	KeyHandles.Reset();
	bHasDefaultValue = false;
}

void FMovieSceneIntegerChannel::Optimize(const FKeyDataOptimizationParams& InParameters)
{
	UE::MovieScene::Optimize(this, InParameters);
}

void FMovieSceneIntegerChannel::Offset(FFrameNumber DeltaPosition)
{
	GetData().Offset(DeltaPosition);
}

FKeyHandle FMovieSceneIntegerChannel::GetHandle(int32 Index)
{
	return GetData().GetHandle(Index);
}

int32 FMovieSceneIntegerChannel::GetIndex(FKeyHandle Handle)
{
	return GetData().GetIndex(Handle);
}

void FMovieSceneIntegerChannel::ClearDefault()
{
	bHasDefaultValue = false;
}
