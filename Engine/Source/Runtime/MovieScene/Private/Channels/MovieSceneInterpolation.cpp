// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/MovieSceneInterpolation.h"
#include "Curves/CurveEvaluation.h"
#include "Algo/Partition.h"
#include "HAL/IConsoleManager.h"


namespace UE::MovieScene::Interpolation
{

static bool GAvoidOverlappingWeightedTangents = true;
static FAutoConsoleVariableRef CAvoidOverlappingWeightedTangents(
	TEXT("ControlRig.AvoidOverlappingWeightedTangents"),
	GAvoidOverlappingWeightedTangents,
	TEXT("(Default: true) If true weighted tangents won't overlap in time, legacy behavior was to have them overlap which can cause discontinuities"),
	ECVF_Default
);

bool FInterpolationExtents::IsValid() const
{
	return MinValue != std::numeric_limits<double>::max() && MaxValue != std::numeric_limits<double>::lowest();
}

void FInterpolationExtents::AddPoint(double Value, FFrameTime Time)
{
	if (Value < MinValue)
	{
		MinValue = Value;
		MinValueTime = Time;
	}
	if (Value > MaxValue)
	{
		MaxValue = Value;
		MaxValueTime = Time;
	}
}

void FInterpolationExtents::Combine(const FInterpolationExtents& Other)
{
	if (Other.IsValid())
	{
		AddPoint(Other.MinValue, Other.MinValueTime);
		AddPoint(Other.MaxValue, Other.MaxValueTime);
	}
}

FCachedInterpolationRange FCachedInterpolationRange::Empty()
{
	return FCachedInterpolationRange{ 0, -1 };
}
FCachedInterpolationRange FCachedInterpolationRange::Finite(FFrameNumber InStart, FFrameNumber InEnd)
{
	return FCachedInterpolationRange{ InStart, InEnd };
}
FCachedInterpolationRange FCachedInterpolationRange::Infinite()
{
	return FCachedInterpolationRange{ TNumericLimits<FFrameNumber>::Lowest(), TNumericLimits<FFrameNumber>::Max() };
}
FCachedInterpolationRange FCachedInterpolationRange::Only(FFrameNumber InTime)
{
	const FFrameNumber EndTime = InTime < TNumericLimits<FFrameNumber>::Max()
		? InTime + 1
		: InTime;

	return FCachedInterpolationRange{ InTime, EndTime };
}
FCachedInterpolationRange FCachedInterpolationRange::From(FFrameNumber InStart)
{
	return FCachedInterpolationRange{ InStart, TNumericLimits<FFrameNumber>::Max() };
}
FCachedInterpolationRange FCachedInterpolationRange::Until(FFrameNumber InEnd) 
{
	return FCachedInterpolationRange{ TNumericLimits<FFrameNumber>::Lowest(), InEnd };
}
bool FCachedInterpolationRange::Contains(FFrameNumber FrameNumber) const
{
	return FrameNumber >= Start && (FrameNumber < End || End == TNumericLimits<FFrameNumber>::Max());
}

bool FCachedInterpolationRange::IsEmpty() const
{
	return Start >= End;
}

FCachedInterpolation::FCachedInterpolation()
	: Data(TInPlaceType<FInvalidValue>())
	, Range(FCachedInterpolationRange::Empty())
{
}

FCachedInterpolation::FCachedInterpolation(const FCachedInterpolationRange& InRange, const FConstantValue& Constant)
	: Data(TInPlaceType<FConstantValue>(), Constant)
	, Range(InRange)
{
}

FCachedInterpolation::FCachedInterpolation(const FCachedInterpolationRange& InRange, const FLinearInterpolation& Linear)
	: Data(TInPlaceType<FLinearInterpolation>(), Linear)
	, Range(InRange)
{
}

FCachedInterpolation::FCachedInterpolation(const FCachedInterpolationRange& InRange, const FQuadraticInterpolation& Quadratic)
	: Data(TInPlaceType<FQuadraticInterpolation>(), Quadratic)
	, Range(InRange)
{
}

FCachedInterpolation::FCachedInterpolation(const FCachedInterpolationRange& InRange, const FCubicInterpolation& Cubic)
	: Data(TInPlaceType<FCubicInterpolation>(), Cubic)
	, Range(InRange)
{
}

FCachedInterpolation::FCachedInterpolation(const FCachedInterpolationRange& InRange, const FQuarticInterpolation& Quartic)
	: Data(TInPlaceType<FQuarticInterpolation>(), Quartic)
	, Range(InRange)
{
}

FCachedInterpolation::FCachedInterpolation(const FCachedInterpolationRange& InRange, const FCubicBezierInterpolation& Cubic)
	: Data(TInPlaceType<FCubicBezierInterpolation>(), Cubic)
	, Range(InRange)
{
}

FCachedInterpolation::FCachedInterpolation(const FCachedInterpolationRange& InRange, const FWeightedCubicInterpolation& WeightedCubic)
	: Data(TInPlaceType<FWeightedCubicInterpolation>(), WeightedCubic)
	, Range(InRange)
{
}

bool FCachedInterpolation::IsValid() const
{
	return !Range.IsEmpty();
}

bool FCachedInterpolation::IsCacheValidForTime(FFrameNumber FrameNumber) const
{
	return Range.Contains(FrameNumber);
}

FCachedInterpolationRange FCachedInterpolation::GetRange() const
{
	return Range;
}

FInterpolationExtents FCachedInterpolation::ComputeExtents() const
{
	return ComputeExtents(Range.Start, Range.End);
}

FInterpolationExtents FCachedInterpolation::ComputeExtents(FFrameTime From, FFrameTime To) const
{
	FInterpolationExtents Extents;

	if (const FConstantValue* Constant = Data.TryGet<FConstantValue>())
	{
		Extents.AddPoint(Constant->Value, From);
		Extents.AddPoint(Constant->Value, To);
	}
	else if (const FLinearInterpolation* Linear = Data.TryGet<FLinearInterpolation>())
	{
		Extents.AddPoint(Linear->Evaluate(From), From);
		Extents.AddPoint(Linear->Evaluate(To), To);
	}
	else if (const FQuadraticInterpolation* Quadratic = Data.TryGet<FQuadraticInterpolation>())
	{
		Extents.AddPoint(Quadratic->Evaluate(From), From);
		Extents.AddPoint(Quadratic->Evaluate(To), To);

		FLinearInterpolation Derivative = Quadratic->Derivative();

		FFrameTime Solutions[1];
		const int32 NumSolutions = Derivative.SolveWithin(From, To, 0.0, Solutions);
		check(NumSolutions <= 1);
		if (NumSolutions > 0)
		{
			Extents.AddPoint(Quadratic->Evaluate(Solutions[0]), Solutions[0]);
		}
	}
	else if (const FCubicInterpolation* Cubic = Data.TryGet<FCubicInterpolation>())
	{
		Extents.AddPoint(Cubic->Evaluate(From), From);
		Extents.AddPoint(Cubic->Evaluate(To), To);

		FQuadraticInterpolation Derivative = Cubic->Derivative();

		FFrameTime Solutions[2];
		const int32 NumSolutions = Derivative.SolveWithin(From, To, 0.0, Solutions);
		check(NumSolutions <= 2);
		for (int32 Index = 0; Index < NumSolutions; ++Index)
		{
			Extents.AddPoint(Cubic->Evaluate(Solutions[Index]), Solutions[Index]);
		}
	}
	else if (const FQuarticInterpolation* Quartic = Data.TryGet<FQuarticInterpolation>())
	{
		Extents.AddPoint(Quartic->Evaluate(From), From);
		Extents.AddPoint(Quartic->Evaluate(To), To);

		FCubicInterpolation Derivative = Quartic->Derivative();

		FFrameTime Solutions[3];
		const int32 NumSolutions = Derivative.SolveWithin(From, To, 0.0, Solutions);
		check(NumSolutions <= 3);
		for (int32 Index = 0; Index < NumSolutions; ++Index)
		{
			Extents.AddPoint(Quartic->Evaluate(Solutions[Index]), Solutions[Index]);
		}
	}
	else if (const FCubicBezierInterpolation* CubicBezier = Data.TryGet<FCubicBezierInterpolation>())
	{
		const double Origin = CubicBezier->Origin.Value;
		const double DX = CubicBezier->DX;

		Extents.AddPoint(CubicBezier->Evaluate(From), From);
		Extents.AddPoint(CubicBezier->Evaluate(To), To);

		FQuadraticInterpolation Derivative = CubicBezier->Derivative();

		FFrameTime Solutions[2];
		const int32 NumSolutions = Derivative.SolveWithin(From, To, 0.0, Solutions);
		check(NumSolutions <= 2);
		for (int32 Index = 0; Index < NumSolutions; ++Index)
		{
			Extents.AddPoint(CubicBezier->Evaluate(Solutions[Index]), Solutions[Index]);
		}
	}
	else if (const FWeightedCubicInterpolation* WeightedCubic = Data.TryGet<FWeightedCubicInterpolation>())
	{
		Extents.AddPoint(WeightedCubic->Evaluate(From), From);
		Extents.AddPoint(WeightedCubic->Evaluate(To),   To);
	}

	ensureMsgf(
		Extents.MinValueTime >= From && Extents.MinValueTime <= To &&
		Extents.MaxValueTime >= From && Extents.MaxValueTime <= To,
		TEXT("ComputeExtents resulted in Min: %f and Max: %f, but expected to be between From: %f To: %f"),
		Extents.MinValueTime.AsDecimal(), Extents.MaxValueTime.AsDecimal(), From.AsDecimal(), To.AsDecimal());

	return Extents;
}

bool FCachedInterpolation::Evaluate(FFrameTime Time, double& OutResult) const
{
	if (const FConstantValue* Constant = Data.TryGet<FConstantValue>())
	{
		OutResult = Constant->Value;
		return true;
	}
	else if (const FLinearInterpolation* Linear = Data.TryGet<FLinearInterpolation>())
	{
		OutResult = Linear->Evaluate(Time);
		return true;
	}
	else if (const FQuadraticInterpolation* Quadratic = Data.TryGet<FQuadraticInterpolation>())
	{
		OutResult = Quadratic->Evaluate(Time);
		return true;
	}
	else if (const FCubicInterpolation* Cubic = Data.TryGet<FCubicInterpolation>())
	{
		OutResult = Cubic->Evaluate(Time);
		return true;
	}
	else if (const FQuarticInterpolation* Quartic = Data.TryGet<FQuarticInterpolation>())
	{
		OutResult = Quartic->Evaluate(Time);
		return true;
	}
	else if (const FCubicBezierInterpolation* CubicBezier = Data.TryGet<FCubicBezierInterpolation>())
	{
		OutResult = CubicBezier->Evaluate(Time);
		return true;
	}
	else if (const FWeightedCubicInterpolation* WeightedCubic = Data.TryGet<FWeightedCubicInterpolation>())
	{
		OutResult = WeightedCubic->Evaluate(Time);
		return true;
	}

	return false;
}

void FCachedInterpolation::Offset(double Amount)
{
	if (FConstantValue* Constant = Data.TryGet<FConstantValue>())
	{
		Constant->Value += Amount;
	}
	else if (FLinearInterpolation* Linear = Data.TryGet<FLinearInterpolation>())
	{
		Linear->Constant += Amount;
	}
	else if (FQuadraticInterpolation* Quadratic = Data.TryGet<FQuadraticInterpolation>())
	{
		Quadratic->Constant += Amount;
	}
	else if (FCubicInterpolation* Cubic = Data.TryGet<FCubicInterpolation>())
	{
		Cubic->Constant += Amount;
	}
	else if (FQuarticInterpolation* Quartic = Data.TryGet<FQuarticInterpolation>())
	{
		Quartic->Constant += Amount;
	}
	else if (FCubicBezierInterpolation* CubicBezier = Data.TryGet<FCubicBezierInterpolation>())
	{
		CubicBezier->P0 += Amount;
		CubicBezier->P1 += Amount;
		CubicBezier->P2 += Amount;
		CubicBezier->P3 += Amount;
	}
	else if (FWeightedCubicInterpolation* WeightedCubic = Data.TryGet<FWeightedCubicInterpolation>())
	{
		WeightedCubic->StartKeyValue += Amount;
		WeightedCubic->EndKeyValue   += Amount;
	}
}

TOptional<FCachedInterpolation> FCachedInterpolation::ComputeIntegral(double ConstantOffset) const
{
	if (const FConstantValue* Constant = Data.TryGet<FConstantValue>())
	{
		return FCachedInterpolation(Range, Constant->Integral(ConstantOffset));
	}
	else if (const FLinearInterpolation* Linear = Data.TryGet<FLinearInterpolation>())
	{
		return FCachedInterpolation(Range, Linear->Integral(ConstantOffset));
	}
	else if (const FQuadraticInterpolation* Quadratic = Data.TryGet<FQuadraticInterpolation>())
	{
		return FCachedInterpolation(Range, Quadratic->Integral(ConstantOffset));
	}
	else if (const FCubicInterpolation* Cubic = Data.TryGet<FCubicInterpolation>())
	{
		return FCachedInterpolation(Range, Cubic->Integral(ConstantOffset));
	}
	else if (const FQuarticInterpolation* Quartic = Data.TryGet<FQuarticInterpolation>())
	{
		checkf(false, TEXT("Unable to compute the integral of a quartic. Although the math is easy, quintic curves are practically impossible to solve."));
	}
	else if (const FCubicBezierInterpolation* CubicBezier = Data.TryGet<FCubicBezierInterpolation>())
	{
		return FCachedInterpolation(Range, CubicBezier->Integral(ConstantOffset));
	}
	else if (const FWeightedCubicInterpolation* WeightedCubic = Data.TryGet<FWeightedCubicInterpolation>())
	{
	}

	return TOptional<FCachedInterpolation>();
}

TOptional<FCachedInterpolation> FCachedInterpolation::ComputeDerivative() const
{
	if (const FConstantValue* Constant = Data.TryGet<FConstantValue>())
	{
		return FCachedInterpolation(Range, Constant->Derivative());
	}
	else if (const FLinearInterpolation* Linear = Data.TryGet<FLinearInterpolation>())
	{
		return FCachedInterpolation(Range, Linear->Derivative());
	}
	else if (const FQuadraticInterpolation* Quadratic = Data.TryGet<FQuadraticInterpolation>())
	{
		return FCachedInterpolation(Range, Quadratic->Derivative());
	}
	else if (const FCubicInterpolation* Cubic = Data.TryGet<FCubicInterpolation>())
	{
		return FCachedInterpolation(Range, Cubic->Derivative());
	}
	else if (const FQuarticInterpolation* Quartic = Data.TryGet<FQuarticInterpolation>())
	{
		return FCachedInterpolation(Range, Quartic->Derivative());
	}
	else if (const FCubicBezierInterpolation* CubicBezier = Data.TryGet<FCubicBezierInterpolation>())
	{
		return FCachedInterpolation(Range, CubicBezier->Derivative());
	}
	else if (const FWeightedCubicInterpolation* WeightedCubic = Data.TryGet<FWeightedCubicInterpolation>())
	{
	}

	return TOptional<FCachedInterpolation>();
}

int32 FCachedInterpolation::InverseEvaluate(double InValue, TInterpSolutions<FFrameTime, 4> OutResults) const
{
	int32 NumSolutions = 0;
	if (const FConstantValue* Constant = Data.TryGet<FConstantValue>())
	{
		if (InValue == Constant->Value)
		{
			if (Range.Start == TNumericLimits<FFrameNumber>::Lowest())
			{
				if (Range.End == TNumericLimits<FFrameNumber>::Lowest())
				{
					OutResults[0] = FFrameTime();
				}
				else
				{
					OutResults[0] = Range.End;
				}
			}
			else
			{
				OutResults[0] = Range.Start;
			}
			NumSolutions = 1;
		}

		check(NumSolutions <= 1);
	}
	else if (const FLinearInterpolation* Linear = Data.TryGet<FLinearInterpolation>())
	{
		NumSolutions = Linear->Solve(InValue, OutResults);
		check(NumSolutions <= 1);
	}
	else if (const FQuadraticInterpolation* Quadratic = Data.TryGet<FQuadraticInterpolation>())
	{
		NumSolutions = Quadratic->Solve(InValue, OutResults);
		check(NumSolutions <= 2);
	}
	else if (const FCubicInterpolation* Cubic = Data.TryGet<FCubicInterpolation>())
	{
		NumSolutions = Cubic->Solve(InValue, OutResults);
		check(NumSolutions <= 3);
	}
	else if (const FQuarticInterpolation* Quartic = Data.TryGet<FQuarticInterpolation>())
	{
		NumSolutions = Quartic->Solve(InValue, OutResults);
		check(NumSolutions <= 4);
	}
	else if (const FCubicBezierInterpolation* CubicBezier = Data.TryGet<FCubicBezierInterpolation>())
	{
		NumSolutions = CubicBezier->Solve(InValue, OutResults);
		check(NumSolutions <= 2);
	}
	else if (const FWeightedCubicInterpolation* WeightedCubic = Data.TryGet<FWeightedCubicInterpolation>())
	{
		NumSolutions = WeightedCubic->Solve(InValue, OutResults);
		check(NumSolutions <= 2);
	}

	// Only accept solutions within our acceptable range
	int32 WritePos = 0;
	for (int32 Index = 0; Index < NumSolutions; ++Index)
	{
		if (OutResults[Index] >= Range.Start && OutResults[Index] <= Range.End)
		{
			if (Index != WritePos)
			{
				OutResults[WritePos] = OutResults[Index];
			}
			++WritePos;
		}
	}

	return WritePos;
}

FConstantValue FConstantValue::Derivative() const
{
	return FConstantValue(Origin, 0.0);
}

FLinearInterpolation FConstantValue::Integral(double ConstantOffset) const
{
	return FLinearInterpolation(Origin, Value, ConstantOffset);
}

double FLinearInterpolation::Evaluate(FFrameTime InTime) const
{
	return Coefficient*(InTime - Origin).AsDecimal() + Constant;
}

int32 FLinearInterpolation::Solve(double Value, TInterpSolutions<FFrameTime, 1> OutResults) const
{
	if (Coefficient != 0.0)
	{
		OutResults[0] = FFrameTime::FromDecimal((Value - Constant) / Coefficient) + Origin;
		return 1;
	}
	else if (Value == Constant)
	{
		OutResults[0] = FFrameTime::FromDecimal(Constant);
		return 1;
	}
	return 0;
}

int32 FLinearInterpolation::SolveWithin(FFrameTime Start, FFrameTime End, double Value, TInterpSolutions<FFrameTime, 1> OutResults) const
{
	if (Value == Constant)
	{
		OutResults[0] = Start;
		return 1;
	}
	if (Solve(Value, OutResults) == 1 && OutResults[0] >= Start && OutResults[0] < End)
	{
		return 1;
	}
	return 0;
}

FConstantValue FLinearInterpolation::Derivative() const
{
	return FConstantValue(Origin, Coefficient);
}

FQuadraticInterpolation FLinearInterpolation::Integral(double ConstantOffset) const
{
	return FQuadraticInterpolation(Origin, 0.5*Coefficient, Constant, ConstantOffset);
}

double FQuadraticInterpolation::Evaluate(FFrameTime InTime) const
{
	const double X = (InTime - Origin).AsDecimal();
	return A*X*X + B*X + Constant;
}

int32 FQuadraticInterpolation::Solve(double Value, TInterpSolutions<FFrameTime, 2> OutResults) const
{
	const double C = Constant - Value;

	// Solve using the quadratic formula
	OutResults[0] = FFrameTime::FromDecimal( (-B + FMath::Sqrt(B*B - 4.0*A*C)) / (2.0*A) ) + Origin;
	OutResults[1] = FFrameTime::FromDecimal( (-B - FMath::Sqrt(B*B - 4.0*A*C)) / (2.0*A) ) + Origin;

	return 2;
}

int32 FQuadraticInterpolation::SolveWithin(FFrameTime Start, FFrameTime End, double Value, TInterpSolutions<FFrameTime, 2> OutResults) const
{
	Solve(Value, OutResults);

	if (OutResults[0] < Start || OutResults[0] >= End)
	{
		if (OutResults[1] < Start || OutResults[1] >= End)
		{
			return 0;
		}
		OutResults[0] = OutResults[1];
		return 1;
	}
	if (OutResults[1] < Start || OutResults[1] >= End)
	{
		return 1;
	}
	return 2;
}

FLinearInterpolation FQuadraticInterpolation::Derivative() const
{
	return FLinearInterpolation(Origin, 2.0*A, B);
}

FCubicInterpolation FQuadraticInterpolation::Integral(double ConstantOffset) const
{
	return FCubicInterpolation(Origin, A/3.0, B/2.0, Constant, ConstantOffset);
}

double FCubicInterpolation::Evaluate(FFrameTime InTime) const
{
	if (FMath::IsNearlyEqual(DX, 0.0))
	{
		return A;
	}

	const double X = (InTime - Origin).AsDecimal() / DX;
	return A*X*X*X + B*X*X + C*X + Constant;
}

int32 FCubicInterpolation::Solve(double Value, TInterpSolutions<FFrameTime, 3> OutResults) const
{
	double Coefficients[4] = {
		Constant-Value,
		C,
		B,
		A,
	};

	double Solutions[3];
	const int32 NumRealSolutions = UE::Curves::SolveCubic(Coefficients, Solutions);

	for (int32 Index = 0; Index < NumRealSolutions; ++Index)
	{
		OutResults[Index] = DX*FFrameTime::FromDecimal(Solutions[Index])+Origin;
	}

	return NumRealSolutions;
}

int32 FCubicInterpolation::SolveWithin(FFrameTime Start, FFrameTime End, double Value, TInterpSolutions<FFrameTime, 3> OutResults) const
{
	int32 NumSolutions = Solve(Value, OutResults);

	// Only accept solutions within our acceptable range
	int32 WritePos = 0;
	for (int32 Index = 0; Index < NumSolutions; ++Index)
	{
		if (OutResults[Index] >= Start && OutResults[Index] <= End)
		{
			if (Index != WritePos)
			{
				OutResults[WritePos] = OutResults[Index];
			}
			++WritePos;
		}
	}

	return WritePos;
}

FQuadraticInterpolation FCubicInterpolation::Derivative() const
{
	return FQuadraticInterpolation(Origin, 3.0*A/(DX*DX*DX), 2.0*B/(DX*DX), C/DX);
}

FQuarticInterpolation FCubicInterpolation::Integral(double ConstantOffset) const
{
	return FQuarticInterpolation(Origin, A*DX/4.0, B*DX/3.0, C*DX/2.0, DX*Constant, ConstantOffset, DX);
}

double FQuarticInterpolation::Evaluate(FFrameTime InTime) const
{
	if (FMath::IsNearlyEqual(DX, 0.0))
	{
		return A;
	}

	const double X = (InTime - Origin).AsDecimal() / DX;
	return A*X*X*X*X + B*X*X*X + C*X*X + D*X + Constant;
}

FCubicInterpolation FQuarticInterpolation::Derivative() const
{
	return FCubicInterpolation(Origin, 4.0*A, 3.0*B, 2.0*C, D, DX);
}

int32 FQuarticInterpolation::Solve(double Value, TInterpSolutions<FFrameTime, 4> OutResults) const
{
	// Solving ax^4 + bx^3 + cx^2 + dx + e = 0 is very difficult, but we can solve it using some tricks.
	// First off, convert our coefficients to make this a monic quartic of the form X^4 + bx^3 + cx^2 + dx + e:
	const double A_ = 1.0;
	const double B_ = B/A;
	const double C_ = C/A;
	const double D_ = D/A;
	const double E_ = (Constant - Value)/A;

	// Now convert the monic quartic to a depressed quartic of the form y^4 + py^2 + qy + r by substituting x = y - b/4 (since a=1.0)
	const double P = C_ - (3.0*B_*B_) / 8.0;
	const double Q = D_ - (B_*C_)/2.0 + (B_*B_*B_)/8.0;
	const double R = E_ - (B_*D_)/4.0 + (B_*B_*C_)/16.0 - (3.0*B_*B_*B_*B_)/256.0;

	// Step 2: Factor the depressed quartic into two quadratics:
	//         (y^2 + sy + t)(y^2 + uy + v)
	//
	//         which results in
	//         y^4 + (s+u)y^3 + (t+v+su)y^2 + (sv+tu)y + tv
	//
	//         Since y^3 is 0 in our depressed quartic, we can deduce that
	//         s+u = 0
	//           p = t + v + su
	//           q = sv + tu
	//           r = tv
	//
	//         From this we can deduce that s = -u:
	//         p + u^2 = t + v
	//               q = u(t-v)
	//               r = tv
	//
	//         Eliminating t and v by substitution and substituting U = u^2:
	//         U^3 + 2pU^2 + (p^2 - 4r)U - q^2 = 0

	// Solve the cubic for U using Cardano's algorithm:
	// SolveCubic expects coefficients in increasing exponent order
	//    ie d + cx + bx^2 + ax^3
	double Coefficients[4] = {
		-(Q*Q),         // d
		(P*P - 4.0*R),  // c
		(2.0*P),        // b
		1.0,            // a
	};

	double Solutions[3];
	const int32 NumSolutions = UE::Curves::SolveCubic(Coefficients, Solutions);

	int32 SolutionToUse = INDEX_NONE;

	// Any real solution will do, but fallback to a complex solution if there's only one
	for (int32 Index = 0; Index < NumSolutions; ++Index)
	{
		SolutionToUse = Index;

		// Use first real solution if possible
		if (Solutions[Index] > 0.0)
		{
			break;
		}
	}

	int32 SolutionIndex = 0;
	if (SolutionToUse != INDEX_NONE)
	{
		const double U = FMath::Sqrt(FMath::Abs(Solutions[0]));
		const double S = -U;
		const double T = (U*U*U + P*U + Q) / (2.0*U);
		const double V = T - (Q / U);

		const double USquaredMinusFourV = U*U - 4.0*V;
		const double SSquaredMinusFourT = S*S - 4.0*T;
		const double BOverFourA         = B_ / (4.0*A_);

		if (USquaredMinusFourV >= 0.0)
		{
			OutResults[SolutionIndex++] = FFrameTime::FromDecimal(( (-U + FMath::Sqrt(USquaredMinusFourV)) / 2.0 ) - BOverFourA) + Origin;
			OutResults[SolutionIndex++] = FFrameTime::FromDecimal(( (-U - FMath::Sqrt(USquaredMinusFourV)) / 2.0 ) - BOverFourA) + Origin;
		}
		if (SSquaredMinusFourT >= 0.0)
		{
			OutResults[SolutionIndex++] = FFrameTime::FromDecimal(( (-S + FMath::Sqrt(SSquaredMinusFourT)) / 2.0 ) - BOverFourA) + Origin;
			OutResults[SolutionIndex++] = FFrameTime::FromDecimal(( (-S - FMath::Sqrt(SSquaredMinusFourT)) / 2.0 ) - BOverFourA) + Origin;
		}
	}

	return SolutionIndex;
}

int32 FQuarticInterpolation::SolveWithin(FFrameTime Start, FFrameTime End, double Value, TInterpSolutions<FFrameTime, 4> OutResults) const
{
	int32 NumSolutions = Solve(Value, OutResults);

	// Only accept solutions within our acceptable range
	int32 WritePos = 0;
	for (int32 Index = 0; Index < NumSolutions; ++Index)
	{
		if (OutResults[Index] >= Start && OutResults[Index] <= End)
		{
			if (Index != WritePos)
			{
				OutResults[WritePos] = OutResults[Index];
			}
			++WritePos;
		}
	}

	return WritePos;
}

FCubicBezierInterpolation::FCubicBezierInterpolation(FFrameNumber InOrigin, double InDX, double InStartValue, double InEndValue, double InStartTangent, double InEndTangent)
	: DX(InDX)
	, Origin(InOrigin)
{
	constexpr double OneThird = 1.0 / 3.0;

	P0 = InStartValue;
	P1 = P0 + (InStartTangent * DX * OneThird);
	P3 = InEndValue;
	P2 = P3 - (InEndTangent * DX * OneThird);
}

FCubicInterpolation FCubicBezierInterpolation::AsCubic() const
{
	// Beziers are interpolated using a normalized input so we have to factor out that normalization to
	//    each term of the resulting polynomial.

	const double OneOverDxCubed   = 1.0 / (DX * DX * DX);
	const double OneOverDxSquared = 1.0 / (DX * DX);
	const double OneOverDx        = 1.0 / (DX);

	const double A = OneOverDxCubed*(-P0 + 3.0*P1 - 3.0*P2 + P3);
	const double B = OneOverDxSquared*(3.0*P0 - 6.0*P1 + 3.0*P2);
	const double C = OneOverDx*3.0*(-P0 + P1);
	const double D = P0;

	return FCubicInterpolation(
		Origin,
		A,
		B,
		C,
		D,
		1.0);// DX);
}

double FCubicBezierInterpolation::Evaluate(FFrameTime InTime) const
{
	if (FMath::IsNearlyEqual(DX, 0.0))
	{
		return P3;
	}

	const float Interp = static_cast<float>((InTime - Origin).AsDecimal() / DX);
	return UE::Curves::BezierInterp(P0, P1, P2, P3, Interp);
}

FQuadraticInterpolation FCubicBezierInterpolation::Derivative() const
{
	return AsCubic().Derivative();
}
FQuarticInterpolation FCubicBezierInterpolation::Integral(double ConstantOffset) const
{
	return AsCubic().Integral(ConstantOffset);
}

int32 FCubicBezierInterpolation::Solve(double InValue, TInterpSolutions<FFrameTime, 4> OutResults) const
{
	// Offset the curve by InValue to find roots

	// SolveCubic expects coefficients in increasing exponent order
	//    ie d + cx + bx^2 + ax^3
	double Coefficients[4] = {
		(P0-InValue),                       // d
		(-3.0 * P0 + 3.0 * P1),             // c
		(3.0 * P0 - 6.0 * P1 + 3.0 * P2),   // b
		(-P0 + 3.0 * P1 - 3.0 * P2 + P3),   // a
	};

	double Solutions[3];
	const int32 NumRealSolutions = UE::Curves::SolveCubic(Coefficients, Solutions);

	int32 NumSolutionsWithinRange = 0;
	for (int32 Index = 0; Index < NumRealSolutions; ++Index)
	{
		double Solution = Solutions[Index];
		// Only accept solutions within the 0-1 range, allowing for some rounding discrepancies
		if (Solution >= -UE_SMALL_NUMBER && Solution <= 1.0 + UE_SMALL_NUMBER)
		{
			Solution = FMath::Clamp(Solution, 0.0, 1.0);
			OutResults[NumSolutionsWithinRange++] = FFrameTime::FromDecimal(Solution*DX) + Origin;
		}
	}

	return NumSolutionsWithinRange;
}

FWeightedCubicInterpolation::FWeightedCubicInterpolation(
	FFrameRate TickResolution,
	FFrameNumber InOrigin,

	FFrameNumber StartTime,
	double StartValue,
	double StartTangent,
	double StartTangentWeight,
	bool bStartIsWeighted,

	FFrameNumber EndTime,
	double EndValue,
	double EndTangent,
	double EndTangentWeight,
	bool bEndIsWeighted)
{
	constexpr double OneThird = 1.0 / 3.0;

	const float TimeInterval = TickResolution.AsInterval();
	const float ToSeconds = 1.0f / TimeInterval;

	const double Time1 = TickResolution.AsSeconds(StartTime);
	const double Time2 = TickResolution.AsSeconds(EndTime);
	const double DXInSeconds = Time2 - Time1;

	Origin = InOrigin;
	DX = (EndTime - StartTime).Value;
	StartKeyValue = StartValue;
	EndKeyValue = EndValue;

	double StartCosAngle, StartSinAngle, StartAngle;
	// ---------------------------------------------------------------------------------
	// Initialize the start key parameters
	StartAngle = FMath::Atan(StartTangent * ToSeconds);
	FMath::SinCos(&StartSinAngle, &StartCosAngle, StartAngle);

	if (bStartIsWeighted)
	{
		StartWeight = StartTangentWeight;
	}
	else
	{
		const double LeaveTangentNormalized = StartTangent / TimeInterval;
		const double DY = LeaveTangentNormalized * DXInSeconds;
		StartWeight = FMath::Sqrt(DXInSeconds*DXInSeconds + DY*DY) * OneThird;
	}

	double StartKeyTanX = StartCosAngle * StartWeight + Time1;
	StartKeyTanY              = StartSinAngle * StartWeight + StartValue;

	double EndCosAngle, EndSinAngle, EndAngle;
	// ---------------------------------------------------------------------------------
	// Initialize the end key parameters
	EndAngle = FMath::Atan(EndTangent * ToSeconds);
	FMath::SinCos(&EndSinAngle, &EndCosAngle, EndAngle);

	if (bEndIsWeighted)
	{
		EndWeight =  EndTangentWeight;
	}
	else
	{
		const double ArriveTangentNormalized = EndTangent / TimeInterval;
		const double DY = ArriveTangentNormalized * DXInSeconds;
		EndWeight = FMath::Sqrt(DXInSeconds*DXInSeconds + DY*DY) * OneThird;
	}

	double EndKeyTanX = -EndCosAngle * EndWeight + Time2;
	EndKeyTanY              = -EndSinAngle * EndWeight + EndValue;

	if (GAvoidOverlappingWeightedTangents) //if prevent overlapping tangents 
	{
		if (EndKeyTanX < StartKeyTanX)
		{
			//shrink tangents based upon proportional distance 
			const double Overlap = StartKeyTanX - EndKeyTanX;
			double FirstXDistance = StartKeyTanX - Time1;
			double SecondXDistance = Time2 - EndKeyTanX;
			const double Total = FirstXDistance + SecondXDistance;
			const double FirstProportion = FirstXDistance / Total;
			const double SecondProption = SecondXDistance / Total;
			const double MoveFirstBack = Overlap * FirstProportion;
			const double MoveSecondForward = Overlap * SecondProption;

			StartKeyTanX -= MoveFirstBack;
			FirstXDistance = StartKeyTanX - Time1;
			const double FirstYDistance = (FirstXDistance)*FMath::Tan(StartAngle);
			StartKeyTanY = FirstYDistance + StartValue;

			EndKeyTanX += MoveSecondForward;
			SecondXDistance = Time2 - EndKeyTanX;
			const double SecondYDistance = (SecondXDistance) * -FMath::Tan(EndAngle);
			EndKeyTanY = SecondYDistance + EndValue;

		}
	}
	NormalizedStartTanDX = (StartKeyTanX - Time1) / DXInSeconds;
	NormalizedEndTanDX = (EndKeyTanX - Time1) / DXInSeconds;
}

double FWeightedCubicInterpolation::Evaluate(FFrameTime InTime) const
{
	const double Interp = (InTime - Origin).AsDecimal() / DX;

	double Coeff[4];
	double Results[3];

	//Convert Bezier to Power basis, also float to double for precision for root finding.
	UE::Curves::BezierToPower(
		0.0, NormalizedStartTanDX, NormalizedEndTanDX, 1.0,
		&(Coeff[3]), &(Coeff[2]), &(Coeff[1]), &(Coeff[0])
	);

	Coeff[0] = Coeff[0] - Interp;

	const int32 NumResults = UE::Curves::SolveCubic(Coeff, Results);
	double NewInterp = Interp;
	if (NumResults == 1)
	{
		NewInterp = Results[0];
	}
	else
	{
		NewInterp = TNumericLimits<double>::Lowest(); //just need to be out of range
		for (double Result : Results)
		{
			if ((Result > 0.0 || FMath::IsNearlyEqual(Result, 0.0)) && (Result < 1.0 || FMath::IsNearlyEqual(Result, 1.0))) 
			{
				if (NewInterp < 0.0 || Result > NewInterp)
				{
					NewInterp = Result;
				}
			}
		}

		if (NewInterp == TNumericLimits<double>::Lowest())
		{
			NewInterp = 0.0;
		}
	}

	//now use NewInterp and adjusted tangents plugged into the Y (Value) part of the graph.
	const double P0 = StartKeyValue;
	const double P1 = StartKeyTanY;
	const double P3 = EndKeyValue;
	const double P2 = EndKeyTanY;

	return UE::Curves::BezierInterp(P0, P1, P2, P3, static_cast<float>(NewInterp));
}

int32 FWeightedCubicInterpolation::Solve(double InValue, TInterpSolutions<FFrameTime, 4> OutResults) const
{
	// Solve the control points first
	const double P0 = StartKeyValue;
	const double P1 = StartKeyTanY;
	const double P3 = EndKeyValue;
	const double P2 = EndKeyTanY;

	// Offset the curve by InValue to find roots

	// SolveCubic expects coefficients in increasing exponent order
	//    ie d + cx + bx^2 + ax^3
	double Coefficients[4] = {
		(P0-InValue),                       // d
		(-3.0 * P0 + 3.0 * P1),             // c
		(3.0 * P0 - 6.0 * P1 + 3.0 * P2),   // b
		(-P0 + 3.0 * P1 - 3.0 * P2 + P3),   // a
	};

	double Solutions[3];
	const int32 NumRealSolutions = UE::Curves::SolveCubic(Coefficients, Solutions);

	FFrameTime InitialSolutions[3];
	int32 NumSolutionsWithinRange = 0;
	for (int32 Index = 0; Index < NumRealSolutions; ++Index)
	{
		double Solution = Solutions[Index];
		// Only accept solutions within the 0-1 range, allowing for some rounding discrepancies
		if (Solution >= -UE_SMALL_NUMBER && Solution <= 1.0 + UE_SMALL_NUMBER)
		{
			Solution = FMath::Clamp(Solution, 0.0, 1.0);

			// Now solve this interp for the power basis
			Solution = UE::Curves::BezierInterp(0.0, NormalizedStartTanDX, NormalizedEndTanDX, 1.0, static_cast<float>(Solution));
			OutResults[NumSolutionsWithinRange++] = FFrameTime::FromDecimal(Solution*DX) + Origin;
		}
	}

	return NumSolutionsWithinRange;
}

} // namespace UE::MovieScene



