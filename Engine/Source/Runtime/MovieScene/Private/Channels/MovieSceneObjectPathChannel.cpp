// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/MovieSceneObjectPathChannel.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneObjectPathChannel)

bool FMovieSceneObjectPathChannelKeyValue::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == NAME_SoftObjectProperty)
	{
		FSoftObjectPtr OldProperty;
		Slot << OldProperty;

		SoftPtr = OldProperty.ToSoftObjectPath();
		if (OldProperty.ToSoftObjectPath().GetSubPathString().Len() == 0)
		{
			// Forcibly load the old property so we can store it as a hard reference, only if it was not referencing an actor or other sub object
			UObject* RawObject = OldProperty.LoadSynchronous();

			// Do not store raw ptrs to actors or other objects that exist in worlds
			if (RawObject && !RawObject->GetTypedOuter<UWorld>())
			{
				HardPtr = RawObject;
			}
		}
		return true;
	}
	return false;
}

FMovieSceneObjectPathChannelKeyValue::FMovieSceneObjectPathChannelKeyValue(UObject* InObject)
	: SoftPtr(nullptr)
	, HardPtr(nullptr)
{
	*this = InObject;
}

FMovieSceneObjectPathChannelKeyValue& FMovieSceneObjectPathChannelKeyValue::operator=(UObject* NewObject)
{
	// Do not store raw ptrs to actors or other objects that exist in worlds
	if (!NewObject || NewObject->GetTypedOuter<UWorld>())
	{
		HardPtr = nullptr;
	}
	else
	{
		HardPtr = NewObject;
	}

	SoftPtr = NewObject;
	return *this;
}

UObject* FMovieSceneObjectPathChannelKeyValue::Get() const
{
	if (!HardPtr && !SoftPtr.IsNull())
	{
		UObject* ResolvedPtr = SoftPtr.Get();
		if (!ResolvedPtr)
		{
			ResolvedPtr = SoftPtr.LoadSynchronous();
		}

		// Do not store raw ptrs to actors or other objects that exist in worlds
		if (ResolvedPtr && !ResolvedPtr->GetTypedOuter<UWorld>())
		{
			HardPtr = ResolvedPtr;
		}
		return ResolvedPtr;
	}

	return HardPtr;
}

bool FMovieSceneObjectPathChannel::Evaluate(FFrameTime InTime, UObject*& OutValue) const
{
	if (Times.Num())
	{
		const int32 Index = FMath::Max(0, Algo::UpperBound(Times, InTime.FrameNumber)-1);
		OutValue = Values[Index].Get();
		return true;
	}
	else if (!DefaultValue.GetSoftPtr().IsNull())
	{
		OutValue = DefaultValue.Get();
		return true;
	}

	return false;
}

void FMovieSceneObjectPathChannel::GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles)
{
	GetData().GetKeys(WithinRange, OutKeyTimes, OutKeyHandles);
}

void FMovieSceneObjectPathChannel::GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes)
{
	GetData().GetKeyTimes(InHandles, OutKeyTimes);
}

void FMovieSceneObjectPathChannel::SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes)
{
	GetData().SetKeyTimes(InHandles, InKeyTimes);
}

void FMovieSceneObjectPathChannel::DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles)
{
	GetData().DuplicateKeys(InHandles, OutNewHandles);
}

void FMovieSceneObjectPathChannel::DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore)
{
	// Insert a key at the current time to maintain evaluation
	if (GetData().GetTimes().Num() > 0)
	{
		UObject* Value = nullptr;
		if (Evaluate(InTime, Value))
		{
			GetData().UpdateOrAddKey(InTime, Value);
		}
	}

	GetData().DeleteKeysFrom(InTime, bDeleteKeysBefore);
}

void FMovieSceneObjectPathChannel::DeleteKeys(TArrayView<const FKeyHandle> InHandles)
{
	GetData().DeleteKeys(InHandles);
}

void FMovieSceneObjectPathChannel::RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer)
{
	GetData().RemapTimes(Retimer);
}

TRange<FFrameNumber> FMovieSceneObjectPathChannel::ComputeEffectiveRange() const
{
	return GetData().GetTotalRange();
}

int32 FMovieSceneObjectPathChannel::GetNumKeys() const 
{
	return Times.Num();
}

void FMovieSceneObjectPathChannel::Reset() 
{
	return GetData().Reset();
}

void FMovieSceneObjectPathChannel::Offset(FFrameNumber DeltaPosition)
{
	GetData().Offset(DeltaPosition);
}

FKeyHandle FMovieSceneObjectPathChannel::GetHandle(int32 Index)
{
	return GetData().GetHandle(Index);
}

int32 FMovieSceneObjectPathChannel::GetIndex(FKeyHandle Handle)
{
	return GetData().GetIndex(Handle);
}

void FMovieSceneObjectPathChannel::Optimize(const FKeyDataOptimizationParams& InParameters)
{}

void FMovieSceneObjectPathChannel::ClearDefault() 
{
	RemoveDefault();
}

void FMovieSceneObjectPathChannel::SetDefault(const FMovieSceneObjectPathChannelKeyValue& InDefaultValue)
{
	DefaultValue = InDefaultValue;
}

