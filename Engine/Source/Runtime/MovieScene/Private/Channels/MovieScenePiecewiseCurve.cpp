// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/MovieScenePiecewiseCurve.h"
#include "Channels/MovieScenePiecewiseCurveUtils.inl"

namespace UE::MovieScene
{

FPiecewiseCurve FPiecewiseCurve::Integral() const
{
	using namespace Interpolation;

	FPiecewiseCurve IntegralCurve;

	double IntegralOffset = 0.0;

	for (const FCachedInterpolation& Value : Values)
	{
		TOptional<FCachedInterpolation> IntegralPiece = Value.ComputeIntegral(IntegralOffset);
		if (ensure(IntegralPiece.IsSet()))
		{
			IntegralPiece->Evaluate(Value.GetRange().End, IntegralOffset);
			IntegralCurve.Values.Emplace(IntegralPiece.GetValue());
		}
	}

	return IntegralCurve;
}


FPiecewiseCurve FPiecewiseCurve::Derivative() const
{
	using namespace Interpolation;

	FPiecewiseCurve DerivativeCurve;

	for (const FCachedInterpolation& Value : Values)
	{
		TOptional<FCachedInterpolation> DerivativePiece = Value.ComputeDerivative();
		if (ensure(DerivativePiece.IsSet()))
		{
			DerivativeCurve.Values.Emplace(DerivativePiece.GetValue());
		}
	}
	return DerivativeCurve;
}


void FPiecewiseCurve::Offset(double Amount)
{
	using namespace Interpolation;

	for (FCachedInterpolation& Value : Values)
	{
		Value.Offset(Amount);
	}
}


Interpolation::FCachedInterpolation FPiecewiseCurve::GetInterpolationForTime(FFrameTime InTime) const
{
	using namespace Interpolation;

	for (const FCachedInterpolation& Interp : Values)
	{
		if (Interp.GetRange().Contains(InTime.FrameNumber))
		{
			return Interp;
		}
	}

	return FCachedInterpolation();
}


bool FPiecewiseCurve::Evaluate(FFrameTime InTime, double& OutResult) const
{
	using namespace Interpolation;

	FCachedInterpolation Interp = GetInterpolationForTime(InTime);
	if (Interp.IsValid())
	{
		return Interp.Evaluate(InTime, OutResult);
	}
	return false;
}


TOptional<FFrameTime> FPiecewiseCurve::InverseEvaluate(double InValue, FFrameTime InTimeHint, EInverseEvaluateFlags Flags) const
{
	using namespace UE::MovieScene::Interpolation;


	if (Values.Num() == 0)
	{
		// No solution
		return TOptional<FFrameTime>();
	}

	// Never walk more than this number of iterations away from the time hint unless we have a cycle with offset mode
	int32 MaxIterations = Values.Num();

	const FFrameNumber MinFrame = Values[0].GetRange().Start;
	const FFrameNumber MaxFrame = Values.Last().GetRange().End;
	const int32 TimeHintCycle = 0;//CycleTime(MinFrame, MaxFrame, InTimeHint).CycleCount;

	// Use the hint to find our first interpolation
	FCachedInterpolation NextInterp = GetInterpolationForTime(InTimeHint);
	if (!NextInterp.IsValid())
	{
		return TOptional<FFrameTime>();
	}

	// Compute the preceeding interpolation
	FCachedInterpolation PrevInterp = EnumHasAnyFlags(Flags, EInverseEvaluateFlags::Backwards)
		? GetInterpolationForTime(NextInterp.GetRange().Start-1)
		: FCachedInterpolation();
	

	// Choose the nearest of all the solutions
	FFrameTime TmpSolutions[4];
	TOptional<FFrameTime> Result;
	TOptional<int32> ResultCycleDiff;
	
	double Difference = std::numeric_limits<double>::max();

	int32 IterationCount = 0;


	auto ReportSolution = [Flags, InTimeHint, TimeHintCycle, MinFrame, MaxFrame, &Result, &Difference, &ResultCycleDiff](FFrameTime InResult)
	{
		// Reject solutions that occur at the same time hint if we're not searching with the equal flag
		if (!EnumHasAnyFlags(Flags, EInverseEvaluateFlags::Equal) && InResult == InTimeHint)
		{
			return false;
		}

		// Reject solutions that occur before the time hint if we're not searching backwards
		if (!EnumHasAnyFlags(Flags, EInverseEvaluateFlags::Backwards) && InResult < InTimeHint)
		{
			return false;
		}

		// Reject solutions that occur after the time hint if we're not searching forwards
		if (!EnumHasAnyFlags(Flags, EInverseEvaluateFlags::Forwards) && InResult > InTimeHint)
		{
			return false;
		}

		const int32 SolutionCycle = 0;//CycleTime(MinFrame, MaxFrame, InResult).CycleCount;

		// Reject solutions that occur in a different cycle if we don't allow cycling
		if(!EnumHasAnyFlags(Flags, EInverseEvaluateFlags::Cycle) && SolutionCycle != TimeHintCycle)
		{
			return false;
		}

		const double ThisDiff  = FMath::Abs((InResult - InTimeHint).AsDecimal());
		const int32  CycleDiff = FMath::Abs(SolutionCycle - TimeHintCycle);
		
		if (Result.IsSet())
		{
			if (CycleDiff > ResultCycleDiff.GetValue())
			{
				return false;
			}
			if (ThisDiff > Difference)
			{
				return false;
			}
		}

		Result = InResult;
		Difference = ThisDiff;
		ResultCycleDiff = CycleDiff;
		return true;
	};


	// Walk forwards
	while (NextInterp.IsValid() && IterationCount < MaxIterations)
	{
		++IterationCount;

		const int32 LocalNumSolutions = NextInterp.InverseEvaluate(InValue, TmpSolutions);

		for (int32 SolutionIndex = 0; SolutionIndex < LocalNumSolutions; ++SolutionIndex)
		{
			ReportSolution(TmpSolutions[SolutionIndex]);
		}

		if (!Result.IsSet() && EnumHasAnyFlags(Flags, EInverseEvaluateFlags::Forwards))
		{
			// Move on to the next one if possible
			FFrameNumber ThisInterpEnd = NextInterp.GetRange().End;
			if (ThisInterpEnd < TNumericLimits<FFrameNumber>::Max())
			{
				NextInterp = GetInterpolationForTime(ThisInterpEnd+1);
				continue;
			}
		}

		// Should only get here if there were solutions, or we're at the end of the range
		break;
	}

	// Walk backwards
	while (PrevInterp.IsValid() && IterationCount < MaxIterations)
	{
		++IterationCount;

		const int32 LocalNumSolutions = PrevInterp.InverseEvaluate(InValue, TmpSolutions);
		for (int32 SolutionIndex = 0; SolutionIndex < LocalNumSolutions; ++SolutionIndex)
		{
			ReportSolution(TmpSolutions[SolutionIndex]);
		}

		if (!Result.IsSet())
		{
			// Move on to the previous one if possible
			FFrameNumber ThisInterpStart = PrevInterp.GetRange().Start;
			if (ThisInterpStart > TNumericLimits<FFrameNumber>::Lowest())
			{
				PrevInterp = GetInterpolationForTime(ThisInterpStart-1);
				continue;
			}
		}

		// Should only get here if there were solutions, or we're at the start of the range
		break;
	}

	return Result;
}


bool FPiecewiseCurve::InverseEvaluateBetween(double InValue, FFrameTime StartTime, FFrameTime EndTime, const TFunctionRef<bool(FFrameTime)>& VisitorCallback) const
{
	using namespace UE::MovieScene::Interpolation;

	if (Values.Num() == 0)
	{
		// No solution
		return true;
	}

	FFrameTime TmpSolutions[4];

	FCachedInterpolation Interp = GetInterpolationForTime(StartTime);
	while (Interp.IsValid())
	{
		const int32 LocalNumSolutions = Interp.InverseEvaluate(InValue, TmpSolutions);

		for (int32 SolutionIndex = 0; SolutionIndex < LocalNumSolutions; ++SolutionIndex)
		{
			if (!VisitorCallback(TmpSolutions[SolutionIndex]))
			{
				return false;
			}
		}

		// Move on to the next one
		FFrameNumber ThisInterpEnd = Interp.GetRange().End;
		if (ThisInterpEnd != TNumericLimits<FFrameNumber>::Max() && ThisInterpEnd < EndTime)
		{
			Interp = GetInterpolationForTime(ThisInterpEnd+1);
		}
		else
		{
			Interp = FCachedInterpolation();
		}
	}

	return true;
}


bool FPiecewiseCurveData::HasDefaultValue() const
{
	return false;
}

double FPiecewiseCurveData::GetDefaultValue() const
{
	return 0.0;
}

double FPiecewiseCurveData::PreExtrapolate(const FFrameTime& Time) const
{
	return 0.0;
}

double FPiecewiseCurveData::PostExtrapolate(const FFrameTime& Time) const
{
	return 0.0;
}

int32 FPiecewiseCurveData::NumPieces() const
{
	return Channel->Values.Num();
}

int32 FPiecewiseCurveData::GetIndexOfPieceByTime(const FFrameTime& Time) const
{
	using namespace Interpolation;

	const int32 Index = Algo::UpperBoundBy(Channel->Values, Time.FrameNumber, [](const Interpolation::FCachedInterpolation& In) { return In.GetRange().Start; })-1;
	if (Index >= 0 && Index < Channel->Values.Num() && Channel->Values[Index].GetRange().Contains(Time.FrameNumber))
	{
		return Index;
	}

	return INDEX_NONE;
}

Interpolation::FCachedInterpolation FPiecewiseCurveData::GetPieceByIndex(int32 Index) const
{
	return Channel->Values[Index];
}

Interpolation::FCachedInterpolation FPiecewiseCurveData::GetPieceByTime(const FFrameTime& Time) const
{
	int32 Index = GetIndexOfPieceByTime(Time);
	return Channel->Values.IsValidIndex(Index) ? GetPieceByIndex(Index) : Interpolation::FCachedInterpolation();
}

FFrameNumber FPiecewiseCurveData::GetFiniteStart() const
{
	return Channel->Values[0].GetRange().Start;
}

FFrameNumber FPiecewiseCurveData::GetFiniteEnd() const
{
	return Channel->Values.Last().GetRange().End;
}

ERichCurveExtrapolation FPiecewiseCurveData::GetPreExtrapolation() const
{
	return RCCE_None;
}

ERichCurveExtrapolation FPiecewiseCurveData::GetPostExtrapolation() const
{
	return RCCE_None;
}

double FPiecewiseCurveData::GetStartingValue() const
{
	double Value = 0.0;
	Channel->Values[0].Evaluate(Channel->Values[0].GetRange().Start, Value);
	return Value;
}

double FPiecewiseCurveData::GetEndingValue() const
{
	double Value = 0.0;
	Channel->Values.Last().Evaluate(Channel->Values.Last().GetRange().End, Value);
	return Value;
}


} // namespace UE::MovieScene