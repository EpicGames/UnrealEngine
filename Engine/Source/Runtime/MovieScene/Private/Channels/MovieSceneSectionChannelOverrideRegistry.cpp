// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/MovieSceneSectionChannelOverrideRegistry.h"
#include "Channels/MovieSceneChannelEditorData.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneChannelOverrideContainer.h"
#include "Channels/IMovieSceneChannelOverrideProvider.h"
#include "EntitySystem/MovieSceneEntityBuilder.h"
#include "MovieSceneSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSectionChannelOverrideRegistry)

UMovieSceneSectionChannelOverrideRegistry::UMovieSceneSectionChannelOverrideRegistry()
{
}

void UMovieSceneSectionChannelOverrideRegistry::AddChannel(FName ChannelName, UMovieSceneChannelOverrideContainer* ChannelContainer)
{
	Modify();
	Overrides.Emplace(ChannelName, ChannelContainer);
}

bool UMovieSceneSectionChannelOverrideRegistry::ContainsChannel(FName ChannelName) const
{
	return Overrides.Contains(ChannelName);
}

int32 UMovieSceneSectionChannelOverrideRegistry::NumChannels() const
{
	return Overrides.Num();
}

UMovieSceneChannelOverrideContainer* UMovieSceneSectionChannelOverrideRegistry::GetChannel(FName ChannelName) const
{
	return Overrides.FindRef(ChannelName);
}

void UMovieSceneSectionChannelOverrideRegistry::RemoveChannel(FName ChannelName)
{
	Modify();
	Overrides.Remove(ChannelName);
}

void UMovieSceneSectionChannelOverrideRegistry::ImportEntityImpl(const UE::MovieScene::FChannelOverrideEntityImportParams& OverrideParams, const UE::MovieScene::FEntityImportParams& ImportParams, UE::MovieScene::FImportedEntity* OutImportedEntity)
{
	TObjectPtr<UMovieSceneChannelOverrideContainer>* ContainerPtr = Overrides.Find(OverrideParams.ChannelName);
	if (ensure(ContainerPtr))
	{
		(*ContainerPtr)->ImportEntityImpl(OverrideParams, ImportParams, OutImportedEntity);
	}
}

void UMovieSceneSectionChannelOverrideRegistry::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder, UMovieSceneSection& OwnerSection)
{
	using namespace UE::MovieScene;

	IMovieSceneChannelOverrideProvider* OverrideProvider = Cast<IMovieSceneChannelOverrideProvider>(GetOuter());
	if (ensure(OverrideProvider))
	{
		// We only want to allocate entities for those channels overrides that are actually in use.
		// In order to know which channels are in use, we check the channel proxy.
		TSet<FMovieSceneChannel*> ActiveChannels;
		FMovieSceneChannelProxy& ChannelProxy = OwnerSection.GetChannelProxy();
		for (const FMovieSceneChannelEntry& ChannelEntry : ChannelProxy.GetAllEntries())
		{

			TArrayView<FMovieSceneChannel* const> Channels = ChannelEntry.GetChannels();
			const int32 NumChannels = Channels.Num();

#if WITH_EDITOR
			TArrayView<const FMovieSceneChannelMetaData> ChannelMetaDatas = ChannelEntry.GetMetaData();
			checkf(Channels.Num() == ChannelMetaDatas.Num(), TEXT("Expected matching arrays between channels and channel metadatas!"));
#endif

			for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
			{
#if WITH_EDITOR
				const FMovieSceneChannelMetaData& ChannelMetaData = ChannelMetaDatas[ChannelIndex];
				const bool bIsChannelEnabled = ChannelMetaData.bEnabled;
#else
				// We don't have the metadata for the channel... we assume all channels are enabled. This will
				// create potentially useless entries in the evaluation field, but hopefully the entity providers
				// will return empty entity builders for those.
				const bool bIsChannelEnabled = true;
#endif

				if (bIsChannelEnabled)
				{
					ActiveChannels.Add(Channels[ChannelIndex]);
				}
			}
		}

		FChannelOverrideProviderTraitsHandle ChannelOverrideTraits = OverrideProvider->GetChannelOverrideProviderTraits();
		check(ChannelOverrideTraits.IsValid());
		for (const TPair<FName, TObjectPtr<UMovieSceneChannelOverrideContainer>>& Override : Overrides)
		{
			if (ActiveChannels.Contains(Override.Value->GetChannel()))
			{
				const int32 EntityID = ChannelOverrideTraits->GetChannelOverrideEntityID(Override.Key);
				const int32 EntityIndex = OutFieldBuilder->FindOrAddEntity(&OwnerSection, EntityID);
				const int32 MetaDataIndex = OutFieldBuilder->AddMetaData(InMetaData);
				OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex, MetaDataIndex);
			}
		}
	}
}

#if WITH_EDITOR

void UMovieSceneSectionChannelOverrideRegistry::OnPostPaste()
{
	ClearFlags(RF_Transient);

	for (TPair<FName, TObjectPtr<UMovieSceneChannelOverrideContainer>> Pair : Overrides)
	{
		if (ensure(Pair.Value))
		{
			Pair.Value->ClearFlags(RF_Transient);
		}
	}
}

void UMovieSceneSectionChannelOverrideRegistry::PostEditUndo()
{
	Super::PostEditUndo();

	if (UMovieSceneSection* Section = GetTypedOuter<UMovieSceneSection>())
	{
		if (IMovieSceneChannelOverrideProvider* OverrideProvider = Cast<IMovieSceneChannelOverrideProvider>(Section))
		{
			OverrideProvider->OnChannelOverridesChanged();
		}
	}
}

#endif

