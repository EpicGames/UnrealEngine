// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/MovieSceneTimeWarpChannel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneTimeWarpChannel)

void Dilate(FMovieSceneTimeWarpChannel* InChannel, FFrameNumber Origin, double DilationFactor)
{
	if (InChannel->Domain == UE::MovieScene::ETimeWarpChannelDomain::PlayRate)
	{
		// Inverse dilate the values if we are in the play-rate domain
		for (FMovieSceneDoubleValue& Value : InChannel->GetData().GetValues())
		{
			Value.Value /= DilationFactor;
		}
	}

	// The default implementation dilates the keytimes
	Dilate(static_cast<FMovieSceneDoubleChannel*>(InChannel), Origin, DilationFactor);
}
