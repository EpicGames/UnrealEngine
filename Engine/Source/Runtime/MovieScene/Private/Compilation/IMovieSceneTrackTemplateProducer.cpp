// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compilation/IMovieSceneTrackTemplateProducer.h"
#include "Compilation/IMovieSceneTemplateGenerator.h"
#include "Evaluation/MovieSceneEvaluationTrack.h"
#include "Evaluation/MovieSceneEvalTemplate.h"

#include "MovieSceneTrack.h"
#include "MovieSceneSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IMovieSceneTrackTemplateProducer)

FMovieSceneTrackCompilerArgs::FMovieSceneTrackCompilerArgs(UMovieSceneTrack* InTrack, IMovieSceneTemplateGenerator* InGenerator)
	: Track(InTrack)
	, Generator(InGenerator)
{
	check(Track);

	if (UMovieSceneSequence* OuterSequence = Track->GetTypedOuter<UMovieSceneSequence>())
	{
		DefaultCompletionMode = OuterSequence->DefaultCompletionMode;
	}
	else
	{
		DefaultCompletionMode = EMovieSceneCompletionMode::RestoreState;
	}
}

FMovieSceneEvaluationTrack IMovieSceneTrackTemplateProducer::GenerateTrackTemplate(UMovieSceneTrack* SourceTrack) const
{
	FMovieSceneEvaluationTrack TrackTemplate = FMovieSceneEvaluationTrack(FGuid());

	// For this path, we don't have a generator, so we just pass through a stub
	struct FTemplateGeneratorStub : IMovieSceneTemplateGenerator
	{
		virtual void AddOwnedTrack(FMovieSceneEvaluationTrack&& InTrackTemplate, const UMovieSceneTrack& SourceTrack) override {}
	} Generator;

	TrackTemplate.SetSourceTrack(SourceTrack);

	FMovieSceneTrackCompilerArgs Args(SourceTrack, &Generator);
	if (CustomCompile(TrackTemplate, Args) == EMovieSceneCompileResult::Unimplemented)
	{
		Compile(TrackTemplate, Args);
	}

	return TrackTemplate;
}


void IMovieSceneTrackTemplateProducer::GenerateTemplate(const FMovieSceneTrackCompilerArgs& Args) const
{
	FMovieSceneEvaluationTrack NewTrackTemplate(Args.ObjectBindingId);

	NewTrackTemplate.SetPreAndPostrollConditions(Args.Track->EvalOptions.bEvaluateInPreroll, Args.Track->EvalOptions.bEvaluateInPostroll);

	EMovieSceneCompileResult Result = CustomCompile(NewTrackTemplate, Args);
	if (Result == EMovieSceneCompileResult::Unimplemented)
	{
		Result = Compile(NewTrackTemplate, Args);
	}

	if (Result == EMovieSceneCompileResult::Success)
	{
		NewTrackTemplate.SetSourceTrack(Args.Track);
		PostCompile(NewTrackTemplate, Args);

		Args.Generator->AddOwnedTrack(MoveTemp(NewTrackTemplate), *Args.Track);
	}
}


EMovieSceneCompileResult IMovieSceneTrackTemplateProducer::Compile(FMovieSceneEvaluationTrack& OutTrack, const FMovieSceneTrackCompilerArgs& Args) const
{
	for (const UMovieSceneSection* Section : Args.Track->GetAllSections())
	{
		const TRange<FFrameNumber> SectionRange = Section->GetTrueRange();
		if (!Section->IsActive() || SectionRange.IsEmpty())
		{
			continue;
		}

		FMovieSceneEvalTemplatePtr NewTemplate = CreateTemplateForSection(*Section);
		if (NewTemplate.IsValid())
		{
			NewTemplate->SetCompletionMode(Section->GetCompletionMode() == EMovieSceneCompletionMode::ProjectDefault ? Args.DefaultCompletionMode : Section->GetCompletionMode());
			NewTemplate->SetSourceSection(Section);

			OutTrack.AddChildTemplate(MoveTemp(NewTemplate));
		}
	}

	return EMovieSceneCompileResult::Success;
}
