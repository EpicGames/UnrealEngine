// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compilation/MovieSceneCompiledDataManager.h"
#include "Compilation/IMovieSceneTemplateGenerator.h"
#include "Compilation/IMovieSceneTrackTemplateProducer.h"
#include "Compilation/IMovieSceneDeterminismSource.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "Evaluation/MovieSceneEvaluationCustomVersion.h"
#include "Evaluation/MovieSceneRootOverridePath.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "Sections/MovieSceneSubSection.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "Decorations/IMovieSceneDecoration.h"
#include "Decorations/IMovieSceneSectionProviderDecoration.h"
#include "Decorations/MovieSceneTimeWarpDecoration.h"
#include "IMovieSceneModule.h"
#include "MovieSceneTimeHelpers.h"
#include "MovieSceneTransformTypes.h"

#include "Algo/Sort.h"
#include "Algo/Unique.h"

#include "Containers/SortedMap.h"

#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"
#include "UObject/PackageReload.h"
#include "MovieSceneCommonHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneCompiledDataManager)


FString GMovieSceneCompilerVersion = "7D4B98092FAC4A6B964ECF72D8279EF8";
FAutoConsoleVariableRef CVarMovieSceneCompilerVersion(
	TEXT("Sequencer.CompilerVersion"),
	GMovieSceneCompilerVersion,
	TEXT("Defines a global identifer for moviescene compiler logic.\n"),
	ECVF_Default
);


TAutoConsoleVariable<bool> CVarAddKeepStateDeterminismFences(
	TEXT("Sequencer.AddKeepStateDeterminismFences"),
	true,
	TEXT("Whether the Sequencer compiler should auto-add determinism fences for the last frame of KeepState sections. "
		 "This ensures that the last possible value of the section is consistently evaluated regardless of framerate, "
		 "at the cost of an extra evaluation on frames that cross over KeepState sections' end time.\n"),
	ECVF_Default);

TSet<UMovieSceneCompiledDataManager*> UMovieSceneCompiledDataManager::ActiveManagers;


IMovieSceneModule& GetMovieSceneModule()
{
	static TWeakPtr<IMovieSceneModule> WeakMovieSceneModule;

	TSharedPtr<IMovieSceneModule> Shared = WeakMovieSceneModule.Pin();
	if (!Shared.IsValid())
	{
		WeakMovieSceneModule = IMovieSceneModule::Get().GetWeakPtr();
		Shared = WeakMovieSceneModule.Pin();
	}
	check(Shared.IsValid());

	return *Shared;
}


struct FMovieSceneCompileDataManagerGenerator : public IMovieSceneTemplateGenerator
{
	FMovieSceneCompileDataManagerGenerator(UMovieSceneCompiledDataManager* InCompiledDataManager)
	{
		CompiledDataManager = InCompiledDataManager;
		Entry               = nullptr;
		Template            = nullptr;
	}

	void Reset(FMovieSceneCompiledDataEntry* InEntry)
	{
		check(InEntry);

		Entry    = InEntry;
		Template = CompiledDataManager->TrackTemplates.Find(Entry->DataID.Value);
	}

	virtual void AddOwnedTrack(FMovieSceneEvaluationTrack&& InTrackTemplate, const UMovieSceneTrack& SourceTrack) override
	{
		check(Entry);

		if (!Template)
		{
			Template = &CompiledDataManager->TrackTemplates.FindOrAdd(Entry->DataID.Value);
		}

		Template->AddTrack(SourceTrack.GetSignature(), MoveTemp(InTrackTemplate));
	}

private:

	UMovieSceneCompiledDataManager* CompiledDataManager;
	FMovieSceneCompiledDataEntry*   Entry;
	FMovieSceneEvaluationTemplate*  Template;
};


struct FCompileOnTheFlyData
{
	/** Primary sort - group */
	uint16 GroupEvaluationPriority;
	/** Secondary sort - Hierarchical bias */
	int16 HierarchicalBias;
	/** Tertiary sort - Eval priority */
	int16 EvaluationPriority;
	/** Quaternary sort - Child priority */
	int16 ChildPriority;
	/**  */
	FName EvaluationGroup;
	/** Whether the track requires initialization or not */
	bool bRequiresInit;
	bool bPriorityTearDown;

	FMovieSceneEvaluationFieldTrackPtr Track;
	FMovieSceneFieldEntry_ChildTemplate Child;
};


/** Gathered data for a given time or range */
struct FMovieSceneGatheredCompilerData
{
	/** Tree of tracks to evaluate */
	TMovieSceneEvaluationTree<FCompileOnTheFlyData> TrackTemplates;
	/** Tree of active sequences */
	TMovieSceneEvaluationTree<FMovieSceneSequenceID> Sequences;
	FMovieSceneEntityComponentField* EntityField = nullptr;

	FMovieSceneDeterminismData DeterminismData;

	EMovieSceneSequenceFlags InheritedFlags = EMovieSceneSequenceFlags::None;
	EMovieSceneSequenceCompilerMask AccumulatedMask = EMovieSceneSequenceCompilerMask::None;
};

/** Parameter structure used for gathering entities for a given time or range */
struct FGatherParameters
{
	FGatherParameters()
		: SequenceID(MovieSceneSequenceID::Root)
		, RootClampRange(TRange<FFrameNumber>::All())
		, LocalClampRange(RootClampRange)
		, Flags(ESectionEvaluationFlags::None)
		, HierarchicalBias(0)
		, AccumulatedFlags(EMovieSceneSubSectionFlags::None)
	{}

	FGatherParameters CreateForSubData(const FMovieSceneSubSequenceData& SubData, FMovieSceneSequenceID InSubSequenceID) const
	{
		using namespace UE::MovieScene;

		FGatherParameters SubParams;

		SubParams.SequenceID                = InSubSequenceID;
		SubParams.RootClampRange            = this->RootClampRange;
		SubParams.LocalClampRange           = UE::MovieScene::ConvertToDiscreteRange(SubData.RootToSequenceTransform.ComputeTraversedHull(this->RootClampRange));
		SubParams.Flags                     = this->Flags;
		SubParams.RootToSequenceTransform   = SubData.RootToSequenceTransform;
#if WITH_EDITORONLY_DATA
		SubParams.RootToUnwarpedLocalTransform = SubData.RootToUnwarpedLocalTransform;
		SubParams.StartTimeBreadcrumbs				= SubData.StartTimeBreadcrumbs;
		SubParams.EndTimeBreadcrumbs				= SubData.EndTimeBreadcrumbs;
#else
		SubParams.StartTimeBreadcrumbs      = this->StartTimeBreadcrumbs;
		SubParams.EndTimeBreadcrumbs        = this->EndTimeBreadcrumbs;
		SubParams.RootToSequenceTransform.TransformTime(DiscreteInclusiveLower(SubData.ParentPlayRange.Value), FTransformTimeParams().AppendBreadcrumbs(SubParams.StartTimeBreadcrumbs));
		SubParams.RootToSequenceTransform.TransformTime(DiscreteExclusiveUpper(SubData.ParentPlayRange.Value), FTransformTimeParams().AppendBreadcrumbs(SubParams.EndTimeBreadcrumbs));
#endif
		SubParams.HierarchicalBias          = SubData.HierarchicalBias;
		SubParams.AccumulatedFlags          = SubData.AccumulatedFlags;
		SubParams.NetworkMask               = this->NetworkMask;

		return SubParams;
	}

	void SetClampRange(TRange<FFrameNumber> InNewRootClampRange)
	{
		RootClampRange  = InNewRootClampRange;
		LocalClampRange = UE::MovieScene::ConvertToDiscreteRange(RootToSequenceTransform.ComputeTraversedHull(InNewRootClampRange));
	}

	/** Clamp the specified range to the current clamp range (in root space) */
	TRange<FFrameNumber> ClampRoot(const TRange<FFrameNumber>& InRootRange) const
	{
		return TRange<FFrameNumber>::Intersection(RootClampRange, InRootRange);
	}

	void TransformLocalRange(const TRange<FFrameNumber>& InLocalRange, TFunctionRef<bool(TRange<FFrameTime>)> InVisitor) const
	{
		using namespace UE::MovieScene;

		TRange<FFrameTime> Range = ConvertToFrameTimeRange(InLocalRange);

		FMovieSceneInverseSequenceTransform SequenceToRootTransform = RootToSequenceTransform.Inverse();

		// Linear transforms are easy
		if (SequenceToRootTransform.IsLinear())
		{
			FMovieSceneTimeTransform LinearTransform = SequenceToRootTransform.AsLinear();

			if (!Range.GetLowerBound().IsOpen())
			{
				Range.SetLowerBoundValue(Range.GetLowerBoundValue() * LinearTransform);
			}
			if (!Range.GetUpperBound().IsOpen())
			{
				Range.SetUpperBoundValue(Range.GetUpperBoundValue() * LinearTransform);
			}

			// Normalize inside-out ranges due to negative TimeScale
			if (Range.GetLowerBound().IsClosed() && Range.GetUpperBound().IsClosed()
				&& Range.GetLowerBoundValue() > Range.GetUpperBoundValue())
			{
				const auto OldLower = Range.GetLowerBound();
				const auto OldUpper = Range.GetUpperBound();
				Range.SetLowerBound(OldUpper);
				Range.SetUpperBound(OldLower);
			}

			InVisitor(Range);
			return;
		}

		// Warping transforms are a bit harder

	
		// First off, intersect with the clamp range
		if (Range.GetLowerBound().IsOpen() || Range.GetUpperBound().IsOpen())
		{
			Range = TRange<FFrameTime>::Intersection(Range, RootToSequenceTransform.ComputeTraversedHull(ConvertToFrameTimeRange(RootClampRange)));
		}

		// Make the range finite based on clamp ranges if possible
		if (Range.GetLowerBound().IsOpen() && !RootClampRange.GetLowerBound().IsOpen())
		{
			TRangeBound<FFrameNumber> LowerBound = RootClampRange.GetLowerBound();
			FFrameTime NewTime = RootToSequenceTransform.TransformTime(LowerBound.GetValue());
			if (LowerBound.IsInclusive())
			{
				Range.SetLowerBound(TRangeBound<FFrameTime>::Inclusive(NewTime));
			}
			else
			{
				Range.SetLowerBound(TRangeBound<FFrameTime>::Exclusive(NewTime));
			}
		}
		if (Range.GetUpperBound().IsOpen() && !RootClampRange.GetUpperBound().IsOpen())
		{
			TRangeBound<FFrameNumber> UpperBound = RootClampRange.GetUpperBound();
			FFrameTime NewTime = RootToSequenceTransform.TransformTime(UpperBound.GetValue());
			if (UpperBound.IsInclusive())
			{
				Range.SetUpperBound(TRangeBound<FFrameTime>::Inclusive(NewTime));
			}
			else
			{
				Range.SetUpperBound(TRangeBound<FFrameTime>::Exclusive(NewTime));
			}
		}

		// Normalize inside-out ranges due to negative TimeScale
		if (Range.GetLowerBound().IsClosed() && Range.GetUpperBound().IsClosed()
			&& Range.GetLowerBoundValue() > Range.GetUpperBoundValue())
		{
			const auto OldLower = Range.GetLowerBound();
			const auto OldUpper = Range.GetUpperBound();
			Range.SetLowerBound(OldUpper);
			Range.SetUpperBound(OldLower);
		}

		if (Range.GetLowerBound().IsOpen() && Range.GetUpperBound().IsOpen())
		{
			// If the range is infinite we just have to add it all since there's no way for us to transform it.
			InVisitor(Range);
		}
		else if (!Range.GetLowerBound().IsOpen() && !Range.GetUpperBound().IsOpen())
		{
			// We have a finite range so transform it as many times as it exists in the root space
			SequenceToRootTransform.TransformFiniteRangeWithinRange(Range, InVisitor, StartTimeBreadcrumbs, EndTimeBreadcrumbs);
		}
		else if (Range.GetLowerBound().IsOpen())
		{
			// Open lower bound so just transform the the upper bound once and compile that
			TOptional<FFrameTime> Time = SequenceToRootTransform.TryTransformTime(Range.GetUpperBoundValue(), EndTimeBreadcrumbs);
			if (Time)
			{
				Range.SetUpperBoundValue(Time->FloorToFrame());
				InVisitor(Range);
			}
		}
		else if (Range.GetUpperBound().IsOpen())
		{
			// Open upper bound so just transform the the lower bound once and compile that
			TOptional<FFrameTime> Time = SequenceToRootTransform.TryTransformTime(Range.GetLowerBoundValue(), StartTimeBreadcrumbs);
			if (Time)
			{
				Range.SetLowerBoundValue(Time->FloorToFrame());
				InVisitor(Range);
			}
		}
	}

	/** The ID of the sequence being compiled */
	FMovieSceneSequenceID SequenceID;

	/** A range to clamp compilation to in the root's time-space */
	TRange<FFrameNumber> RootClampRange;
	/** A range to clamp compilation to in the current sequence's time-space */
	TRange<FFrameNumber> LocalClampRange;

	/** Evaluation flags for the current sequence */
	ESectionEvaluationFlags Flags;

	/** Transform from the root time-space to the current sequence's time-space */
	FMovieSceneSequenceTransform RootToSequenceTransform;
#if WITH_EDITORONLY_DATA
	/** The transform from root space to this sub-sequence's unwarped local space. */
	FMovieSceneSequenceTransform RootToUnwarpedLocalTransform;
#endif
	FMovieSceneTransformBreadcrumbs StartTimeBreadcrumbs;
	FMovieSceneTransformBreadcrumbs EndTimeBreadcrumbs;

	/** Current accumulated hierarchical bias */
	int16 HierarchicalBias;

	/** Current accumulated sub-section flags */
	EMovieSceneSubSectionFlags AccumulatedFlags;

	EMovieSceneServerClientMask NetworkMask;
};

/** Parameter structure used for gathering entities for a given time or range */
struct FTrackGatherParameters : FGatherParameters
{
	FTrackGatherParameters(UMovieSceneCompiledDataManager* InCompiledDataManager)
		: TemplateGenerator(InCompiledDataManager)
	{}

	FTrackGatherParameters CreateForSubData(const FMovieSceneSubSequenceData& SubData, FMovieSceneSequenceID InSubSequenceID) const
	{
		FTrackGatherParameters SubParams;
		static_cast<FGatherParameters&>(SubParams) = FGatherParameters::CreateForSubData(SubData, InSubSequenceID);
		SubParams.TemplateGenerator = this->TemplateGenerator;
		return SubParams;
	}


	/** Store from which to retrieve templates */
	mutable FMovieSceneCompileDataManagerGenerator TemplateGenerator;

private:
	FTrackGatherParameters()
		: TemplateGenerator(nullptr)
	{}
};


bool SortPredicate(const FCompileOnTheFlyData& A, const FCompileOnTheFlyData& B)
{
	if (A.GroupEvaluationPriority != B.GroupEvaluationPriority)
	{
		return A.GroupEvaluationPriority > B.GroupEvaluationPriority;
	}
	else if (A.HierarchicalBias != B.HierarchicalBias)
	{
		return A.HierarchicalBias < B.HierarchicalBias;
	}
	else if (A.EvaluationPriority != B.EvaluationPriority)
	{
		return A.EvaluationPriority > B.EvaluationPriority;
	}
	else
	{
		return A.ChildPriority > B.ChildPriority;
	}
}

void AddPtrsToGroup(
	FMovieSceneEvaluationGroup* OutGroup,
	TArray<FMovieSceneFieldEntry_EvaluationTrack>& InitTrackLUT,
	TArray<FMovieSceneFieldEntry_ChildTemplate>&   InitSectionLUT,
	TArray<FMovieSceneFieldEntry_EvaluationTrack>& EvalTrackLUT,
	TArray<FMovieSceneFieldEntry_ChildTemplate>&   EvalSectionLUT
	)
{
	if (!InitTrackLUT.Num() && !EvalTrackLUT.Num())
	{
		return;
	}

	FMovieSceneEvaluationGroupLUTIndex Index;
	Index.NumInitPtrs = InitTrackLUT.Num();
	Index.NumEvalPtrs = EvalTrackLUT.Num();

	OutGroup->LUTIndices.Add(Index);
	OutGroup->TrackLUT.Append(InitTrackLUT);
	OutGroup->TrackLUT.Append(EvalTrackLUT);

	OutGroup->SectionLUT.Append(InitSectionLUT);
	OutGroup->SectionLUT.Append(EvalSectionLUT);

	InitTrackLUT.Reset();
	InitSectionLUT.Reset();
	EvalTrackLUT.Reset();
	EvalSectionLUT.Reset();
}

FMovieSceneCompiledDataEntry::FMovieSceneCompiledDataEntry()
	: AccumulatedFlags(EMovieSceneSequenceFlags::None)
	, AccumulatedMask(EMovieSceneSequenceCompilerMask::None)
{}

UMovieSceneSequence* FMovieSceneCompiledDataEntry::GetSequence() const
{
	return CastChecked<UMovieSceneSequence>(SequenceKey.ResolveObjectPtr(), ECastCheckedType::NullAllowed);
}

UMovieSceneCompiledData::UMovieSceneCompiledData()
{
	AccumulatedMask = EMovieSceneSequenceCompilerMask::None;
	AllocatedMask = EMovieSceneSequenceCompilerMask::None;
	AccumulatedFlags = EMovieSceneSequenceFlags::None;
}

void UMovieSceneCompiledData::Reset()
{
	EvaluationTemplate = FMovieSceneEvaluationTemplate();
	Hierarchy = FMovieSceneSequenceHierarchy();
	EntityComponentField = FMovieSceneEntityComponentField();
	TrackTemplateField = FMovieSceneEvaluationField();
	DeterminismFences.Reset();
	CompiledSignature.Invalidate();
	CompilerVersion.Invalidate();
	AccumulatedMask = EMovieSceneSequenceCompilerMask::None;
	AllocatedMask = EMovieSceneSequenceCompilerMask::None;
	AccumulatedFlags = EMovieSceneSequenceFlags::None;
}

#if WITH_EDITORONLY_DATA
void UMovieSceneCompiledData::AppendToClassSchema(FAppendToClassSchemaContext& Context)
{
	Super::AppendToClassSchema(Context);

	// Specify the compiler version to the iterative cooker. Any changes to the schema of
	//    compiled data should update the version to ensure that compiled data is invalidated
	//    for the purposes of iterative cooking.

	FGuid ParsedCompilerVersion;
	if (FGuid::Parse(GMovieSceneCompilerVersion, ParsedCompilerVersion))
	{
		Context.Update(&ParsedCompilerVersion, sizeof(ParsedCompilerVersion));
	}
}
#endif

UMovieSceneCompiledDataManager::UMovieSceneCompiledDataManager()
{
	const bool bParsed = FGuid::Parse(GMovieSceneCompilerVersion, CompilerVersion);
	ensureMsgf(bParsed, TEXT("Invalid compiler version specified - this will break any persistent compiled data"));

	IConsoleManager::Get().RegisterConsoleVariableSink_Handle(FConsoleCommandDelegate::CreateUObject(this, &UMovieSceneCompiledDataManager::ConsoleVariableSink));

	ReallocationVersion = 0;
	NetworkMask = EMovieSceneServerClientMask::All;

	auto OnPackageReloaded = [this](const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent)
	{
		if (InPackageReloadPhase != EPackageReloadPhase::OnPackageFixup)
		{
			return;
		}

		for (const TPair<UObject*, UObject*>& Pair : InPackageReloadedEvent->GetRepointedObjects())
		{
			UMovieSceneSequence* OldSequence = Cast<UMovieSceneSequence>(Pair.Key);
			UMovieSceneSequence* NewSequence = Cast<UMovieSceneSequence>(Pair.Value);
			if (OldSequence && NewSequence)
			{
				FMovieSceneCompiledDataID DataID = this->SequenceToDataIDs.FindRef(OldSequence);
				if (DataID.IsValid())
				{
					// Repoint the data ID for the old sequence to the new sequence
					{
						FMovieSceneCompiledDataEntry& Entry = CompiledDataEntries[DataID.Value];
						this->SequenceToDataIDs.Remove(Entry.SequenceKey);

						// Entry is a ref here, so care is taken to ensure we do not allocate CompiledDataEntries while the ref is around
						Entry = FMovieSceneCompiledDataEntry();
						Entry.SequenceKey = NewSequence;
						Entry.DataID = DataID;

						this->SequenceToDataIDs.Add(Entry.SequenceKey, DataID);
					}

					// Destroy all the old compiled data as it is no longer valid
					this->Hierarchies.Remove(DataID.Value);
					this->TrackTemplates.Remove(DataID.Value);
					this->TrackTemplateFields.Remove(DataID.Value);
					this->EntityComponentFields.Remove(DataID.Value);

					++this->ReallocationVersion;
				}
			}
		}
	};

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		FCoreUObjectDelegates::OnPackageReloaded.AddWeakLambda(this, OnPackageReloaded);
		ActiveManagers.Add(this);
	}
}

void UMovieSceneCompiledDataManager::BeginDestroy()
{
	ActiveManagers.Remove(this);
	Super::BeginDestroy();
}

void UMovieSceneCompiledDataManager::ReportSequenceDestroyed(UMovieSceneSequence* InSequence)
{
	if (!GExitPurge)
	{
		for (UMovieSceneCompiledDataManager* Manager : ActiveManagers)
		{
			Manager->Reset(InSequence);
		}
	}
}

void UMovieSceneCompiledDataManager::DestroyAllData()
{
	// Eradicate all compiled data
	for (int32 Index = 0; Index < CompiledDataEntries.GetMaxIndex(); ++Index)
	{
		if (CompiledDataEntries.IsAllocated(Index))
		{
			FMovieSceneCompiledDataEntry& Entry = CompiledDataEntries[Index];
			Entry.CompiledSignature       = FGuid();
			Entry.AccumulatedFlags        = EMovieSceneSequenceFlags::None;
			Entry.AccumulatedMask         = EMovieSceneSequenceCompilerMask::None;
		}
	}

	Hierarchies.Empty();
	TrackTemplates.Empty();
	TrackTemplateFields.Empty();
	EntityComponentFields.Empty();
}

void UMovieSceneCompiledDataManager::ConsoleVariableSink()
{
	FGuid NewCompilerVersion;
	const bool bParsed = FGuid::Parse(GMovieSceneCompilerVersion, NewCompilerVersion);
	ensureMsgf(bParsed, TEXT("Invalid compiler version specific - this will break any persistent compiled data"));

	if (CompilerVersion != NewCompilerVersion)
	{
		DestroyAllData();
	}
}

void UMovieSceneCompiledDataManager::CopyCompiledData(UMovieSceneSequence* Sequence)
{
	UMovieSceneCompiledData* CompiledData = Sequence->GetOrCreateCompiledData();
	CompiledData->Reset();

	FMovieSceneCompiledDataID DataID = GetDataID(Sequence);
	Compile(DataID, Sequence);

	if (const FMovieSceneSequenceHierarchy* Hierarchy = FindHierarchy(DataID))
	{
		CompiledData->Hierarchy = *Hierarchy;
		CompiledData->AllocatedMask |= EMovieSceneSequenceCompilerMask::Hierarchy;
	}
	if (const FMovieSceneEvaluationTemplate* TrackTemplate = FindTrackTemplate(DataID))
	{
		CompiledData->EvaluationTemplate = *TrackTemplate;
		CompiledData->AllocatedMask |= EMovieSceneSequenceCompilerMask::EvaluationTemplate;
	}
	if (const FMovieSceneEvaluationField* TrackTemplateField = FindTrackTemplateField(DataID))
	{
		if (Sequence->IsPlayableDirectly())
		{
			CompiledData->TrackTemplateField = *TrackTemplateField;
			CompiledData->AllocatedMask  |= EMovieSceneSequenceCompilerMask::EvaluationTemplateField;
		}
	}
	if (const FMovieSceneEntityComponentField* EntityComponentField = FindEntityComponentField(DataID))
	{
		CompiledData->EntityComponentField = *EntityComponentField;
		CompiledData->AllocatedMask |= EMovieSceneSequenceCompilerMask::EntityComponentField;
	}

	const FMovieSceneCompiledDataEntry& DataEntry = CompiledDataEntries[DataID.Value];
	CompiledData->DeterminismFences = DataEntry.DeterminismFences;
	CompiledData->CompiledSignature = Sequence->GetSignature();
	CompiledData->CompilerVersion = CompilerVersion;
	CompiledData->AccumulatedMask = DataEntry.AccumulatedMask;
	CompiledData->AccumulatedFlags = DataEntry.AccumulatedFlags;
	CompiledData->CompiledFlags = DataEntry.CompiledFlags;
}

void UMovieSceneCompiledDataManager::LoadCompiledData(UMovieSceneSequence* Sequence)
{
	// This can be called during Async Loads
	FScopeLock AsyncLoadLock(&AsyncLoadCriticalSection);

	UMovieSceneCompiledData* CompiledData = Sequence->GetCompiledData();
	if (CompiledData)
	{
		FMovieSceneCompiledDataID DataID = GetDataID(Sequence);

		if (CompiledData->CompilerVersion != CompilerVersion)
		{
			CompiledDataEntries[DataID.Value].AccumulatedFlags |= EMovieSceneSequenceFlags::Volatile;
			return;
		}

		if (EnumHasAnyFlags(CompiledData->AllocatedMask, EMovieSceneSequenceCompilerMask::Hierarchy))
		{
			Hierarchies.Add(DataID.Value, MoveTemp(CompiledData->Hierarchy));
		}
		if (EnumHasAnyFlags(CompiledData->AllocatedMask, EMovieSceneSequenceCompilerMask::EvaluationTemplate))
		{
			TrackTemplates.Add(DataID.Value, MoveTemp(CompiledData->EvaluationTemplate));
		}
		if (EnumHasAnyFlags(CompiledData->AllocatedMask, EMovieSceneSequenceCompilerMask::EvaluationTemplateField))
		{
			TrackTemplateFields.Add(DataID.Value, MoveTemp(CompiledData->TrackTemplateField));
		}
		if (EnumHasAnyFlags(CompiledData->AllocatedMask, EMovieSceneSequenceCompilerMask::EntityComponentField))
		{
			EntityComponentFields.Add(DataID.Value, MoveTemp(CompiledData->EntityComponentField));
		}

		FMovieSceneCompiledDataEntry* EntryPtr = GetEntryPtr(DataID);

		EntryPtr->DeterminismFences = MoveTemp(CompiledData->DeterminismFences);
		EntryPtr->CompiledSignature = CompiledData->CompiledSignature;
		EntryPtr->AccumulatedMask = CompiledData->AccumulatedMask;
		EntryPtr->AccumulatedFlags = CompiledData->AccumulatedFlags;
		EntryPtr->CompiledFlags = CompiledData->CompiledFlags;

		++ReallocationVersion;
	}
	else
	{
		Reset(Sequence);
	}
}

bool UMovieSceneCompiledDataManager::CanMarkSignedObjectAsChangedDuringCook(UMovieSceneSequence* Sequence) const
{
	const FMovieSceneCompiledDataID DataID = FindDataID(Sequence);
	if (!DataID.IsValid())
	{
		// No data ID has been created, so this sequence hasn't been compiled yet.
		// We're OK to modify it.
		return true;
	}

	const FMovieSceneCompiledDataEntry* EntryPtr = GetEntryPtr(DataID);

	// If the compiled signature is set, we have already compiled the sequence. In that
	// case, it's not OK to modify data anymore.
	return !EntryPtr->CompiledSignature.IsValid();
}

void UMovieSceneCompiledDataManager::SetEmulatedNetworkMask(EMovieSceneServerClientMask NewMask)
{
	DestroyAllData();
	NetworkMask = NewMask;
}

void UMovieSceneCompiledDataManager::Reset(UMovieSceneSequence* Sequence)
{
	// Care is taken here not to use GetDataID which _creates_ a new data ID if
	// one is not available. This ensures that calling Reset() does not create
	// new data for sequences that have not yet been encountered
	FMovieSceneCompiledDataID DataID = SequenceToDataIDs.FindRef(Sequence);
	if (DataID.IsValid())
	{
		DestroyData(DataID);
		SequenceToDataIDs.Remove(Sequence);
	}
}

FMovieSceneCompiledDataID UMovieSceneCompiledDataManager::FindDataID(UMovieSceneSequence* Sequence) const
{
	return SequenceToDataIDs.FindRef(Sequence);
}

FMovieSceneCompiledDataID UMovieSceneCompiledDataManager::GetDataID(UMovieSceneSequence* Sequence)
{
	check(Sequence);

	FMovieSceneCompiledDataID ExistingDataID = FindDataID(Sequence);
	if (ExistingDataID.IsValid())
	{
		return ExistingDataID;
	}

	const int32 Index = CompiledDataEntries.Add(FMovieSceneCompiledDataEntry());

	ExistingDataID = FMovieSceneCompiledDataID { Index };
	FMovieSceneCompiledDataEntry& NewEntry = CompiledDataEntries[Index];

	NewEntry.SequenceKey = Sequence;
	NewEntry.DataID = ExistingDataID;
	NewEntry.AccumulatedFlags = Sequence->GetFlags();

	SequenceToDataIDs.Add(Sequence, ExistingDataID);
	return ExistingDataID;
}

FMovieSceneCompiledDataID UMovieSceneCompiledDataManager::GetSubDataID(FMovieSceneCompiledDataID DataID, FMovieSceneSequenceID SubSequenceID)
{
	if (SubSequenceID == MovieSceneSequenceID::Root)
	{
		return DataID;
	}

	const FMovieSceneSequenceHierarchy* Hierarchy = FindHierarchy(DataID);
	if (Hierarchy)
	{
		const FMovieSceneSubSequenceData* SubData     = Hierarchy->FindSubData(SubSequenceID);
		UMovieSceneSequence*              SubSequence = SubData ? SubData->GetSequence() : nullptr;

		if (SubSequence)
		{
			return GetDataID(SubSequence);
		}
	}

	return FMovieSceneCompiledDataID();
}


#if WITH_EDITOR

UMovieSceneCompiledDataManager* UMovieSceneCompiledDataManager::GetPrecompiledData(EMovieSceneServerClientMask EmulatedMask)
{
	ensureMsgf(!GExitPurge, TEXT("Attempting to access precompiled data manager during shutdown - this is undefined behavior since the manager may have already been destroyed, or could be unconstrictible"));

	if (EmulatedMask == EMovieSceneServerClientMask::Client)
	{
		static UMovieSceneCompiledDataManager* GEmulatedClientDataManager = NewObject<UMovieSceneCompiledDataManager>(GetTransientPackage(), "EmulatedClientDataManager", RF_MarkAsRootSet);
		GEmulatedClientDataManager->NetworkMask = EMovieSceneServerClientMask::Client;
		return GEmulatedClientDataManager;
	}

	if (EmulatedMask == EMovieSceneServerClientMask::Server)
	{
		static UMovieSceneCompiledDataManager* GEmulatedServerDataManager = NewObject<UMovieSceneCompiledDataManager>(GetTransientPackage(), "EmulatedServerDataManager", RF_MarkAsRootSet);
		GEmulatedServerDataManager->NetworkMask = EMovieSceneServerClientMask::Server;
		return GEmulatedServerDataManager;
	}

	static UMovieSceneCompiledDataManager* GPrecompiledDataManager = NewObject<UMovieSceneCompiledDataManager>(GetTransientPackage(), "PrecompiledDataManager", RF_MarkAsRootSet);
	return GPrecompiledDataManager;
}

#else // WITH_EDITOR

UMovieSceneCompiledDataManager* UMovieSceneCompiledDataManager::GetPrecompiledData()
{
	ensureMsgf(!GExitPurge, TEXT("Attempting to access precompiled data manager during shutdown - this is undefined behavior since the manager may have already been destroyed, or could be unconstrictible"));

	static UMovieSceneCompiledDataManager* GPrecompiledDataManager = NewObject<UMovieSceneCompiledDataManager>(GetTransientPackage(), "PrecompiledDataManager", RF_MarkAsRootSet);
	return GPrecompiledDataManager;
}

#endif // WITH_EDITOR

void UMovieSceneCompiledDataManager::DestroyData(FMovieSceneCompiledDataID DataID)
{
	check(DataID.IsValid() && CompiledDataEntries.IsValidIndex(DataID.Value));

	Hierarchies.Remove(DataID.Value);
	TrackTemplates.Remove(DataID.Value);
	TrackTemplateFields.Remove(DataID.Value);
	EntityComponentFields.Remove(DataID.Value);

	CompiledDataEntries.RemoveAt(DataID.Value);
}

void UMovieSceneCompiledDataManager::DestroyTemplate(FMovieSceneCompiledDataID DataID)
{
	check(DataID.IsValid() && CompiledDataEntries.IsValidIndex(DataID.Value));

	// Remove the lookup entry for this sequence/network mask combination
	const FMovieSceneCompiledDataEntry& Entry = CompiledDataEntries[DataID.Value];
	SequenceToDataIDs.Remove(Entry.SequenceKey);

	DestroyData(DataID);
}

bool UMovieSceneCompiledDataManager::IsDirty(const FMovieSceneCompiledDataEntry& Entry) const
{
	if (!Entry.GetSequence())
	{
		return false;
	}

	if (Entry.CompiledSignature != Entry.GetSequence()->GetSignature())
	{
		return true;
	}

	if (const FMovieSceneSequenceHierarchy* Hierarchy = FindHierarchy(Entry.DataID))
	{
		for (const TTuple<FMovieSceneSequenceID, FMovieSceneSubSequenceData>& Pair : Hierarchy->AllSubSequenceData())
		{
			if (UMovieSceneSequence* SubSequence = Pair.Value.GetSequence())
			{
				FMovieSceneCompiledDataID SubDataID = FindDataID(SubSequence);
				if (!SubDataID.IsValid() || CompiledDataEntries[SubDataID.Value].CompiledSignature != SubSequence->GetSignature())
				{
					return true;
				}
			}
			else
			{
				return true;
			}
		}
	}

	return false;
}

bool UMovieSceneCompiledDataManager::IsDirty(FMovieSceneCompiledDataID CompiledDataID) const
{
	check(CompiledDataID.IsValid() && CompiledDataEntries.IsValidIndex(CompiledDataID.Value));
	return IsDirty(CompiledDataEntries[CompiledDataID.Value]);
}

bool UMovieSceneCompiledDataManager::IsDirty(UMovieSceneSequence* Sequence) const
{
	FMovieSceneCompiledDataID ExistingDataID = FindDataID(Sequence);
	if (ExistingDataID.IsValid())
	{
		check(CompiledDataEntries.IsValidIndex(ExistingDataID.Value));
		FMovieSceneCompiledDataEntry Entry = CompiledDataEntries[ExistingDataID.Value];
		return IsDirty(Entry);
	}

	return true;
}

bool UMovieSceneCompiledDataManager::ValidateEntry(FMovieSceneCompiledDataID DataID, UMovieSceneSequence* Sequence) const
{
	if (!ensureMsgf(
			CompiledDataEntries.IsValidIndex(DataID.Value),
			TEXT("Given DataID %d is not valid! (%d entries in the data manager)"), DataID.Value, CompiledDataEntries.Num()))
	{
		return false;
	}

	const FMovieSceneCompiledDataEntry& Entry = CompiledDataEntries[DataID.Value];
	UMovieSceneSequence* EntrySequence = Entry.GetSequence();
	if (!ensureMsgf(
			EntrySequence == Sequence,
			TEXT("Unexpected sequence for data ID! Expected '%s', but data manager has '%s'."), *GetNameSafe(Sequence), *GetNameSafe(EntrySequence)))
	{
		return false;
	}

	return true;
}

void UMovieSceneCompiledDataManager::Compile(FMovieSceneCompiledDataID DataID)
{
	Compile(DataID, NetworkMask);
}


void UMovieSceneCompiledDataManager::Compile(FMovieSceneCompiledDataID DataID, EMovieSceneServerClientMask InNetworkMask)
{
	check(DataID.IsValid() && CompiledDataEntries.IsValidIndex(DataID.Value));
	UMovieSceneSequence* Sequence = CompiledDataEntries[DataID.Value].GetSequence();
	check(Sequence);
	Compile(DataID, Sequence, InNetworkMask);
}

FMovieSceneCompiledDataID UMovieSceneCompiledDataManager::Compile(UMovieSceneSequence* Sequence)
{
	FMovieSceneCompiledDataID DataID = GetDataID(Sequence);
	Compile(DataID, Sequence);
	return DataID;
}

void UMovieSceneCompiledDataManager::Compile(FMovieSceneCompiledDataID DataID, UMovieSceneSequence* Sequence)
{
	Compile(DataID, Sequence, NetworkMask);
}

void UMovieSceneCompiledDataManager::Compile(FMovieSceneCompiledDataID DataID, UMovieSceneSequence* Sequence, EMovieSceneServerClientMask InNetworkMask)
{
	check(DataID.IsValid() && CompiledDataEntries.IsValidIndex(DataID.Value));
	FMovieSceneCompiledDataEntry Entry = CompiledDataEntries[DataID.Value];
	if (!IsDirty(Entry))
	{
		return;
	}

	FMovieSceneGatheredCompilerData GatheredData;
	FTrackGatherParameters Params(this);

	Entry.DeterminismFences.Empty();
	Entry.AccumulatedFlags = Sequence->GetFlags();
	Params.TemplateGenerator.Reset(&Entry);
	Params.NetworkMask = InNetworkMask;

	// Clear list of generated conditions
	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (ensure(MovieScene))
	{
		for (TObjectPtr<UObject> DecorationObject : MovieScene->GetDecorations())
		{
			if (IMovieSceneDecoration* Decoration = Cast<IMovieSceneDecoration>(DecorationObject))
			{
				Decoration->OnPreDecorationCompiled();
			}
		}

		MovieScene->ResetGeneratedConditions();
	}

	// ---------------------------------------------------------------------------------------------------
	// Step 1 - Always ensure the hierarchy information is completely up to date first
	FMovieSceneSequenceHierarchy NewHierarchy;
	const bool bHasHierarchy = CompileHierarchy(Sequence, Params, &NewHierarchy);

	// If the network mask of the compiled data manager is 'all', but the sequence has been created with client-only and/or server-only subsections,
	// then we mark the sequence volatile as we may need to recompile it at runtime in order to exclude these subsections depending on the net mode at runtime.
	if (Params.NetworkMask == EMovieSceneServerClientMask::All && NewHierarchy.GetAccumulatedNetworkMask() != EMovieSceneServerClientMask::All)
	{
		Entry.AccumulatedFlags |= EMovieSceneSequenceFlags::Volatile;
	}

	if (IMovieSceneDeterminismSource* DeterminismSource = Cast<IMovieSceneDeterminismSource>(Sequence))
	{
		DeterminismSource->PopulateDeterminismData(GatheredData.DeterminismData, TRange<FFrameNumber>::All());
	}

	TSet<FGuid> GatheredSignatures;

	{
		if (ensure(MovieScene))
		{
			// Ensure we have no null tracks. This should really not be the case, but we have crash reporter cases of this happening
			ensure(!MovieScene->RemoveNullTracks());

			for (const FMovieSceneMarkedFrame& Mark : MovieScene->GetMarkedFrames())
			{
				if (Mark.bIsDeterminismFence)
				{
					GatheredData.DeterminismData.Fences.Emplace(Mark.FrameNumber, Mark.bIsInclusiveTime);
				}
			}

			if (UMovieSceneTrack* Track = MovieScene->GetCameraCutTrack())
			{
				CompileTrack(&Entry, nullptr, Track, Params, &GatheredSignatures, &GatheredData);
			}

			for (UMovieSceneTrack* Track : MovieScene->GetTracks())
			{	
				CompileTrack(&Entry, nullptr, Track, Params, &GatheredSignatures, &GatheredData);
			}

			for (const FMovieSceneBinding& ObjectBinding : ((const UMovieScene*)MovieScene)->GetBindings())
			{
				for (UMovieSceneTrack* Track : ObjectBinding.GetTracks())
				{	
					CompileTrack(&Entry, &ObjectBinding, Track, Params, &GatheredSignatures, &GatheredData);
				}
			}
		}
	}

	// ---------------------------------------------------------------------------------------------------
	// Step 2 - Gather compilation data
	FMovieSceneEntityComponentField ThisSequenceEntityField;

	{
		GatheredData.EntityField = &ThisSequenceEntityField;
		Gather(Entry, Sequence, Params, &GatheredData);
		GatheredData.EntityField = nullptr;
	}

	// ---------------------------------------------------------------------------------------------------
	// Step 3 - Assign entity field from data gathered for _this sequence only_
	if (ThisSequenceEntityField.IsEmpty())
	{
		EntityComponentFields.Remove(DataID.Value);
	}
	else
	{
		// EntityComponent data is not flattened so we assign that now after the initial gather
		EntityComponentFields.FindOrAdd(DataID.Value) = MoveTemp(ThisSequenceEntityField);
		GatheredData.AccumulatedMask |= EMovieSceneSequenceCompilerMask::EntityComponentField;
	}

	// ---------------------------------------------------------------------------------------------------
	// Step 4 - If we have a hierarchy, perform a gather for sub sequences
	if (bHasHierarchy)
	{
		CompileSubSequences(NewHierarchy, Params, &GatheredData);
		Entry.AccumulatedFlags |= GatheredData.InheritedFlags;
		Entry.AccumulatedMask |= GatheredData.AccumulatedMask;
	}

	// ---------------------------------------------------------------------------------------------------
	// Step 5 - Consolidate track template data from gathered data
	if (FMovieSceneEvaluationTemplate* TrackTemplate = TrackTemplates.Find(Entry.DataID.Value))
	{
		TrackTemplate->RemoveStaleData(GatheredSignatures);
	}

	CompileTrackTemplateField(&Entry, NewHierarchy, &GatheredData);

	// ---------------------------------------------------------------------------------------------------
	// Step 6 - Reassign or remove the new hierarchy
	if (bHasHierarchy)
	{
		Hierarchies.FindOrAdd(DataID.Value) = MoveTemp(NewHierarchy);
	}
	else
	{
		Hierarchies.Remove(DataID.Value);
	}

	// ---------------------------------------------------------------------------------------------------
	// Step 7: Apply the final state to the entry
	Entry.CompiledFlags.bParentSequenceRequiresLowerFence = GatheredData.DeterminismData.bParentSequenceRequiresLowerFence;
	Entry.CompiledFlags.bParentSequenceRequiresUpperFence = GatheredData.DeterminismData.bParentSequenceRequiresUpperFence;
	Entry.CompiledSignature = Sequence->GetSignature();
	Entry.AccumulatedMask = GatheredData.AccumulatedMask;
	Entry.DeterminismFences = MoveTemp(GatheredData.DeterminismData.Fences);
	if (Entry.DeterminismFences.Num())
	{
		Algo::SortBy(Entry.DeterminismFences, &FMovieSceneDeterminismFence::FrameNumber);
		const int32 NewNum = Algo::Unique(Entry.DeterminismFences);
		if (NewNum != Entry.DeterminismFences.Num())
		{
			Entry.DeterminismFences.SetNum(NewNum);
		}
	}

	CompiledDataEntries[DataID.Value] = Entry;
	++ReallocationVersion;

	for (TObjectPtr<UObject> DecorationObject : Sequence->GetMovieScene()->GetDecorations())
	{
		if (IMovieSceneDecoration* Decoration = Cast<IMovieSceneDecoration>(DecorationObject))
		{
			Decoration->OnPostDecorationCompiled();
		}
	}

#if 0
#if !NO_LOGGING
	if (bHasHierarchy)
	{
		FMovieSceneSequenceHierarchy* HierarchyToLog = Hierarchies.Find(DataID.Value);
		if (ensure(HierarchyToLog))
		{
			UE_LOGF(LogMovieScene, Log, "Newly compiled sequence hierarchy:");
			HierarchyToLog->LogHierarchy();
			HierarchyToLog->LogSubSequenceTree();
		}
	}
	else
	{
		UE_LOGF(LogMovieScene, Log, "No sequence hierarchy");
}
#endif
#endif
}


void UMovieSceneCompiledDataManager::Gather(const FMovieSceneCompiledDataEntry& Entry, UMovieSceneSequence* Sequence, const FTrackGatherParameters& Params, FMovieSceneGatheredCompilerData* OutCompilerData) const
{
	const FMovieSceneEvaluationTemplate* TrackTemplate = FindTrackTemplate(Entry.DataID);

	UMovieScene* MovieScene = Sequence->GetMovieScene();

	if (ensure(MovieScene))
	{
		// Allow decorations on the movie scene to define entities in the field
		if (OutCompilerData->EntityField)
		{
			FMovieSceneEntityComponentFieldBuilder FieldBuilder(OutCompilerData->EntityField);
			for (TObjectPtr<UObject> DecorationObject : MovieScene->GetDecorations())
			{
				if (IMovieSceneEntityProvider* Provider = Cast<IMovieSceneEntityProvider>(DecorationObject))
				{
					FMovieSceneEvaluationFieldEntityMetaData MetaData;
					Provider->PopulateEvaluationField(TRange<FFrameNumber>::All(), MetaData, &FieldBuilder);
				}
			}
		}

		if (UMovieSceneTrack* Track = MovieScene->GetCameraCutTrack())
		{
			GatherTrack(nullptr, Track, Params, TrackTemplate, OutCompilerData);
		}

		for (UMovieSceneTrack* Track : MovieScene->GetTracks())
		{
			GatherTrack(nullptr, Track, Params, TrackTemplate, OutCompilerData);
		}

		for (const FMovieSceneBinding& ObjectBinding : ((const UMovieScene*)MovieScene)->GetBindings())
		{
			for (UMovieSceneTrack* Track : ObjectBinding.GetTracks())
			{
				GatherTrack(&ObjectBinding, Track, Params, TrackTemplate, OutCompilerData);
			}
		}
	}
}

void UMovieSceneCompiledDataManager::CompileSubSequences(const FMovieSceneSequenceHierarchy& Hierarchy, const FTrackGatherParameters& Params, FMovieSceneGatheredCompilerData* OutCompilerData)
{
	using namespace UE::MovieScene;

	OutCompilerData->AccumulatedMask |= EMovieSceneSequenceCompilerMask::Hierarchy;

	// Ensure all sub sequences are compiled
	for (const TTuple<FMovieSceneSequenceID, FMovieSceneSubSequenceData>& Pair : Hierarchy.AllSubSequenceData())
	{
		if (UMovieSceneSequence* SubSequence = Pair.Value.GetSequence())
		{
			Compile(SubSequence);
		}
	}

	const TMovieSceneEvaluationTree<FMovieSceneSubSequenceTreeEntry>& SubSequenceTree = Hierarchy.GetTree();

	// When adding determinism fences for sub sequences, we track the iteration index for each sequence ID so that
	// we only add a fence when the sub sequence truly ends or begins, not for every segmentation of the sub sequence tree
	struct FSubSequenceItMetaData
	{
		int32 LastIterIndex = INDEX_NONE;
		TOptional<FFrameNumber> TrailingFence;

	};
	TSortedMap<FMovieSceneSequenceID, FSubSequenceItMetaData> ItMetaData;

	// Start iterating the field from the lower bound of the compile range
	FMovieSceneEvaluationTreeRangeIterator SubSequenceIt = SubSequenceTree.IterateFromLowerBound(Params.RootClampRange.GetLowerBound());
	for ( int32 ItIndex = 0; SubSequenceIt && SubSequenceIt.Range().Overlaps(Params.RootClampRange); ++SubSequenceIt, ++ItIndex)
	{
		// Iterate all sub sequences in the current range
		for (const FMovieSceneSubSequenceTreeEntry& SubSequenceEntry : SubSequenceTree.GetAllData(SubSequenceIt.Node()))
		{
			FMovieSceneSequenceID SubSequenceID = SubSequenceEntry.SequenceID;

			const FMovieSceneSubSequenceData* SubData = Hierarchy.FindSubData(SubSequenceID);
			checkf(SubData, TEXT("Sub data could not be found for a sequence that exists in the sub sequence tree - this indicates an error while populating the sub sequence hierarchy tree."));

			UMovieSceneSequence* SubSequence = SubData->GetSequence();
			if (SubSequence)
			{
				FTrackGatherParameters SubSectionGatherParams = Params.CreateForSubData(*SubData, SubSequenceID);
				SubSectionGatherParams.Flags |= SubSequenceEntry.Flags;
				SubSectionGatherParams.SetClampRange(SubSequenceIt.Range());

				// Access the sub entry data after compilation
				FMovieSceneCompiledDataID SubDataID = GetDataID(SubSequence);
				check(SubDataID.IsValid());

				// Gather track template data for the sub sequence
				FMovieSceneCompiledDataEntry SubEntry = CompiledDataEntries[SubDataID.Value];
				if (TrackTemplates.Contains(SubDataID.Value))
				{
					Gather(SubEntry, SubSequence, SubSectionGatherParams, OutCompilerData);
				}

				// Inherit flags from sub sequences (if a sub sequence is volatile, so must this be)
				OutCompilerData->InheritedFlags |= (CompiledDataEntries[SubDataID.Value].AccumulatedFlags & EMovieSceneSequenceFlags::InheritedFlags);
				OutCompilerData->AccumulatedMask |= SubEntry.AccumulatedMask;

				FSubSequenceItMetaData* MetaData = &ItMetaData.FindOrAdd(SubSequenceID);

				const bool bWasEvaluatedLastFrame = MetaData->LastIterIndex != INDEX_NONE && MetaData->LastIterIndex == ItIndex-1;
				if (SubEntry.CompiledFlags.bParentSequenceRequiresLowerFence && bWasEvaluatedLastFrame == false)
				{
					OutCompilerData->DeterminismData.Fences.Add(DiscreteInclusiveLower(SubSequenceIt.Range()));
				}
				if (SubEntry.CompiledFlags.bParentSequenceRequiresUpperFence)
				{
					MetaData->TrailingFence = DiscreteExclusiveUpper(SubSequenceIt.Range());
				}

				// Add determinism fences for boundary conditions
				if (!SubData->OuterToInnerTransform.IsLinear() && (SubEntry.CompiledFlags.bParentSequenceRequiresUpperFence || SubEntry.CompiledFlags.bParentSequenceRequiresLowerFence) )
				{
					SubData->OuterToInnerTransform.ExtractBoundariesWithinRange(SubSequenceIt.Range().GetLowerBoundValue(), SubSequenceIt.Range().GetUpperBoundValue(), [OutCompilerData](FFrameTime FrameTime)
					{
						OutCompilerData->DeterminismData.Fences.Add(FrameTime.FrameNumber);
						return true;
					});
				}

				MetaData->LastIterIndex = ItIndex;
			}
		}

		for (TPair<FMovieSceneSequenceID, FSubSequenceItMetaData>& Pair : ItMetaData)
		{
			if (Pair.Value.LastIterIndex == ItIndex-1 && Pair.Value.TrailingFence.IsSet())
			{
				OutCompilerData->DeterminismData.Fences.Add(Pair.Value.TrailingFence.GetValue());
				Pair.Value.TrailingFence.Reset();
			}
		}
	}
}


void UMovieSceneCompiledDataManager::CompileTrackTemplateField(FMovieSceneCompiledDataEntry* OutEntry, const FMovieSceneSequenceHierarchy& Hierarchy, FMovieSceneGatheredCompilerData* InCompilerData)
{
	if (!EnumHasAnyFlags(InCompilerData->AccumulatedMask, EMovieSceneSequenceCompilerMask::EvaluationTemplate))
	{
		TrackTemplateFields.Remove(OutEntry->DataID.Value);
		return;
	}


	FMovieSceneEvaluationField* TrackTemplateField = &TrackTemplateFields.FindOrAdd(OutEntry->DataID.Value);

	// Wipe the current evaluation field for the template
	*TrackTemplateField = FMovieSceneEvaluationField();

	InCompilerData->AccumulatedMask |= EMovieSceneSequenceCompilerMask::EvaluationTemplateField;

	TArray<FCompileOnTheFlyData> CompileData;
	for (FMovieSceneEvaluationTreeRangeIterator It(InCompilerData->TrackTemplates); It; ++It)
	{
		CompileData.Reset();

		TRange<FFrameNumber> FieldRange = It.Range();
		for (const FCompileOnTheFlyData& TrackData : InCompilerData->TrackTemplates.GetAllData(It.Node()))
		{
			CompileData.Add(TrackData);
		}

		// Sort the compilation data based on (in order):
		//  1. Group
		//  2. Hierarchical bias
		//  3. Evaluation priority
		CompileData.Sort(SortPredicate);

		// Generate the evaluation group by gathering initialization and evaluation ptrs for each unique group
		FMovieSceneEvaluationGroup EvaluationGroup;
		PopulateEvaluationGroup(CompileData, &EvaluationGroup);

		// Compute meta data for this segment
		TMovieSceneEvaluationTreeDataIterator<FMovieSceneSubSequenceTreeEntry> SubSequences = Hierarchy.GetTree().GetAllData(Hierarchy.GetTree().IterateFromLowerBound(FieldRange.GetLowerBound()).Node());

		FMovieSceneEvaluationMetaData MetaData;
		PopulateMetaData(Hierarchy, CompileData, SubSequences, &MetaData);

		TrackTemplateField->Add(FieldRange, MoveTemp(EvaluationGroup), MoveTemp(MetaData));
	}
}


void UMovieSceneCompiledDataManager::PopulateEvaluationGroup(const TArray<FCompileOnTheFlyData>& SortedCompileData, FMovieSceneEvaluationGroup* OutGroup)
{
	check(OutGroup);
	if (SortedCompileData.Num() == 0)
	{
		return;
	}

	static TArray<FMovieSceneFieldEntry_EvaluationTrack> InitTrackLUT;
	static TArray<FMovieSceneFieldEntry_ChildTemplate>   InitSectionLUT;

	static TArray<FMovieSceneFieldEntry_EvaluationTrack> EvalTrackLUT;
	static TArray<FMovieSceneFieldEntry_ChildTemplate>   EvalSectionLUT;

	InitTrackLUT.Reset();
	InitSectionLUT.Reset();
	EvalTrackLUT.Reset();
	EvalSectionLUT.Reset();

	// Now iterate the tracks and insert indices for initialization and evaluation
	FName LastEvaluationGroup = SortedCompileData[0].EvaluationGroup;

	int32 Index = 0;
	while (Index < SortedCompileData.Num())
	{
		const FCompileOnTheFlyData& Data = SortedCompileData[Index];

		// Check for different evaluation groups
		if (Data.EvaluationGroup != LastEvaluationGroup)
		{
			// If we're now in a different flush group, add the ptrs to the group
			AddPtrsToGroup(OutGroup, InitTrackLUT, InitSectionLUT, EvalTrackLUT, EvalSectionLUT);
		}
		LastEvaluationGroup = Data.EvaluationGroup;

		// Add all subsequent entries that relate to the same track
		FMovieSceneEvaluationFieldTrackPtr MatchTrack = Data.Track;

		uint16 NumChildren = 0;
		for ( ; Index < SortedCompileData.Num() && SortedCompileData[Index].Track == MatchTrack; ++Index)
		{
			if (SortedCompileData[Index].Child.ChildIndex != uint16(-1))
			{
				++NumChildren;
				// If this track requires initialization, add it to the init array
				if (Data.bRequiresInit)
				{
					InitSectionLUT.Add(SortedCompileData[Index].Child);
				}
				EvalSectionLUT.Add(SortedCompileData[Index].Child);
			}
		}

		FMovieSceneFieldEntry_EvaluationTrack Entry{ Data.Track, NumChildren };
		if (Data.bRequiresInit)
		{
			InitTrackLUT.Add(Entry);
		}
		EvalTrackLUT.Add(Entry);
	}

	AddPtrsToGroup(OutGroup, InitTrackLUT, InitSectionLUT, EvalTrackLUT, EvalSectionLUT);
}


void UMovieSceneCompiledDataManager::PopulateMetaData(const FMovieSceneSequenceHierarchy& RootHierarchy, const TArray<FCompileOnTheFlyData>& SortedCompileData, TMovieSceneEvaluationTreeDataIterator<FMovieSceneSubSequenceTreeEntry> SubSequences, FMovieSceneEvaluationMetaData* OutMetaData)
{
	check(OutMetaData);
	OutMetaData->Reset();

	uint16 SetupIndex    = 0;
	uint16 TearDownIndex = 0;
	for (const FCompileOnTheFlyData& CompileData : SortedCompileData)
	{
		if (CompileData.bRequiresInit)
		{
			uint32 ChildIndex = CompileData.Child.ChildIndex == uint16(-1) ? uint32(-1) : CompileData.Child.ChildIndex;

			FMovieSceneEvaluationKey TrackKey(CompileData.Track.SequenceID, CompileData.Track.TrackIdentifier, ChildIndex);
			OutMetaData->ActiveEntities.Add(FMovieSceneOrderedEvaluationKey{ TrackKey, SetupIndex++, (CompileData.bPriorityTearDown ? TearDownIndex : uint16(MAX_uint16-TearDownIndex)) });
			++TearDownIndex;
		}
	}

	// Then all the eval tracks
	for (const FCompileOnTheFlyData& CompileData : SortedCompileData)
	{
		if (!CompileData.bRequiresInit)
		{
			uint32 ChildIndex = CompileData.Child.ChildIndex == uint16(-1) ? uint32(-1) : CompileData.Child.ChildIndex;

			FMovieSceneEvaluationKey TrackKey(CompileData.Track.SequenceID, CompileData.Track.TrackIdentifier, ChildIndex);
			OutMetaData->ActiveEntities.Add(FMovieSceneOrderedEvaluationKey{ TrackKey, SetupIndex++, (CompileData.bPriorityTearDown ? TearDownIndex : uint16(MAX_uint16-TearDownIndex)) });
			++TearDownIndex;
		}
	}

	Algo::SortBy(OutMetaData->ActiveEntities, &FMovieSceneOrderedEvaluationKey::Key);

	{
		OutMetaData->ActiveSequences.Reset();
		OutMetaData->ActiveSequences.Add(MovieSceneSequenceID::Root);

		for (const FMovieSceneSubSequenceTreeEntry& SubSequenceEntry : SubSequences)
		{
			OutMetaData->ActiveSequences.Add(SubSequenceEntry.SequenceID);
		}

		OutMetaData->ActiveSequences.Sort();
	}
}


void UMovieSceneCompiledDataManager::CompileTrack(FMovieSceneCompiledDataEntry* OutEntry, const FMovieSceneBinding* ObjectBinding, UMovieSceneTrack* Track, const FTrackGatherParameters& Params, TSet<FGuid>* OutCompiledSignatures, FMovieSceneGatheredCompilerData* OutCompilerData)
{
	using namespace UE::MovieScene;

	check(Track);
	check(OutCompiledSignatures);

	const bool bTrackMatchesFlags = ( Params.Flags == ESectionEvaluationFlags::None )
		|| ( EnumHasAnyFlags(Params.Flags, ESectionEvaluationFlags::PreRoll)  && Track->EvalOptions.bEvaluateInPreroll  )
		|| ( EnumHasAnyFlags(Params.Flags, ESectionEvaluationFlags::PostRoll) && Track->EvalOptions.bEvaluateInPostroll );

	if (!bTrackMatchesFlags)
	{
		return;
	}

	if (Track->IsEvalDisabled())
	{
		return;
	}

	UMovieSceneSequence* Sequence = OutEntry->GetSequence();
	check(Sequence);

	// -------------------------------------------------------------------------------------------------------------------------------------
	// Step 1 - ensure that track templates exist for any track that implements IMovieSceneTrackTemplateProducer
	FMovieSceneTrackIdentifier TrackIdentifier;
	FMovieSceneEvaluationTemplate* TrackTemplate = nullptr;
	if (const IMovieSceneTrackTemplateProducer* TrackTemplateProducer = Cast<const IMovieSceneTrackTemplateProducer>(Track))
	{
		TrackTemplate = &TrackTemplates.FindOrAdd(OutEntry->DataID.Value);

		check(TrackTemplate);

		TrackIdentifier = TrackTemplate->GetLedger().FindTrackIdentifier(Track->GetSignature());

		if (!TrackIdentifier)
		{
			// If the track doesn't exist - we need to generate it from scratch
			FMovieSceneTrackCompilerArgs Args(Track, &Params.TemplateGenerator);
			if (ObjectBinding)
			{
				Args.ObjectBindingId = ObjectBinding->GetObjectGuid();
			}

			Args.DefaultCompletionMode = Sequence->DefaultCompletionMode;

			TrackTemplateProducer->GenerateTemplate(Args);

			TrackIdentifier = TrackTemplate->GetLedger().FindTrackIdentifier(Track->GetSignature());
		}

		if (TrackIdentifier)
		{
			OutCompiledSignatures->Add(Track->GetSignature());
		}

		OutCompilerData->AccumulatedMask |= EMovieSceneSequenceCompilerMask::EvaluationTemplate;
	}

	// -------------------------------------------------------------------------------------------------------------------------------------
	// Step 2 - let the track or its sections add determinism fences
	if (IMovieSceneDeterminismSource* DeterminismSource = Cast<IMovieSceneDeterminismSource>(Track))
	{
		DeterminismSource->PopulateDeterminismData(OutCompilerData->DeterminismData, TRange<FFrameNumber>::All());
	}

	const FMovieSceneTrackEvaluationField& EvaluationField = Track->GetEvaluationField();
	const EMovieSceneCompletionMode DefaultCompletionMode = Sequence->DefaultCompletionMode;
	const bool bAddKeepStateDeterminismFences = CVarAddKeepStateDeterminismFences.GetValueOnGameThread();
	for (const FMovieSceneTrackEvaluationFieldEntry& Entry : EvaluationField.Entries)
	{
		if (bAddKeepStateDeterminismFences && Entry.Section)
		{
			// If a section is KeepState, we need to make sure to evaluate it on its last frame so that the value that "sticks" is correct.
			const TRange<FFrameNumber> SectionRange = Entry.Section->GetRange();
			const EMovieSceneCompletionMode SectionCompletionMode = Entry.Section->GetCompletionMode();
			if (SectionRange.HasUpperBound() &&
					(SectionCompletionMode == EMovieSceneCompletionMode::KeepState ||
					 (SectionCompletionMode == EMovieSceneCompletionMode::ProjectDefault && DefaultCompletionMode == EMovieSceneCompletionMode::KeepState)))
			{
				// We simply use the end time of the section for the fence, regardless of whether it's inclusive or exclusive.
				// When exclusive, the ECS system will query entities just before that time, but still pass that time for
				// evaluation purposes, so we will get the correct evaluated values.
				const FFrameNumber FenceTime(SectionRange.GetUpperBoundValue());
				OutCompilerData->DeterminismData.Fences.Add(FenceTime);
			}
		}

		IMovieSceneDeterminismSource* DeterminismSource = Cast<IMovieSceneDeterminismSource>(Entry.Section);
		if (DeterminismSource)
		{
			DeterminismSource->PopulateDeterminismData(OutCompilerData->DeterminismData, Entry.Range);
		}
	}
}

void UMovieSceneCompiledDataManager::GatherTrack(const FMovieSceneBinding* ObjectBinding, UMovieSceneTrack* Track, const FTrackGatherParameters& Params, const FMovieSceneEvaluationTemplate* TrackTemplate, FMovieSceneGatheredCompilerData* OutCompilerData) const
{
	using namespace UE::MovieScene;

	check(Track);

	const bool bTrackMatchesFlags = ( Params.Flags == ESectionEvaluationFlags::None )
		|| ( EnumHasAnyFlags(Params.Flags, ESectionEvaluationFlags::PreRoll)  && Track->EvalOptions.bEvaluateInPreroll  )
		|| ( EnumHasAnyFlags(Params.Flags, ESectionEvaluationFlags::PostRoll) && Track->EvalOptions.bEvaluateInPostroll );

	if (!bTrackMatchesFlags)
	{
		return;
	}

	if (Track->IsEvalDisabled())
	{
		return;
	}

	// Some tracks could want to do some custom pre-compilation things.
	FMovieSceneTrackPreCompileResult PreCompileResult;
	Track->PreCompile(PreCompileResult);

	const FMovieSceneTrackEvaluationField& EvaluationField = Track->GetEvaluationField();

	// -------------------------------------------------------------------------------------------------------------------------------------
	// Step 1 - Handle any entity producers that exist within the field
	if (OutCompilerData->EntityField)
	{
		FMovieSceneEntityComponentFieldBuilder FieldBuilder(OutCompilerData->EntityField);

		if (ObjectBinding)
		{
			FieldBuilder.GetSharedMetaData().ObjectBindingID = ObjectBinding->GetObjectGuid();
		}

		for (UObject* Decoration : Track->GetActiveDecorations())
		{
			if (IMovieSceneEntityProvider* Provider = Cast<IMovieSceneEntityProvider>(Decoration))
			{
				FMovieSceneEvaluationFieldEntityMetaData MetaData(PreCompileResult.DefaultMetaData);
				MetaData.bEvaluateInSequencePreRoll  = Track->EvalOptions.bEvaluateInPreroll;
				MetaData.bEvaluateInSequencePostRoll = Track->EvalOptions.bEvaluateInPostroll;
				MetaData.Condition = Track->ConditionContainer.Condition;

				Provider->PopulateEvaluationField(Params.LocalClampRange, MetaData, &FieldBuilder);
			}

			// Handle decorations that provide sections for evaluation
			if (IMovieSceneSectionProviderDecoration* SectionProvider = Cast<IMovieSceneSectionProviderDecoration>(Decoration))
			{
				for (UMovieSceneSection* Section : SectionProvider->GetSections())
				{
					if (!Section || !Section->IsActive())
					{
						continue;
					}

					TRange<FFrameNumber> SectionRange = Section->GetRange();
					TRange<FFrameNumber> EffectiveRange = TRange<FFrameNumber>::Intersection(Params.LocalClampRange, SectionRange);
					if (EffectiveRange.IsEmpty())
					{
						continue;
					}

					FMovieSceneEvaluationFieldEntityMetaData MetaData(PreCompileResult.DefaultMetaData);
					MetaData.bEvaluateInSequencePreRoll  = Track->EvalOptions.bEvaluateInPreroll;
					MetaData.bEvaluateInSequencePostRoll = Track->EvalOptions.bEvaluateInPostroll;
					MetaData.Condition = Track->ConditionContainer.Condition;

					// If the section is an entity provider, let it populate itself
					IMovieSceneEntityProvider* SectionEntityProvider = Cast<IMovieSceneEntityProvider>(Section);
					if (SectionEntityProvider)
					{
						if (!SectionEntityProvider->PopulateEvaluationField(EffectiveRange, MetaData, &FieldBuilder))
						{
							const int32 EntityIndex   = FieldBuilder.FindOrAddEntity(Section, 0);
							const int32 MetaDataIndex = FieldBuilder.AddMetaData(MetaData);
							FieldBuilder.AddPersistentEntity(EffectiveRange, EntityIndex, MetaDataIndex);
						}
					}
					else
					{
						const int32 EntityIndex   = FieldBuilder.FindOrAddEntity(Section, 0);
						const int32 MetaDataIndex = FieldBuilder.AddMetaData(MetaData);
						FieldBuilder.AddPersistentEntity(EffectiveRange, EntityIndex, MetaDataIndex);
					}
				}
			}
		}

		IMovieSceneEntityProvider* TrackEntityProvider = Cast<IMovieSceneEntityProvider>(Track);

		// If the track is an entity provider, allow it to add entries first
		if (TrackEntityProvider)
		{
			FMovieSceneEvaluationFieldEntityMetaData MetaData(PreCompileResult.DefaultMetaData);
			MetaData.bEvaluateInSequencePreRoll  = Track->EvalOptions.bEvaluateInPreroll;
			MetaData.bEvaluateInSequencePostRoll = Track->EvalOptions.bEvaluateInPostroll;
			MetaData.Condition = Track->ConditionContainer.Condition;

			TrackEntityProvider->PopulateEvaluationField(Params.LocalClampRange, MetaData, &FieldBuilder);
		}
		else for (const FMovieSceneTrackEvaluationFieldEntry& Entry : EvaluationField.Entries)
		{
			if (Entry.Section && Track->IsRowEvalDisabled(Entry.Section->GetRowIndex()))
			{
				continue;
			}

#if WITH_EDITOR
			if (Entry.Section && Entry.Section->IsLocalEvalDisabled())
			{
				continue;
			}
#endif

			IMovieSceneEntityProvider* EntityProvider = Cast<IMovieSceneEntityProvider>(Entry.Section);
			if (!EntityProvider)
			{
				continue;
			}

			// This codepath should only ever execute for the highest level so we do not need to do any transformations
			TRange<FFrameNumber> EffectiveRange = TRange<FFrameNumber>::Intersection(Params.LocalClampRange, Entry.Range);
			if (!EffectiveRange.IsEmpty())
			{
				FMovieSceneEvaluationFieldEntityMetaData MetaData(PreCompileResult.DefaultMetaData);

				MetaData.ForcedTime = Entry.ForcedTime;
				MetaData.Flags      = Entry.Flags;
				MetaData.bEvaluateInSequencePreRoll  = Track->EvalOptions.bEvaluateInPreroll;
				MetaData.bEvaluateInSequencePostRoll = Track->EvalOptions.bEvaluateInPostroll;
				
				MetaData.Condition = MovieSceneHelpers::GetSequenceCondition(Track, Entry.Section, true);

				if (!EntityProvider->PopulateEvaluationField(EffectiveRange, MetaData, &FieldBuilder))
				{
					const int32 EntityIndex   = FieldBuilder.FindOrAddEntity(Entry.Section, 0);
					const int32 MetaDataIndex = FieldBuilder.AddMetaData(MetaData);

					FieldBuilder.AddPersistentEntity(EffectiveRange, EntityIndex, MetaDataIndex);
				}
			}
		}
	}

	// -------------------------------------------------------------------------------------------------------------------------------------
	// Step 2 - Handle the track being a template producer
	FMovieSceneTrackIdentifier TrackIdentifier = TrackTemplate ? TrackTemplate->GetLedger().FindTrackIdentifier(Track->GetSignature()) : FMovieSceneTrackIdentifier();
	if (TrackIdentifier)
	{
		// Iterate everything in the field
		for (const FMovieSceneTrackEvaluationFieldEntry& Entry : EvaluationField.Entries)
		{
			// Iterate all the valid ranges this translates to in the root
			FMovieSceneInverseSequenceTransform SequenceToRootTransform  = Params.RootToSequenceTransform.Inverse();

			auto VisitWarpedRootRange = [this, &Entry, &Params, &OutCompilerData, TrackIdentifier, TrackTemplate, Track](TRange<FFrameTime> InRange)
			{
				TRange<FFrameNumber> ClampedRangeRoot = Params.ClampRoot(ConvertToDiscreteRange(InRange));
				UMovieSceneSection*  Section          = Entry.Section;

				if (Section && Track->IsRowEvalDisabled(Section->GetRowIndex()))
				{
					return true;
				}

#if WITH_EDITOR
				if (Section && Section->IsLocalEvalDisabled())
				{
					return true;
				}
#endif

				if (ClampedRangeRoot.IsEmpty())
				{
					return true;
				}

				const FMovieSceneEvaluationTrack* EvaluationTrack = TrackTemplate->FindTrack(TrackIdentifier);
				check(EvaluationTrack);

				// Get the correct template for the sub sequence
				FCompileOnTheFlyData CompileData;

				CompileData.Track                   = FMovieSceneEvaluationFieldTrackPtr(Params.SequenceID, TrackIdentifier);
				CompileData.EvaluationPriority      = EvaluationTrack->GetEvaluationPriority();
				CompileData.EvaluationGroup         = EvaluationTrack->GetEvaluationGroup();
				CompileData.GroupEvaluationPriority = GetMovieSceneModule().GetEvaluationGroupParameters(CompileData.EvaluationGroup).EvaluationPriority;
				CompileData.HierarchicalBias        = Params.HierarchicalBias;
				CompileData.bPriorityTearDown       = EvaluationTrack->HasTearDownPriority();

				auto FindChildWithSection = [Section](const FMovieSceneEvalTemplatePtr& ChildTemplate)
				{
					return ChildTemplate.IsValid() && ChildTemplate->GetSourceSection() == Section;
				};

				const int32 ChildTemplateIndex = Section ? EvaluationTrack->GetChildTemplates().IndexOfByPredicate(FindChildWithSection) : INDEX_NONE;
				if (ChildTemplateIndex != INDEX_NONE)
				{
					check(ChildTemplateIndex >= 0 && ChildTemplateIndex < TNumericLimits<uint16>::Max());

					ESectionEvaluationFlags Flags = Params.Flags == ESectionEvaluationFlags::None ? Entry.Flags : Params.Flags;

					if (EnumHasAnyFlags(Params.AccumulatedFlags, EMovieSceneSubSectionFlags::OverrideRestoreState))
					{
						Flags |= ESectionEvaluationFlags::ForceRestoreState;
					}
					else if (EnumHasAnyFlags(Params.AccumulatedFlags, EMovieSceneSubSectionFlags::OverrideKeepState))
					{
						Flags |= ESectionEvaluationFlags::ForceKeepState;
					}

					CompileData.ChildPriority = Entry.LegacySortOrder;
					CompileData.Child         = FMovieSceneFieldEntry_ChildTemplate((uint16)ChildTemplateIndex, Flags, Entry.ForcedTime);
					CompileData.bRequiresInit = EvaluationTrack->GetChildTemplate(ChildTemplateIndex).RequiresInitialization();
				}
				else
				{
					CompileData.ChildPriority = 0;
					CompileData.Child         = FMovieSceneFieldEntry_ChildTemplate{};
					CompileData.bRequiresInit = false;
				}

				OutCompilerData->TrackTemplates.Add(ClampedRangeRoot, CompileData);
				return true;
			};

			Params.TransformLocalRange(Entry.Range, VisitWarpedRootRange);
		}
	}
}

bool UMovieSceneCompiledDataManager::CompileHierarchy(UMovieSceneSequence* Sequence, FMovieSceneSequenceHierarchy* InOutHierarchy, EMovieSceneServerClientMask InNetworkMask)
{
	FGatherParameters Params;
	Params.NetworkMask = InNetworkMask;
	return CompileHierarchy(Sequence, Params, InOutHierarchy);
}

bool UMovieSceneCompiledDataManager::CompileHierarchy(UMovieSceneSequence* Sequence, const FGatherParameters& Params, FMovieSceneSequenceHierarchy* InOutHierarchy)
{
	using namespace UE::MovieScene;

	UE::MovieScene::FSubSequencePath RootPath;

	const FGatherParameters* ParamsToUse = &Params;

	bool bContainsTimeWarp = false;

	if (Params.SequenceID == MovieSceneSequenceID::Root)
	{
		UMovieSceneTimeWarpDecoration* TimeWarp = Sequence->GetMovieScene()->FindDecoration<UMovieSceneTimeWarpDecoration>();
		if (TimeWarp)
		{
			FMovieSceneSequenceTransform TimeWarpTransform = TimeWarp->GenerateTransform();

			// Don't do anything for identity transforms
			if (!TimeWarpTransform.IsIdentity())
			{
				InOutHierarchy->SetRootTransform(FMovieSceneSequenceTransform(MoveTemp(TimeWarpTransform)));
				bContainsTimeWarp = true;
			}
		}
	}

	// Compile all the sub data for every part of the hierarchy
	const bool bContainsSubSequences = GenerateSubSequenceData(Sequence, *ParamsToUse, FMovieSceneEvaluationOperand(), &RootPath, InOutHierarchy);

	// Populate the sub sequence tree that defines which sub sequences happen at a given time
	PopulateSubSequenceTree(Sequence, *ParamsToUse, &RootPath, InOutHierarchy);

	return bContainsSubSequences || bContainsTimeWarp;
}

bool UMovieSceneCompiledDataManager::GenerateSubSequenceData(UMovieSceneSequence* SubSequence, const FGatherParameters& Params, const FMovieSceneEvaluationOperand& Operand, UE::MovieScene::FSubSequencePath* RootPath, FMovieSceneSequenceHierarchy* InOutHierarchy)
{
	using namespace UE::MovieScene;

	UMovieScene* MovieScene = SubSequence ? SubSequence->GetMovieScene() : nullptr;
	if (!MovieScene)
	{
		return false;
	}

	check(RootPath && InOutHierarchy);

	bool bContainsSubSequences = false;

	for (UMovieSceneTrack* Track : MovieScene->GetTracks())
	{
		if (UMovieSceneSubTrack* SubTrack = Cast<UMovieSceneSubTrack>(Track))
		{
			bContainsSubSequences |= GenerateSubSequenceData(SubTrack, Params, Operand, RootPath, InOutHierarchy);
		}
	}

	for (const FMovieSceneBinding& ObjectBinding : ((const UMovieScene*)MovieScene)->GetBindings())
	{
		for (UMovieSceneTrack* Track : ObjectBinding.GetTracks())
		{
			if (UMovieSceneSubTrack* SubTrack = Cast<UMovieSceneSubTrack>(Track))
			{
				const FMovieSceneEvaluationOperand ChildOperand(Params.SequenceID, ObjectBinding.GetObjectGuid());

				bContainsSubSequences |= GenerateSubSequenceData(SubTrack, Params, ChildOperand, RootPath, InOutHierarchy);
			}
		}
	}

	return bContainsSubSequences;
}

bool UMovieSceneCompiledDataManager::GenerateSubSequenceData(UMovieSceneSubTrack* SubTrack, const FGatherParameters& Params, const FMovieSceneEvaluationOperand& Operand, UE::MovieScene::FSubSequencePath* RootPath, FMovieSceneSequenceHierarchy* InOutHierarchy)
{
	using namespace UE::MovieScene;

	bool bContainsSubSequences = false;

	check(SubTrack && RootPath);

	const FMovieSceneSequenceID ParentSequenceID = Params.SequenceID;

	for (UMovieSceneSection* Section : SubTrack->GetAllSections())
	{
		if (SubTrack->IsRowEvalDisabled(Section->GetRowIndex()))
		{
			continue;
		}

		UMovieSceneSubSection* SubSection  = Cast<UMovieSceneSubSection>(Section);
		if (!SubSection)
		{
			continue;
		}

		// Note: we always compile FMovieSceneSubSequenceData for all entries of a hierarchy, even if excluded from the network mask
		// to ensure that hierarchical information is still available when emulating different network masks

		UMovieSceneSequence* SubSequence = SubSection->GetSequence();
		if (!SubSequence)
		{
			continue;
		}

		UMovieScene* MovieScene = SubSequence->GetMovieScene();
		if (!MovieScene)
		{
			continue;
		}

		const FMovieSceneSequenceID InnerSequenceID = RootPath->ResolveChildSequenceID(SubSection->GetSequenceID());

		FSubSequenceInstanceDataParams InstanceParams{ InnerSequenceID, Operand };
		FMovieSceneSubSequenceData     NewSubData = SubSection->GenerateSubSequenceData(InstanceParams);

		// LocalClampRange here is in SubTrack's space, so we need to multiply that by the OuterToInnerTransform 
		// (which is the same as RootToSequenceTransform here before we transform it)
		TRange<FFrameNumber> InnerClampRange = (Params.LocalClampRange.GetLowerBound().IsOpen() || Params.LocalClampRange.GetUpperBound().IsOpen())
			? Params.LocalClampRange
			: ConvertToDiscreteRange(NewSubData.OuterToInnerTransform.ComputeTraversedHull(Params.LocalClampRange));

		// Put the root play range in the new root space
		NewSubData.PlayRange               = TRange<FFrameNumber>::Intersection(InnerClampRange, NewSubData.PlayRange.Value);
		NewSubData.RootToSequenceTransform = NewSubData.RootToSequenceTransform * Params.RootToSequenceTransform;
#if WITH_EDITORONLY_DATA
		NewSubData.RootToUnwarpedLocalTransform = NewSubData.RootToUnwarpedLocalTransform * Params.RootToUnwarpedLocalTransform;
#endif
		NewSubData.HierarchicalBias        = Params.HierarchicalBias + NewSubData.HierarchicalBias;
		NewSubData.AccumulatedFlags        = UE::MovieScene::AccumulateChildSubSectionFlags(Params.AccumulatedFlags, NewSubData.AccumulatedFlags);

#if WITH_EDITORONLY_DATA
		NewSubData.StartTimeBreadcrumbs.CombineWithOuterBreadcrumbs(Params.StartTimeBreadcrumbs);
		NewSubData.EndTimeBreadcrumbs.CombineWithOuterBreadcrumbs(Params.EndTimeBreadcrumbs);
#endif // WITH_EDITORONLY_DATA

		// Add the sub data to the root hierarchy
		InOutHierarchy->Add(NewSubData, InnerSequenceID, ParentSequenceID);

		// Iterate into the sub sequence
		FGatherParameters SubParams = Params.CreateForSubData(NewSubData, InnerSequenceID);

		// This is a bit of hack to make sure that LocalClampRange gets sent through to the next GenerateSubSequenceData call,
		// but we do not set RootClampRange because it would be ambiguous to do so w.r.t looping sub sequences
		SubParams.LocalClampRange = NewSubData.PlayRange.Value;

		RootPath->PushGeneration(InnerSequenceID, NewSubData.DeterministicSequenceID);
		GenerateSubSequenceData(SubSequence, SubParams, Operand, RootPath, InOutHierarchy);
		RootPath->PopGenerations(1);

		bContainsSubSequences = true;
	}

	return bContainsSubSequences;
}

void UMovieSceneCompiledDataManager::PopulateSubSequenceTree(UMovieSceneSequence* SubSequence, const FGatherParameters& Params, UE::MovieScene::FSubSequencePath* RootPath, FMovieSceneSequenceHierarchy* InOutHierarchy)
{
	UMovieScene* MovieScene = SubSequence ? SubSequence->GetMovieScene() : nullptr;
	if (!MovieScene)
	{
		return;
	}

	check(RootPath && InOutHierarchy);

	for (UMovieSceneTrack* Track : MovieScene->GetTracks())
	{
		if (UMovieSceneSubTrack* SubTrack = Cast<UMovieSceneSubTrack>(Track))
		{
			PopulateSubSequenceTree(SubTrack, Params, RootPath, InOutHierarchy);
		}
	}

	for (const FMovieSceneBinding& ObjectBinding : ((const UMovieScene*)MovieScene)->GetBindings())
	{
		for (UMovieSceneTrack* Track : ObjectBinding.GetTracks())
		{
			if (UMovieSceneSubTrack* SubTrack = Cast<UMovieSceneSubTrack>(Track))
			{
				PopulateSubSequenceTree(SubTrack, Params, RootPath, InOutHierarchy);
			}
		}
	}
}

void UMovieSceneCompiledDataManager::PopulateSubSequenceTree(UMovieSceneSubTrack* SubTrack, const FGatherParameters& Params, UE::MovieScene::FSubSequencePath* RootPath, FMovieSceneSequenceHierarchy* InOutHierarchy)
{
	using namespace UE::MovieScene;

	check(SubTrack && RootPath);

	const bool bTrackMatchesFlags = ( Params.Flags == ESectionEvaluationFlags::None )
		|| ( EnumHasAnyFlags(Params.Flags, ESectionEvaluationFlags::PreRoll)  && SubTrack->EvalOptions.bEvaluateInPreroll  )
		|| ( EnumHasAnyFlags(Params.Flags, ESectionEvaluationFlags::PostRoll) && SubTrack->EvalOptions.bEvaluateInPostroll );

	if (!bTrackMatchesFlags)
	{
		return;
	}

	if (SubTrack->IsEvalDisabled())
	{
		return;
	}

	UMovieSceneSequence* OuterSequence = SubTrack->GetTypedOuter<UMovieSceneSequence>();
	if (!OuterSequence)
	{
		return;
	}

	for (const FMovieSceneTrackEvaluationFieldEntry& Entry : SubTrack->GetEvaluationField().Entries)
	{
		UMovieSceneSubSection* SubSection  = Cast<UMovieSceneSubSection>(Entry.Section);
		if (!SubSection || SubSection->GetSequence() == nullptr || SubSection->GetSequence()->GetMovieScene() == nullptr)
		{
			continue;
		}

		if (SubTrack->IsRowEvalDisabled(SubSection->GetRowIndex()))
		{
			continue;
		}

		EMovieSceneServerClientMask NewMask = Params.NetworkMask & SubSection->GetNetworkMask();
		if (NewMask == EMovieSceneServerClientMask::None)
		{
			continue;
		}

		InOutHierarchy->AccumulateNetworkMask(SubSection->GetNetworkMask());

		const FMovieSceneSequenceID       SubSequenceID = RootPath->ResolveChildSequenceID(SubSection->GetSequenceID());
		const FMovieSceneSubSequenceData* SubData = InOutHierarchy->FindSubData(SubSequenceID);

		checkf(SubData, TEXT("Unable to locate sub-data for a sub section that appears in the track's evaluation field - this indicates that the section is being evaluated even though it is not active"));

		auto AddRange = [&Params, &Entry, &RootPath, InOutHierarchy, NewMask, SubData, SubSequenceID](TRange<FFrameTime> Range)
		{
			TRange<FFrameNumber> FrameRange = Params.ClampRoot(ConvertToDiscreteRange(Range));

			if (!FrameRange.IsEmpty())
			{
				FGatherParameters SubParams = Params.CreateForSubData(*SubData, SubSequenceID);
				SubParams.SetClampRange(FrameRange);
				SubParams.Flags |= Entry.Flags;
				SubParams.NetworkMask = NewMask;

				const ESectionEvaluationFlags SubEntryFlags = Entry.Flags | Params.Flags;

				InOutHierarchy->AddRange(FrameRange, SubSequenceID, SubEntryFlags);

				// Recurse into the sub sequence
				RootPath->PushGeneration(SubSequenceID, SubData->DeterministicSequenceID);
				{
					PopulateSubSequenceTree(SubData->GetSequence(), SubParams, RootPath, InOutHierarchy);
				}
				RootPath->PopGenerations(1);
			}

			return true;
		};

		Params.TransformLocalRange(Entry.Range, AddRange);
	}
}

TOptional<FFrameNumber> UMovieSceneCompiledDataManager::GetLoopingSubSectionEndTime(const UMovieSceneSequence* InRootSequence, const UMovieSceneSubSection* SubSection, const FGatherParameters& Params)
{
	using namespace UE::MovieScene;

	TRangeBound<FFrameNumber> SectionRangeEnd = SubSection->SectionRange.GetUpperBound();
	if (!SectionRangeEnd.IsOpen())
	{
		return DiscreteExclusiveUpper(SectionRangeEnd);
	}

	// This section is open ended... we don't want to compile its sub-sequence in an infinite loop so we'll bound
	// that by the playback end of is own sequence.
	if (const UMovieScene* MovieScene = InRootSequence->GetMovieScene())
	{
		const TRange<FFrameNumber> PlaybackRange = MovieScene->GetPlaybackRange();
		if (!PlaybackRange.GetUpperBound().IsOpen())
		{
			return DiscreteExclusiveUpper(PlaybackRange.GetUpperBound());
		}
	}

	// Sadly, the root sequence is also open ended, so we effectively would need to loop the sub-sequence
	// indefinitely... we don't support that yet.
	return TOptional<FFrameNumber>();
}

