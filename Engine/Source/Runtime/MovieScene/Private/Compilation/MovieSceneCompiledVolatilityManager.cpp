// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compilation/MovieSceneCompiledVolatilityManager.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "MovieSceneFwd.h"

namespace UE
{
namespace MovieScene
{

#if WITH_EDITOR

int32 GVolatileSequencesInEditor = 1;
FAutoConsoleVariableRef CVarVolatileSequencesInEditor(
	TEXT("Sequencer.VolatileSequencesInEditor"),
	GVolatileSequencesInEditor,
	TEXT("(Default: 1) When non-zero, all assets will be treated as volatile in editor. Can be disabled to bypass volatility checks in-editor for more representative runtime performance metrics.\n"),
	ECVF_Default
);

#endif


FORCEINLINE EMovieSceneSequenceFlags GetEditorVolatilityFlags()
{
#if WITH_EDITOR
	return GVolatileSequencesInEditor ? EMovieSceneSequenceFlags::Volatile : EMovieSceneSequenceFlags::None;
#else
	return EMovieSceneSequenceFlags::None;
#endif
}


TUniquePtr<FCompiledDataVolatilityManager> FCompiledDataVolatilityManager::Construct(TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	FMovieSceneCompiledDataID RootDataID = SharedPlaybackState->GetRootCompiledDataID();
	UMovieSceneCompiledDataManager* CompiledDataManager = SharedPlaybackState->GetCompiledDataManager();

	const FMovieSceneCompiledDataEntry& Entry = CompiledDataManager->GetEntryRef(RootDataID);
	EMovieSceneSequenceFlags SequenceFlags = Entry.AccumulatedFlags | GetEditorVolatilityFlags();
	if (!EnumHasAnyFlags(SequenceFlags, EMovieSceneSequenceFlags::Volatile))
	{
		// If the entry has a valid compiled signature, it is assumed to be non-volatile if it does
		// not explicitly have the volatile flag. Otherwise we assume this sequence was added to the
		// manager but never compiled, and as such must be volatile since it needs compiling.
		if (Entry.CompiledSignature.IsValid())
		{
			return nullptr;
		}
	}

	TUniquePtr<FCompiledDataVolatilityManager> VolatilityManager = MakeUnique<FCompiledDataVolatilityManager>(SharedPlaybackState);
	VolatilityManager->ConditionalRecompile();
	return VolatilityManager;
}

FCompiledDataVolatilityManager::FCompiledDataVolatilityManager(TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
	: WeakSharedPlaybackState(SharedPlaybackState)
{
}

bool FCompiledDataVolatilityManager::HasBeenRecompiled() const
{
	TSharedPtr<const FSharedPlaybackState> SharedPlaybackState = WeakSharedPlaybackState.Pin();
	FMovieSceneCompiledDataID RootDataID = SharedPlaybackState->GetRootCompiledDataID();
	UMovieSceneCompiledDataManager* CompiledDataManager = SharedPlaybackState->GetCompiledDataManager();

	if (HasSequenceBeenRecompiled(RootDataID, MovieSceneSequenceID::Root))
	{
		return true;
	}

	if (const FMovieSceneSequenceHierarchy* Hierarchy = CompiledDataManager->FindHierarchy(RootDataID))
	{
		for (const TTuple<FMovieSceneSequenceID, FMovieSceneSubSequenceData>& Pair : Hierarchy->AllSubSequenceData())
		{
			FMovieSceneCompiledDataID SubDataID = CompiledDataManager->GetSubDataID(RootDataID, Pair.Key);
			if (SubDataID.IsValid() && HasSequenceBeenRecompiled(SubDataID, Pair.Key))
			{
				return true;
			}
		}
	}

	return false;
}

bool FCompiledDataVolatilityManager::HasSequenceBeenRecompiled(FMovieSceneCompiledDataID DataID, FMovieSceneSequenceID SequenceID) const
{
	const FGuid* CachedSignature = CachedCompilationSignatures.Find(SequenceID);

	TSharedPtr<const FSharedPlaybackState> SharedPlaybackState = WeakSharedPlaybackState.Pin();
	UMovieSceneCompiledDataManager* CompiledDataManager = SharedPlaybackState->GetCompiledDataManager();
	const FMovieSceneCompiledDataEntry& CompiledEntry = CompiledDataManager->GetEntryRef(DataID);
	return CachedSignature == nullptr || *CachedSignature != CompiledEntry.CompiledSignature;
}

bool FCompiledDataVolatilityManager::ConditionalRecompile()
{
	bool bRecompiled = false;

	TSharedPtr<const FSharedPlaybackState> SharedPlaybackState = WeakSharedPlaybackState.Pin();
	FMovieSceneCompiledDataID RootDataID = SharedPlaybackState->GetRootCompiledDataID();
	UMovieSceneCompiledDataManager* CompiledDataManager = SharedPlaybackState->GetCompiledDataManager();
	if (CompiledDataManager->IsDirty(RootDataID))
	{
		// We override the network mask from the compiled data manager here simply because it may not be correct.
		// In a non-editor/PIE executable, we have a single global compiled data manager, and in its current 'global static' implementation,
		// it may not have been created at a time with enough context to determine the net mode.
		// In certain edge cases, such as networked games where both server and client are compiled using a single target executable of TargetType.Game,
		// we may not have the context at Sequence cook/compile time to determine which subsections may be included/excluded at runtime based on NetworkMask,
		// and so these Sequences are marked as volatile on compile time. Therefore, it's here, upon conditional recompile, when we need to know the correct
		// network mask to use, which we override and apply here.
		EMovieSceneServerClientMask NetworkMask = CompiledDataManager->GetNetworkMask();
		UObject* PlaybackContext = SharedPlaybackState->GetPlaybackContext();
		UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;

		if (World)
		{
			ENetMode NetMode = World->GetNetMode();
			if (NetMode == ENetMode::NM_DedicatedServer)
			{
				NetworkMask = EMovieSceneServerClientMask::Server;
			}
			else if (NetMode == ENetMode::NM_Client)
			{
				NetworkMask = EMovieSceneServerClientMask::Client;
			}
		}
		CompiledDataManager->Compile(RootDataID, NetworkMask);
		bRecompiled = true;
	}
	else
	{
		bRecompiled = HasBeenRecompiled();
	}

	if (bRecompiled)
	{
		UpdateCachedSignatures();
	}

	return bRecompiled;
}

void FCompiledDataVolatilityManager::UpdateCachedSignatures()
{
	CachedCompilationSignatures.Reset();

	TSharedPtr<const FSharedPlaybackState> SharedPlaybackState = WeakSharedPlaybackState.Pin();
	FMovieSceneCompiledDataID RootDataID = SharedPlaybackState->GetRootCompiledDataID();
	UMovieSceneCompiledDataManager* CompiledDataManager = SharedPlaybackState->GetCompiledDataManager();
	FMovieSceneEvaluationState* State = SharedPlaybackState->FindCapability<FMovieSceneEvaluationState>();

	{
		const FMovieSceneCompiledDataEntry& RootEntry = CompiledDataManager->GetEntryRef(RootDataID);
		CachedCompilationSignatures.Add(MovieSceneSequenceID::Root, RootEntry.CompiledSignature);

		UMovieSceneSequence* RootSequence = RootEntry.GetSequence();
		if (RootSequence && State)
		{
			State->AssignSequence(MovieSceneSequenceID::Root, *RootSequence, SharedPlaybackState.ToSharedRef());
		}
	}

	if (const FMovieSceneSequenceHierarchy* Hierarchy = CompiledDataManager->FindHierarchy(RootDataID))
	{
		for (const TTuple<FMovieSceneSequenceID, FMovieSceneSubSequenceData>& SubData : Hierarchy->AllSubSequenceData())
		{
			const FMovieSceneCompiledDataID     SubDataID = CompiledDataManager->GetSubDataID(RootDataID, SubData.Key);
			if (!SubDataID.IsValid())
			{
				UE_LOGF(LogMovieScene, Error, "Invalid SubDataID for: %ls", *SubData.Value.Sequence.ToString());
				continue;
			}

			const FMovieSceneCompiledDataEntry& SubEntry  = CompiledDataManager->GetEntryRef(SubDataID);

			CachedCompilationSignatures.Add(SubData.Key, SubEntry.CompiledSignature);

			UMovieSceneSequence* Sequence = SubData.Value.GetSequence();
			if (Sequence && State)
			{
				State->AssignSequence(SubData.Key, *Sequence, SharedPlaybackState.ToSharedRef());
			}
		}
	}
}


} // namespace MovieScene
} // namespace UE
