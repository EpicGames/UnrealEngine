// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compilation/MovieSceneCompilerRules.h"
#include "MovieSceneSection.h"

TOptional<FMovieSceneSegment> MovieSceneSegmentCompiler::EvaluateNearestSegment(const TRange<FFrameNumber>& Range, const FMovieSceneSegment* PreviousSegment, const FMovieSceneSegment* NextSegment)
{
	if (PreviousSegment)
	{
		// There is a preceeding segment
		FFrameNumber PreviousSegmentRangeBound = PreviousSegment->Range.GetUpperBoundValue();

		FMovieSceneSegment EmptySpace(Range);
		for (FSectionEvaluationData Data : PreviousSegment->Impls)
		{
			EmptySpace.Impls.Add(FSectionEvaluationData(Data.ImplIndex, PreviousSegmentRangeBound));
		}
		return EmptySpace;
	}
	else if (NextSegment)
	{
		// Before any sections
		FFrameNumber NextSegmentRangeBound = NextSegment->Range.GetLowerBoundValue();

		FMovieSceneSegment EmptySpace(Range);
		for (FSectionEvaluationData Data : NextSegment->Impls)
		{
			EmptySpace.Impls.Add(FSectionEvaluationData(Data.ImplIndex, NextSegmentRangeBound));
		}
		return EmptySpace;
	}

	return TOptional<FMovieSceneSegment>();
}

bool MovieSceneSegmentCompiler::AlwaysEvaluateSection(const FMovieSceneSectionData& InSectionData)
{
	return InSectionData.Section->GetBlendType().IsValid() || EnumHasAnyFlags(InSectionData.Flags, ESectionEvaluationFlags::PreRoll | ESectionEvaluationFlags::PostRoll);
}

void MovieSceneSegmentCompiler::FilterOutUnderlappingSections(FSegmentBlendData& BlendData)
{
	if (!BlendData.Num())
	{
		return;
	}

	int32 HighestOverlap = TNumericLimits<int32>::Lowest();
	for (const FMovieSceneSectionData& SectionData : BlendData)
	{
		if (!AlwaysEvaluateSection(SectionData))
		{
			HighestOverlap = FMath::Max(HighestOverlap, SectionData.Section->GetOverlapPriority());
		}
	}

	// Remove anything that's not the highest priority, (excluding pre/postroll sections)
	for (int32 RemoveAtIndex = BlendData.Num() - 1; RemoveAtIndex >= 0; --RemoveAtIndex)
	{
		const FMovieSceneSectionData& SectionData = BlendData[RemoveAtIndex];
		if (SectionData.Section->GetOverlapPriority() != HighestOverlap && !AlwaysEvaluateSection(SectionData))
		{
			BlendData.RemoveAt(RemoveAtIndex, EAllowShrinking::No);
		}
	}
}

void MovieSceneSegmentCompiler::ChooseLowestRowIndex(FSegmentBlendData& BlendData)
{
	if (!BlendData.Num())
	{
		return;
	}

	int32 LowestRowIndex = TNumericLimits<int32>::Max();
	for (const FMovieSceneSectionData& SectionData : BlendData)
	{
		if (!AlwaysEvaluateSection(SectionData))
		{
			LowestRowIndex = FMath::Min(LowestRowIndex, SectionData.Section->GetRowIndex());
		}
	}

	// Remove anything that's not the highest priority, (excluding pre/postroll sections)
	for (int32 RemoveAtIndex = BlendData.Num() - 1; RemoveAtIndex >= 0; --RemoveAtIndex)
	{
		const FMovieSceneSectionData& SectionData = BlendData[RemoveAtIndex];
		if (SectionData.Section->GetRowIndex() > LowestRowIndex && !AlwaysEvaluateSection(SectionData))
		{
			BlendData.RemoveAt(RemoveAtIndex, EAllowShrinking::No);
		}
	}
}

// Reduces the evaluated sections to only the section that resides last in the source data. Legacy behaviour from various track instances.
void MovieSceneSegmentCompiler::BlendSegmentLegacySectionOrder(FSegmentBlendData& BlendData)
{
	if (BlendData.Num() > 1)
	{
		Algo::SortBy(BlendData, &FMovieSceneSectionData::TemplateIndex);
		BlendData.RemoveAt(1, BlendData.Num() - 1, EAllowShrinking::No);
	}
}