// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compilation/MovieSceneEvaluationTreePopulationRules.h"
#include "MovieSceneSection.h"
#include "MovieSceneTimeHelpers.h"

namespace UE
{
namespace MovieScene
{

void FEvaluationTreePopulationRules::SortSections(TArrayView<UMovieSceneSection* const> Sections, TArray<FSortedSection, TInlineAllocator<16>>& SortedSections, FSectionSortPredicate Predicate)
{
	for (int32 SectionIndex = 0; SectionIndex < Sections.Num(); ++SectionIndex)
	{
		UMovieSceneSection* Section = Sections[SectionIndex];

		if (Section && Section->IsActive())
		{
			const TRange<FFrameNumber> SectionRange = Section->GetRange();
			if (!SectionRange.IsEmpty())
			{
				SortedSections.Add(FSortedSection(*Section, SectionIndex));
			}
		}
	}

	SortedSections.Sort(Predicate);
}

void FEvaluationTreePopulationRules::Blended(TArrayView<UMovieSceneSection* const> Sections, TMovieSceneEvaluationTree<FMovieSceneTrackEvaluationData>& OutTree)
{
	for (UMovieSceneSection* Section : Sections)
	{
		if (Section && Section->IsActive())
		{
			const TRange<FFrameNumber> SectionRange = Section->GetRange();
			if (!SectionRange.IsEmpty())
			{
				OutTree.Add(SectionRange, FMovieSceneTrackEvaluationData::FromSection(Section));
			}
		}
	}
}

void FEvaluationTreePopulationRules::Blended(TArrayView<UMovieSceneSection* const> Sections, TMovieSceneEvaluationTree<FMovieSceneTrackEvaluationData>& OutTree, FSectionSortPredicate Predicate)
{
	TArray<FSortedSection, TInlineAllocator<16>> SortedSections;
	SortSections(Sections, SortedSections, Predicate);

	for (const FSortedSection& SectionEntry : SortedSections)
	{
		UMovieSceneSection* Section = Sections[SectionEntry.Index];
		OutTree.Add(Section->GetRange(), FMovieSceneTrackEvaluationData::FromSection(Section));
	}
}

void FEvaluationTreePopulationRules::HighPass(TArrayView<UMovieSceneSection* const> Sections, TMovieSceneEvaluationTree<FMovieSceneTrackEvaluationData>& OutTree)
{
	HighPassCustom(Sections, OutTree, FSortedSection::SortByOverlapPriorityAndRow);
}

void FEvaluationTreePopulationRules::HighPassPerRow(TArrayView<UMovieSceneSection* const> Sections, TMovieSceneEvaluationTree<FMovieSceneTrackEvaluationData>& OutTree)
{
	HighPassCustomPerRow(Sections, OutTree, FSortedSection::SortByOverlapPriorityAndRow);
}

void FEvaluationTreePopulationRules::HighPassCustom(TArrayView<UMovieSceneSection* const> Sections, TMovieSceneEvaluationTree<FMovieSceneTrackEvaluationData>& OutTree, FSectionSortPredicate Predicate)
{
	if (Sections.Num() == 0)
	{
		return;
	}

	TArray<FSortedSection, TInlineAllocator<16>> SortedSections;
	SortSections(Sections, SortedSections, Predicate);

	bool bCurrentHasBlendType = false;

	auto CanBlendOrNothingExistsAtTime = [&OutTree, &bCurrentHasBlendType](FMovieSceneEvaluationTreeNodeHandle Node)
	{
		return bCurrentHasBlendType || (OutTree.GetAllData(Node).IsValid() == false);
	};

	for (const FSortedSection& SectionEntry : SortedSections)
	{
		UMovieSceneSection* Section = Sections[SectionEntry.Index];
		bCurrentHasBlendType = Section->GetBlendType().IsValid();
		OutTree.AddSelective(Section->GetRange(), FMovieSceneTrackEvaluationData::FromSection(Section), CanBlendOrNothingExistsAtTime);
	}
}

void FEvaluationTreePopulationRules::HighPassCustomPerRow(TArrayView<UMovieSceneSection* const> Sections, TMovieSceneEvaluationTree<FMovieSceneTrackEvaluationData>& OutTree, FSectionSortPredicate Predicate)
{
	if (Sections.Num() == 0)
	{
		return;
	}

	TArray<FSortedSection, TInlineAllocator<16>> SortedSections;
	SortSections(Sections, SortedSections, Predicate);

	int32 CurrentRowIndex = 0;
	bool bCurrentHasBlendType = false;

	auto CanBlendOrRowIsVacantAtTime = [&OutTree, &CurrentRowIndex, &bCurrentHasBlendType](FMovieSceneEvaluationTreeNodeHandle Node)
	{
		if (bCurrentHasBlendType)
		{
			return true;
		}

		for (FMovieSceneTrackEvaluationData Data : OutTree.GetAllData(Node))
		{
			if (Data.Section.Get()->GetRowIndex() == CurrentRowIndex)
			{
				return false;
			}
		}

		return true;
	};

	for (const FSortedSection& SectionEntry : SortedSections)
	{
		UMovieSceneSection* Section = Sections[SectionEntry.Index];
		CurrentRowIndex = SectionEntry.Row();
		bCurrentHasBlendType = Section->GetBlendType().IsValid();
		OutTree.AddSelective(Section->GetRange(), FMovieSceneTrackEvaluationData::FromSection(Section), CanBlendOrRowIsVacantAtTime);
	}
}

void FEvaluationTreePopulationRules::PopulateNearestSection(TArrayView<UMovieSceneSection* const> Sections, TMovieSceneEvaluationTree<FMovieSceneTrackEvaluationData>& OutTree)
{
	if (OutTree.IsEmpty())
	{
		return;
	}

	// Fill in gaps
	TArray<TTuple<TRange<FFrameNumber>, FMovieSceneTrackEvaluationData>> RangesToInsert;

	for (FMovieSceneEvaluationTreeRangeIterator It(OutTree); It; ++It)
	{
		const bool bContainsSection = OutTree.GetAllData(It.Node()).IsValid();
		if (!bContainsSection)
		{
			FFrameNumber ForcedTime;

			FMovieSceneEvaluationTreeRangeIterator NodeToCopy = It.Next();
			if (!NodeToCopy)
			{
				NodeToCopy = It.Previous();
				ForcedTime = UE::MovieScene::DiscreteExclusiveUpper(NodeToCopy.Range());
			}
			else
			{
				ForcedTime = UE::MovieScene::DiscreteInclusiveLower(NodeToCopy.Range());
			}

			if (NodeToCopy)
			{
				TMovieSceneEvaluationTreeDataIterator<FMovieSceneTrackEvaluationData> DataIt = OutTree.GetAllData(NodeToCopy.Node());
				while (DataIt)
				{
					FMovieSceneTrackEvaluationData DataItCopy = *DataIt;
					DataItCopy.ForcedTime = ForcedTime;

					RangesToInsert.Add(MakeTuple(It.Range(), DataItCopy));
					++DataIt;
				}
			}
		}
	}

	for (const TTuple<TRange<FFrameNumber>, FMovieSceneTrackEvaluationData>& Pair : RangesToInsert)
	{
		OutTree.Add(Pair.Get<0>(), Pair.Get<1>());
	}
}

} // namespace MovieScene
} // namespace UE
