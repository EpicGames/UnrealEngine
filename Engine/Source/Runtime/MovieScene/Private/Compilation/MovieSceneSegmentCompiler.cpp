// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compilation/MovieSceneSegmentCompiler.h"
#include "Compilation/MovieSceneCompilerRules.h"
#include "Evaluation/MovieSceneEvaluationTree.h"
#include "MovieSceneSection.h"

bool FMovieSceneAdditiveCameraTrackBlender::SortByStartTime(const FMovieSceneSectionData& A, const FMovieSceneSectionData& B)
{
	return TRangeBound<FFrameNumber>::MinLower(A.Section->GetRange().GetLowerBound(), B.Section->GetRange().GetLowerBound()) == A.Section->GetRange().GetLowerBound();
}
