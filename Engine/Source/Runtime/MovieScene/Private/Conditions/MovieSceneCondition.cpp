// Copyright Epic Games, Inc. All Rights Reserved.

#include "Conditions/MovieSceneCondition.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneCondition)

bool UMovieSceneCondition::EvaluateCondition(FGuid BindingGuid, FMovieSceneSequenceID SequenceID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const
{

#if WITH_EDITORONLY_DATA
	if (bEditorForceTrue)
	{
		return true;
	}
	else
#endif
	{
		bool bConditionResult = EvaluateConditionInternal(BindingGuid, SequenceID, SharedPlaybackState);

		return bInvert ? !bConditionResult : bConditionResult;
	}

}

uint32 UMovieSceneCondition::ComputeCacheKey(FGuid BindingGuid, FMovieSceneSequenceID SequenceID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, UObject* EntityOwner) const
{
	// We use our own ptr in the cache key to ensure multiple instances of the same condition class with different parameters evaluate separately,
	// while a shared condition across multiple bindings/entities can be cached if its Scope is Global.
	uint32 HashResult = GetTypeHash(this);
	
	EMovieSceneConditionScope Scope = GetScopeInternal();
	if (Scope == EMovieSceneConditionScope::Binding)
	{
		HashResult = HashCombineFast(HashCombineFast(HashResult, GetTypeHash(BindingGuid)), GetTypeHash(SequenceID));
	}
	else if (Scope == EMovieSceneConditionScope::OwnerObject && EntityOwner != nullptr)
	{
		HashResult = HashCombineFast(HashResult, GetTypeHash(EntityOwner));
	}

	return HashResult;
}

bool UMovieSceneCondition::EvaluateConditionInternal(FGuid BindingGuid, FMovieSceneSequenceID SequenceID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const
{
	TArray<TObjectPtr<UObject>> BoundObjects;
	Algo::Transform(SharedPlaybackState->FindBoundObjects(BindingGuid, SequenceID), BoundObjects, [](const TWeakObjectPtr<> BoundObject) { return BoundObject.Get(); });

	FMovieSceneConditionContext ConditionContext{ SharedPlaybackState->GetPlaybackContext(), FMovieSceneBindingProxy(BindingGuid, SharedPlaybackState->GetSequence(SequenceID)), BoundObjects};

	return BP_EvaluateCondition(ConditionContext);
}

EMovieSceneConditionScope UMovieSceneCondition::BP_GetScope_Implementation() const
{
	return EMovieSceneConditionScope::Global;
}

EMovieSceneConditionScope UMovieSceneCondition::GetScopeInternal() const 
{ 
	return BP_GetScope();
}

EMovieSceneConditionCheckFrequency UMovieSceneCondition::BP_GetCheckFrequency_Implementation() const
{
	return EMovieSceneConditionCheckFrequency::Once;
}

EMovieSceneConditionCheckFrequency UMovieSceneCondition::GetCheckFrequencyInternal() const
{
	return BP_GetCheckFrequency();
}

bool UMovieSceneCondition::CanCacheResult(TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const
{ 
#if WITH_EDITOR
	// Specifically in editor worlds we don't cache condition results- this is because it's too difficult to know
	// what sort of things the user might change to invalidate the cached results.
	if (UObject* Context = SharedPlaybackState->GetPlaybackContext())
	{
		if (UWorld* World = Context->GetWorld())
		{
			if (World->IsEditorWorld() && !World->IsPlayInEditor())
			{
				return false;
			}
		}
	}
#endif
	
	return GetCheckFrequencyInternal() != EMovieSceneConditionCheckFrequency::OnTick; 
}

EMovieSceneConditionScope UMovieSceneCondition::GetConditionScope() const
{
	return GetScopeInternal();
}
