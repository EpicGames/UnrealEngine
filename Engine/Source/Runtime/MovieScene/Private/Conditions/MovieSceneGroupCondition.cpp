// Copyright Epic Games, Inc. All Rights Reserved.

#include "Conditions/MovieSceneGroupCondition.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "Engine/World.h"
#include "MovieSceneCommonHelpers.h"
#include "Algo/AllOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneGroupCondition)


uint32 UMovieSceneGroupCondition::ComputeCacheKey(FGuid BindingGuid, FMovieSceneSequenceID SequenceID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, UObject* EntityOwner) const
{
	// Combine our pointer's hash with sub-conditions hashes
	uint32 HashResult = GetTypeHash(this);

	for(const FMovieSceneConditionContainer& SubCondition : SubConditions)
	{
		if (SubCondition.Condition)
		{
			HashResult = HashCombineFast(HashResult, SubCondition.Condition->ComputeCacheKey(BindingGuid, SequenceID, SharedPlaybackState, EntityOwner));
		}
	}

	return HashResult;
}

bool UMovieSceneGroupCondition::EvaluateConditionInternal(FGuid BindingGuid, FMovieSceneSequenceID SequenceID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const
{
	bool bResult = (Operator == EMovieSceneGroupConditionOperator::And);
	UMovieSceneSignedObject* ConditionOwner = GetTypedOuter<UMovieSceneSignedObject>();
	for (const FMovieSceneConditionContainer& ConditionContainer : SubConditions)
	{
		if (ConditionContainer.Condition)
		{
			// We use MovieSceneHelpers::EvaluateSequenceCondition below to allow cacheing of calls to our sub-conditions where relevant.
			bool bLocalResult = MovieSceneHelpers::EvaluateSequenceCondition(BindingGuid, SequenceID, ConditionContainer.Condition, ConditionOwner, SharedPlaybackState);
			switch (Operator)
			{
			case EMovieSceneGroupConditionOperator::And:
				if (!bLocalResult)
				{
					return false;
				}
				break;
			case EMovieSceneGroupConditionOperator::Or:
				if (bLocalResult)
				{
					return true;
				}
				break;
			case EMovieSceneGroupConditionOperator::Xor:
				if (bLocalResult && !bResult)
				{
					bResult = true;
				}
				else if (bLocalResult && bResult)
				{
					return false;
				}
				break;
			}
		}
	}
	return bResult;
}

bool UMovieSceneGroupCondition::CanCacheResult(TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const
{
	return Algo::AllOf(SubConditions, [SharedPlaybackState](const FMovieSceneConditionContainer& SubCondition) { return !SubCondition.Condition || SubCondition.Condition->CanCacheResult(SharedPlaybackState);});
}
