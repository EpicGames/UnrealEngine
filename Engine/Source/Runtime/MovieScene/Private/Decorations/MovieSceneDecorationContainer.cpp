// Copyright Epic Games, Inc. All Rights Reserved.

#include "Decorations/MovieSceneDecorationContainer.h"

#include "MovieScene.h"
#include "MovieSceneTrack.h"
#include "MovieSceneSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneDecorationContainer)


UObject* FMovieSceneDecorationContainer::FindDecoration(const TSubclassOf<UObject>& InClass) const
{
	if (UClass* Class = InClass.Get())
	{
		// Do an exact match - we intentionally do not support derived decorations
		for (UObject* Decoration : Decorations)
		{
			if (Decoration && Decoration->IsA(Class))
			{
				return Decoration;
			}
		}
	}
	return nullptr;
}

void FMovieSceneDecorationContainer::AddDecoration(UObject* InDecoration, UObject* Outer, TFunctionRef<void(UObject*)> Event)
{
	check(InDecoration && Outer);

	if (!ensureMsgf(FindDecoration(InDecoration->GetClass()) == nullptr, TEXT("Attempting to add a decoration when one of the same type already exists. This request will be ignored.")))
	{
		return;
	}

	if (!ensureMsgf(InDecoration->IsIn(Outer->GetOutermost()), TEXT("Attempting to add a decoration from a different pacakge - this is not allowed.")))
	{
		return;
	}

	Decorations.Add(InDecoration);
	Event(InDecoration);
}

UObject* FMovieSceneDecorationContainer::GetOrCreateDecoration(const TSubclassOf<UObject>& InClass, UObject* Outer, TFunctionRef<void(UObject*)> Event)
{
	UObject* Found = FindDecoration(InClass);
	if (!Found)
	{
		Found = NewObject<UObject>(Outer, InClass, NAME_None, RF_Transactional);
		Decorations.Add(Found);
		Event(Found);
	}
	return Found;
}

void FMovieSceneDecorationContainer::RemoveDecoration(const TSubclassOf<UObject>& InClass, TFunctionRef<void(UObject*)> Event)
{
	if (UClass* Class = InClass.Get())
	{
		for (int32 Index = Decorations.Num()-1; Index >= 0; --Index)
		{
			if (Decorations[Index] && Decorations[Index]->IsA(Class))
			{
				Event(Decorations[Index]);
				Decorations.RemoveAtSwap(Index);
			}
		}
	}
}

TArrayView<const TObjectPtr<UObject>> FMovieSceneDecorationContainer::GetDecorations() const
{
	return Decorations;
}

void UMovieSceneDecorationContainerObject::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	// Remove null decorations for safety
	Decorations.RemoveNulls();

	// Clean up disabled set- remove entries for decorations that no longer exist
	if (Ar.IsLoading())
	{
		DisabledDecorations.Remove(nullptr);
	}
}

void UMovieSceneDecorationContainerObject::AddDecoration(UObject* InDecoration)
{
	Decorations.AddDecoration(InDecoration, this, [this](UObject* Decoration){
		this->OnDecorationAdded(Decoration);
	});
	MarkAsChanged();
}

UObject* UMovieSceneDecorationContainerObject::GetOrCreateDecoration(const TSubclassOf<UObject>& InClass)
{
	UObject* Result = Decorations.GetOrCreateDecoration(InClass, this, [this](UObject* Decoration){
		this->OnDecorationAdded(Decoration);
	});
	MarkAsChanged();
	return Result;
}

void UMovieSceneDecorationContainerObject::RemoveDecoration(const TSubclassOf<UObject>& InClass)
{
	Decorations.RemoveDecoration(InClass, [this](UObject* Decoration){
		MutedDecorations.Remove(Decoration);
		DisabledDecorations.Remove(Decoration);
		this->OnDecorationRemoved(Decoration);
	});
	MarkAsChanged();
}

bool UMovieSceneDecorationContainerObject::IsDecorationMuted(const UObject* Decoration) const
{
	return MutedDecorations.Contains(Decoration);
}

void UMovieSceneDecorationContainerObject::SetDecorationMuted(UObject* Decoration, bool bMuted)
{
	if (bMuted)
	{
		if (MutedDecorations.Add(Decoration).IsValidId())
		{
			MarkAsChanged();
		}
	}
	else
	{
		if (MutedDecorations.Remove(Decoration) > 0)
		{
			MarkAsChanged();
		}
	}
}

bool UMovieSceneDecorationContainerObject::IsDecorationDisabled(const UObject* Decoration) const
{
	return DisabledDecorations.Contains(Decoration);
}

void UMovieSceneDecorationContainerObject::SetDecorationDisabled(UObject* Decoration, bool bDisabled)
{
	const bool bCurrentlyDisabled = DisabledDecorations.Contains(Decoration);
	if (bDisabled == bCurrentlyDisabled)
	{
		return;
	}

	Modify();

	if (bDisabled)
	{
		DisabledDecorations.Add(Decoration);
	}
	else
	{
		DisabledDecorations.Remove(Decoration);
	}

	MarkAsChanged();
}

bool UMovieSceneDecorationContainerObject::IsDecorationActive(const UObject* Decoration) const
{
	return !IsDecorationMuted(Decoration) && !IsDecorationDisabled(Decoration);
}

TArray<UObject*> UMovieSceneDecorationContainerObject::GetActiveDecorations() const
{
	TArray<UObject*> Result;
	for (const TObjectPtr<UObject>& Decoration : Decorations.GetDecorations())
	{
		if (Decoration && IsDecorationActive(Decoration))
		{
			Result.Add(Decoration);
		}
	}
	return Result;
}
