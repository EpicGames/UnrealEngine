// Copyright Epic Games, Inc. All Rights Reserved.

#include "Decorations/MovieSceneLanguagePreviewDecoration.h"
#include "MovieSceneSection.h"
#include "MovieScene.h"
#include "Misc/PackageName.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneLanguagePreviewDecoration)

UObject* UMovieSceneLanguagePreviewDecoration::FindLocalizedAsset(UObject* InUnlocalizedAsset, const UMovieSceneSection* Section)
{
	check(Section);

#if WITH_EDITORONLY_DATA

	if (InUnlocalizedAsset != nullptr)
	{
		// Handle custom language preview if necessary
		UMovieSceneLanguagePreviewDecoration* This = Section->GetTypedOuter<UMovieScene>()->FindDecoration<UMovieSceneLanguagePreviewDecoration>();
		if (This)
		{
			return This->FindLocalizedAsset(InUnlocalizedAsset);
		}
	}

#endif

	// Asset will have already been localized if we're in game by package-level localization
	return InUnlocalizedAsset;
}

UObject* UMovieSceneLanguagePreviewDecoration::FindLocalizedAsset(UObject* InUnlocalizedAsset) const
{

#if WITH_EDITORONLY_DATA

	// Handle custom language preview if necessary
	if (InUnlocalizedAsset != nullptr && PreviewLanguage.Len() != 0)
	{
		FString LocalizedPackageName = FPackageName::GetLocalizedPackagePath(InUnlocalizedAsset->GetOutermost()->GetPathName(), PreviewLanguage);
		if (LocalizedPackageName.Len() != 0)
		{
			// Attempt to load the localized asset
			UObject* LocalizedAsset = LoadObject<UObject>(nullptr, *FString(LocalizedPackageName + TEXT(".") + InUnlocalizedAsset->GetName()));
			if (LocalizedAsset)
			{
				return LocalizedAsset;
			}
		}
	}

#endif

	return InUnlocalizedAsset;
}
