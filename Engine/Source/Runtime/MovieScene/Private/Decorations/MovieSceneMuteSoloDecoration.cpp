// Copyright Epic Games, Inc. All Rights Reserved.

#include "Decorations/MovieSceneMuteSoloDecoration.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneMuteSoloDecoration)

UMovieSceneMuteSoloDecoration::UMovieSceneMuteSoloDecoration()
{
	// Mute Solo Decoration is not meant to be saved
	SetFlags(RF_Transient);
}
