// Copyright Epic Games, Inc. All Rights Reserved.

#include "Decorations/MovieSceneScalingAnchors.h"
#include "Decorations/MovieSceneTimeWarpDecoration.h"
#include "Variants/MovieSceneTimeWarpVariant.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneInterpolation.h"
#include "MovieSceneTransformTypes.h"
#include "MovieSceneTimeHelpers.h"
#include "MovieScene.h"
#include "Evaluation/MovieSceneSequenceTransform.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneScalingAnchors)

#if 0
uint32 UMovieSceneScalingAnchors::DefaultTimeWarpEntityID = MAX_uint32;
#endif

UMovieSceneScalingAnchors::UMovieSceneScalingAnchors()
{
}

void UMovieSceneScalingAnchors::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{
		// Remove null drivers for safety
		ScalingDrivers.Remove(nullptr);
	}
}

void UMovieSceneScalingAnchors::OnDecorationAdded(UMovieScene* MovieScene)
{
	UMovieSceneTimeWarpDecoration* TimeWarp = MovieScene->GetOrCreateDecoration<UMovieSceneTimeWarpDecoration>();
	TimeWarp->AddTimeWarpSource(this);

#if 0
	// Anchors provide a default timewarp to all entities in the local sequence
	UMovieSceneTimeWarpRegistryDecoration* TimewarpRegistry = MovieScene->GetOrCreateDecoration<UMovieSceneTimeWarpRegistryDecoration>();
	TimewarpRegistry->SetDefaultTimeWarp(this);
#endif
}

void UMovieSceneScalingAnchors::OnDecorationRemoved()
{	
	UMovieScene* MovieScene = GetTypedOuter<UMovieScene>();
	if (!MovieScene)
	{
		return;
	}

	if (UMovieSceneTimeWarpDecoration* TimeWarp = MovieScene->FindDecoration<UMovieSceneTimeWarpDecoration>())
	{
		TimeWarp->RemoveTimeWarpSource(this);

		if (!TimeWarp->HasAnySources())
		{
			MovieScene->RemoveDecoration<UMovieSceneTimeWarpDecoration>();
		}
	}
#if 0
	if (UMovieSceneTimeWarpRegistryDecoration* TimeWarp = MovieScene->FindDecoration<UMovieSceneTimeWarpRegistryDecoration>())
	{
		TimeWarp->SetDefaultTimeWarp(nullptr);

		if (!TimeWarp->HasAnySources())
		{
			MovieScene->RemoveDecoration<UMovieSceneTimeWarpDecoration>();
		}
	}
#endif
}

void UMovieSceneScalingAnchors::ResetScaling()
{
	PlayRate.Reset();
	bUpToDate = false;
	bPlayRateCurveIsUpToDate = false;
}

FMovieSceneNestedSequenceTransform UMovieSceneScalingAnchors::GenerateTimeWarpTransform()
{
	return FMovieSceneNestedSequenceTransform(0, FMovieSceneTimeWarpVariant(this));
}

bool UMovieSceneScalingAnchors::IsTimeWarpActive() const
{
	return true;
}

void UMovieSceneScalingAnchors::SetIsTimeWarpActive(bool bInActive)
{
	// Unimplemented - always comes first
}

int32 UMovieSceneScalingAnchors::GetTimeWarpSortOrder() const
{
	return MIN_int32;
}

void UMovieSceneScalingAnchors::AddScalingDriver(TScriptInterface<IMovieSceneScalingDriver> InDriver)
{
	ScalingDrivers.AddUnique(InDriver);
	ResetScaling();
}

void UMovieSceneScalingAnchors::RemoveScalingDriver(TScriptInterface<IMovieSceneScalingDriver> InDriver)
{
	ScalingDrivers.Remove(InDriver);
	ResetScaling();
}

bool UMovieSceneScalingAnchors::HasAnyScalingDrivers() const
{
	return !ScalingDrivers.IsEmpty();
}

FMovieSceneAnchorsScalingGroup& UMovieSceneScalingAnchors::GetOrCreateScalingGroup(const FGuid& Guid)
{
	return ScalingGroups.FindOrAdd(Guid);
}

FMovieSceneAnchorsScalingGroup* UMovieSceneScalingAnchors::FindScalingGroup(const FGuid& Guid)
{
	return ScalingGroups.Find(Guid);
}

const TMap<FGuid, FMovieSceneScalingAnchor>& UMovieSceneScalingAnchors::GetInitialAnchors() const
{
	return InitialAnchors;
}

const TMap<FGuid, FMovieSceneScalingAnchor>& UMovieSceneScalingAnchors::GetCurrentAnchors() const
{
	return CurrentAnchors;
}

void UMovieSceneScalingAnchors::RemoveScalingGroup(const FGuid& Guid)
{
	ScalingGroups.Remove(Guid);
}

const TMap<FGuid, FMovieSceneAnchorsScalingGroup>& UMovieSceneScalingAnchors::GetScalingGroups() const
{
	return ScalingGroups;
}

void UMovieSceneScalingAnchors::OnPreDecorationCompiled()
{
	InitialAnchors.Reset();

	for (TScriptInterface<IMovieSceneScalingDriver> Source : ScalingDrivers)
	{
		Source->PopulateInitialAnchors(InitialAnchors);
	}

	ResetScaling();
}

EMovieSceneChannelProxyType UMovieSceneScalingAnchors::PopulateChannelProxy(FMovieSceneChannelProxyData& OutProxyData, EAllowTopLevelChannels AllowTopLevel)
{
#if WITH_EDITOR
	UMovieScene* MovieScene = GetTypedOuter<UMovieScene>();

	FMovieSceneChannelMetaData ChannelMetaData;
	ChannelMetaData.Name = "Anchors";
	ChannelMetaData.bCanCollapseToTrack = (AllowTopLevel == EAllowTopLevelChannels::Yes);
	ChannelMetaData.DisplayText = NSLOCTEXT("MovieSceneScalingAnchors", "Anchors_Label", "Anchors");
	ChannelMetaData.WeakOwningObject = this;
	ChannelMetaData.bRelativeToSection = true;

	OutProxyData.Add(PlayRate, ChannelMetaData);

#else
	OutProxyData.Add(PlayRate);
#endif

	return EMovieSceneChannelProxyType::Static;
}

bool UMovieSceneScalingAnchors::DeleteChannel(FMovieSceneTimeWarpVariant& OutVariant, FName ChannelName)
{
	if (ChannelName == "PlayRate")
	{
		OutVariant.Set(1.0);
		return true;
	}
	return false;
}

void UMovieSceneScalingAnchors::UpdateCurve(UMovieScenePlayRateCurve* Curve) const
{
	if (bPlayRateCurveIsUpToDate)
	{
		return;
	}

	bPlayRateCurveIsUpToDate = true;

	Curve->bUpToDate = false;
	Curve->PlayRate.Reset();

	FFrameNumber PlaybackStart = UE::MovieScene::DiscreteInclusiveLower(GetTypedOuter<UMovieScene>()->GetPlaybackRange());

	Curve->AnchorMode = EMovieScenePlayRateCurveAnchorMode::Manual;
	Curve->PlaybackStartFrame = 0;//PlaybackStart;

	CurrentAnchors.Reset();
	// Remove null drivers for safety
	for (TScriptInterface<IMovieSceneScalingDriver> Source : ScalingDrivers)
	{
		Source->PopulateAnchors(CurrentAnchors);
	}

	if (CurrentAnchors.Num() == 0)
	{
		Curve->PlayRate.SetDefault(1.0);
		return;
	}

	struct FSort
	{
		bool operator()(const TPair<FGuid, FMovieSceneScalingAnchor>& A, const TPair<FGuid, FMovieSceneScalingAnchor>& B) const
		{
			return A.Get<1>().Position < B.Get<1>().Position;
		}
	};

	// Sort all the keys
	struct FScaledAnchor
	{
		FGuid ID;
		FMovieSceneScalingAnchor InitialAnchor;
		FMovieSceneScalingAnchor CurrentAnchor;
		FMovieSceneScalingAnchor ScaledAnchor;
	};
	TArray<FScaledAnchor> SortedAnchors;
	for (TPair<FGuid, FMovieSceneScalingAnchor> Pair : InitialAnchors)
	{
		if (FMovieSceneScalingAnchor* NewAnchor = CurrentAnchors.Find(Pair.Key))
		{
			// Insert, allowing for duplicates at the same time
			int32 Index = Algo::LowerBoundBy(SortedAnchors, Pair.Value.Position, [](FScaledAnchor In) { return In.InitialAnchor.Position; });
			SortedAnchors.Insert(FScaledAnchor{ Pair.Key, Pair.Value, *NewAnchor, Pair.Value }, Index);
		}
	}

	// The time to add the next play ate key at
	FFrameNumber KeyTime = 0;
	// A linear offset to apply to any non-overlapping anchors to accomodate for earlier scaled anchors
	FFrameNumber CumulativeScaleOffset = 0;
	// The scaled position of the last scale point encountered
	TOptional<FFrameNumber> LastAnchorPosition;
	// The initial position of the last scale point encountered
	TOptional<FFrameNumber> LastAnchorInitialPosition;

	// Place a key just before the first anchor to initialize the play rate to one
	Curve->PlayRate.AddConstantKey(SortedAnchors[0].ScaledAnchor.Position - 100, 1.0);


	TArray<int32> OverlappingIndex;

	// Go through all scaling anchors by their initial order and compute the scaling,
	//      taking account of overlapping anchors
	for (int32 Index = 0; Index < SortedAnchors.Num(); ++Index)
	{
		FScaledAnchor& Entry = SortedAnchors[Index];

		// Any entry we encounter in this outer loop is guaranteed to not be overlapping any other
		//    since those will have been handled by the inner loop
		Entry.ScaledAnchor.Position = CumulativeScaleOffset + Entry.CurrentAnchor.Position;
		Entry.ScaledAnchor.Duration = Entry.CurrentAnchor.Duration;

		// Move on the scale offset by how much this anchor moved (ie, the empty pace between the last point and this has scaled)
		CumulativeScaleOffset += (Entry.CurrentAnchor.Position - Entry.InitialAnchor.Position);

		// If we have a previous anchor position, add play rate for the empty spae up until this anchor
		if (LastAnchorInitialPosition.IsSet() && !FMath::IsNearlyZero(double(Entry.InitialAnchor.Position.Value - LastAnchorInitialPosition->Value)))
		{
			const double PlayRateToThisAnchor = double(Entry.ScaledAnchor.Position.Value - LastAnchorPosition->Value) / double(Entry.InitialAnchor.Position.Value - LastAnchorInitialPosition->Value);
			Curve->PlayRate.AddConstantKey(KeyTime, PlayRateToThisAnchor);
		}

		// Keep track of this anchor for the next iteration
		KeyTime                   = Entry.ScaledAnchor.Position;
		LastAnchorPosition        = Entry.ScaledAnchor.Position;
		LastAnchorInitialPosition = Entry.InitialAnchor.Position;

		// Store this anchor's final position for future external reference
		CurrentAnchors.Add(Entry.ID, Entry.ScaledAnchor);

		if (Entry.InitialAnchor.Duration <= 0.0 || Entry.CurrentAnchor.Duration <= 0.0)
		{
			// No duration on this anchor - just move on
			continue;
		}


		// This anchor has duration: we need to scale anything that overlaps it proportionally
		FFrameNumber OverlapBoundary        = Entry.ScaledAnchor.Position + Entry.ScaledAnchor.Duration;
		FFrameNumber InitialOverlapBoundary = Entry.InitialAnchor.Position + Entry.InitialAnchor.Duration;

		FFrameNumber MaxTime = OverlapBoundary;
		FFrameNumber InitialMaxTime = InitialOverlapBoundary;


		// Recursively process overlapping anchors until we find empty space
		OverlappingIndex.Reset(1);
		OverlappingIndex.Add(Index);

		while (Index < SortedAnchors.Num()-1 && OverlappingIndex.Num() != 0)
		{
			const FScaledAnchor& LastOverlap = SortedAnchors[OverlappingIndex.Last()];

			// Position everything proportionally along this anchor's duration that starts within the range
			//     making sure not to stretch durations
			for (int32 NextIndex = Index+1; NextIndex < SortedAnchors.Num(); ++NextIndex, ++Index)
			{
				FScaledAnchor& NextEntry = SortedAnchors[NextIndex];

				if (NextEntry.InitialAnchor.Position >= LastOverlap.InitialAnchor.Position + LastOverlap.InitialAnchor.Duration)
				{
					// This anchor does not fall within the duration of the one we're processing
					//     so pop the last overlap and start again from the next
					OverlappingIndex.Pop();
					break;
				}

				const double DurationScale = double(LastOverlap.ScaledAnchor.Duration) / double(LastOverlap.InitialAnchor.Duration);

				// Reposition this anchor to be scaled proportionally within the most recent anchor's range
				double NextAnchorPosition = LastOverlap.ScaledAnchor.Position.Value + (NextEntry.CurrentAnchor.Position.Value - LastOverlap.CurrentAnchor.Position.Value) * DurationScale;
				NextEntry.ScaledAnchor.Position = FMath::RoundToInt32(NextAnchorPosition);
				NextEntry.ScaledAnchor.Duration = NextEntry.CurrentAnchor.Duration;

				// Store this anchor's final position for future external reference
				CurrentAnchors.Add(NextEntry.ID, NextEntry.ScaledAnchor);

				// If this anchor has a range that overflows the current maximum, we need to add a scale between
				//    up until this boundary, and move on to start from the current boundary
				if (NextEntry.ScaledAnchor.Position + NextEntry.ScaledAnchor.Duration > OverlapBoundary)
				{
					// Add a play rate key for the current boundary and move the boundary forward
					const double OverlapScale = double(InitialOverlapBoundary.Value - LastAnchorInitialPosition->Value) / double(OverlapBoundary.Value - LastAnchorPosition->Value);
					Curve->PlayRate.AddConstantKey(KeyTime, OverlapScale);

					// The next play rate range should start from the current boundary
					KeyTime = OverlapBoundary;
					LastAnchorPosition = OverlapBoundary;
					LastAnchorInitialPosition = InitialOverlapBoundary;

					// The new boundary point will be the end of this anchor
					OverlapBoundary = NextEntry.ScaledAnchor.Position + NextEntry.ScaledAnchor.Duration;
					InitialOverlapBoundary = NextEntry.InitialAnchor.Position + NextEntry.InitialAnchor.Duration;

					// Keep track of the maximums for the CumulativeScaleOffset
					MaxTime = OverlapBoundary;
					if (InitialOverlapBoundary > InitialMaxTime)
					{
						InitialMaxTime = InitialOverlapBoundary;
					}
				}

				// If this anchor has duration we re-run this whole loop with the new anchor
				if (NextEntry.CurrentAnchor.Duration > 0)
				{
					OverlappingIndex.Add(NextIndex);
					++Index;
					break;
				}
			}
		}

		// Accumulate the scale offset for this whole group
		CumulativeScaleOffset += (MaxTime - Entry.ScaledAnchor.Position) - (InitialMaxTime - Entry.InitialAnchor.Position);

		// Process the remaining space that is left from these overlaps (or the entire range if there are no overlaps)
		const FFrameNumber InitialRemainingRange = InitialOverlapBoundary - LastAnchorInitialPosition.GetValue();
		const FFrameNumber RemainingRange        = OverlapBoundary - LastAnchorPosition.GetValue();
		if (InitialRemainingRange != 0 && RemainingRange != 0)
		{
			// Add a play rate key for the current boundary and move the boundary forward
			const double OverlapScale = double(InitialRemainingRange.Value) / double(RemainingRange.Value);
			Curve->PlayRate.AddConstantKey(KeyTime, OverlapScale);

			// Move on from this point
			KeyTime = OverlapBoundary;
			LastAnchorPosition = OverlapBoundary;
			LastAnchorInitialPosition = InitialOverlapBoundary;
		}
	}

	// Add final play rate to cause subsequent regions to all play-back at 1.0x
	Curve->PlayRate.AddConstantKey(*LastAnchorPosition, 1.0);
}

UMovieScenePlayRateCurve* UMovieSceneScalingAnchors::Initialize(TSharedPtr<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const
{
	UMovieScenePlayRateCurve* ContextPlayRate = NewObject<UMovieScenePlayRateCurve>(SharedPlaybackState->GetPlaybackContext());
	UpdateCurve(ContextPlayRate);

#if 0
	// // Update our desired duration
	UMovieScene* MovieScene = GetTypedOuter<UMovieScene>();
	TRange<FFrameNumber> PlaybackRange = MovieScene->GetPlaybackRange();

	FFrameNumber StartTime = PlaybackRange.GetLowerBoundValue();
	FFrameNumber EndTime   = PlaybackRange.GetLowerBoundValue();

	TOptional<FFrameTime> NewEndTime = Curve->GetTimeWarpCurve().InverseEvaluate(EndTime.Value, EndTime, UE::MovieScene::EInverseEvaluateFlags::AnyDirection);

	NewDuration = NewEndTime.Get(FFrameTime(EndTime)) - StartTime;
#endif
	return ContextPlayRate;
}

void UMovieSceneScalingAnchors::UpdateFromSource() const
{
	UpdateCurve(const_cast<UMovieSceneScalingAnchors*>(this));
}

TRange<FFrameTime> UMovieSceneScalingAnchors::ComputeTraversedHull(const TRange<FFrameTime>& Range) const
{
	UpdateFromSource();
	return Super::ComputeTraversedHull(Range);
}

FFrameTime UMovieSceneScalingAnchors::RemapTime(FFrameTime In) const
{
	UpdateFromSource();
	return Super::RemapTime(In);
}

TOptional<FFrameTime> UMovieSceneScalingAnchors::InverseRemapTimeCycled(FFrameTime InValue, FFrameTime InTimeHint, const UE::MovieScene::FInverseTransformTimeParams& Params) const
{
	UpdateFromSource();
	return Super::InverseRemapTimeCycled(InValue, InTimeHint, Params);
}

bool UMovieSceneScalingAnchors::InverseRemapTimeWithinRange(FFrameTime InTime, FFrameTime RangeStart, FFrameTime RangeEnd, const TFunctionRef<bool(FFrameTime)>& VisitorCallback) const
{
	UpdateFromSource();
	return Super::InverseRemapTimeWithinRange(InTime, RangeStart, RangeEnd, VisitorCallback);
}

void UMovieSceneScalingAnchors::ScaleBy(double UnwarpedScaleFactor)
{
	// Cannot Scale Anchors
}
