// Copyright Epic Games, Inc. All Rights Reserved.

#include "Decorations/MovieSceneSectionAnchorsDecoration.h"
#include "MovieScene.h"
#include "Decorations/MovieSceneScalingAnchors.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSectionAnchorsDecoration)


void UMovieSceneSectionAnchorsDecoration::OnReconstruct(UMovieScene* MovieScene)
{
	MovieScene->Modify();

	UMovieSceneScalingAnchors* Anchors = MovieScene->GetOrCreateDecoration<UMovieSceneScalingAnchors>();

	Anchors->Modify();
	Anchors->AddScalingDriver(this);

	StartAnchor = FGuid::NewGuid();
}

void UMovieSceneSectionAnchorsDecoration::OnDestroy(UMovieScene* MovieScene)
{
	if (UMovieSceneScalingAnchors* Anchors = MovieScene->FindDecoration<UMovieSceneScalingAnchors>())
	{
		Anchors->Modify();
		Anchors->RemoveScalingDriver(this);

		if (!Anchors->HasAnyScalingDrivers())
		{
			MovieScene->RemoveDecoration<UMovieSceneScalingAnchors>();
		}
	}
}

void UMovieSceneSectionAnchorsDecoration::PopulateInitialAnchors(TMap<FGuid, FMovieSceneScalingAnchor>& OutAnchors)
{
	IMovieSceneScalingDriver* SectionImpl = Cast<IMovieSceneScalingDriver>(GetTypedOuter<UMovieSceneSection>());
	if (SectionImpl)
	{
		SectionImpl->PopulateInitialAnchors(OutAnchors);
	}
}

void UMovieSceneSectionAnchorsDecoration::PopulateAnchors(TMap<FGuid, FMovieSceneScalingAnchor>& OutAnchors)
{
	IMovieSceneScalingDriver* SectionImpl = Cast<IMovieSceneScalingDriver>(GetTypedOuter<UMovieSceneSection>());
	if (SectionImpl)
	{
		SectionImpl->PopulateAnchors(OutAnchors);
	}
}

void UMovieSceneSectionAnchorsDecoration::PostEditImport()
{
	// Do not allow duplicate anchor GUIDs
	StartAnchor = FGuid::NewGuid();
}
