// Copyright Epic Games, Inc. All Rights Reserved.

#include "Decorations/MovieSceneTimeWarpDecoration.h"
#include "Evaluation/MovieSceneSequenceTransform.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneTimeWarpDecoration)


FMovieSceneSequenceTransform UMovieSceneTimeWarpDecoration::GenerateTransform() const
{
	FMovieSceneSequenceTransform CombinedTransform;

	Algo::SortBy(Sources, &IMovieSceneTimeWarpSource::GetTimeWarpSortOrder);

	for (TScriptInterface<IMovieSceneTimeWarpSource> Source : Sources)
	{
		FMovieSceneNestedSequenceTransform TimeWarpTransform = Source->GenerateTimeWarpTransform();

		// Don't do anything for identity transforms
		if (!TimeWarpTransform.IsIdentity())
		{
			CombinedTransform.Add(TimeWarpTransform);
		}
	}


	return CombinedTransform;
}

void UMovieSceneTimeWarpDecoration::OnCompiled()
{

}

void UMovieSceneTimeWarpDecoration::AddTimeWarpSource(TScriptInterface<IMovieSceneTimeWarpSource> InSource)
{
	Sources.AddUnique(InSource);
}

void UMovieSceneTimeWarpDecoration::RemoveTimeWarpSource(TScriptInterface<IMovieSceneTimeWarpSource> InSource)
{
	Sources.Remove(InSource);
}

void UMovieSceneTimeWarpDecoration::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{

}
bool UMovieSceneTimeWarpDecoration::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	return true;
}

void UMovieSceneTimeWarpDecoration::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	if (Ar.IsLoading())
	{
		Sources.Remove(nullptr);
	}
}
