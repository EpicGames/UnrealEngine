// Copyright Epic Games, Inc. All Rights Reserved.

#include "Decorations/MovieSceneTrackRowDecoration.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneTrackRowDecoration)

UMovieSceneTrackRowDecoration::UMovieSceneTrackRowDecoration()
{
	// Track Row Decoration is not meant to be saved
	SetFlags(RF_Transient); 
}

void UMovieSceneTrackRowDecoration::SetMuted(int32 InRowIndex, bool bMuted)
{
	if (!MuteSoloData.Contains(InRowIndex))
	{
		MuteSoloData.Add(InRowIndex, FMovieSceneMuteSoloData());
	}

	MuteSoloData[InRowIndex].bMuted = bMuted;
}

bool UMovieSceneTrackRowDecoration::IsMuted(int32 InRowIndex) const
{
	if (MuteSoloData.Contains(InRowIndex))
	{
		return MuteSoloData[InRowIndex].bMuted;
	}
	return false;
}

void UMovieSceneTrackRowDecoration::SetSoloed(int32 InRowIndex, bool bSoloed)
{
	if (!MuteSoloData.Contains(InRowIndex))
	{
		MuteSoloData.Add(InRowIndex, FMovieSceneMuteSoloData());
	}

	MuteSoloData[InRowIndex].bSoloed = bSoloed;
}

bool UMovieSceneTrackRowDecoration::IsSoloed(int32 InRowIndex) const
{
	if (MuteSoloData.Contains(InRowIndex))
	{
		return MuteSoloData[InRowIndex].bSoloed;
	}
	return false;
}

void UMovieSceneTrackRowDecoration::OnRowIndicesChanged(const TMap<int32, int32>& NewToOldRowIndices)
{
	TMap<int32, FMovieSceneMuteSoloData> OldMuteSoloData = MuteSoloData;

	MuteSoloData.Empty();
	for (const TPair<int32, int32>& NewToOldRowIndex : NewToOldRowIndices)
	{
		if (OldMuteSoloData.Contains(NewToOldRowIndex.Value))
		{
			MuteSoloData.Add(NewToOldRowIndex.Key, OldMuteSoloData[NewToOldRowIndex.Value]);
		}
	}
}