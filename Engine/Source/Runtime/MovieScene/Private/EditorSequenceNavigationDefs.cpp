// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorSequenceNavigationDefs.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorSequenceNavigationDefs)

const FNavigationToolSerializedTreeNode* FNavigationToolSerializedTreeNode::GetParentTreeNode() const
{
	if (!OwningTree || &OwningTree->GetRootNode() == this)
	{
		return nullptr;
	}

	const FNavigationToolSerializedTreeNode* FoundParentTreeNode = nullptr;

	if (const FNavigationToolSerializedItem* const ParentItem = OwningTree->GetItemAtIndex(ParentIndex))
	{
		FoundParentTreeNode = OwningTree->FindTreeNode(*ParentItem);
	}

	return FoundParentTreeNode ? FoundParentTreeNode : &OwningTree->GetRootNode();
}

int32 FNavigationToolSerializedTreeNode::CalculateHeight() const
{
	int32 Height = 0;

	const FNavigationToolSerializedTreeNode* ParentTreeNode = GetParentTreeNode();
	while (ParentTreeNode)
	{
		++Height;
		ParentTreeNode = ParentTreeNode->GetParentTreeNode();
	}

	return Height;
}

TArray<const FNavigationToolSerializedTreeNode*> FNavigationToolSerializedTreeNode::FindPath(const TArray<const FNavigationToolSerializedTreeNode*>& InItems) const
{
	TArray<const FNavigationToolSerializedTreeNode*> Path;
	for (const FNavigationToolSerializedTreeNode* Item : InItems)
	{
		Path.Reset();
		const FNavigationToolSerializedTreeNode* CurrentItem = Item;
		while (CurrentItem)
		{
			if (this == CurrentItem)
			{
				Algo::Reverse(Path);
				return Path;
			}
			Path.Add(CurrentItem);
			CurrentItem = CurrentItem->GetParentTreeNode();
		}
	}
	return TArray<const FNavigationToolSerializedTreeNode*>();
}

void FNavigationToolSerializedTreeNode::Reset()
{
	GlobalIndex = INDEX_NONE;
	LocalIndex = INDEX_NONE;
	ParentIndex = INDEX_NONE;
	ChildrenIndices.Reset();
}

FNavigationToolSerializedTree::FNavigationToolSerializedTree()
{
	RootNode.OwningTree = this;
}

void FNavigationToolSerializedTree::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		UpdateTreeNodes();
	}
}

FNavigationToolSerializedTreeNode* FNavigationToolSerializedTree::FindTreeNode(const FNavigationToolSerializedItem& InItem)
{
	if (InItem.IsValid())
	{
		return ItemTreeMap.Find(InItem);
	}
	return nullptr;
}

const FNavigationToolSerializedTreeNode* FNavigationToolSerializedTree::FindTreeNode(const FNavigationToolSerializedItem& InItem) const
{
	if (InItem.IsValid())
	{
		return ItemTreeMap.Find(InItem);
	}
	return nullptr;
}

const FNavigationToolSerializedItem* FNavigationToolSerializedTree::GetItemAtIndex(int32 InIndex) const
{
	if (SceneItems.IsValidIndex(InIndex))
	{
		return &SceneItems[InIndex];
	}
	return nullptr;
}

FNavigationToolSerializedTreeNode& FNavigationToolSerializedTree::GetOrAddTreeNode(const FNavigationToolSerializedItem& InItem, const FNavigationToolSerializedItem& InParentItem)
{
	if (FNavigationToolSerializedTreeNode* const ExistingNode = FindTreeNode(InItem))
	{
		return *ExistingNode;
	}

	// If Item Tree Map did not find the Item, Scene Items should not have it too
	checkSlow(!SceneItems.Contains(InItem));

	FNavigationToolSerializedTreeNode* ParentNode = FindTreeNode(InParentItem);
	if (!ParentNode)
	{
		ParentNode = &RootNode;
	}

	FNavigationToolSerializedTreeNode TreeNode;
	TreeNode.GlobalIndex = SceneItems.Add(InItem);
	TreeNode.LocalIndex  = ParentNode->ChildrenIndices.Add(TreeNode.GlobalIndex);
	TreeNode.ParentIndex = ParentNode->GlobalIndex;
	TreeNode.OwningTree  = this;

	return ItemTreeMap.Add(InItem, MoveTemp(TreeNode));
}

const FNavigationToolSerializedTreeNode* FNavigationToolSerializedTree::FindLowestCommonAncestor(const TArray<const FNavigationToolSerializedTreeNode*>& InItems)
{
	TSet<const FNavigationToolSerializedTreeNode*> IntersectedAncestors;

	for (const FNavigationToolSerializedTreeNode* Item : InItems)
	{
		const FNavigationToolSerializedTreeNode* Parent = Item->GetParentTreeNode();
		TSet<const FNavigationToolSerializedTreeNode*> ItemAncestors;

		while (Parent)
		{
			ItemAncestors.Add(Parent);
			Parent = Parent->GetParentTreeNode();
		}

		if (IntersectedAncestors.Num() == 0)
		{
			IntersectedAncestors = ItemAncestors;
		}
		else
		{
			IntersectedAncestors = IntersectedAncestors.Intersect(ItemAncestors);

			if (IntersectedAncestors.Num() == 1)
			{
				break;
			}
		}
	}

	const FNavigationToolSerializedTreeNode* LowestCommonAncestor = nullptr;
	for (const FNavigationToolSerializedTreeNode* Item : IntersectedAncestors)
	{
		if (!LowestCommonAncestor || Item->CalculateHeight() > LowestCommonAncestor->CalculateHeight())
		{
			LowestCommonAncestor = Item;
		}
	}
	return LowestCommonAncestor;
}

bool FNavigationToolSerializedTree::CompareTreeItemOrder(const FNavigationToolSerializedTreeNode* InA, const FNavigationToolSerializedTreeNode* InB)
{
	if (!InA || !InB)
	{
		return false;
	}
	if (const FNavigationToolSerializedTreeNode* LowestCommonAncestor = FindLowestCommonAncestor({InA, InB}))
	{
		const TArray<const FNavigationToolSerializedTreeNode*> PathToA = LowestCommonAncestor->FindPath({InA});
		const TArray<const FNavigationToolSerializedTreeNode*> PathToB = LowestCommonAncestor->FindPath({InB});

		int32 Index = 0;

		int32 PathAIndex = -1;
		int32 PathBIndex = -1;

		while (PathAIndex == PathBIndex)
		{
			if (!PathToA.IsValidIndex(Index))
			{
				return true;
			}
			if (!PathToB.IsValidIndex(Index))
			{
				return false;
			}

			PathAIndex = PathToA[Index]->GetLocalIndex();
			PathBIndex = PathToB[Index]->GetLocalIndex();
			Index++;
		}
		return PathAIndex < PathBIndex;
	}
	return false;
}

void FNavigationToolSerializedTree::Reset()
{
	RootNode.Reset();
	ItemTreeMap.Reset();
	SceneItems.Reset();
}

void FNavigationToolSerializedTree::UpdateTreeNodes()
{
	for (TPair<FNavigationToolSerializedItem, FNavigationToolSerializedTreeNode>& Pair : ItemTreeMap)
	{
		FNavigationToolSerializedTreeNode& Node = Pair.Value;
		Node.OwningTree = this;
	}
}
