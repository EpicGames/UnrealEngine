// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/BuiltInComponentTypes.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneInterpolation.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneComponentRegistry.h"
#include "EntitySystem/MovieSceneEntityFactoryTemplates.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieScenePropertyBinding.h"
#include "Templates/Greater.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BuiltInComponentTypes)

namespace UE
{
namespace MovieScene
{

static constexpr int16 GInvalidBlendTarget = std::numeric_limits<int16>::lowest();
static const int16 GInitialBlendTargets[7] = {
	GInvalidBlendTarget, GInvalidBlendTarget, GInvalidBlendTarget, GInvalidBlendTarget,
	GInvalidBlendTarget, GInvalidBlendTarget, GInvalidBlendTarget
};

FHierarchicalBlendTarget::FHierarchicalBlendTarget()
	: Capacity(InlineCapacity)
{
	// Fill with invalid values
	static_assert(sizeof(Data) == sizeof(GInitialBlendTargets));
	FMemory::Memcpy(this->Data, GInitialBlendTargets, sizeof(Data));
}

FHierarchicalBlendTarget::FHierarchicalBlendTarget(const FHierarchicalBlendTarget& RHS)
	: Capacity(InlineCapacity)
{
	// Make an allocation if necessary
	if (RHS.Capacity != InlineCapacity)
	{
		int16* NewAllocation = new int16[RHS.Capacity];
		Capacity = RHS.Capacity;
		*reinterpret_cast<int16**>(Data) = NewAllocation;
	}

	// Copy the data
	FMemory::Memcpy(this->GetMemory(), RHS.GetMemory(), sizeof(int16)*Capacity);
}

FHierarchicalBlendTarget& FHierarchicalBlendTarget::operator=(const FHierarchicalBlendTarget& RHS)
{
	// Release our allocation if it's the wrong size
	if (this->Capacity != RHS.Capacity)
	{
		FreeAllocation();
	}

	// Make an allocation if necessary
	if (RHS.Capacity != InlineCapacity)
	{
		int16* NewAllocation = new int16[RHS.Capacity];
		*reinterpret_cast<int16**>(Data) = NewAllocation;
	}

	Capacity = RHS.Capacity;

	// Copy the data
	FMemory::Memcpy(this->GetMemory(), RHS.GetMemory(), sizeof(int16)*Capacity);
	return *this;
}

FHierarchicalBlendTarget::FHierarchicalBlendTarget(FHierarchicalBlendTarget&& RHS)
	: Capacity(RHS.Capacity)
{
	FMemory::Memcpy(this->Data, RHS.Data, sizeof(Data));

	// Reset RHS
	RHS.Capacity = InlineCapacity;
	static_assert(sizeof(Data) == sizeof(GInitialBlendTargets));
	FMemory::Memcpy(RHS.Data, GInitialBlendTargets, sizeof(Data));
}

FHierarchicalBlendTarget& FHierarchicalBlendTarget::operator=(FHierarchicalBlendTarget&& RHS)
{
	// Free our allocation if it is allocated
	FreeAllocation();

	// Steal RHS
	Capacity = RHS.Capacity;
	FMemory::Memcpy(this->Data, RHS.Data, sizeof(Data));

	// Reset RHS
	RHS.Capacity = InlineCapacity;
	static_assert(sizeof(Data) == sizeof(GInitialBlendTargets));
	FMemory::Memcpy(RHS.Data, GInitialBlendTargets, sizeof(Data));

	return *this;
}

FHierarchicalBlendTarget::~FHierarchicalBlendTarget()
{
	FreeAllocation();
}

void FHierarchicalBlendTarget::FreeAllocation()
{
	// Free our allocation if it is allocated
	if (Capacity != InlineCapacity)
	{
		delete[] GetMemory();
		// Reset the capacity to be on the safe side.
		// Note: this does not reinitialize the inline array to the default state
		//       since we leave that to the caller to do if necessary
		Capacity = InlineCapacity;
	}
}

void FHierarchicalBlendTarget::Grow(uint16 NewCapacity)
{
	int16* NewAllocation = new int16[NewCapacity];
	int16* OldAllocation = GetMemory();

	// Copy old allocation
	FMemory::Memmove(NewAllocation, OldAllocation, Capacity*sizeof(int16));

	// Initialize the tail of the new allocation
	for (uint16 Index = Capacity; Index < NewCapacity; ++Index)
	{
		NewAllocation[Index] = GInvalidBlendTarget;
	}

	if (Capacity != InlineCapacity)
	{
		delete[] OldAllocation;
	}

	Capacity = NewCapacity;
	*reinterpret_cast<int16**>(Data) = NewAllocation;
}

void FHierarchicalBlendTarget::Add(int16 InValue)
{
	TArrayView<int16> HBiasChain = GetAllEntries();

	const int32 Index = Algo::LowerBound(HBiasChain, InValue, TGreater<>());
	
	if (Index < HBiasChain.Num() && HBiasChain[Index] == InValue)
	{
		// Already exists
		return;
	}

	// If our insertion index is beyond our capacity we need to grow.
	const bool bAtMaxCapacity = HBiasChain[Capacity-1] != GInvalidBlendTarget;
	const bool bNeedToGrow    = bAtMaxCapacity || Index >= Capacity;

	if (bNeedToGrow)
	{
		if (Capacity == InlineCapacity)
		{
			Grow(16u);
		}
		else
		{
			Grow(Capacity + 16u);
		}

		// Re-retrieve the memory
		HBiasChain = GetAllEntries();
	}

	// Right shift by 1 if necessary
	if (Index < Capacity-1 && HBiasChain[Index] != GInvalidBlendTarget)
	{
		// Example: Inserting 500 into the following:
		// [ 1000, 100, 50, 10, 5, 0, -32768 ] w/ Capacity=7
		// Results in:
		// [ 1000, 500, 100, 50, 10, 5, 0 ] 
		// Index=1
		// NumToRelocate=5
		void* Src = &HBiasChain[Index];
		void* Dst = &HBiasChain[Index + 1];
		int32 NumToRelocate = Capacity-Index-1; // -1 to avoid buffer overrun. The last one will always get overwritten. See comment above.
		FMemory::Memmove(Dst, Src, sizeof(int16)*NumToRelocate);
	}

	// Set the value
	HBiasChain[Index] = InValue;
}

int32 FHierarchicalBlendTarget::Num() const
{
	TArrayView<const int16> HBiasChain = GetAllEntries();
	return Algo::LowerBound(HBiasChain, GInvalidBlendTarget, TGreater<>());
}

int16 FHierarchicalBlendTarget::operator[](int32 Index) const
{
	check(Index < static_cast<int32>(Capacity));
	return GetAllEntries()[Index];
}

int16* FHierarchicalBlendTarget::GetMemory()
{
	return const_cast<int16*>(const_cast<const FHierarchicalBlendTarget*>(this)->GetMemory());
}

const int16* FHierarchicalBlendTarget::GetMemory() const
{
	if (Capacity == InlineCapacity)
	{
		return reinterpret_cast<const int16*>(Data);
	}
	else
	{
		return *reinterpret_cast<const int16* const*>(Data);
	}
}

TArrayView<int16> FHierarchicalBlendTarget::GetAllEntries()
{
	return MakeArrayView(GetMemory(), Capacity);
}

TArrayView<const int16> FHierarchicalBlendTarget::GetAllEntries() const
{
	return MakeArrayView(GetMemory(), Capacity);
}

TArrayView<const int16> FHierarchicalBlendTarget::AsArray() const
{
	return GetAllEntries().Left(Num());
}

bool FObjectComponent::IsStrongReference() const
{
	return ObjectKey == FObjectKey();
}

UObject* FObjectComponent::GetObject() const
{
	if (IsStrongReference())
	{
		return ObjectPtr;
	}
	return ObjectKey.ResolveObjectPtr();
}

void AddReferencedObjectForComponent(FReferenceCollector* ReferenceCollector, FObjectComponent* ComponentData)
{
	if (ComponentData->IsStrongReference())
	{
		ReferenceCollector->AddReferencedObject(ComponentData->ObjectPtr);
	}
}

static bool GMovieSceneBuiltInComponentTypesDestroyed = false;
static TUniquePtr<FBuiltInComponentTypes> GMovieSceneBuiltInComponentTypes;

struct FBoundObjectKeyInitializer : IMutualComponentInitializer
{
	void Run(const FEntityRange& Range, const FEntityAllocationWriteContext& WriteContext) override
	{
		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

		TComponentReader<UObject*>   Objects    = Range.Allocation->ReadComponents(BuiltInComponents->BoundObject);
		TComponentWriter<FObjectKey> ObjectKeys = Range.Allocation->WriteComponents(BuiltInComponents->BoundObjectKey, WriteContext);

		for (int32 Index = 0; Index < Range.Num; ++Index)
		{
			ObjectKeys[Range.ComponentStartOffset + Index] = Objects[Range.ComponentStartOffset + Index];
		}
	}
};

FBuiltInComponentTypes::FBuiltInComponentTypes()
{
	using namespace UE::MovieScene;

	FComponentRegistry* ComponentRegistry = UMovieSceneEntitySystemLinker::GetComponents();

	ComponentRegistry->NewComponentType(&ParentEntity,          TEXT("Parent Entity"));
	ComponentRegistry->NewComponentType(&SequenceID,            TEXT("Sequence ID"));
	ComponentRegistry->NewComponentType(&InstanceHandle,        TEXT("Instance Handle"));
	ComponentRegistry->NewComponentType(&RootInstanceHandle,    TEXT("Root Instance Handle"));

	ComponentRegistry->NewComponentType(&EvalTime,              TEXT("Eval Time"));
	ComponentRegistry->NewComponentType(&EvalSeconds,           TEXT("Eval Seconds"));

	ComponentRegistry->NewComponentType(&Group,					TEXT("Entity Group"));

	ComponentRegistry->NewComponentType(&BoundObjectKey,        TEXT("Bound Object Key"));
	// Intentionally hidden from the reference graph because they are always accompanied by a BoundObjectKey which is used for garbage collection
	ComponentRegistry->NewComponentTypeNoAddReferencedObjects(&BoundObject, TEXT("Bound Object"));

	ComponentRegistry->NewComponentType(&PropertyBinding,         TEXT("Property Binding"), EComponentTypeFlags::CopyToOutput);
	ComponentRegistry->NewComponentType(&GenericObjectBinding,    TEXT("Generic Object Binding ID"));
	ComponentRegistry->NewComponentType(&BoundObjectResolver,     TEXT("Bound Object Resolver"));
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ComponentRegistry->NewComponentType(&SceneComponentBinding,   TEXT("[DEPRECATED] USceneComponent Binding ID"));
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	ComponentRegistry->NewComponentType(&SpawnableBinding,        TEXT("Spawnable Binding"));
	ComponentRegistry->NewComponentType(&TrackInstance,           TEXT("Track Instance"));
	ComponentRegistry->NewComponentType(&BoolChannel,             TEXT("Bool Channel"));
	ComponentRegistry->NewComponentType(&ByteChannel,             TEXT("Byte Channel"));
	ComponentRegistry->NewComponentType(&IntegerChannel,          TEXT("Integer Channel"));
	ComponentRegistry->NewComponentType(&FloatChannel[0],         TEXT("Float Channel 0"));
	ComponentRegistry->NewComponentType(&FloatChannel[1],         TEXT("Float Channel 1"));
	ComponentRegistry->NewComponentType(&FloatChannel[2],         TEXT("Float Channel 2"));
	ComponentRegistry->NewComponentType(&FloatChannel[3],         TEXT("Float Channel 3"));
	ComponentRegistry->NewComponentType(&FloatChannel[4],         TEXT("Float Channel 4"));
	ComponentRegistry->NewComponentType(&FloatChannel[5],         TEXT("Float Channel 5"));
	ComponentRegistry->NewComponentType(&FloatChannel[6],         TEXT("Float Channel 6"));
	ComponentRegistry->NewComponentType(&FloatChannel[7],         TEXT("Float Channel 7"));
	ComponentRegistry->NewComponentType(&FloatChannel[8],         TEXT("Float Channel 8"));
	ComponentRegistry->NewComponentType(&DoubleChannel[0],        TEXT("Double Channel 0"));
	ComponentRegistry->NewComponentType(&DoubleChannel[1],        TEXT("Double Channel 1"));
	ComponentRegistry->NewComponentType(&DoubleChannel[2],        TEXT("Double Channel 2"));
	ComponentRegistry->NewComponentType(&DoubleChannel[3],        TEXT("Double Channel 3"));
	ComponentRegistry->NewComponentType(&DoubleChannel[4],        TEXT("Double Channel 4"));
	ComponentRegistry->NewComponentType(&DoubleChannel[5],        TEXT("Double Channel 5"));
	ComponentRegistry->NewComponentType(&DoubleChannel[6],        TEXT("Double Channel 6"));
	ComponentRegistry->NewComponentType(&DoubleChannel[7],        TEXT("Double Channel 7"));
	ComponentRegistry->NewComponentType(&DoubleChannel[8],        TEXT("Double Channel 8"));
	ComponentRegistry->NewComponentType(&WeightChannel,           TEXT("Weight Channel"));
	ComponentRegistry->NewComponentType(&StringChannel,           TEXT("String Channel"));
	ComponentRegistry->NewComponentType(&TextChannel,             TEXT("Text Channel"));
	ComponentRegistry->NewComponentType(&ObjectPathChannel,       TEXT("Object Path Channel"));

	ComponentRegistry->NewComponentType(&CachedInterpolation[0],   TEXT("Cached Interpolation [0]"));
	ComponentRegistry->NewComponentType(&CachedInterpolation[1],   TEXT("Cached Interpolation [1]"));
	ComponentRegistry->NewComponentType(&CachedInterpolation[2],   TEXT("Cached Interpolation [2]"));
	ComponentRegistry->NewComponentType(&CachedInterpolation[3],   TEXT("Cached Interpolation [3]"));
	ComponentRegistry->NewComponentType(&CachedInterpolation[4],   TEXT("Cached Interpolation [4]"));
	ComponentRegistry->NewComponentType(&CachedInterpolation[5],   TEXT("Cached Interpolation [5]"));
	ComponentRegistry->NewComponentType(&CachedInterpolation[6],   TEXT("Cached Interpolation [6]"));
	ComponentRegistry->NewComponentType(&CachedInterpolation[7],   TEXT("Cached Interpolation [7]"));
	ComponentRegistry->NewComponentType(&CachedInterpolation[8],   TEXT("Cached Interpolation [8]"));
	ComponentRegistry->NewComponentType(&CachedWeightChannelInterpolation, TEXT("Cached Weight Channel Interpolation"));

	ComponentRegistry->NewComponentType(&Easing,                  TEXT("Easing"));
	ComponentRegistry->NewComponentType(&EasingResult,            TEXT("Easing Result"));
	ComponentRegistry->NewComponentType(&HierarchicalEasingChannel,  TEXT("Hierarchical Easing Channel"));
	ComponentRegistry->NewComponentType(&HierarchicalBlendTarget,    TEXT("Hierarchical Blend Target"));
	ComponentRegistry->NewComponentType(&HierarchicalEasingProvider, TEXT("Hierarchical Easing Provider"));

	ComponentRegistry->NewComponentType(&BlenderType,           TEXT("Blender System Type"), EComponentTypeFlags::CopyToChildren);
	ComponentRegistry->NewComponentType(&BlendingOrder,			TEXT("Blending Order"));
	ComponentRegistry->NewComponentType(&BlendChannelInput,     TEXT("Blend Channel Input"));
	ComponentRegistry->NewComponentType(&HierarchicalBias,      TEXT("Hierarchical Bias"));
	ComponentRegistry->NewComponentType(&BlendChannelOutput,    TEXT("Blend Channel Output"));
	ComponentRegistry->NewComponentType(&InitialValueIndex,     TEXT("Initial Value Index"), EComponentTypeFlags::MigrateToOutput);

	ComponentRegistry->NewComponentType(&CustomPropertyIndex,   TEXT("Custom Property Index"));			// Not EComponentTypeFlags::Preserved because the system property manager will always ensure that the component is added to the correct entity
	ComponentRegistry->NewComponentType(&FastPropertyOffset,    TEXT("Fast Property Offset"));			// Not EComponentTypeFlags::Preserved because the system property manager will always ensure that the component is added to the correct entity
	ComponentRegistry->NewComponentType(&SlowProperty,          TEXT("Slow Property Binding"));			// Not EComponentTypeFlags::Preserved because the system property manager will always ensure that the component is added to the correct entity
	ComponentRegistry->NewComponentType(&VariantPropertyTypeIndex,TEXT("Variant Property Type Index"));
	ComponentRegistry->NewComponentType(&BoolResult,            TEXT("Bool Result"));
	ComponentRegistry->NewComponentType(&ByteResult,            TEXT("Byte Result"));
	ComponentRegistry->NewComponentType(&IntegerResult,         TEXT("Integer Result"));
	ComponentRegistry->NewComponentType(&DoubleResult[0],       TEXT("Double Result 0"));
	ComponentRegistry->NewComponentType(&DoubleResult[1],       TEXT("Double Result 1"));
	ComponentRegistry->NewComponentType(&DoubleResult[2],       TEXT("Double Result 2"));
	ComponentRegistry->NewComponentType(&DoubleResult[3],       TEXT("Double Result 3"));
	ComponentRegistry->NewComponentType(&DoubleResult[4],       TEXT("Double Result 4"));
	ComponentRegistry->NewComponentType(&DoubleResult[5],       TEXT("Double Result 5"));
	ComponentRegistry->NewComponentType(&DoubleResult[6],       TEXT("Double Result 6"));
	ComponentRegistry->NewComponentType(&DoubleResult[7],       TEXT("Double Result 7"));
	ComponentRegistry->NewComponentType(&DoubleResult[8],       TEXT("Double Result 8"));
	ComponentRegistry->NewComponentType(&StringResult,          TEXT("String Result"));
	ComponentRegistry->NewComponentType(&TextResult,            TEXT("Text Result"));
	ComponentRegistry->NewComponentType(&ObjectResult,          TEXT("Object Result"));
	ComponentRegistry->NewComponentType(&TimeWarp,              TEXT("Time Warp"));

	ComponentRegistry->NewComponentType(&BaseByte,			    TEXT("Base Byte"));
	ComponentRegistry->NewComponentType(&BaseInteger,			TEXT("Base Integer"));
	ComponentRegistry->NewComponentType(&BaseDouble[0],         TEXT("Base Double 0"));
	ComponentRegistry->NewComponentType(&BaseDouble[1],         TEXT("Base Double 1"));
	ComponentRegistry->NewComponentType(&BaseDouble[2],         TEXT("Base Double 2"));
	ComponentRegistry->NewComponentType(&BaseDouble[3],         TEXT("Base Double 3"));
	ComponentRegistry->NewComponentType(&BaseDouble[4],         TEXT("Base Double 4"));
	ComponentRegistry->NewComponentType(&BaseDouble[5],         TEXT("Base Double 5"));
	ComponentRegistry->NewComponentType(&BaseDouble[6],         TEXT("Base Double 6"));
	ComponentRegistry->NewComponentType(&BaseDouble[7],         TEXT("Base Double 7"));
	ComponentRegistry->NewComponentType(&BaseDouble[8],         TEXT("Base Double 8"));

	ComponentRegistry->NewComponentType(&BaseValueEvalTime,     TEXT("Base Value Eval Time"));
	ComponentRegistry->NewComponentType(&BaseValueEvalSeconds,  TEXT("Base Value Eval Seconds"));

	ComponentRegistry->NewComponentType(&WeightResult,          TEXT("Weight Result"));
	ComponentRegistry->NewComponentType(&WeightAndEasingResult, TEXT("Weight/Easing Result"));

	ComponentRegistry->NewComponentType(&TrackInstance,         TEXT("Track Instance"));
	ComponentRegistry->NewComponentType(&TrackInstanceInput,    TEXT("Track Instance Input"));

	ComponentRegistry->NewComponentType(&EvaluationHook,        TEXT("Evaluation Hook"));
	ComponentRegistry->NewComponentType(&EvaluationHookFlags,   TEXT("Evaluation Hook Flags"), EComponentTypeFlags::Preserved);
	ComponentRegistry->NewComponentType(&Condition, TEXT("Condition"));

	ComponentRegistry->NewComponentType(&Interrogation.InputKey,  TEXT("Interrogation Input"));
	ComponentRegistry->NewComponentType(&Interrogation.Instance,  TEXT("Interrogation Instance"));
	ComponentRegistry->NewComponentType(&Interrogation.OutputKey, TEXT("Interrogation Output"));

	ComponentRegistry->NewComponentType(&BindingLifetime, TEXT("Binding Lifetime"));

	Tags.RestoreState            = ComponentRegistry->NewTag(TEXT("Is Restore State Entity"));
	Tags.IgnoreHierarchicalBias  = ComponentRegistry->NewTag(TEXT("Ignore Hierarchical Bias"));
	Tags.BlendHierarchicalBias   = ComponentRegistry->NewTag(TEXT("Blend Hierarchical Bias"));

	Tags.AbsoluteBlend           = ComponentRegistry->NewTag(TEXT("Is Absolute Blend"));
	Tags.RelativeBlend           = ComponentRegistry->NewTag(TEXT("Is Relative Blend"));
	Tags.AdditiveBlend           = ComponentRegistry->NewTag(TEXT("Is Additive Blend"));
	Tags.AdditiveFromBaseBlend   = ComponentRegistry->NewTag(TEXT("Is Additive From Base Blend"));
	Tags.OverrideBlend			 = ComponentRegistry->NewTag(TEXT("Is Override Blend"));

	Tags.AdditiveAnimation       = ComponentRegistry->NewTag(TEXT("Is Additive Animation"));

	Tags.NeedsLink               = ComponentRegistry->NewTag(TEXT("Needs Link"));
	Tags.NeedsUnlink             = ComponentRegistry->NewTag(TEXT("Needs Unlink"));
	Tags.HasUnresolvedBinding    = ComponentRegistry->NewTag(TEXT("Has Unresolved Binding"));
	Tags.HasAssignedInitialValue = ComponentRegistry->NewTag(TEXT("Has Assigned Initial Value"), EComponentTypeFlags::MigrateToOutput);
	Tags.Root                    = ComponentRegistry->NewTag(TEXT("Root"));
	Tags.SubInstance             = ComponentRegistry->NewTag(TEXT("Sub Instance"), EComponentTypeFlags::CopyToChildren);
	Tags.ImportedEntity          = ComponentRegistry->NewTag(TEXT("Imported Entity"));
	Tags.Finished                = ComponentRegistry->NewTag(TEXT("Finished Evaluating"));
	Tags.Ignored                 = ComponentRegistry->NewTag(TEXT("Ignored"));
	Tags.DontOptimizeConstants   = ComponentRegistry->NewTag(TEXT("Don't Optimize Constants"));
	Tags.FixedTime               = ComponentRegistry->NewTag(TEXT("Fixed Time"));
	Tags.PreRoll                 = ComponentRegistry->NewTag(TEXT("Pre Roll"));
	Tags.SectionPreRoll          = ComponentRegistry->NewTag(TEXT("Section Pre Roll"));
	Tags.AlwaysCacheInitialValue = ComponentRegistry->NewTag(TEXT("Always Cache Initial Value"), EComponentTypeFlags::CopyToOutput);
	Tags.OldStyleSpawnable       = ComponentRegistry->NewTag(TEXT("Old Style Spawnable"));
	Tags.ExternalBlending        = ComponentRegistry->NewTag(TEXT("External Blending"));

	SymbolicTags.CreatesEntities = ComponentRegistry->NewTag(TEXT("~~ SYMBOLIC ~~ Creates Entities"));

	FinishedMask.SetAll({ Tags.NeedsUnlink, Tags.Finished });

	{
		FMutuallyInclusiveComponentParams ObjectKeyParams;
		ObjectKeyParams.CustomInitializer = MakeUnique<FBoundObjectKeyInitializer>();
		ObjectKeyParams.Type = EMutuallyInclusiveComponentType::Mandatory;

		ComponentRegistry->Factories.DefineMutuallyInclusiveComponents(BoundObject, { BoundObjectKey }, MoveTemp(ObjectKeyParams));
	}

	// New children always need link
	ComponentRegistry->Factories.DefineChildComponent(Tags.NeedsLink);

	// Always copy these tags over to children
	ComponentRegistry->Factories.DefineChildComponent(Tags.RestoreState,  Tags.RestoreState);
	ComponentRegistry->Factories.DefineChildComponent(Tags.IgnoreHierarchicalBias,  Tags.IgnoreHierarchicalBias);
	ComponentRegistry->Factories.DefineChildComponent(Tags.BlendHierarchicalBias,  Tags.BlendHierarchicalBias);
	ComponentRegistry->Factories.DefineChildComponent(Tags.AbsoluteBlend, Tags.AbsoluteBlend);
	ComponentRegistry->Factories.DefineChildComponent(Tags.RelativeBlend, Tags.RelativeBlend);
	ComponentRegistry->Factories.DefineChildComponent(Tags.AdditiveBlend, Tags.AdditiveBlend);
	ComponentRegistry->Factories.DefineChildComponent(Tags.OverrideBlend, Tags.OverrideBlend);
	ComponentRegistry->Factories.DefineChildComponent(Tags.AdditiveFromBaseBlend, Tags.AdditiveFromBaseBlend);
	ComponentRegistry->Factories.DefineChildComponent(Tags.AdditiveAnimation, Tags.AdditiveAnimation);
	ComponentRegistry->Factories.DefineChildComponent(Tags.FixedTime,     Tags.FixedTime);
	ComponentRegistry->Factories.DefineChildComponent(Tags.PreRoll,       Tags.PreRoll);
	ComponentRegistry->Factories.DefineChildComponent(Tags.SectionPreRoll,Tags.SectionPreRoll);
	ComponentRegistry->Factories.DefineChildComponent(Tags.AlwaysCacheInitialValue,Tags.AlwaysCacheInitialValue);

	ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(Tags.SectionPreRoll,Tags.PreRoll);

	ComponentRegistry->Factories.DuplicateChildComponent(EvalTime);
	ComponentRegistry->Factories.DuplicateChildComponent(EvalSeconds);
	ComponentRegistry->Factories.DuplicateChildComponent(BaseValueEvalTime);
	ComponentRegistry->Factories.DuplicateChildComponent(BaseValueEvalSeconds);

	ComponentRegistry->Factories.DuplicateChildComponent(SequenceID);
	ComponentRegistry->Factories.DuplicateChildComponent(InstanceHandle);
	ComponentRegistry->Factories.DuplicateChildComponent(RootInstanceHandle);
	ComponentRegistry->Factories.DuplicateChildComponent(PropertyBinding);
	ComponentRegistry->Factories.DuplicateChildComponent(HierarchicalBias);
	ComponentRegistry->Factories.DuplicateChildComponent(BlendingOrder);
	ComponentRegistry->Factories.DuplicateChildComponent(BindingLifetime);
	ComponentRegistry->Factories.ConditionallyDuplicateChildComponent(GenericObjectBinding, FComponentMask({BindingLifetime}));


	// Children always need a Parent - these are initialized by the tasks that create them
	{
		ComponentRegistry->Factories.DefineChildComponent(FComponentTypeID::Invalid(), ParentEntity);
	}
	
	// Bool channel relationships
	{
		ComponentRegistry->Factories.DuplicateChildComponent(BoolChannel);
		ComponentRegistry->Factories.DuplicateChildComponent(BoolResult);
		ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(BoolChannel, BoolResult);
		ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(BoolResult, EvalTime);
	}

	// Byte channel relationships
	{
		ComponentRegistry->Factories.DuplicateChildComponent(ByteChannel);
		ComponentRegistry->Factories.DuplicateChildComponent(ByteResult);
		ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(ByteChannel, ByteResult);
		ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(ByteChannel, EvalTime);

		ComponentRegistry->Factories.DefineComplexInclusiveComponents(
				FComplexInclusivityFilter::All({ ByteResult, BaseValueEvalTime, Tags.AdditiveFromBaseBlend }),
				BaseByte);
	}
	
	// Integer channel relationships
	{
		ComponentRegistry->Factories.DuplicateChildComponent(IntegerChannel);
		ComponentRegistry->Factories.DuplicateChildComponent(IntegerResult);
		ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(IntegerChannel, IntegerResult);
		ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(IntegerChannel, EvalTime);

		ComponentRegistry->Factories.DefineComplexInclusiveComponents(
				FComplexInclusivityFilter::All({ IntegerResult, BaseValueEvalTime, Tags.AdditiveFromBaseBlend }),
				BaseInteger);
	}

	// Float channel relationships
	{
		static_assert(
				UE_ARRAY_COUNT(FloatChannel) == UE_ARRAY_COUNT(BaseDouble), 
				"Base floats and float results should have the same size.");
		static_assert(
				UE_ARRAY_COUNT(FloatChannel) == UE_ARRAY_COUNT(CachedInterpolation),
				"Float channel flags and flor channels should have the same size.");

		// Duplicate float channels
		for (int32 Index = 0; Index < UE_ARRAY_COUNT(FloatChannel); ++Index)
		{
			ComponentRegistry->Factories.DuplicateChildComponent(FloatChannel[Index]);
			ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(FloatChannel[Index], DoubleResult[Index]);
			ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(FloatChannel[Index], EvalTime);
			ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(FloatChannel[Index], CachedInterpolation[Index]);
		}
	}

	// Double channel relationships
	{
		static_assert(
				UE_ARRAY_COUNT(DoubleChannel) == UE_ARRAY_COUNT(BaseDouble), 
				"Base doubles and double results should have the same size.");
		static_assert(
				UE_ARRAY_COUNT(DoubleChannel) == UE_ARRAY_COUNT(CachedInterpolation),
				"Double channel flags and flor channels should have the same size.");

		// Duplicate double channels
		for (int32 Index = 0; Index < UE_ARRAY_COUNT(DoubleChannel); ++Index)
		{
			ComponentRegistry->Factories.DuplicateChildComponent(DoubleChannel[Index]);
			ComponentRegistry->Factories.DuplicateChildComponent(DoubleResult[Index]);
			ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(DoubleChannel[Index], DoubleResult[Index]);
			ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(DoubleChannel[Index], EvalTime);
			ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(DoubleChannel[Index], CachedInterpolation[Index]);
		}
	}

	{
		// Associate result and base values.
		for (int32 Index = 0; Index < UE_ARRAY_COUNT(DoubleResult); ++Index)
		{
			ResultToBase.Add(DoubleResult[Index], BaseDouble[Index]);
		}

		// Create base double components for any channels that are meant to be additive from base.
		for (int32 Index = 0; Index < UE_ARRAY_COUNT(BaseDouble); ++Index)
		{
			ComponentRegistry->Factories.DefineComplexInclusiveComponents(
					FComplexInclusivityFilter::All({ DoubleResult[Index], BaseValueEvalTime, Tags.AdditiveFromBaseBlend }),
					BaseDouble[Index]);
		}

		// If normal evaluation requires the time in seconds, the base value evaluation probably also requires
		// a base eval time in seconds.
		ComponentRegistry->Factories.DefineComplexInclusiveComponents(
				FComplexInclusivityFilter::All({ EvalSeconds, BaseValueEvalTime }),
				BaseValueEvalSeconds);
	}

	// Easing component relationships
	{
		// Easing components should be duplicated to children
		ComponentRegistry->Factories.DuplicateChildComponent(Easing);
		ComponentRegistry->Factories.DuplicateChildComponent(HierarchicalEasingChannel);

		// Easing needs a time to evaluate
		ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(Easing, EvalTime);
		ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(Easing, EasingResult);
	}

	// Weight channel relationships
	{
		// Weight channel components should be duplicated to children
		ComponentRegistry->Factories.DuplicateChildComponent(WeightChannel);
		ComponentRegistry->Factories.DuplicateChildComponent(WeightResult);

		// Weight channel components need a time and result to evaluate
		ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(WeightChannel, EvalTime);
		ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(WeightChannel, WeightResult);
		ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(WeightChannel, CachedWeightChannelInterpolation);
	}

	// Weight and easing result component relationship
	{
		ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(HierarchicalEasingChannel, WeightAndEasingResult);
		ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(EasingResult, WeightAndEasingResult);
		ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(WeightResult, WeightAndEasingResult);
	}
	
	// String channel relationships
	{
		ComponentRegistry->Factories.DuplicateChildComponent(StringChannel);
		ComponentRegistry->Factories.DuplicateChildComponent(StringResult);

		ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(StringChannel, EvalTime);
		ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(StringChannel, StringResult);
	}

	// Text channel relationships
	{
		ComponentRegistry->Factories.DuplicateChildComponent(TextChannel);
		ComponentRegistry->Factories.DuplicateChildComponent(TextResult);

		ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(TextChannel, EvalTime);
		ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(TextChannel, TextResult);
	}

	// Object path channel relationships
	{
		// Object channel components should be duplicated to children
		ComponentRegistry->Factories.DuplicateChildComponent(ObjectPathChannel);
		ComponentRegistry->Factories.DuplicateChildComponent(ObjectResult);

		// Object channel components need a time and result to evaluate
		ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(ObjectPathChannel, EvalTime);
		ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(ObjectPathChannel, ObjectResult);
	}

	// Track instances always produce inputs
	{
		auto InitInput = [](const FMovieSceneTrackInstanceComponent& InInstance, FTrackInstanceInputComponent& OutInput)
		{
			OutInput.Section = InInstance.Owner;
		};
		ComponentRegistry->Factories.DefineChildComponent(TrackInstance, TrackInstanceInput, InitInput);
	}

	{
		ComponentRegistry->Factories.DefineChildComponent(EvaluationHook, EvaluationHook);
		ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(EvaluationHook, EvalTime);
		ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(EvaluationHook, EvaluationHookFlags);
	}

	RequiresInstantiationMask.Set(Tags.NeedsLink);
}

FBuiltInComponentTypes::~FBuiltInComponentTypes()
{
}

void FBuiltInComponentTypes::Destroy()
{
	GMovieSceneBuiltInComponentTypes.Reset();
	GMovieSceneBuiltInComponentTypesDestroyed = true;
}

FBuiltInComponentTypes* FBuiltInComponentTypes::Get()
{
	if (!GMovieSceneBuiltInComponentTypes.IsValid())
	{
		check(!GMovieSceneBuiltInComponentTypesDestroyed);
		GMovieSceneBuiltInComponentTypes.Reset(new FBuiltInComponentTypes);
	}
	return GMovieSceneBuiltInComponentTypes.Get();
}

FComponentTypeID FBuiltInComponentTypes::GetBaseValueComponentType(const FComponentTypeID& InResultComponentType)
{
	if (FComponentTypeID* BaseComponentType = ResultToBase.Find(InResultComponentType))
	{
		return *BaseComponentType;
	}
	return FComponentTypeID::Invalid();
}

} // namespace MovieScene
} // namespace UE

