// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/EntityAllocationIterator.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "EntitySystem/MovieSceneEntityManager.h"


namespace UE
{
namespace MovieScene
{

FEntityAllocationIterator::FEntityAllocationIterator(const FEntityManager* InManager)
	: Filter(nullptr)
	, Manager(InManager)
	, AllocationIndex(Manager->EntityAllocationMasks.GetMaxIndex())
{}

FEntityAllocationIterator::FEntityAllocationIterator(const FEntityManager* InManager, const FEntityComponentFilter* InFilter)
	: Filter(InFilter)
	, Manager(InManager)
{
	Manager->EnterIteration();
	AllocationIndex = FindMatchingAllocationStartingAt(0);
}

FEntityAllocationIterator::FEntityAllocationIterator(FEntityAllocationIterator&& RHS)
{
	*this = MoveTemp(RHS);
}

FEntityAllocationIterator& FEntityAllocationIterator::operator=(FEntityAllocationIterator&& RHS)
{
	Manager = RHS.Manager;
	Filter = RHS.Filter;
	AllocationIndex = RHS.AllocationIndex;

	// Wipe out the filter so it doesn't decrement the iteration count on destruction
	RHS.Filter= nullptr;
	return *this;
}

FEntityAllocationIterator::~FEntityAllocationIterator()
{
	if (Filter)
	{
		Manager->ExitIteration();
	}
}

bool FEntityAllocationIterator::operator!=(const FEntityAllocationIterator& Other) const
{
	return AllocationIndex != Other.AllocationIndex;
}

FEntityAllocationIterator& FEntityAllocationIterator::operator++()
{
	AllocationIndex = FindMatchingAllocationStartingAt(AllocationIndex + 1);
	return *this;
}


FEntityAllocationIteratorItem FEntityAllocationIterator::operator*() const
{
	return FEntityAllocationIteratorItem(Manager, AllocationIndex);
}


int32 FEntityAllocationIterator::FindMatchingAllocationStartingAt(int32 Index) const
{
	const FEntityComponentFilter& GlobalIterationFilter = Manager->GetGlobalIterationFilter();
	const bool bHasGlobalFilter = GlobalIterationFilter.IsValid();
	for ( ; Index < Manager->EntityAllocationMasks.GetMaxIndex(); ++Index)
	{
		if (!Manager->EntityAllocationMasks.IsAllocated(Index))
		{
			continue;
		}

		if (bHasGlobalFilter == true)
		{
			if (!GlobalIterationFilter.Match(Manager->EntityAllocationMasks[Index]))
			{
				continue;
			}
		}

		if (Filter->Match(Manager->EntityAllocationMasks[Index]) && Manager->EntityAllocations[Index]->Num() > 0)
		{
			return Index;
		}
	}

	return Index;
}


} // namespace MovieScene
} // namespace UE
