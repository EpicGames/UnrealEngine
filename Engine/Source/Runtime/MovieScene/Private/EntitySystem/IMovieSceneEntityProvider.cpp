// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntityManager.h"

#include "Evaluation/MovieSceneEvaluationField.h"

#include "MovieSceneSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IMovieSceneEntityProvider)

namespace UE
{
namespace MovieScene
{

FGuid FEntityImportParams::GetObjectBindingID() const
{
	return SharedMetaData ? SharedMetaData->ObjectBindingID : FGuid();
}

FMovieSceneEntityID FImportedEntity::Manufacture(const FEntityImportParams& Params, FEntityManager* EntityManager)
{
	using namespace MovieScene;

	FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();

	auto BaseBuilder = FEntityBuilder()
	.AddTag(Components->Tags.NeedsLink)
	.AddTag(Components->Tags.ImportedEntity)
	.AddTagConditional(Components->Tags.AlwaysCacheInitialValue, Params.Sequence.bDynamicWeighting)
	.AddConditional(Components->SequenceID, Params.Sequence.SequenceID, Params.Sequence.SequenceID != MovieSceneSequenceID::Root)
	.AddConditional(Components->RootInstanceHandle, Params.Sequence.RootInstanceHandle, Params.Sequence.RootInstanceHandle.IsValid())
	.AddConditional(Components->InstanceHandle, Params.Sequence.InstanceHandle, Params.Sequence.InstanceHandle.IsValid())
	.AddConditional(Components->Condition, Params.EntityMetaData ? Params.EntityMetaData->Condition.Get() : nullptr, Params.EntityMetaData && Params.EntityMetaData->Condition.IsValid());

	FEntityAllocationWriteContext WriteContext(*EntityManager);

	bool bAddMutualComponents = false;

	FComponentMask NewMask;
	BaseBuilder.GenerateType(EntityManager, NewMask, bAddMutualComponents);
	for (TInlineValue<IEntityBuilder>& Builder : Builders)
	{
		Builder->GenerateType(EntityManager, NewMask, bAddMutualComponents);
	}

	FMutualComponentInitializers MutualInitializers;
	EMutuallyInclusiveComponentType MutualTypes = bAddMutualComponents ? EMutuallyInclusiveComponentType::All : EMutuallyInclusiveComponentType::Mandatory;
	EntityManager->GetComponents()->Factories.ComputeMutuallyInclusiveComponents(MutualTypes, NewMask, MutualInitializers);

	FEntityInfo NewEntity = EntityManager->AllocateEntity(NewMask);

	BaseBuilder.Initialize(EntityManager, NewEntity);
	for (TInlineValue<IEntityBuilder>& Builder : Builders)
	{
		Builder->Initialize(EntityManager, NewEntity);
	}

	// Run mutual initializers after the builder has actually constructed the entity
	// otherwise mutual components would be reading garbage
	MutualInitializers.Execute(NewEntity.Data.AsRange(), WriteContext);

	return NewEntity.EntityID;
}

} // namespace MovieScene
} // namespace UE

void IMovieSceneEntityProvider::ImportEntity(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	ImportEntityImpl(EntityLinker, Params, OutImportedEntity);
}

void IMovieSceneEntityProvider::InterrogateEntity(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	InterrogateEntityImpl(EntityLinker, Params, OutImportedEntity);
}
