// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/IMovieSceneTaskScheduler.h"
#include "EntitySystem/MovieSceneTaskScheduler.h"

namespace UE::MovieScene
{

FTaskID IEntitySystemScheduler::AddNullTask()
{
	return static_cast<FEntitySystemScheduler*>(this)->AddNullTask();
}

FTaskID IEntitySystemScheduler::AddTask(const FTaskParams& InParams, TSharedPtr<ITaskContext> InTaskContext, TaskFunctionPtr InTaskFunction)
{
	return static_cast<FEntitySystemScheduler*>(this)->AddTask(InParams, InTaskContext, InTaskFunction);
}

FTaskID IEntitySystemScheduler::CreateForkedAllocationTask(const FTaskParams& InParams, TSharedPtr<ITaskContext> InTaskContext, TaskFunctionPtr InTaskFunction, TFunctionRef<void(FEntityAllocationIteratorItem,TArray<FPreLockedDataPtr>&)> InPreLockFunc, const FEntityComponentFilter& Filter, const FComponentMask& ReadDeps, const FComponentMask& WriteDeps)
{
	return static_cast<FEntitySystemScheduler*>(this)->CreateForkedAllocationTask(InParams, InTaskContext, InTaskFunction, InPreLockFunc, Filter, ReadDeps, WriteDeps);
}

void IEntitySystemScheduler::AddPrerequisite(FTaskID Prerequisite, FTaskID Subsequent)
{
	return static_cast<FEntitySystemScheduler*>(this)->AddPrerequisite(Prerequisite, Subsequent);
}

void IEntitySystemScheduler::AddChildBack(FTaskID Parent, FTaskID Child)
{
	return static_cast<FEntitySystemScheduler*>(this)->AddChildBack(Parent, Child);
}

void IEntitySystemScheduler::AddChildFront(FTaskID Parent, FTaskID Child)
{
	return static_cast<FEntitySystemScheduler*>(this)->AddChildFront(Parent, Child);
}

} // namespace UE::MovieScene