// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneBindingLifetimeSystem.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/BuiltInComponentTypes.h"

#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "IMovieScenePlayer.h"
#include "MovieSceneSequence.h"

#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "EntitySystem/MovieSceneEntityRange.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntityInstantiatorSystem.h"
#include "Evaluation/MovieSceneEvaluationOperand.h"

#include "MovieScene.h"
#include "MovieSceneExecutionToken.h"
#include "IMovieScenePlaybackClient.h"
#include "EntitySystem/MovieSceneSpawnablesSystem.h"
#include "MovieSceneBindingEventReceiverInterface.h"
#include "EntitySystem/MovieSceneBoundObjectInstantiator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneBindingLifetimeSystem)
#define LOCTEXT_NAMESPACE "MovieSceneBindingLifetimeSystem"


UMovieSceneBindingLifetimeSystem::UMovieSceneBindingLifetimeSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	Phase = UE::MovieScene::ESystemPhase::Spawn | UE::MovieScene::ESystemPhase::Instantiation;
	FBuiltInComponentTypes* BuiltInComponentTypes = FBuiltInComponentTypes::Get();
	RelevantComponent = BuiltInComponentTypes->BindingLifetime;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(GetClass(), UMovieSceneSpawnablesSystem::StaticClass());
		DefineImplicitPrerequisite(UMovieSceneGenericBoundObjectInstantiator::StaticClass(), GetClass());
	}
}

void UMovieSceneBindingLifetimeSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;
	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	if (!Linker->EntityManager.Contains(FEntityComponentFilter().Any({ BuiltInComponents->Tags.NeedsLink, BuiltInComponents->Tags.NeedsUnlink })))
	{
		return;
	}

	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();
	TSharedRef<FMovieSceneEntitySystemRunner> Runner = Linker->GetRunner();

	if (Runner->GetCurrentPhase() == ESystemPhase::Spawn)
	{
		bool bLink = false;
		auto SetBindingActivation = [InstanceRegistry, &bLink](FMovieSceneEntityID EntityID, FGuid ObjectBindingID, const UObject* const* OptionalBoundObject, FInstanceHandle InstanceHandle, const FMovieSceneBindingLifetimeComponentData& BindingLifetime)
			{
				const FSequenceInstance& SequenceInstance = InstanceRegistry->GetInstance(InstanceHandle);

				FMovieSceneSequenceID SequenceID = SequenceInstance.GetSequenceID();
				TSharedRef<const FSharedPlaybackState> SharedPlaybackState = SequenceInstance.GetSharedPlaybackState();
				FMovieSceneEvaluationState* EvaluationState = SharedPlaybackState->FindCapability<FMovieSceneEvaluationState>();
				IMovieScenePlayer* Player = FPlayerIndexPlaybackCapability::GetPlayer(SharedPlaybackState);
				if (EvaluationState)
				{
					// For now we use the linking/unlinking of the inactive ranges to set the binding activations
					if (BindingLifetime.BindingLifetimeState == EMovieSceneBindingLifetimeState::Inactive)
					{
						EvaluationState->SetBindingActivation(ObjectBindingID, SequenceID, !bLink);
					}
					else if (!bLink)
					{
						if (OptionalBoundObject)
						{
							if (UObject* BoundObject = const_cast<UObject*>(*OptionalBoundObject))
							{
								if (BoundObject->Implements<UMovieSceneBindingEventReceiverInterface>())
								{
									const TScriptInterface<IMovieSceneBindingEventReceiverInterface> BindingEventReceiver = BoundObject;
									if (BindingEventReceiver.GetObject() && Player)
									{
										FMovieSceneObjectBindingID BindingID = UE::MovieScene::FRelativeObjectBindingID(MovieSceneSequenceID::Root, SequenceID, ObjectBindingID, SharedPlaybackState);
										// The OnBound event is handled by the BindingLifetimeActivationSystem
										IMovieSceneBindingEventReceiverInterface::Execute_OnObjectUnboundBySequencer(BindingEventReceiver.GetObject(), Cast<UMovieSceneSequencePlayer>(Player->AsUObject()), BindingID);
									}
								}
							}
						}

						// Invalidate the binding, forcing it to be rebound
						EvaluationState->Invalidate(ObjectBindingID, SequenceID);
					}
				}
			};
		// Unlink stale bindinglifetime entities
		FEntityTaskBuilder()
			.ReadEntityIDs()
			.Read(BuiltInComponents->GenericObjectBinding)
			.ReadOptional(BuiltInComponents->BoundObject)
			.Read(BuiltInComponents->InstanceHandle)
			.Read(BuiltInComponents->BindingLifetime)
			.FilterAll({ BuiltInComponents->Tags.NeedsUnlink })
			.Iterate_PerEntity(&Linker->EntityManager, SetBindingActivation);

		// Link new bindinglifetime entities
		bLink = true;
		FEntityTaskBuilder()
			.ReadEntityIDs()
			.Read(BuiltInComponents->GenericObjectBinding)
			.ReadOptional(BuiltInComponents->BoundObject)
			.Read(BuiltInComponents->InstanceHandle)
			.Read(BuiltInComponents->BindingLifetime)
			.FilterAll({ BuiltInComponents->Tags.NeedsLink })
			.Iterate_PerEntity(&Linker->EntityManager, SetBindingActivation);
	}
	else // Instantiation- we only care here about linking activate entities
	{
		auto SendBoundMessage = [InstanceRegistry](FMovieSceneEntityID EntityID, FGuid ObjectBindingID, FInstanceHandle InstanceHandle, const FMovieSceneBindingLifetimeComponentData& BindingLifetime, UObject* BoundObject)
			{
				const FSequenceInstance& SequenceInstance = InstanceRegistry->GetInstance(InstanceHandle);

				FMovieSceneSequenceID SequenceID = SequenceInstance.GetSequenceID();
				TSharedRef<const FSharedPlaybackState> SharedPlaybackState = SequenceInstance.GetSharedPlaybackState();
				IMovieScenePlayer* Player = FPlayerIndexPlaybackCapability::GetPlayer(SharedPlaybackState);
				// For now we use the linking/unlinking of the inactive ranges to set the binding activations
				if (BindingLifetime.BindingLifetimeState == EMovieSceneBindingLifetimeState::Active)
				{
					if (BoundObject && BoundObject->Implements<UMovieSceneBindingEventReceiverInterface>())
					{
						TScriptInterface<IMovieSceneBindingEventReceiverInterface> BindingEventReceiver = BoundObject;
						if (BindingEventReceiver.GetObject() && Player)
						{
							FMovieSceneObjectBindingID BindingID = UE::MovieScene::FRelativeObjectBindingID(MovieSceneSequenceID::Root, SequenceID, ObjectBindingID, SharedPlaybackState);
							// The OnBound event is handled by the BindingLifetimeActivationSystem
							IMovieSceneBindingEventReceiverInterface::Execute_OnObjectBoundBySequencer(BindingEventReceiver.GetObject(), Cast<UMovieSceneSequencePlayer>(Player->AsUObject()), BindingID);
						}
					}
				}
			};
		
		FEntityTaskBuilder()
			.ReadEntityIDs()
			.Read(BuiltInComponents->GenericObjectBinding)
			.Read(BuiltInComponents->InstanceHandle)
			.Read(BuiltInComponents->BindingLifetime)
			.Read(BuiltInComponents->BoundObject)
			.FilterAll({ BuiltInComponents->Tags.NeedsLink })
			.Iterate_PerEntity(&Linker->EntityManager, SendBoundMessage);
	}
}

#undef LOCTEXT_NAMESPACE

