// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneBlenderSystem.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/BuiltInComponentTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneBlenderSystem)

namespace UE
{
namespace MovieScene
{

static TMap<FMovieSceneBlenderSystemID, TSubclassOf<UMovieSceneBlenderSystem>> GBlenderSystemRegistry;

} // namespace MovieScene
} // namespace UE

TSubclassOf<UMovieSceneBlenderSystem> UMovieSceneBlenderSystem::GetBlenderSystemClass(FMovieSceneBlenderSystemID InSystemID)
{
	using namespace UE::MovieScene;

	TSubclassOf<UMovieSceneBlenderSystem>* BlenderClass = GBlenderSystemRegistry.Find(InSystemID);
	return BlenderClass ? *BlenderClass : TSubclassOf<UMovieSceneBlenderSystem>();
}

UMovieSceneBlenderSystem::UMovieSceneBlenderSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	SystemCategories = UE::MovieScene::EEntitySystemCategory::BlenderSystems;
	SelectionPriority = DefaultPriority;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		// No need to register these for the base class
		if (GetClass() != UMovieSceneBlenderSystem::StaticClass())
		{
			// You can only ever register blender systems, never unregister them.
			SystemID = FMovieSceneBlenderSystemID(GBlenderSystemRegistry.Num());
			GBlenderSystemRegistry.Add(SystemID, GetClass());

			FComponentRegistry* ComponentRegistry = UMovieSceneEntitySystemLinker::GetComponents();
			BlenderTypeTag = ComponentRegistry->NewTag(*GetName());
		}
	}
	else 
	{
		UMovieSceneBlenderSystem* CDO = GetClass()->GetDefaultObject<UMovieSceneBlenderSystem>();

		SystemID = CDO->SystemID;
		BlenderTypeTag = CDO->BlenderTypeTag;

		checkf(SystemID.IsValid(), TEXT("Blender system wasn't registered correctly on init!"));
	}
}

FMovieSceneBlenderSystemID UMovieSceneBlenderSystem::GetBlenderSystemID() const
{
	return SystemID;
}

FMovieSceneBlendChannelID UMovieSceneBlenderSystem::AllocateBlendChannel()
{
	int32 NewBlendChannel = AllocatedBlendChannels.FindAndSetFirstZeroBit();
	if (NewBlendChannel == INDEX_NONE)
	{
		NewBlendChannel = AllocatedBlendChannels.Add(true);
	}

	checkf(NewBlendChannel < TNumericLimits<uint16>::Max(), TEXT("Maximum number of active blends reached - this indicates either a leak, or more than 65535 blend channels are genuinely required"));
	return FMovieSceneBlendChannelID { SystemID, static_cast<uint16>(NewBlendChannel) };
}


void UMovieSceneBlenderSystem::ReleaseBlendChannel(FMovieSceneBlendChannelID BlendID)
{
	ensureMsgf(BlendID.SystemID == SystemID, TEXT("This given blend channel wasn't allocated by this blender system!"));
	AllocatedBlendChannels[BlendID.ChannelID] = false;
}


bool UMovieSceneBlenderSystem::IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	return AllocatedBlendChannels.Find(true) != INDEX_NONE;
}


void UMovieSceneBlenderSystem::CompactBlendChannels()
{
	// @todo: scheduled routine maintenance like this to optimize memory layouts
	const int32 LastBlendIndex = AllocatedBlendChannels.FindLast(true);
	if (LastBlendIndex == INDEX_NONE)
	{
		AllocatedBlendChannels.Empty();
	}
	else if (LastBlendIndex < AllocatedBlendChannels.Num() - 1)
	{
		AllocatedBlendChannels.RemoveAt(LastBlendIndex + 1, AllocatedBlendChannels.Num() - LastBlendIndex - 1);
	}
}

