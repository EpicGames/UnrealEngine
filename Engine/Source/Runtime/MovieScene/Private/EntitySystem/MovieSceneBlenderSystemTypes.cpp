// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneBlenderSystemTypes.h"
#include "EntitySystem/MovieSceneBlenderSystem.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"

FMovieSceneBlenderSystemID FMovieSceneBlenderSystemID::Invalid;

TSubclassOf<UMovieSceneBlenderSystem> FMovieSceneBlendChannelID::GetSystemClass() const
{
	return UMovieSceneBlenderSystem::GetBlenderSystemClass(SystemID);
}

UMovieSceneBlenderSystem* FMovieSceneBlendChannelID::FindSystem(const UMovieSceneEntitySystemLinker* Linker) const
{
	TSubclassOf<UMovieSceneBlenderSystem> SystemClass = GetSystemClass();
	return CastChecked<UMovieSceneBlenderSystem>(Linker->FindSystem(SystemClass));
}

