// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneBoundObjectInstantiator.h"
#include "EntitySystem/MovieSceneEntityInstantiatorSystem.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneInstanceRegistry.h"
#include "EntitySystem/MovieSceneComponentRegistry.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntityFactoryTemplates.h"

#include "IMovieScenePlayer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneBoundObjectInstantiator)

UMovieSceneGenericBoundObjectInstantiator::UMovieSceneGenericBoundObjectInstantiator(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();
	RelevantComponent = Components->GenericObjectBinding;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineComponentProducer(GetClass(), Components->BoundObject);
		DefineComponentProducer(GetClass(), Components->SymbolicTags.CreatesEntities);
	}
}

void UMovieSceneGenericBoundObjectInstantiator::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();

	UnlinkStaleObjectBindings(Components->GenericObjectBinding);

	FBoundObjectTask BoundObjectTask(Linker);

	// Gather all newly instanced entities with an object binding ID
	FEntityTaskBuilder()
	.ReadEntityIDs()
	.Read(Components->InstanceHandle)
	.Read(Components->GenericObjectBinding)
	.ReadOptional(Components->BoundObjectResolver)
	.FilterAny({ Components->Tags.NeedsLink, Components->Tags.HasUnresolvedBinding })
	.FilterNone({ Components->Tags.NeedsUnlink })
	.RunInline_PerAllocation(&Linker->EntityManager, BoundObjectTask);
}

