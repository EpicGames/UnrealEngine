// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneBoundSceneComponentInstantiator.h"
#include "EntitySystem/MovieSceneBoundObjectInstantiator.h"
#include "EntitySystem/MovieSceneEntityInstantiatorSystem.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/MovieSceneEntityFactoryTemplates.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneInstanceRegistry.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntityMutations.h"

#include "Evaluation/MovieSceneEvaluationState.h"
#include "MovieSceneCommonHelpers.h"
#include "IMovieScenePlayer.h"

#include "Components/SceneComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneBoundSceneComponentInstantiator)

PRAGMA_DISABLE_DEPRECATION_WARNINGS

UMovieSceneBoundSceneComponentInstantiator::UMovieSceneBoundSceneComponentInstantiator(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();
	RelevantComponent = Components->SceneComponentBinding;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(GetClass(), UMovieSceneGenericBoundObjectInstantiator::StaticClass());
	}
}

void UMovieSceneBoundSceneComponentInstantiator::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();

	struct FSceneComponentBindingMutation : IMovieSceneEntityMutation
	{
		virtual void CreateMutation(FEntityManager* EntityManager, FComponentMask* InOutEntityComponentTypes) const override
		{
			FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();
			InOutEntityComponentTypes->SetAll({ Components->GenericObjectBinding, Components->BoundObjectResolver });
		}
		virtual void InitializeAllocation(FEntityAllocation* Allocation, const FComponentMask& AllocationType) const
		{
			FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();

			const FGuid*          SceneComponentBindings = Allocation->ReadComponents(Components->SceneComponentBinding).AsPtr();
			FGuid*                GenericObjectBindings  = Allocation->WriteComponents(Components->GenericObjectBinding, FEntityAllocationWriteContext::NewAllocation()).AsPtr();
			FBoundObjectResolver* BoundObjectResolvers   = Allocation->WriteComponents(Components->BoundObjectResolver, FEntityAllocationWriteContext::NewAllocation()).AsPtr();

			const int32 Num = Allocation->Num();
			FMemory::Memcpy(GenericObjectBindings, SceneComponentBindings, sizeof(FGuid)*Num);

			for (int32 Index = 0; Index < Num; ++Index)
			{
				BoundObjectResolvers[Index] = MovieSceneHelpers::ResolveSceneComponentBoundObject;
			}
		}
	} Mutation;


	Linker->EntityManager.MutateAll(
		FEntityComponentFilter().All({ Components->SceneComponentBinding }), Mutation);
}


PRAGMA_ENABLE_DEPRECATION_WARNINGS

