// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneCachedEntityFilterResult.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/EntityAllocationIterator.h"

namespace UE
{
namespace MovieScene
{

ECachedEntityManagerState FCachedEntityManagerState::Update(const FEntityManager& InEntityManager)
{
	if (InEntityManager.HasStructureChangedSince(LastSystemVersion))
	{
		LastSystemVersion = InEntityManager.GetSystemSerial();
		return ECachedEntityManagerState::Stale;
	}

	return ECachedEntityManagerState::UpToDate;
}


bool FCachedEntityFilterResult_Match::Matches(const FEntityManager& InEntityManager)
{
	if (Cache.Update(InEntityManager) == ECachedEntityManagerState::Stale)
	{
		bContainsMatch = InEntityManager.Contains(Filter);
	}

	return bContainsMatch;
}

void FCachedEntityFilterResult_Match::Invalidate()
{
	Cache.Invalidate();
	bContainsMatch = false;
}




TArrayView<FEntityAllocation* const> FCachedEntityFilterResult_Allocations::GetMatchingAllocations(const FEntityManager& InEntityManager)
{
	if (Cache.Update(InEntityManager) == ECachedEntityManagerState::Stale)
	{
		MatchedEntityAllocations.Empty();
		for (FEntityAllocation* Allocation : InEntityManager.Iterate(&Filter))
		{
			MatchedEntityAllocations.Add(Allocation);
		}
	}

	return MatchedEntityAllocations;
}



void FCachedEntityFilterResult_Allocations::Invalidate()
{
	Cache.Invalidate();
	MatchedEntityAllocations.Empty();
}


} // namespace MovieScene
} // namespace UE

