// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneComponentAccessors.h"
#include "EntitySystem/MovieSceneComponentRegistry.h"
#include "EntitySystem/MovieSceneEntityManager.h"


namespace UE
{
namespace MovieScene
{

#if UE_MOVIESCENE_ENTITY_DEBUG

void AccessorToString(const FReadAccess* In, FEntityManager* EntityManager, FString& OutString)
{
	const FComponentTypeInfo& ComponentTypeInfo = EntityManager->GetComponents()->GetComponentTypeChecked(In->ComponentType);
	OutString += FString::Printf(TEXT("\n\tRead: %s %s"), *ComponentTypeInfo.DebugInfo->DebugName, ComponentTypeInfo.DebugInfo->DebugTypeName);
}
void AccessorToString(const FWriteAccess* In, FEntityManager* EntityManager, FString& OutString)
{
	const FComponentTypeInfo& ComponentTypeInfo = EntityManager->GetComponents()->GetComponentTypeChecked(In->ComponentType);
	OutString += FString::Printf(TEXT("\n\tWrite: %s %s"), *ComponentTypeInfo.DebugInfo->DebugName, ComponentTypeInfo.DebugInfo->DebugTypeName);
}
void AccessorToString(const FOptionalReadAccess* In, FEntityManager* EntityManager, FString& OutString)
{
	if (In->ComponentType)
	{
		const FComponentTypeInfo& ComponentTypeInfo = EntityManager->GetComponents()->GetComponentTypeChecked(In->ComponentType);
		OutString += FString::Printf(TEXT("\n\tRead (Optional): %s %s"), *ComponentTypeInfo.DebugInfo->DebugName, ComponentTypeInfo.DebugInfo->DebugTypeName);
	}
}
void AccessorToString(const FOptionalWriteAccess* In, FEntityManager* EntityManager, FString& OutString)
{
	if (In->ComponentType)
	{
		const FComponentTypeInfo& ComponentTypeInfo = EntityManager->GetComponents()->GetComponentTypeChecked(In->ComponentType);
		OutString += FString::Printf(TEXT("\n\tWrite (Optional): %s %s"), *ComponentTypeInfo.DebugInfo->DebugName, ComponentTypeInfo.DebugInfo->DebugTypeName);
	}
}
void AccessorToString(const FEntityIDAccess*, FEntityManager* EntityManager, FString& OutString)
{
	OutString += TEXT("\n\tRead: Entity IDs");
}
void OneOfAccessorToString(const FOptionalReadAccess* In, FEntityManager* EntityManager, FString& OutString)
{
	if (In->ComponentType)
	{
		const FComponentTypeInfo& ComponentTypeInfo = EntityManager->GetComponents()->GetComponentTypeChecked(In->ComponentType);
		OutString = FString::Printf(TEXT("%s %s"), *ComponentTypeInfo.DebugInfo->DebugName, ComponentTypeInfo.DebugInfo->DebugTypeName);
	}
}

#endif // UE_MOVIESCENE_ENTITY_DEBUG


} // namespace MovieScene
} // namespace UE