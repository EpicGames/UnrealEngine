// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneComponentRegistry.h"
#include "EntitySystem/MovieSceneComponentTypeInfo.h"

namespace UE
{
namespace MovieScene
{


FComponentTypeID FComponentRegistry::NewTag(const TCHAR* const DebugName, EComponentTypeFlags Flags)
{
	FComponentTypeInfo NewTypeInfo;

	NewTypeInfo.Sizeof = 0;
	NewTypeInfo.Alignment = 0;
	NewTypeInfo.bIsZeroConstructType = 1;
	NewTypeInfo.bIsTriviallyDestructable = 1;
	NewTypeInfo.bIsTriviallyCopyAssignable = 1;
	NewTypeInfo.bIsPreserved = EnumHasAnyFlags(Flags, EComponentTypeFlags::Preserved);
	NewTypeInfo.bIsCopiedToOutput = EnumHasAnyFlags(Flags, EComponentTypeFlags::CopyToOutput);
	NewTypeInfo.bIsMigratedToOutput = EnumHasAnyFlags(Flags, EComponentTypeFlags::MigrateToOutput);

#if UE_MOVIESCENE_ENTITY_DEBUG
	NewTypeInfo.DebugInfo                = MakeUnique<FComponentTypeDebugInfo>();
	NewTypeInfo.DebugInfo->DebugName     = DebugName;
	NewTypeInfo.DebugInfo->DebugTypeName = TEXT("TAG");
#endif

	FComponentTypeID NewType = NewComponentTypeInternal(MoveTemp(NewTypeInfo));

	if (EnumHasAnyFlags(Flags, EComponentTypeFlags::CopyToChildren))
	{
		Factories.DefineChildComponent(NewType, NewType);
	}
	return NewType;
}

FComponentTypeID FComponentRegistry::NewComponentTypeInternal(FComponentTypeInfo&& TypeInfo)
{
	check(ComponentTypes.Num() < MaximumNumComponentsSupported-1);

	const bool bAddToPreservationMask = TypeInfo.bIsPreserved;
	const bool bAddToCopyMask         = TypeInfo.bIsCopiedToOutput;
	const bool bAddToMigrationMask    = TypeInfo.bIsMigratedToOutput;
	const bool bIsTag                 = TypeInfo.IsTag();

	const int32 InsertedIndex = ComponentTypes.Add(MoveTemp(TypeInfo));
	// ~~~~~~~~~~~~~~~~~ TypeInfo is now garbage ~~~~~~~~~~~~~~~~~

	FComponentTypeID NewComponentType = FComponentTypeID::FromBitIndex(InsertedIndex);

	if (bAddToPreservationMask)
	{
		PreservationMask.Set(NewComponentType);
	}

	if (bAddToCopyMask)
	{
		CopyAndMigrationMask.Set(NewComponentType);
	}

	if (bAddToMigrationMask)
	{
		MigrationMask.Set(NewComponentType);
		CopyAndMigrationMask.Set(NewComponentType);
	}

	if (!bIsTag)
	{
		NonTagComponentMask.Set(NewComponentType);
	}

	return FComponentTypeID::FromBitIndex(InsertedIndex);
}

const FComponentTypeInfo& FComponentRegistry::GetComponentTypeChecked(FComponentTypeID ComponentTypeID) const
{
	check(ComponentTypes.IsValidIndex(ComponentTypeID.BitIndex()));
	return ComponentTypes[ComponentTypeID.BitIndex()];
}



}	// using namespace MovieScene
}	// using namespace UE
