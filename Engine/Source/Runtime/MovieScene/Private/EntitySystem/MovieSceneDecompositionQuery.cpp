// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneDecompositionQuery.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneDecompositionQuery)

namespace UE
{
namespace MovieScene
{

void FDecomposedValue::Decompose(
		FMovieSceneEntityID EntityID, 
		FWeightedValue& ThisValue, EDecomposedValueBlendType& OutBlendType, 
		FAccumulatedWeightedValue& Absolutes, 
		FAccumulatedWeightedValue& Additives) const
{
	// Look through all our decomposed values and isolate the one that corresponds to the particular given entity that
	// we're interested in. All other decomposed values get accumulated in the given output parameters.
	for (TTuple<FMovieSceneEntityID, FWeightedValue> Pair : DecomposedAbsolutes)
	{
		if (Pair.Get<0>() == EntityID)
		{
			ThisValue = Pair.Value;
			OutBlendType = EDecomposedValueBlendType::Absolute;
		}
		else
		{
			Absolutes.AccumulateThis(Pair.Value);
		}
	}
	for (TTuple<FMovieSceneEntityID, FWeightedValue> Pair : DecomposedAdditives)
	{
		if (Pair.Get<0>() == EntityID)
		{
			ThisValue = Pair.Value;
			OutBlendType = (Pair.Value.BaseValue != 0.f) ? 
				EDecomposedValueBlendType::AdditiveFromBase : EDecomposedValueBlendType::Additive;
		}
		else
		{
			Additives.AccumulateThis(Pair.Value);
		}
	}
	for (TTuple<FMovieSceneEntityID, FWeightedValue> Pair : DecomposedOverrides)
	{
		if (Pair.Get<0>() == EntityID)
		{
			ThisValue = Pair.Value;
			OutBlendType = EDecomposedValueBlendType::Override;
		}
	}
}
static void AddToValue(FWeightedValue& Data, double& OutCurrentValue)
{
	if (Data.bIsAdditive)
	{
		OutCurrentValue += (Data.Value * Data.Weight);
	}
	else //override do weighted blend
	{
		if (Data.Weight >= 1.0)
		{
			OutCurrentValue = Data.Value;
		}
		else
		{
			OutCurrentValue = (OutCurrentValue * (1.0 - Data.Weight)) +
				(Data.Value * Data.Weight);
		}
	}
}
double FDecomposedValue::Recompose(FMovieSceneEntityID RecomposeEntity, double CurrentValue, const double* InitialValue) const
{
	// First, a little reminder... the formula for blending values is either:
	//
	// Value = Absolutes + Additives
	//
	// Absolutes = (Abs1 * AbsWeight1 + ... + AbsN * AbsWeightN) / (AbsWeight1 + ... + AbsWeightN)
	// Additives = (Add1 * AddWeight1 + ... AddN * AddWeightN)
	//
	// Or if an Override is present it's Absolutes  plus the Additives and Overrides in blending order
	// Value = Absolutes + BlendOrder1 +  BlendOrder2
	// 
	// Where the BlendOrder is either
	// AdditivesN = (AddN * AddWeightN) or
	// OverrideN = (OverrideN * OverrideWeightN) + (Value * (1.0 -OverrideWeightN))
	// 
	// Sort through all the data we have so that the contribution of RecomposeEntity is set aside in Channel.  The
	// contributions of other decomposed entities (if any) are combined into OtherAbsolute and OtherAdditive, which
	// should also be combined with Result (which contains non-decomposed entities) to get the full picture.
	FWeightedValue Channel;
	EDecomposedValueBlendType BlendType(EDecomposedValueBlendType::Absolute);
	FAccumulatedWeightedValue OtherAbsolute;
	FAccumulatedWeightedValue OtherAdditive;
	Decompose(RecomposeEntity, Channel, BlendType, OtherAbsolute, OtherAdditive);
	
	FAccumulatedWeightedValue ResultAbsolute = Result.Absolute;
	const bool bIsAdditive = (BlendType == EDecomposedValueBlendType::Additive ||
		BlendType == EDecomposedValueBlendType::AdditiveFromBase);
	const bool bIsOverride = (BlendType == EDecomposedValueBlendType::Override);
	float TotalAbsoluteWeight = ResultAbsolute.TotalWeight + OtherAbsolute.TotalWeight;
	if (!bIsAdditive && !bIsOverride)
	{
		TotalAbsoluteWeight += Channel.Weight;
	}
	if (TotalAbsoluteWeight < 1.f && InitialValue != nullptr)
	{
		// If all the absolutes we know about (the decomposed and the non-decomposed) do not amount up to 100% weight,
		// let's fill the gap with the initial value.
		const float InitialValueWeight = (1.f - TotalAbsoluteWeight);
		ResultAbsolute.Total = (*InitialValue) * InitialValueWeight + ResultAbsolute.Total;
		ResultAbsolute.TotalWeight = InitialValueWeight + ResultAbsolute.TotalWeight;
		TotalAbsoluteWeight = 1.f;
	}

	// If this channel is the only thing we decomposed, we just need to adjust it by the difference
	// between what we have and what the desired current value is.
	if (OtherAbsolute.TotalWeight == 0.f && OtherAdditive.TotalWeight == 0.f)
	{
		//for overrides we need to iterate over everthing since it's order dependent since 
		//there can be an override track anywhere
		if (bIsOverride && AllDecomposedOverrides.Num() > 0) //if override we go up to this one's BlendValue
		{
			AllDecomposedOverrides.Sort();
			double OverrideValue = ResultAbsolute.Total; //current value that's set
			double PreviousValue = OverrideValue;    //value set right before this one
			double ExtraValue = 0.0;
			for(FWeightedValue& Data: AllDecomposedOverrides)
			{
				if (Data.BlendingOrder < Channel.BlendingOrder)
				{
					AddToValue(Data, OverrideValue);
					PreviousValue = OverrideValue;
				}
				else if(Data.BlendingOrder > Channel.BlendingOrder)
				{
					AddToValue(Data, ExtraValue);
				}
			}
			if (Channel.bIsAdditive)
			{
				const double Difference = CurrentValue - ExtraValue - PreviousValue;
				return (Channel.Weight == 0.f ? Difference : (Difference / Channel.Weight));
			}
			else
			{
				const double Difference = (CurrentValue - ExtraValue) - (PreviousValue * (1.0 - Channel.Weight));
				return (Channel.Weight == 0.f ? Difference : (Difference / Channel.Weight));

			}
		}
		else if (bIsAdditive)
		{
			// For an additive channel, we just put the missing difference on it, adjusted by its weight.
			const double WeightedAdditiveResult = CurrentValue - ResultAbsolute.Normalize() - Result.Additive;
			return (Channel.Weight == 0.f ? WeightedAdditiveResult : WeightedAdditiveResult / Channel.Weight) + Channel.BaseValue;
		}
		else // is absolute
		{
			// Absolutes get all added together, and normalized by the total weight, so we can't simply find the
			// missing difference like we do with additives. The overall formula for our case is this:
			//
			// Value = (WeightedResultAbsoluteValue + WeightedChannelValue) / (ResultAbsoluteWeight + ChannelWeight) + ResultAdditive
			//
			// Therefore:
			//
			// WeightedChannelValue = (Value - ResultAdditive) * (ResultAbsoluteWeight + ChannelWeight) - WeightedResultAbsoluteValue
			// ChannelValue = WeightedChannelValue / ChannelWeight
			//
			// Note that if the channel's weight is zero, it doesn't matter what value we key, we can never reach
			// whatever value we want to hit. In order to provide a better default (and avoid a division by zero), we
			// single this case out and set the channel value as if it was weighted to 100%.
			//
			// Note: this assumes that weights are always positive or zero though (we don't support negative weights
			// that could, when combined with equal positive weights, have a sum of zero).
			//
			if (Channel.Weight == 0.f)
			{
				// Note that we use the pure result here (Result instead of AbsoluteResult). This is because
				// AbsoluteResult may include the initial value. As mentioned, we key the channel as if it had 100%
				// weight. If it has 100% weight, the initial value wouldn't be involved, so we shouldn't count it in.
				return (CurrentValue - Result.Additive) * (Result.Absolute.TotalWeight + 1.f) - Result.Absolute.Total;
			}
			else
			{
				// Normal use-case... just run the formula from above.
				const double WeightedAbsoluteResult = (CurrentValue - Result.Additive) * TotalAbsoluteWeight - ResultAbsolute.Total;
				return (Channel.Weight == 0.f ? WeightedAbsoluteResult : WeightedAbsoluteResult / Channel.Weight);
			}
		}
	}

	// If we're here, we have multiple channels (entities) that we are decomposing for.
	// 
	// This generally means we are keying multiple channels at once, and we're expecting this function to be called once
	// for each channel being decomposed.  This is important because we will be spreading the needed value changes over
	// these multiple channels, as needed, which means only passing some of it onto the channel (entity) given as the
	// first parameter.

	// First, check if the current channel has any weight. If it doesn't, let's just ignore it and hope to be able to do
	// something better in subsequent calls to this function for which we look at another channel.
	if (Channel.Weight == 0.f)
	{
		return Channel.Value;
	}

	// Main case for multiple decomposed values. The gist of it is as follows:
	//
	// - Mix of absolutes and additives: leave the absolutes alone, and spread the required changes among additives,
	//      based on these additives' proportional values (so channels with larger values get a bigger slice of the pie).
	//
	// - Additives only: as above, we proportionally spread the required changes among the additive channels.
	//
	// - Absolutes only: we proportionally spread the required changes among the absolute channels based on their
	//		weight (so channels with greater weight get a bigger slice of the pie).
	//
	if (bIsAdditive)
	{
		const double ThisAdditive = Channel.Get();
		const double AllOtherAbsolutes = ResultAbsolute.Accumulate(OtherAbsolute).Normalize();

		// We are keying an additive channel, but it's currently zero and no other additives are involved.
		// Key our channel with the difference to reach the desired value.
		if (OtherAdditive.Total + ThisAdditive == 0.f)
		{
			const double WeightedChannelAdditive = (CurrentValue - AllOtherAbsolutes - Result.Additive);
			return (Channel.Weight == 0.f) ? (WeightedChannelAdditive + Channel.BaseValue) : (WeightedChannelAdditive / Channel.Weight + Channel.BaseValue);
		}

		// Scale up/down each of the additives we want to key proportionally to their value. That is, bigger additives
		// will be changed proportionally more than smaller additives. But the sum of all increases or decreases will
		// amount to whatever is needed to reach the desired final value.
		const double DecomposeFactor = ThisAdditive / (OtherAdditive.Total + ThisAdditive);
		const double WeightedChannelAdditive = (CurrentValue - AllOtherAbsolutes - Result.Additive) * DecomposeFactor;
		return (Channel.Weight == 0.f) ? (WeightedChannelAdditive + Channel.BaseValue) : (WeightedChannelAdditive / Channel.Weight + Channel.BaseValue);
	}
	else if (DecomposedAdditives.Num() != 0)
	{
		// We are currently dealing with keying an absolute channel, but we're also keying additives in the same
		// operation (since some of them are decomposed). Let's put the full weight of keying on the additives, and
		// leave the absolute value alone.
		return Channel.Value;
	}
	else
	{
		// We are keying multiple absolute channels only. We just need to take the difference between the current value
		// and the desired value, and spread that across all our absolute channels proportionally based on weight. This
		// proportion, for a given channel, is:
		//
		// DecomposeFactor = Channel.Weight / (AllOtherWeights + Channel.Weight)
		//
		// The current value is:
		//
		// AllOtherAbsolutes = (WeightedResultAbsolute + WeightedOtherAbsolute)
		// AllOtherWeights = (ResultAbsoluteWeight + OtherAbsoluteWeight)
		// ActualValue = (WeightedChannelValue + AllOtherAbsolutes) / (ChannelWeight + AllOtherWeights) + ResultAdditive
		// 
		// The missing difference for the current channel is:
		//
		// ChannelDelta = (Value - ActualValue) * DecomposeFactor
		//
		// So we want the channel to be increased by ChannelDelta.
		//
		const FAccumulatedWeightedValue TotalAbsolutes = ResultAbsolute.Accumulate(OtherAbsolute).Accumulate(Channel);
		if (TotalAbsolutes.TotalWeight == 0.f)
		{
			const double ChannelDelta = CurrentValue - Result.Additive;
			return Channel.Value + ChannelDelta;
		}
		else
		{
			const double ActualValue = TotalAbsolutes.Normalize() + Result.Additive;
			const double ChannelDelta = (CurrentValue - ActualValue);
			return Channel.Value + ChannelDelta;
		}
	}
}

} // namespace MovieScene
} // namespace UE

