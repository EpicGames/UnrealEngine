// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneEntityFactory.h"
#include "EntitySystem/MovieSceneComponentRegistry.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntityFactoryTemplates.h"
#include "MovieSceneFwd.h"
#include "Conditions/MovieSceneCondition.h"

namespace UE
{
namespace MovieScene
{

int32 FChildEntityFactory::Num() const
{
	return ParentEntityOffsets.Num();
}

int32 FChildEntityFactory::GetCurrentIndex() const
{
	if (const int32* CurrentOffset = CurrentEntityOffsets.GetData())
	{
		return CurrentOffset - ParentEntityOffsets.GetData();
	}
	return INDEX_NONE;
}

void FChildEntityFactory::Apply(UMovieSceneEntitySystemLinker* Linker, FEntityAllocationProxy ParentAllocationProxy)
{
	const FComponentMask ParentType = ParentAllocationProxy.GetAllocationType();

	FComponentMask DerivedEntityType;
	FMutualComponentInitializers MutualInitializers;

	{
		GenerateDerivedType(DerivedEntityType);

		Linker->EntityManager.GetComponents()->Factories.ComputeChildComponents(ParentType, DerivedEntityType);
		Linker->EntityManager.GetComponents()->Factories.ComputeMutuallyInclusiveComponents(EMutuallyInclusiveComponentType::All, DerivedEntityType, MutualInitializers);
	}

	const bool bHasAnyType = DerivedEntityType.Find(true) != INDEX_NONE;
	if (!bHasAnyType)
	{
		return;
	}

	const int32 NumToAdd = Num();

	int32 CurrentParentOffset = 0;
	const FEntityAllocation* ParentAllocation = ParentAllocationProxy.GetAllocation();
	FEntityAllocationWriteContext WriteContext(Linker->EntityManager);

	// We attempt to allocate all the linker entities contiguously in memory for efficient initialization,
	// but we may reach capacity constraints within allocations so we may have to run the factories more than once
	while(CurrentParentOffset < NumToAdd)
	{
		// Ask to allocate as many as possible - we may only manage to allocate a smaller number contiguously this iteration however
		int32 NumAdded = NumToAdd - CurrentParentOffset;

		FEntityDataLocation NewLinkerEntities = Linker->EntityManager.AllocateContiguousEntities(DerivedEntityType, &NumAdded);
		FEntityRange ChildRange{ NewLinkerEntities.Allocation, NewLinkerEntities.ComponentOffset, NumAdded };

		CurrentEntityOffsets = MakeArrayView(ParentEntityOffsets.GetData() + CurrentParentOffset, NumAdded);

		if (TOptionalComponentWriter<FMovieSceneEntityID> ParentEntityIDs =
			ChildRange.Allocation->TryWriteComponents(FBuiltInComponentTypes::Get()->ParentEntity, WriteContext))
		{
			TArrayView<const FMovieSceneEntityID> ParentIDs = ParentAllocation->GetEntityIDs();
			for (int32 Index = 0; Index < ChildRange.Num; ++Index)
			{
				const int32 ParentIndex = CurrentEntityOffsets[Index];
				const int32 ChildIndex  = ChildRange.ComponentStartOffset + Index;

				ParentEntityIDs[ChildIndex] = ParentIDs[ParentIndex];
			}
		}

		// Initialize the bound objects before we call child initializers
		InitializeAllocation(Linker, ParentType, DerivedEntityType, ParentAllocation, CurrentEntityOffsets, ChildRange);

		MutualInitializers.Execute(ChildRange, WriteContext);
		Linker->EntityManager.InitializeChildAllocation(ParentType, DerivedEntityType, ParentAllocation, CurrentEntityOffsets, ChildRange);

		CurrentParentOffset += NumAdded;
	}

	PostInitialize(Linker);
}


void FObjectFactoryBatch::Add(int32 EntityIndex, UObject* BoundObject)
{
	ParentEntityOffsets.Add(EntityIndex);
	ObjectsToAssign.Add(BoundObject);
}

void FObjectFactoryBatch::GenerateDerivedType(FComponentMask& OutNewEntityType)
{
	OutNewEntityType.Set(FBuiltInComponentTypes::Get()->BoundObject);
}

void FObjectFactoryBatch::InitializeAllocation(UMovieSceneEntitySystemLinker* Linker, const FComponentMask& ParentType, const FComponentMask& ChildType, const FEntityAllocation* ParentAllocation, TArrayView<const int32> ParentAllocationOffsets, const FEntityRange& InChildEntityRange)
{
	TSortedMap<UObject*, FMovieSceneEntityID, TInlineAllocator<8>> ChildMatchScratch;

	TComponentTypeID<UObject*>            BoundObject = FBuiltInComponentTypes::Get()->BoundObject;
	TComponentTypeID<FMovieSceneEntityID> ParentEntity = FBuiltInComponentTypes::Get()->ParentEntity;

	TArrayView<const FMovieSceneEntityID> ParentIDs = ParentAllocation->GetEntityIDs();

	FEntityAllocationWriteContext WriteContext = FEntityAllocationWriteContext::NewAllocation();

	const FEntityAllocation* Allocation = nullptr;
	int32 ComponentStartOffset = 0;
	int32 Num = 0;

	TArrayView<const FMovieSceneEntityID> ChildEntityIDs        = InChildEntityRange.Allocation->GetEntityIDs();
	TComponentReader<FMovieSceneEntityID> ParentIDComponents    = InChildEntityRange.Allocation->ReadComponents(ParentEntity);
	TComponentWriter<UObject*>            BoundObjectComponents = InChildEntityRange.Allocation->WriteComponents(BoundObject, WriteContext);

	int32 Index = GetCurrentIndex();
	for (int32 ChildIndex = InChildEntityRange.ComponentStartOffset; ChildIndex < InChildEntityRange.ComponentStartOffset + InChildEntityRange.Num; ++ChildIndex)
	{
		FMovieSceneEntityID Parent = ParentIDComponents[ChildIndex];
		FMovieSceneEntityID Child  = ChildEntityIDs[ChildIndex];

		UObject* Object = ObjectsToAssign[Index++];
		BoundObjectComponents[ChildIndex] = Object;

		if (FMovieSceneEntityID OldEntityToPreserve = StaleEntitiesToPreserve->FindRef(MakeTuple(Object, Parent)))
		{
			PreservedEntities.Add(Child, OldEntityToPreserve);
		}
		Linker->EntityManager.AddChild(Parent, Child);
	}
}

void FObjectFactoryBatch::PostInitialize(UMovieSceneEntitySystemLinker* InLinker)
{
	FComponentMask PreservationMask = InLinker->EntityManager.GetComponents()->GetPreservationMask();

	for (TTuple<FMovieSceneEntityID, FMovieSceneEntityID> Pair : PreservedEntities)
	{
		InLinker->EntityManager.CombineComponents(Pair.Key, Pair.Value, &PreservationMask);
	}
}

FBoundObjectTask::FBoundObjectTask(UMovieSceneEntitySystemLinker* InLinker)
	: Linker(InLinker)
{}

void FBoundObjectTask::Apply()
{
	for (TTuple<FEntityAllocationProxy, FObjectFactoryBatch>& Pair : Batches)
	{
		// Determine the type for the new entities
		if (Pair.Value.Num() != 0)
		{
			Pair.Value.Apply(Linker, Pair.Key);
		}
	}
}

void FBoundObjectTask::ForEachAllocation(FEntityAllocationProxy AllocationProxy, FReadEntityIDs EntityIDs, TRead<FInstanceHandle> Instances, TRead<FGuid> ObjectBindings, TReadOptional<FBoundObjectResolver> Resolvers)
{
	const FEntityAllocation* Allocation = AllocationProxy.GetAllocation();
	const FComponentTypeID TagHasUnresolvedBinding = FBuiltInComponentTypes::Get()->Tags.HasUnresolvedBinding;

	// Check whether every binding in this allocation is currently unresolved
	const bool bWasUnresolvedBinding = Allocation->FindComponentHeader(TagHasUnresolvedBinding) != nullptr;

	FObjectFactoryBatch& Batch = Batches.Emplace(AllocationProxy);
	Batch.StaleEntitiesToPreserve = &StaleEntitiesToPreserve;

	const int32 Num = Allocation->Num();
	
	TOptionalComponentReader<TObjectPtr<const UMovieSceneCondition>> Conditions = Allocation->TryReadComponents(FBuiltInComponentTypes::Get()->Condition);

	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();

	// Keep track of existing bindings so we can preserve any components on them
	TComponentTypeID<UObject*> BoundObjectComponent = FBuiltInComponentTypes::Get()->BoundObject;

	const FBoundObjectResolver* ResolverPtr = Resolvers.AsPtr();

	for (int32 Index = 0; Index < Num; ++Index)
	{
		FMovieSceneEntityID ParentID = EntityIDs[Index];

		// Discard existing children
		const int32 StartNum = EntitiesToDiscard.Num();
		Linker->EntityManager.GetImmediateChildren(ParentID, EntitiesToDiscard);

		// Keep track of any existing object bindings so we can preserve components on them if they are resolved to the same thing
		for (int32 ChildIndex = StartNum; ChildIndex < EntitiesToDiscard.Num(); ++ChildIndex)
		{
			FMovieSceneEntityID ChildID = EntitiesToDiscard[ChildIndex];
			TOptionalComponentReader<UObject*> ObjectPtr = Linker->EntityManager.ReadComponent(ChildID,  BoundObjectComponent);
			if (ObjectPtr)
			{
				StaleEntitiesToPreserve.Add(MakeTuple(*ObjectPtr, ParentID), ChildID);
			}
		}

		bool bIsResolvedBinding = false;

		const FSequenceInstance&     SequenceInstance = InstanceRegistry->GetInstance(Instances[Index]);
		TArrayView<TWeakObjectPtr<>> BoundObjects     = SequenceInstance.GetSharedPlaybackState()->FindBoundObjects(ObjectBindings[Index], SequenceInstance.GetSequenceID());
		const FSequenceInstance&     RootInstance     = InstanceRegistry->GetInstance(SequenceInstance.GetRootInstanceHandle());
		
		bool bCheckedCondition = false;
		for (TWeakObjectPtr<> WeakObject : BoundObjects)
		{
			UObject* Object = WeakObject.Get();

			// Pass the object through the resolver component if necessary
			if (ResolverPtr && Object)
			{
				Object = (ResolverPtr[Index])(Object);
			}

			if (Object)
			{
				if (!ensureMsgf(!FBuiltInComponentTypes::IsBoundObjectGarbage(Object), TEXT("Attempting to bind an object that is garbage or unreachable")))
				{
					continue;
				}

				if (!bCheckedCondition && Conditions && Conditions[Index] && Conditions[Index]->GetConditionScope() != EMovieSceneConditionScope::Global)
				{
					// If this entity has a condition that could depend on a bound object, then it hasn't yet been tested, and we must test it here. \
					// Note that it will only be tested once here, and then the entity ledger will take care of testing it again if it needs to
					// and it is a per-tick condition.
					// Conditions can only be evaluated by the root instance, since only it has a SequenceUpdater
					bCheckedCondition = true;
					if (!RootInstance.EvaluateCondition(ObjectBindings[Index], SequenceInstance.GetSequenceID(), Conditions[Index], Conditions[Index]->GetTypedOuter<UMovieSceneSignedObject>()))
					{
						// Condition has failed, don't add this entity to the batch
						break;
					}
				}

				// Make a child entity for this resolved binding
				Batch.Add(Index, Object);
				bIsResolvedBinding = true;
			}
		}


		if (bIsResolvedBinding && bWasUnresolvedBinding)
		{
			// We have successfully resolved a binding, so remove the HasUnresolvedBinding tag
			constexpr bool bAddComponent = false;
			EntityMutations.Add(FEntityMutationData{ ParentID, TagHasUnresolvedBinding, bAddComponent });
		}
		else if (!bIsResolvedBinding && !bWasUnresolvedBinding)
		{
			// Only bother attempting to add the HasUnresolvedBindingTag if it is not already tagged in such a way
			constexpr bool bAddComponent = true;
			EntityMutations.Add(FEntityMutationData{ ParentID, TagHasUnresolvedBinding, bAddComponent });
		}
	}

}

void FBoundObjectTask::PostTask()
{
	Apply();

	FComponentTypeID NeedsUnlink = FBuiltInComponentTypes::Get()->Tags.NeedsUnlink;
	for (FMovieSceneEntityID Discard : EntitiesToDiscard)
	{
		Linker->EntityManager.AddComponent(Discard, NeedsUnlink, EEntityRecursion::Full);
	}

	for (FEntityMutationData Mutation : EntityMutations)
	{
		if (Mutation.bAddComponent)
		{
			Linker->EntityManager.AddComponent(Mutation.EntityID, Mutation.ComponentTypeID);
		}
		else
		{
			Linker->EntityManager.RemoveComponent(Mutation.EntityID, Mutation.ComponentTypeID);
		}
	}
}

void FEntityFactories::DefineChildComponent(TInlineValue<FChildEntityInitializer>&& InInitializer)
{
	check(InInitializer.IsValid());

	DefineChildComponent(InInitializer->GetParentComponent(), InInitializer->GetChildComponent());
	// Note: after this line, InInitializer is reset
	ChildInitializers.Add(MoveTemp(InInitializer));
}

void FEntityFactories::DefineMutuallyInclusiveComponents(FComponentTypeID InComponentA, std::initializer_list<FComponentTypeID> InMutualComponents)
{
	MutualInclusivityGraph.DefineMutualInclusionRule(InComponentA, InMutualComponents);
}

void FEntityFactories::DefineMutuallyInclusiveComponents(FComponentTypeID InComponentA, std::initializer_list<FComponentTypeID> InMutualComponents, FMutuallyInclusiveComponentParams&& Params)
{
	MutualInclusivityGraph.DefineMutualInclusionRule(InComponentA, InMutualComponents, MoveTemp(Params));
}

void FEntityFactories::DefineComplexInclusiveComponents(const FComplexInclusivityFilter& InFilter, FComponentTypeID InComponent)
{
	MutualInclusivityGraph.DefineComplexInclusionRule(InFilter, { InComponent });
}

void FEntityFactories::DefineComplexInclusiveComponents(const FComplexInclusivityFilter& InFilter, std::initializer_list<FComponentTypeID> InComponents, FMutuallyInclusiveComponentParams&& Params)
{
	MutualInclusivityGraph.DefineComplexInclusionRule(InFilter, InComponents, MoveTemp(Params));
}

int32 FEntityFactories::ComputeChildComponents(const FComponentMask& ParentComponentMask, FComponentMask& ChildComponentMask)
{
	int32 NumNewComponents = 0;

	// Any child components keyed off an invalid parent component type are always relevant
	FComponentTypeID InvalidComponent = FComponentTypeID::Invalid();
	for (auto Child = ParentToChildComponentTypes.CreateConstKeyIterator(InvalidComponent); Child; ++Child)
	{
		if (!ChildComponentMask.Contains(Child.Value()))
		{
			ChildComponentMask.Set(Child.Value());
			++NumNewComponents;
		}
	}

	for (FComponentMaskIterator It = ParentComponentMask.Iterate(); It; ++It)
	{
		FComponentTypeID ParentComponent = FComponentTypeID::FromBitIndex(It.GetIndex());
		for (auto Child = ParentToChildComponentTypes.CreateConstKeyIterator(ParentComponent); Child; ++Child)
		{
			if (!ChildComponentMask.Contains(Child.Value()))
			{
				ChildComponentMask.Set(Child.Value());
				++NumNewComponents;
			}
		}
	}

	return NumNewComponents;
}

int32 FEntityFactories::ComputeMutuallyInclusiveComponents(EMutuallyInclusiveComponentType MutualTypes, FComponentMask& ComponentMask, FMutualComponentInitializers& OutInitializers)
{
	return MutualInclusivityGraph.ComputeMutuallyInclusiveComponents(MutualTypes, ComponentMask, ComponentMask, OutInitializers);
}

void FEntityFactories::RunInitializers(const FComponentMask& ParentType, const FComponentMask& ChildType, const FEntityAllocation* ParentAllocation, TArrayView<const int32> ParentAllocationOffsets, const FEntityRange& InChildEntityRange)
{
	// First off, run child initializers
	for (TInlineValue<FChildEntityInitializer>& ChildInit : ChildInitializers)
	{
		if (ChildInit->IsRelevant(ParentType, ChildType))
		{
			ChildInit->Run(InChildEntityRange, ParentAllocation, ParentAllocationOffsets);
		}
	}
}

}	// using namespace MovieScene
}	// using namespace UE
