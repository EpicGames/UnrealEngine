// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneEntityGroupingSystem.h"

#include "Containers/ArrayView.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneBoundObjectInstantiator.h"
#include "EntitySystem/MovieSceneBoundSceneComponentInstantiator.h"
#include "EntitySystem/MovieSceneRootInstantiatorSystem.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntityMutations.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneEntityGroupingSystem)

namespace UE::MovieScene
{

FEntityGroupBuilder::FEntityGroupBuilder(UMovieSceneEntityGroupingSystem* InOwner, FEntityGroupingPolicyKey InPolicyKey)
	: Owner(InOwner)
	, PolicyKey(InPolicyKey)
{
}

void FEntityGroupBuilder::AddEntityToGroup(const FMovieSceneEntityID& InEntity, const FEntityGroupID& InNewGroupID)
{
	const int32 EntityIndex = InEntity.AsIndex();

	if (ensure(InNewGroupID.HasGroup()))
	{
		const bool bAlreadyHasGroup = Owner->EntityIDToGroup.IsValidIndex(EntityIndex);
		if (bAlreadyHasGroup)
		{
			// Remove it from the old group
			FEntityGroupID OldGroupID = Owner->EntityIDToGroup[EntityIndex];
			if (OldGroupID == InNewGroupID)
			{
				// Already part of this group - nothing to do
				return;
			}
			RemoveEntityFromGroup(InEntity, OldGroupID);
		}

		UMovieSceneEntityGroupingSystem::FEntityGroupInfo& GroupInfo = Owner->Groups.FindOrAdd(InNewGroupID);

		++GroupInfo.NumEntities;
		Owner->EntityIDToGroup.Insert(EntityIndex, InNewGroupID);
	}
}

void FEntityGroupBuilder::RemoveEntityFromGroup(const FMovieSceneEntityID& InEntity, const FEntityGroupID& InPreviousGroupID)
{
	const int32 EntityIndex = InEntity.AsIndex();

	if (ensure(InPreviousGroupID.HasGroup()) && Owner->EntityIDToGroup.IsValidIndex(EntityIndex))
	{
		ensureMsgf(InPreviousGroupID == Owner->EntityIDToGroup[EntityIndex], TEXT("Attempting to remove an entity from the wrong group!"));

		Owner->EntityIDToGroup.RemoveAt(InEntity.AsIndex());

		// Remove the entity from the group. We should find that group, and find that entity inside it.
		UMovieSceneEntityGroupingSystem::FEntityGroupInfo* PreviousGroup = Owner->Groups.Find(InPreviousGroupID);
		if (ensureAlways(PreviousGroup))
		{
			using FEntityGroupingHandlerInfo = UMovieSceneEntityGroupingSystem::FEntityGroupingHandlerInfo;

			--PreviousGroup->NumEntities;
			if (PreviousGroup->NumEntities <= 0)
			{
				ensure(PreviousGroup->NumEntities == 0);

				// The group is now empty, we can re-use its index for a new group... but we don't
				// want to re-use it until later, because we could end up in two situations we
				// want to avoid:
				//
				// 1) At the same time that a group is emptied of all its entities, new entities
				//    come in that belong to that group because they generate the exact same key.
				//    We want that group to effectively "persist" with the same index.
				//
				// 2) If we freed the index right away, we couldn't tell the difference between
				//	  the above situation, and a brand new group that just happens to re-use the
				//	  recently freed index.
				//
				Owner->EmptyGroupIndices.PadToNum(InPreviousGroupID.GroupIndex + 1, false);
				Owner->EmptyGroupIndices[InPreviousGroupID.GroupIndex] = true;
			}
		}
	}
}

void FEntityGroupBuilder::ReportUsedGroupIndex(int32 GroupIndex)
{
	if (Owner->EmptyGroupIndices.IsValidIndex(GroupIndex))
	{
		Owner->EmptyGroupIndices[GroupIndex] = false;
	}
}

int32 FEntityGroupBuilder::AllocateGroupIndex()
{
	return Owner->AllocateGroupIndex(PolicyKey);
}


struct FAddGroupMutation : IMovieSceneEntityMutation
{
	UMovieSceneEntityGroupingSystem* System;
	FBuiltInComponentTypes* BuiltInComponents;

	FAddGroupMutation(UMovieSceneEntityGroupingSystem* InSystem)
		: System(InSystem)
	{
		BuiltInComponents = FBuiltInComponentTypes::Get();
	}

	virtual void CreateMutation(FEntityManager* EntityManager, FComponentMask* InOutEntityComponentTypes) const override
	{
		for (const UMovieSceneEntityGroupingSystem::FEntityGroupingHandlerInfo& HandlerInfo : System->GroupHandlers)
		{
			if (HandlerInfo.ComponentFilter.Match(*InOutEntityComponentTypes))
			{
				InOutEntityComponentTypes->Set(BuiltInComponents->Group);
				break;
			}
		}
	}

	virtual void InitializeAllocation(FEntityAllocation* Allocation, const FComponentMask& AllocationType) const override
	{
		FEntityAllocationWriteContext WriteContext = FEntityAllocationWriteContext::NewAllocation();
		TComponentWriter<FEntityGroupID> GroupIDs = Allocation->WriteComponents(BuiltInComponents->Group, WriteContext);

		// Find the policy key for this allocation. We'll initialize all group IDs to an invalid group but
		// with a valid policy key.
		FEntityGroupingPolicyKey PolicyKey;
		for (auto It = System->GroupHandlers.CreateConstIterator(); It; ++It)
		{
			if (It->ComponentFilter.Match(AllocationType))
			{
				PolicyKey = FEntityGroupingPolicyKey(It.GetIndex());
				break;
			}
		}
		ensure(PolicyKey.IsValid());

		const int32 Num = Allocation->Num();
		for (int32 Index = 0; Index < Num; ++Index)
		{
			GroupIDs[Index] = FEntityGroupID(PolicyKey, INDEX_NONE);
		}
	}
};

struct FUpdateGroupsTask
{
	using FEntityGroupInfo = UMovieSceneEntityGroupingSystem::FEntityGroupInfo;
	using FEntityGroupingHandlerInfo = UMovieSceneEntityGroupingSystem::FEntityGroupingHandlerInfo;

	UMovieSceneEntityGroupingSystem* System;
	FBuiltInComponentTypes* BuiltInComponents;
	bool bFreeGroupIDs = true;

	FUpdateGroupsTask(UMovieSceneEntityGroupingSystem* InSystem, bool bInFreeGroupIDs = true)
		: System(InSystem)
		, BuiltInComponents(nullptr)
		, bFreeGroupIDs(bInFreeGroupIDs)
	{
	}

	void ForEachAllocation(FEntityAllocationIteratorItem Item, FReadEntityIDs EntityIDs, TWrite<FEntityGroupID> GroupComponents)
	{
		TBitArray<> MatchingHandlers;
		const FComponentMask& AllocationType = Item.GetAllocationType();
		GatherMatchingGroupingHandlers(AllocationType, MatchingHandlers);

		ensureMsgf(
				MatchingHandlers.CountSetBits() <= 1, 
				TEXT("Found more than one matching grouping handler for an entity allocation. "
					 "Entities cannot belong to more than one group, so we will only process the first one!"));

		const int32 HandlerIndex = MatchingHandlers.Find(true);
		if (ensureMsgf(
					HandlerIndex != INDEX_NONE,
					TEXT("No matching gropuing handling for entity allocation even though it has a group ID component!")))
		{
			FEntityGroupingHandlerInfo& HandlerInfo = System->GroupHandlers[HandlerIndex];
			FEntityGroupBuilder Builder(System, FEntityGroupingPolicyKey{ HandlerIndex });
			HandlerInfo.Handler->ProcessAllocation(Item, EntityIDs, GroupComponents, &Builder);
		}
	}

	void GatherMatchingGroupingHandlers(const FComponentMask& AllocationType, TBitArray<>& OutMatchingHandlers)
	{
		for (auto It = System->GroupHandlers.CreateConstIterator(); It; ++It)
		{
			if (It->ComponentFilter.Match(AllocationType))
			{
				int32 Index = It.GetIndex();
				OutMatchingHandlers.PadToNum(Index + 1, false);
				OutMatchingHandlers[Index] = true;
			}
		}
	}

	void PostTask()
	{
		if (bFreeGroupIDs)
		{
			System->FreeEmptyGroups();
		}
	}
};

} // namespace UE::MovieScene

UMovieSceneEntityGroupingSystem::UMovieSceneEntityGroupingSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	SystemCategories = EEntitySystemCategory::Core;
	Phase = ESystemPhase::Instantiation;
	RelevantComponent = BuiltInComponents->Group;

	// We know that (as the time of this writing) we only have two systems that support grouping:
	// object properties and materials.
	GroupHandlers.Reserve(2);

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		// We produce the group component. It's in the name.
		DefineComponentProducer(GetClass(), BuiltInComponents->Group);

		// This isn't something we *really* need, but pretty much all use-cases we have will want
		// to group things using bound objects and/or bound scene components, so let's run after the
		// systems that set those up.
		DefineComponentConsumer(GetClass(), BuiltInComponents->BoundObject);
		DefineImplicitPrerequisite(UMovieSceneRootInstantiatorSystem::StaticClass(), GetClass());
	}
}

bool UMovieSceneEntityGroupingSystem::IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	// We are relevant if we have any groupings to do.
	return !GroupHandlers.IsEmpty();
}

int32 UMovieSceneEntityGroupingSystem::AllocateGroupIndex(FEntityGroupingPolicyKey InPolicy)
{
	return AllocatedGroupIndices.Emplace(InPolicy);
}

void UMovieSceneEntityGroupingSystem::FreeEmptyGroups()
{
	if (EmptyGroupIndices.Find(true) == INDEX_NONE)
	{
		return;
	}

	// Free the indices we don't use anymore, and free the group key lookup entry too.
	for (TConstSetBitIterator<> It(EmptyGroupIndices); It; ++It)
	{
		const int32 GroupIndex = It.GetIndex();
		FEntityGroupingPolicyKey Policy = AllocatedGroupIndices[GroupIndex];

		GroupHandlers[Policy.Index].Handler->OnGroupIndexFreed(GroupIndex);
		
		// The group is now empty! Let's remove it.
		Groups.Remove(FEntityGroupID{ Policy, GroupIndex });
		AllocatedGroupIndices.RemoveAt(GroupIndex);
	}
	EmptyGroupIndices.Empty();
}

void UMovieSceneEntityGroupingSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	// Mutate any new allocation that fits any of our grouping policies by adding the group component.
	FEntityComponentFilter BroadFilter;
	BroadFilter.Any({ BuiltInComponents->Tags.NeedsLink });
	FAddGroupMutation Mutation(this);
	Linker->EntityManager.MutateAll(BroadFilter, Mutation);

	// Go over all the entities and update their groups.
	FUpdateGroupsTask GroupTask(this);
	FEntityTaskBuilder()
	.ReadEntityIDs()
	.Write(BuiltInComponents->Group)
	.FilterAny({ BuiltInComponents->Tags.NeedsLink, BuiltInComponents->Tags.NeedsUnlink })
	.RunInline_PerAllocation(&Linker->EntityManager, GroupTask);

#if DO_CHECK && !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	for (int32 EntityIndex = 0; EntityIndex < EntityIDToGroup.GetMaxIndex(); ++EntityIndex)
	{
		if (!EntityIDToGroup.IsValidIndex(EntityIndex))
		{
			continue;
		}

		TReadOptional<FEntityGroupID> GroupComponent = Linker->EntityManager.ReadComponent(FMovieSceneEntityID::FromIndex(EntityIndex), BuiltInComponents->Group);
		ensureMsgf(
				GroupComponent.IsValid() && *GroupComponent == EntityIDToGroup[EntityIndex],
				TEXT("Found mismatch between group cache and group component!"));
	}
#endif
}

void UMovieSceneEntityGroupingSystem::OnLink()
{
#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectsReplaced.AddUObject(this, &UMovieSceneEntityGroupingSystem::OnObjectsReplaced);
#endif
}

void UMovieSceneEntityGroupingSystem::OnUnlink()
{
	const bool bIsEmpty = GroupHandlers.Num() == 0 && Groups.IsEmpty();
	if (!ensure(bIsEmpty))
	{
		GroupHandlers.Empty();
		Groups.Empty();
	}

#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
#endif
}

void UMovieSceneEntityGroupingSystem::OnCleanTaggedGarbage()
{
	using namespace UE::MovieScene;

	// Garbage has been tagged with NeedsUnlink, so visit those and remove them from their groups.
	// In theory, a group with garbage in its group key should get emptied, because we assume that
	// a group key only has garbage in it if the components used to derive it also have garbage in
	// them. And in that case, their entities would have been flagged, and removed from that
	// group.
	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	// Don't free group IDs. We only want to free them on instantiation phases, so that downstream
	// systems don't see any surprisingly re-used IDs from one instantiation frame to another.
	const bool bFreeGroupIDs = false;
	FUpdateGroupsTask GroupTask(this, bFreeGroupIDs);
	FEntityTaskBuilder()
	.ReadEntityIDs()
	.Write(BuiltInComponents->Group)
	.FilterAny({ BuiltInComponents->Tags.NeedsUnlink })
	.RunInline_PerAllocation(&Linker->EntityManager, GroupTask);
}

#if WITH_EDITOR
void UMovieSceneEntityGroupingSystem::OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap)
{
	for (const FEntityGroupingHandlerInfo& HandlerInfo : GroupHandlers)
	{
		HandlerInfo.Handler->OnObjectsReplaced(ReplacementMap);
	}
}
#endif

void UMovieSceneEntityGroupingSystem::RemoveGrouping(FEntityGroupingPolicyKey InPolicyKey)
{
	using namespace UE::MovieScene;

	check(InPolicyKey.IsValid());

	// Get the list of existing groups using this policy, and clean them up.
	TArray<FEntityGroupID> ExistingGroupIDs;
	Groups.GetKeys(ExistingGroupIDs);
	for (const FEntityGroupID& ExistingGroupID : ExistingGroupIDs)
	{
		if (!ensureMsgf(ExistingGroupID.PolicyKey != InPolicyKey, TEXT("Found leftover group from policy being removed")))
		{
			Groups.Remove(ExistingGroupID);
		}
	}

	// Remove the handler itself.
	GroupHandlers.RemoveAt(InPolicyKey.Index);
}
