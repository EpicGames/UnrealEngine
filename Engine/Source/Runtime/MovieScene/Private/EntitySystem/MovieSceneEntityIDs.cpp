// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntityManager.h"

namespace UE
{
namespace MovieScene
{

bool FEntityHandle::IsValid(const FEntityManager& InEntityManager) const
{
	return InEntityManager.IsHandleValid(*this);
}

bool FComponentTypeIDFilter::Passes(const FComponentMask& Type) const
{
	FComponentTypeID ConditionComponent = GetComponentType();
	return !ConditionComponent || Type.Contains(ConditionComponent) == PassFlag;
}

} // namespace MovieScene
} // namespace UE
