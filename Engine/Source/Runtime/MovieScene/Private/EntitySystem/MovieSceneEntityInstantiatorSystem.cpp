// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneEntityInstantiatorSystem.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/BuiltInComponentTypes.h"

#include "MovieSceneObjectBindingID.h"
#include "ProfilingDebugging/CountersTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneEntityInstantiatorSystem)

DECLARE_CYCLE_STAT(TEXT("UnlinkStaleObjectBindings"), MovieSceneEval_UnlinkStaleObjectBindings, STATGROUP_MovieSceneECS);

UMovieSceneEntityInstantiatorSystem::UMovieSceneEntityInstantiatorSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	Phase = UE::MovieScene::ESystemPhase::Instantiation;
}

void UMovieSceneEntityInstantiatorSystem::UnlinkStaleObjectBindings(UE::MovieScene::TComponentTypeID<FGuid> BindingType)
{
	using namespace UE::MovieScene;

	MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_UnlinkStaleObjectBindings);

	check(Linker);

	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();
	if (!InstanceRegistry->HasInvalidatedBindings())
	{
		return;
	}

	FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();

	TArray<FMovieSceneEntityID> StaleEntities;

	auto GatherStaleBindings = [InstanceRegistry, &StaleEntities](FMovieSceneEntityID EntityID, FInstanceHandle InstanceHandle, const FGuid& InObjectBindingID)
	{
		if (InstanceRegistry->IsBindingInvalidated(InObjectBindingID, InstanceHandle))
		{
			StaleEntities.Add(EntityID);
		}
	};

	// Gather all newly instanced entities with an object binding ID
	FEntityTaskBuilder()
	.ReadEntityIDs()
	.Read(Components->InstanceHandle)
	.Read(BindingType)
	.FilterAll({ Components->Tags.ImportedEntity })
	.FilterNone({ Components->Tags.NeedsUnlink })
	.Iterate_PerEntity(&Linker->EntityManager, GatherStaleBindings);

	for (FMovieSceneEntityID Entity : StaleEntities)
	{
		// Tag this as needs link, and all children as needs unlink
		Linker->EntityManager.AddComponent(Entity, Components->Tags.NeedsLink);
		Linker->EntityManager.AddComponent(Entity, Components->Tags.NeedsUnlink, EEntityRecursion::Children);
	}
}


void UMovieSceneEntityInstantiatorSystem::UnlinkStaleObjectBindings(UE::MovieScene::TComponentTypeID<FMovieSceneObjectBindingID> BindingType)
{
	using namespace UE::MovieScene;

	check(Linker);

	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();
	if (!InstanceRegistry->HasInvalidatedBindings())
	{
		return;
	}

	FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();

	TArray<FMovieSceneEntityID> StaleEntities;

	auto GatherStaleBindings = [InstanceRegistry, &StaleEntities](FMovieSceneEntityID EntityID, FInstanceHandle InstanceHandle, FMovieSceneObjectBindingID InObjectBindingID)
	{
		const FSequenceInstance* TargetInstance = &InstanceRegistry->GetInstance(InstanceHandle);

		FMovieSceneSequenceID ThisSequenceID     = TargetInstance->GetSequenceID();
		FMovieSceneSequenceID RemappedSequenceID = InObjectBindingID.ResolveSequenceID(ThisSequenceID, TargetInstance->GetSharedPlaybackState());

		if (RemappedSequenceID == ThisSequenceID)
		{
			if (InstanceRegistry->IsBindingInvalidated(InObjectBindingID.GetGuid(), InstanceHandle))
			{
				StaleEntities.Add(EntityID);
			}
		}
		else
		{
			if (!TargetInstance->IsRootSequence())
			{
				TargetInstance = &InstanceRegistry->GetInstance(TargetInstance->GetRootInstanceHandle());
			}

			FInstanceHandle SubInstance = TargetInstance->FindSubInstance(RemappedSequenceID);
			if (!InstanceRegistry->IsHandleValid(SubInstance) || InstanceRegistry->IsBindingInvalidated(InObjectBindingID.GetGuid(), SubInstance))
			{
				StaleEntities.Add(EntityID);
			}
		}
	};

	// Gather all newly instanced entities with an object binding ID
	FEntityTaskBuilder()
	.ReadEntityIDs()
	.Read(Components->InstanceHandle)
	.Read(BindingType)
	.FilterAll({ Components->Tags.ImportedEntity })
	.FilterNone({ Components->Tags.NeedsUnlink })
	.Iterate_PerEntity(&Linker->EntityManager, GatherStaleBindings);

	for (FMovieSceneEntityID Entity : StaleEntities)
	{
		// Tag this as needs link, and all children as needs unlink
		Linker->EntityManager.AddComponent(Entity, Components->Tags.NeedsLink);
		Linker->EntityManager.AddComponent(Entity, Components->Tags.NeedsUnlink, EEntityRecursion::Children);
	}
}

