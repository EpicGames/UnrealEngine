// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneEntityLedger.h"
#include "EntitySystem/MovieSceneInstanceRegistry.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntityMutations.h"
#include "EntitySystem/IMovieSceneEntityDecorator.h"

#include "Evaluation/MovieSceneEvaluationField.h"
#include "Conditions/MovieSceneCondition.h"

#include "MovieSceneSection.h"
#include "MovieSceneTrack.h"

namespace UE
{
namespace MovieScene
{

void FEntityLedger::UpdateEntities(UMovieSceneEntitySystemLinker* Linker, const FEntityImportSequenceParams& ImportParams, const FMovieSceneEntityComponentField* EntityField, const FMovieSceneEvaluationFieldEntitySet& NewEntities)
{
	FMovieSceneEvaluationFieldEntitySet OutConditionalEntities;
	TMap<uint32, bool> ConditionResultCache;
	UpdateEntities(Linker, ImportParams, EntityField, NewEntities, OutConditionalEntities, ConditionResultCache);
}

void FEntityLedger::UpdateEntities(UMovieSceneEntitySystemLinker* Linker, const FEntityImportSequenceParams& ImportParams, const FMovieSceneEntityComponentField* EntityField, const FMovieSceneEvaluationFieldEntitySet& NewEntities, FMovieSceneEvaluationFieldEntitySet& OutConditionalEntities, TMap<uint32, bool>& ConditionResultCache)
{
	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();
	if (NewEntities.Num() != 0)
	{
		// Destroy any entities that are no longer relevant
		if (ImportedEntities.Num() != 0)
		{
			FComponentMask FinishedMask = FBuiltInComponentTypes::Get()->FinishedMask;

			for (auto It = ImportedEntities.CreateIterator(); It; ++It)
			{
				if (!NewEntities.Contains(It.Key()))
				{
					if (It.Value().EntityID)
					{
						Linker->EntityManager.AddComponents(It.Value().EntityID, FinishedMask, EEntityRecursion::Full);
					}
					It.RemoveCurrent();
				}
			}
		}

		// If we've invalidated or we haven't imported anything yet, we can simply (re)import everything
		if (ImportedEntities.Num() == 0 || bInvalidated)
		{
			for (const FMovieSceneEvaluationFieldEntityQuery& Query : NewEntities)
			{
				ImportEntity(Linker, ImportParams, EntityField, Query, OutConditionalEntities, ConditionResultCache);
			}
		}
		else for (const FMovieSceneEvaluationFieldEntityQuery& Query : NewEntities)
		{
			FImportedEntityData Existing = ImportedEntities.FindRef(Query.Entity.Key);
			if (!Existing.EntityID || Existing.MetaDataIndex != Query.MetaDataIndex)
			{
				ImportEntity(Linker, ImportParams, EntityField, Query, OutConditionalEntities, ConditionResultCache);
			}
		}
	}
	else
	{
		UnlinkEverything(Linker);
	}

	// Nothing is invalidated now
	bInvalidated = false;
}

void FEntityLedger::UpdateOneShotEntities(UMovieSceneEntitySystemLinker* Linker, const FEntityImportSequenceParams& ImportParams, const FMovieSceneEntityComponentField* EntityField, const FMovieSceneEvaluationFieldEntitySet& NewEntities)
{
	TMap<uint32, bool> ConditionResultCache;
	UpdateOneShotEntities(Linker, ImportParams, EntityField, NewEntities, ConditionResultCache);
}

void FEntityLedger::UpdateOneShotEntities(UMovieSceneEntitySystemLinker* Linker, const FEntityImportSequenceParams& ImportParams, const FMovieSceneEntityComponentField* EntityField, const FMovieSceneEvaluationFieldEntitySet& NewEntities, TMap<uint32, bool>& ConditionResultCache)
{
	checkf(OneShotEntities.Num() == 0, TEXT("One shot entities should not be updated multiple times per-evaluation. They must not have gotten cleaned up correctly."));
	if (NewEntities.Num() == 0)
	{
		return;
	}

	FEntityImportParams Params;
	Params.Sequence = ImportParams;

	FMovieSceneEvaluationFieldEntitySet DummyEntitySet;
	for (const FMovieSceneEvaluationFieldEntityQuery& Query : NewEntities)
	{
		UObject* EntityOwner = Query.Entity.Key.EntityOwner.Get();
		IMovieSceneEntityProvider* Provider = Cast<IMovieSceneEntityProvider>(EntityOwner);
		if (!Provider)
		{
			continue;
		}

		Params.EntityID = Query.Entity.Key.EntityID;

		if (EntityField)
		{
			Params.EntityMetaData = EntityField->FindMetaData(Query);
			Params.SharedMetaData = EntityField->FindSharedMetaData(Query);
		}

		if (ImportParams.bPreRoll && (Params.EntityMetaData == nullptr || Params.EntityMetaData->bEvaluateInSequencePreRoll == false))
		{
			return;
		}
		if (ImportParams.bPostRoll && (Params.EntityMetaData == nullptr || Params.EntityMetaData->bEvaluateInSequencePostRoll == false))
		{
			return;
		}

		// Check conditions
		if (!CanImportEntity(Linker, ImportParams, EntityField, Query, DummyEntitySet, ConditionResultCache))
		{
			return;
		}

		FImportedEntity ImportedEntity;
		Provider->ImportEntity(Linker, Params, &ImportedEntity);

		if (!ImportedEntity.IsEmpty())
		{
			if (UMovieSceneDecorationContainerObject* DecorationContainer = Cast<UMovieSceneDecorationContainerObject>(EntityOwner))
			{
				for (UObject* Decoration : DecorationContainer->GetDecorations())
				{
					if (IMovieSceneEntityDecorator* EntityDecorator = Cast<IMovieSceneEntityDecorator>(Decoration))
					{
						EntityDecorator->ExtendEntity(Linker, Params, &ImportedEntity);
					}
				}
			}

			if (UMovieSceneSection* Section = Cast<UMovieSceneSection>(EntityOwner))
			{
				Section->BuildDefaultComponents(Linker, Params, &ImportedEntity);

				if (UMovieSceneTrack* Track = Section->GetTypedOuter<UMovieSceneTrack>())
				{
					for (UObject* Decoration : Track->GetDecorations())
					{
						if (IMovieSceneEntityDecorator* EntityDecorator = Cast<IMovieSceneEntityDecorator>(Decoration))
						{
							EntityDecorator->ExtendEntity(Linker, Params, &ImportedEntity);
						}
					}
				}
			}

			FMovieSceneEntityID NewEntityID = ImportedEntity.Manufacture(Params, &Linker->EntityManager);
			OneShotEntities.Add(NewEntityID);
		}
	}
}

void FEntityLedger::UpdateConditionalEntities(UMovieSceneEntitySystemLinker* Linker, const FEntityImportSequenceParams& ImportParams, const FMovieSceneEntityComponentField* EntityField, const FMovieSceneEvaluationFieldEntitySet& ConditionalEntities)
{
	if (ConditionalEntities.Num() == 0 || EntityField == nullptr)
	{
		return;
	}

	FEntityImportParams Params;
	Params.Sequence = ImportParams;
	FMovieSceneEvaluationFieldEntitySet DummyEntitySet;
	TMap<uint32, bool> ConditionResultCache;
	ConditionResultCache.Reserve(ConditionalEntities.Num());
	for (const FMovieSceneEvaluationFieldEntityQuery& Query : ConditionalEntities)
	{
		Params.EntityID = Query.Entity.Key.EntityID;

		if (EntityField)
		{
			Params.EntityMetaData = EntityField->FindMetaData(Query);
			Params.SharedMetaData = EntityField->FindSharedMetaData(Query);
		}

		// We cache all results here in the temp cache we've made so at least we won't re-run the same condition multiple times each tick
		bool bConditionPassed = CanImportEntity(Linker, ImportParams, EntityField, Query, DummyEntitySet, ConditionResultCache, true);

		FImportedEntityData& EntityData = ImportedEntities.FindOrAdd(Query.Entity.Key);
		if (bConditionPassed && (!EntityData.EntityID || EntityData.MetaDataIndex != Query.MetaDataIndex))
		{
			// A previously failing condition has now passed. Attempt to properly import the entity.
			EntityData.MetaDataIndex = Query.MetaDataIndex;
			UObject* EntityOwner = Query.Entity.Key.EntityOwner.Get();
			IMovieSceneEntityProvider* Provider = Cast<IMovieSceneEntityProvider>(EntityOwner);
			if (!Provider)
			{
				return;
			}

			FImportedEntity ImportedEntity;
			Provider->ImportEntity(Linker, Params, &ImportedEntity);

			if (!ImportedEntity.IsEmpty())
			{
				if (UMovieSceneDecorationContainerObject* DecorationContainer = Cast<UMovieSceneDecorationContainerObject>(EntityOwner))
				{
					for (UObject* Decoration : DecorationContainer->GetDecorations())
					{
						if (IMovieSceneEntityDecorator* EntityDecorator = Cast<IMovieSceneEntityDecorator>(Decoration))
						{
							EntityDecorator->ExtendEntity(Linker, Params, &ImportedEntity);
						}
					}
				}

				if (UMovieSceneSection* Section = Cast<UMovieSceneSection>(EntityOwner))
				{
					Section->BuildDefaultComponents(Linker, Params, &ImportedEntity);

					if (UMovieSceneTrack* Track = Section->GetTypedOuter<UMovieSceneTrack>())
					{
						for (UObject* Decoration : Track->GetDecorations())
						{
							if (IMovieSceneEntityDecorator* EntityDecorator = Cast<IMovieSceneEntityDecorator>(Decoration))
							{
								EntityDecorator->ExtendEntity(Linker, Params, &ImportedEntity);
							}
						}
					}
				}

				FMovieSceneEntityID NewEntityID = ImportedEntity.Manufacture(Params, &Linker->EntityManager);

				Linker->EntityManager.ReplaceEntityID(EntityData.EntityID, NewEntityID);
			}
		}
		else if (!bConditionPassed && EntityData.EntityID)
		{
			// A previously succeeding condition has now failed. Remove the entity.
			FComponentMask FinishedMask = FBuiltInComponentTypes::Get()->FinishedMask;
			Linker->EntityManager.AddComponents(EntityData.EntityID, FinishedMask, EEntityRecursion::Full);
			EntityData.EntityID = FMovieSceneEntityID();
			EntityData.MetaDataIndex = INDEX_NONE;
		}
	}
}

void FEntityLedger::Invalidate()
{
	bInvalidated = true;
}

bool FEntityLedger::IsEmpty() const
{
	return ImportedEntities.Num() == 0;
}

bool FEntityLedger::HasImportedEntity(const FMovieSceneEvaluationFieldEntityKey& EntityKey) const
{
	return ImportedEntities.Contains(EntityKey);
}

FMovieSceneEntityID FEntityLedger::FindImportedEntity(const FMovieSceneEvaluationFieldEntityKey& EntityKey) const
{
	return ImportedEntities.FindRef(EntityKey).EntityID;
}

void FEntityLedger::FindImportedEntities(TWeakObjectPtr<UObject> EntityOwner, TArray<FMovieSceneEntityID>& OutEntityIDs) const
{
	for (const TPair<FMovieSceneEvaluationFieldEntityKey, FImportedEntityData>& Pair : ImportedEntities)
	{
		if (Pair.Key.EntityOwner == EntityOwner)
		{
			OutEntityIDs.Add(Pair.Value.EntityID);
		}
	}
}

bool FEntityLedger::CanImportEntity(UMovieSceneEntitySystemLinker* Linker, const FEntityImportSequenceParams& ImportParams, const FMovieSceneEntityComponentField* EntityField, const FMovieSceneEvaluationFieldEntityQuery& Query, FMovieSceneEvaluationFieldEntitySet& OutPerTickConditionalEntities, TMap<uint32, bool>& ConditionResultCache, bool bUpdatingPerTickEntities)
{
	// If we don't have a condition, just return true
	const FMovieSceneEvaluationFieldEntityMetaData* EntityMetadata = EntityField ? EntityField->FindMetaData(Query) : nullptr;
	if (!EntityMetadata || !EntityMetadata->Condition)
	{
		return true;
	}


	const FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();
	const FSequenceInstance& SequenceInstance = InstanceRegistry->GetInstance(ImportParams.InstanceHandle);

	bool bCanCacheResult = EntityMetadata->Condition->CanCacheResult(SequenceInstance.GetSharedPlaybackState());
	if (!bCanCacheResult)
	{
		// If we can't cache the result, it will need to be checked again next tick
		OutPerTickConditionalEntities.Add(Query);
	}

	FGuid BindingID;

	if (EntityField)
	{
		if (const FMovieSceneEvaluationFieldSharedEntityMetaData* SharedMetadata = EntityField->FindSharedMetaData(Query))
		{
			BindingID = SharedMetadata->ObjectBindingID;
		}
	}

	// If we have a valid binding ID, and the condition depends on object binding, then we must ensure the object binding is resolved
	// before evaluating the condition. To ensure this, we always defer checking the condition for non-global conditions on bound objects to the bound object resolver.
	// We don't do this for updating per-tick entities as the bound object resolver is only run once.
	if (!bUpdatingPerTickEntities && BindingID.IsValid() && EntityMetadata->Condition->GetConditionScope() != EMovieSceneConditionScope::Global)
	{
		return true;
	}

	uint32 CacheKey = EntityMetadata->Condition->ComputeCacheKey(BindingID, ImportParams.SequenceID, SequenceInstance.GetSharedPlaybackState(), Query.Entity.Key.EntityOwner.Get());
	
	if (bool* CachedResult = ConditionResultCache.Find(CacheKey))
	{
		return *CachedResult;
	}
	else
	{
		bool bResult = EntityMetadata->Condition->EvaluateCondition(BindingID, ImportParams.SequenceID, SequenceInstance.GetSharedPlaybackState());
		// We always cache the results for per tick entities as they get thrown away after the tick, and we might as well prevent the same condition from getting re-evaluated multiple times per tick.
		if (bCanCacheResult || bUpdatingPerTickEntities)
		{
			ConditionResultCache.Add(CacheKey, bResult);
		}
		return bResult;
	}
}

void FEntityLedger::ImportEntity(UMovieSceneEntitySystemLinker* Linker, const FEntityImportSequenceParams& ImportParams, const FMovieSceneEntityComponentField* EntityField, const FMovieSceneEvaluationFieldEntityQuery& Query)
{
	FMovieSceneEvaluationFieldEntitySet OutConditionalEntities;
	TMap<uint32, bool> ConditionResultCache;
	ImportEntity(Linker, ImportParams, EntityField, Query, OutConditionalEntities, ConditionResultCache);
}

void FEntityLedger::ImportEntity(UMovieSceneEntitySystemLinker* Linker, const FEntityImportSequenceParams& ImportParams, const FMovieSceneEntityComponentField* EntityField, const FMovieSceneEvaluationFieldEntityQuery& Query, FMovieSceneEvaluationFieldEntitySet& OutPerTickConditionalEntities, TMap<uint32, bool>& ConditionResultCache)
{
	// We always add an entry even if no entity was imported by the provider to ensure that we do not repeatedly try and import the same entity every frame
	FImportedEntityData& EntityData = ImportedEntities.FindOrAdd(Query.Entity.Key);
	EntityData.MetaDataIndex = Query.MetaDataIndex;

	UObject* EntityOwner = Query.Entity.Key.EntityOwner.Get();
	IMovieSceneEntityProvider* Provider = Cast<IMovieSceneEntityProvider>(EntityOwner);
	if (!Provider)
	{
		return;
	}

	FEntityImportParams Params;
	Params.Sequence = ImportParams;
	Params.EntityID = Query.Entity.Key.EntityID;
	
	if (EntityField)
	{
		Params.EntityMetaData = EntityField->FindMetaData(Query);
		Params.SharedMetaData = EntityField->FindSharedMetaData(Query);
	}

	bool bUnsupportedPrePostRoll = false;
	if (ImportParams.bPreRoll && (Params.EntityMetaData == nullptr || Params.EntityMetaData->bEvaluateInSequencePreRoll == false))
	{
		bUnsupportedPrePostRoll = true;
	}
	else if (ImportParams.bPostRoll && (Params.EntityMetaData == nullptr || Params.EntityMetaData->bEvaluateInSequencePostRoll == false))
	{
		bUnsupportedPrePostRoll = true;
	}
	if (bUnsupportedPrePostRoll)
	{
		if (EntityData.EntityID)
		{
			// If we are reimporting an existing entity and we entered a pre/post-roll time range
			// that this entity shouldn't update in, we need to delete that entity.
			Linker->EntityManager.AddComponents(EntityData.EntityID, FBuiltInComponentTypes::Get()->FinishedMask, EEntityRecursion::Full);
			EntityData.EntityID = FMovieSceneEntityID();
		}
		return;
	}

	// Check conditions
	if (!CanImportEntity(Linker, ImportParams, EntityField, Query, OutPerTickConditionalEntities, ConditionResultCache))
	{
		// In case of cache invalidation, we may already have an entity here that we need to mark as finished
		if (EntityData.EntityID)
		{
			FComponentMask FinishedMask = FBuiltInComponentTypes::Get()->FinishedMask;
			Linker->EntityManager.AddComponents(EntityData.EntityID, FinishedMask, EEntityRecursion::Full);
			EntityData.EntityID = FMovieSceneEntityID();
		}
		return;
	}

	FImportedEntity ImportedEntity;
	Provider->ImportEntity(Linker, Params, &ImportedEntity);

	if (!ImportedEntity.IsEmpty())
	{
		if (UMovieSceneDecorationContainerObject* DecorationContainer = Cast<UMovieSceneDecorationContainerObject>(EntityOwner))
		{
			for (UObject* Decoration : DecorationContainer->GetDecorations())
			{
				if (IMovieSceneEntityDecorator* EntityDecorator = Cast<IMovieSceneEntityDecorator>(Decoration))
				{
					EntityDecorator->ExtendEntity(Linker, Params, &ImportedEntity);
				}
			}
		}

		if (UMovieSceneSection* Section = Cast<UMovieSceneSection>(EntityOwner))
		{
			Section->BuildDefaultComponents(Linker, Params, &ImportedEntity);

			if (UMovieSceneTrack* Track = Section->GetTypedOuter<UMovieSceneTrack>())
			{
				for (UObject* Decoration : Track->GetDecorations())
				{
					if (IMovieSceneEntityDecorator* EntityDecorator = Cast<IMovieSceneEntityDecorator>(Decoration))
					{
						EntityDecorator->ExtendEntity(Linker, Params, &ImportedEntity);
					}
				}
			}
		}

		FMovieSceneEntityID NewEntityID = ImportedEntity.Manufacture(Params, &Linker->EntityManager);

		Linker->EntityManager.ReplaceEntityID(EntityData.EntityID, NewEntityID);
	}
}

void FEntityLedger::UnlinkEverything(UMovieSceneEntitySystemLinker* Linker, EUnlinkEverythingMode UnlinkMode)
{
	FComponentTypeID NeedsLink = FBuiltInComponentTypes::Get()->Tags.NeedsLink;
	FComponentMask FinishedMask = FBuiltInComponentTypes::Get()->FinishedMask;

	for (TPair<FMovieSceneEvaluationFieldEntityKey, FImportedEntityData>& Pair : ImportedEntities)
	{
		if (Pair.Value.EntityID)
		{
			if (UnlinkMode == EUnlinkEverythingMode::CleanGarbage)
			{
				Linker->EntityManager.RemoveComponent(Pair.Value.EntityID, NeedsLink, EEntityRecursion::Full);
			}
			Linker->EntityManager.AddComponents(Pair.Value.EntityID, FinishedMask, EEntityRecursion::Full);
		}
	}
	ImportedEntities.Empty();
}

void FEntityLedger::UnlinkOneShots(UMovieSceneEntitySystemLinker* Linker)
{
	FComponentMask FinishedMask = FBuiltInComponentTypes::Get()->FinishedMask;

	for (FMovieSceneEntityID Entity : OneShotEntities)
	{
		Linker->EntityManager.AddComponents(Entity, FinishedMask, EEntityRecursion::Full);
	}
	OneShotEntities.Empty();
}

void FEntityLedger::CleanupLinkerEntities(const TSet<FMovieSceneEntityID>& LinkerEntities)
{
	for (int32 Index = OneShotEntities.Num()-1; Index >= 0; --Index)
	{
		if (LinkerEntities.Contains(OneShotEntities[Index]))
		{
			OneShotEntities.RemoveAtSwap(Index, EAllowShrinking::No);
		}
	}
	for (auto It = ImportedEntities.CreateIterator(); It; ++It)
	{
		FMovieSceneEntityID EntityID = It.Value().EntityID;
		if (EntityID && LinkerEntities.Contains(EntityID))
		{
			It.RemoveCurrent();
		}
	}
}

void FEntityLedger::TagGarbage(UMovieSceneEntitySystemLinker* Linker)
{
	FComponentTypeID NeedsLink = FBuiltInComponentTypes::Get()->Tags.NeedsLink;
	FComponentTypeID NeedsUnlink = FBuiltInComponentTypes::Get()->Tags.NeedsUnlink;

	for (auto It = ImportedEntities.CreateIterator(); It; ++It)
	{
		if (!It.Key().EntityOwner.IsValid())
		{
			if (It.Value().EntityID)
			{
				Linker->EntityManager.RemoveComponent(It.Value().EntityID, NeedsLink, EEntityRecursion::Full);
				Linker->EntityManager.AddComponent(It.Value().EntityID, NeedsUnlink, EEntityRecursion::Full);
			}
			It.RemoveCurrent();
		}
	}
}

bool FEntityLedger::Contains(UMovieSceneEntitySystemLinker* Linker, const FEntityComponentFilter& Filter) const
{
	bool bResult = false;

	auto Visit = [&Filter, &bResult, Linker](FMovieSceneEntityID EntityID)
	{
		bResult = Filter.Match(Linker->EntityManager.GetEntityType(EntityID));
	};

	for (FMovieSceneEntityID EntityID : OneShotEntities)
	{
		Visit(EntityID);
		Linker->EntityManager.IterateChildren_ParentFirst(EntityID, Visit);

		if (bResult)
		{
			return true;
		}
	}

	for (const TPair<FMovieSceneEvaluationFieldEntityKey, FImportedEntityData>& Pair : ImportedEntities)
	{
		Visit(Pair.Value.EntityID);
		Linker->EntityManager.IterateChildren_ParentFirst(Pair.Value.EntityID, Visit);

		if (bResult)
		{
			return true;
		}
	}

	return bResult;
}

void FEntityLedger::MutateAll(UMovieSceneEntitySystemLinker* Linker, const FEntityComponentFilter& Filter, const IMovieScenePerEntityMutation& Mutation) const
{
	auto Visit = [&Filter, &Mutation, Linker](FMovieSceneEntityID EntityID)
	{
		const FComponentMask& ExistingType = Linker->EntityManager.GetEntityType(EntityID);
		if (Filter.Match(ExistingType))
		{
			FComponentMask NewType = ExistingType;
			Mutation.CreateMutation(&Linker->EntityManager, &NewType);

			if (!NewType.CompareSetBits(ExistingType))
			{
				Linker->EntityManager.ChangeEntityType(EntityID, NewType);

				FEntityInfo EntityInfo = Linker->EntityManager.GetEntity(EntityID);
				Mutation.InitializeEntities(EntityInfo.Data.AsRange(), NewType);
			}
		}
	};

	for (FMovieSceneEntityID EntityID : OneShotEntities)
	{
		Visit(EntityID);
		Linker->EntityManager.IterateChildren_ParentFirst(EntityID, Visit);
	}

	for (const TPair<FMovieSceneEvaluationFieldEntityKey, FImportedEntityData>& Pair : ImportedEntities)
	{
		Visit(Pair.Value.EntityID);
		Linker->EntityManager.IterateChildren_ParentFirst(Pair.Value.EntityID, Visit);
	}
}

} // namespace MovieScene
} // namespace UE
