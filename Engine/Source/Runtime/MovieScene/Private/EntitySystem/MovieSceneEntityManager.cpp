// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/MovieSceneEntityMutations.h"
#include "EntitySystem/MovieSceneComponentRegistry.h"

#include "EntitySystem/BuiltInComponentTypes.h"

#include "Algo/Find.h"
#include "UObject/StrongObjectPtr.h"
#include "Containers/SortedMap.h"
#include "AutoRTFM.h"

#include "HAL/PlatformProcess.h"
#include "Misc/FeedbackContext.h"

#include "EntitySystem/EntityAllocationIterator.h"

#include "AutoRTFM.h"

#if DO_GUARD_SLOW

bool GCheckMovieSceneEntityManagerInvariants = false;
FAutoConsoleVariableRef CVarCheckMovieSceneEntityManagerInvariants(
	TEXT("Sequencer.CheckEntityManagerInvariants"),
	GCheckMovieSceneEntityManagerInvariants,
	TEXT("Defines whether FEntityManager invariants should be checked on mutation or not. Note: severely impairs performance.\n"),
	ECVF_Default
);

#endif

namespace UE
{
namespace MovieScene
{

FComponentMask GEntityManagerEmptyMask;

// @todo: this is a very rough initial guess at the break even point for when threaded evaluation becomes beneficial, and will vary highly between platforms and hardware.
// We may wish to make this more flexible in future (such as only threading hot paths such as float channel evaluation) by enabling threading per-task, but more data is required to make such decisions
int32 GThreadedEvaluationAllocationThreshold = 32;
FAutoConsoleVariableRef CVarThreadedEvaluationAllocationThreshold(
	TEXT("Sequencer.ThreadedEvaluation.AllocationThreshold"),
	GThreadedEvaluationAllocationThreshold,
	TEXT("(Default: 32) Defines the entity allocation fragmentation threshold above which threaded evaluation will be used.\n"),
	ECVF_Default
);
int32 GThreadedEvaluationEntityThreshold = 256;
FAutoConsoleVariableRef CVarThreadedEvaluationEntityThreshold(
	TEXT("Sequencer.ThreadedEvaluation.EntityThreshold"),
	GThreadedEvaluationEntityThreshold,
	TEXT("(Default: 256) Defines the number of entities that need to exist to justify threaded evaluation.\n"),
	ECVF_Default
);

#if UE_MOVIESCENE_ENTITY_DEBUG
bool GRichComponentDebuggingInitialized = false;
bool GRichComponentDebugging = false;
FAutoConsoleVariableRef CVarRichComponentDebugging(
	TEXT("Sequencer.RichComponentDebugging"),
	GRichComponentDebugging,
	TEXT("(Default: false. Whether to enable rich component debugging within Sequencer.")
	);
#endif // UE_MOVIESCENE_ENTITY_DEBUG

FEntityManager* GEntityManagerForDebuggingVisualizers = nullptr;

static bool IsValidUint16(int32 Test)
{
	return (Test & 0xFFFF0000) == 0;
}

struct FEntityInitializer
{
	static void AddEntity(FEntityAllocation* Allocation, int32 ActualOffsetWithinAllocation, FMovieSceneEntityID EntityID)
	{
		check( IsValidUint16(ActualOffsetWithinAllocation) );
		check(ActualOffsetWithinAllocation == Allocation->Size);

		// Assign the currently free entry to the specified offset
		Allocation->EntityIDs[ActualOffsetWithinAllocation] = EntityID;
		++Allocation->Size;
	}

	static void MoveEntryIndex(FEntityAllocation* Allocation, int32 InCurrentOffset, int32 InNewOffset)
	{
		check( IsValidUint16(InCurrentOffset) && IsValidUint16(InNewOffset) && InNewOffset < Allocation->Size );

		Allocation->EntityIDs[InNewOffset] = Allocation->EntityIDs[InCurrentOffset];
	}

	static void MigrateAllocation(FEntityAllocation* DestAllocation, FEntityAllocation* SrcAllocation, const FComponentRegistry* InComponentRegistry)
	{
		check(DestAllocation->Capacity - DestAllocation->Size >= SrcAllocation->Size);

		const int32 NumEntities = SrcAllocation->Size;
		const int32 DstStartOffset = DestAllocation->Size;

		DestAllocation->Size += SrcAllocation->Size;

		// Initialize entity IDs
		FMemory::Memcpy(DestAllocation->EntityIDs + DstStartOffset, SrcAllocation->EntityIDs, sizeof(FMovieSceneEntityID)*NumEntities);

		TArrayView<const FComponentHeader> DstHeaders = DestAllocation->GetComponentHeaders();
		TArrayView<const FComponentHeader> SrcHeaders = SrcAllocation->GetComponentHeaders();

		check(DstHeaders.Num() == SrcHeaders.Num());

		for (int32 HeaderIndex = 0; HeaderIndex < DstHeaders.Num(); ++HeaderIndex)
		{
			const FComponentHeader* SrcHeader = &SrcHeaders[HeaderIndex];
			const FComponentHeader* DstHeader = &DstHeaders[HeaderIndex];

			check(DstHeader->ComponentType == SrcHeader->ComponentType);
			if (!SrcHeader->IsTag())
			{
				const FComponentTypeInfo& ComponentTypeInfo = InComponentRegistry->GetComponentTypeChecked(SrcHeader->ComponentType);

				void* DstValue = DstHeader->GetValuePtr(DstStartOffset);
				void* SrcValue = SrcHeader->GetValuePtr(0);

				ComponentTypeInfo.RelocateConstructItems(DstValue, SrcValue, NumEntities);

				// Mark these components as having be relocated, so they don't need to be destructed later.
				SrcHeader->Components = nullptr;
			}
		}

		SrcAllocation->Size = 0;
	}

	static void SetEntryIndex(FEntityAllocation* Allocation, int32 InEntityOffset, FMovieSceneEntityID EntityID)
	{
		check( IsValidUint16(InEntityOffset) && InEntityOffset < Allocation->Size && EntityID );

		Allocation->EntityIDs[InEntityOffset] = EntityID;
	}

	static void FreeEntryIndex(FEntityAllocation* Allocation, int32 EntityIndex)
	{
		check( IsValidUint16(EntityIndex) );

		// This entry is now free
		Allocation->EntityIDs[EntityIndex] = FMovieSceneEntityID::Invalid();
		--Allocation->Size;
	}

	struct FEntityAllocationInitializationInfo
	{
		uint32 AllocationID = (uint32)-1;
		int32 NumComponents = 0;
		uint16 InitialCapacity = 0;
		uint16 MaxCapacity = 0;
		SIZE_T SizeofComponentHeaders = 0;
		SIZE_T SizeofEntityIDs = 0;
		
		// If size is valid and pointer is null, allocate a new data buffer.
		// If size is 0 and pointer is non-null, steal the given allocation's buffer.
		// Both can't be valid at the same time.
		SIZE_T SizeofComponentData = 0;
		FEntityAllocation* MigrateComponentDataFrom = nullptr;
	};

	static FEntityAllocation* Initialize(const FEntityManager& EntityManager, const FComponentMask& EntityComponentMask, const FEntityAllocationInitializationInfo& InitInfo)
	{
		const FEntityAllocationWriteContext WriteContext(EntityManager);
		check(IsValidUint16(InitInfo.NumComponents));

		// Compute the size that we need: struct size + component headers array + entity IDs array.
		const SIZE_T RawStructSize = sizeof(FEntityAllocation);
		const SIZE_T TotalAllocationSize = RawStructSize + 
			alignof(FComponentHeader) + InitInfo.SizeofComponentHeaders + 
			alignof(FMovieSceneEntityID) + InitInfo.SizeofEntityIDs;

		// Allocate the structure.
		uint8* const AllocationStart = (uint8*)FMemory::Malloc(TotalAllocationSize);
		FEntityAllocation* const Allocation = new (AllocationStart) FEntityAllocation();

		// Initialize the structure.
		Allocation->UniqueID = InitInfo.AllocationID;
		Allocation->NumComponents = InitInfo.NumComponents;
		Allocation->Size = 0;
		Allocation->Capacity = InitInfo.InitialCapacity;
		Allocation->MaxCapacity = InitInfo.MaxCapacity;
		Allocation->SerialNumber = WriteContext.GetSystemSerial();

		// Fixup pointer offsets:
		//
		// ComponentHeaders exist right after the main structure.
		uint8* RawComponentHeadersPtr = AllocationStart + sizeof(FEntityAllocation);
		Allocation->ComponentHeaders = reinterpret_cast<FComponentHeader*>(Align(RawComponentHeadersPtr, alignof(FComponentHeader)));

		// EntityIDs exist right after the component headers.
		uint8* RawEntityIDsPtr = reinterpret_cast<uint8*>(Allocation->ComponentHeaders) + InitInfo.SizeofComponentHeaders;
		Allocation->EntityIDs = reinterpret_cast<FMovieSceneEntityID*>(Align(RawEntityIDsPtr, alignof(FMovieSceneEntityID)));

		// Component data buffer: allocate, or re-use/share/migrate.
		check((InitInfo.SizeofComponentData > 0 && InitInfo.MigrateComponentDataFrom == nullptr) || 
				(InitInfo.SizeofComponentData == 0 && InitInfo.MigrateComponentDataFrom != nullptr));
		if (InitInfo.MigrateComponentDataFrom != nullptr)
		{
			Allocation->ComponentData = InitInfo.MigrateComponentDataFrom->ComponentData;

			// Mark the original allocation's buffer as "stolen" so we don't free it later.
			InitInfo.MigrateComponentDataFrom->ComponentData = nullptr;
			for (FComponentHeader& ComponentHeader : InitInfo.MigrateComponentDataFrom->GetComponentHeaders())
			{
				ComponentHeader.Components = nullptr;
			}
		}
		else if (InitInfo.SizeofComponentData > 0)
		{
			uint8* const ComponentDataPtrStart = (uint8*)FMemory::Malloc(InitInfo.SizeofComponentData);
			Allocation->ComponentData = ComponentDataPtrStart;
		}

		// Initialize component headers.
		{
			uint8* ComponentDataPtr = Allocation->ComponentData;
			FComponentHeader* Header = Allocation->ComponentHeaders;

			for (FComponentMaskIterator It = EntityComponentMask.Iterate(); It; ++It, ++Header)
			{
				FComponentTypeID ComponentTypeID = FComponentTypeID::FromBitIndex(It.GetIndex());
				const FComponentTypeInfo& TypeInfo = EntityManager.GetComponents()->GetComponentTypeChecked(ComponentTypeID);

#if UE_MOVIESCENE_ENTITY_DEBUG
				if (GRichComponentDebugging)
				{
					TypeInfo.DebugInfo->InitializeComponentHeader(Header);
					Header->Size = &Allocation->Size;
				}
				else
#endif
				{
					UE_AUTORTFM_OPEN
					{
						new (Header) FComponentHeader();
					};
						
					UE_AUTORTFM_ONABORT(Header)
					{
						Header->~FComponentHeader();
					};
				}

				Header->ComponentType = ComponentTypeID;
				Header->Sizeof = TypeInfo.Sizeof;
				Header->PostWriteComponents(WriteContext);
				if (TypeInfo.IsTag())
				{
					Header->Components = nullptr;
				}
				else
				{
					// We align explicitly to cache lines to remove thread contention on read/write of neighboring component types
					uint8 Alignment = FMath::Max<uint8>(PLATFORM_CACHE_LINE_SIZE, TypeInfo.Alignment);
					ComponentDataPtr = Align(ComponentDataPtr, Alignment);

					Header->ScheduledAccessCount.exchange(0, std::memory_order_relaxed);
					Header->Components = reinterpret_cast<uint8*>(ComponentDataPtr);

					check(IsAligned(Header->Components, TypeInfo.Alignment));

					ComponentDataPtr += TypeInfo.Sizeof * InitInfo.InitialCapacity;

#if UE_MOVIESCENE_ENTITY_DEBUG
					if (GRichComponentDebugging)
					{
						TypeInfo.DebugInfo->InitializeDebugComponentData(*Header, InitInfo.InitialCapacity);
					}
#endif
				}
			}
		}

		return Allocation;
	}

	static void Duplicate(FEntityAllocation* Dest, const FEntityAllocation* Source)
	{
		Dest->Size = Source->Size;
		FMemory::Memcpy(Dest->EntityIDs, Source->EntityIDs, sizeof(FMovieSceneEntityID)*Source->Size);
	}

	static void TearDown(FEntityAllocation* Allocation)
	{
		check(Allocation != nullptr);
		if (Allocation->ComponentData != nullptr)
		{
			FMemory::Free(Allocation->ComponentData);
		}
		FMemory::Free(Allocation);
	}
};


void FFreeEntityOperation::MarkAllocationForFree(int32 AllocationIndex)
{
	AllocationsToDestroy.Add(AllocationIndex);
}

void FFreeEntityOperation::MarkEntityForFree(FMovieSceneEntityID EntityID)
{
	LooseEntitiesToDestroy.Add(EntityID);
}

FFreeEntityOperation::FCommitData FFreeEntityOperation::Commit() const
{
	FFreeEntityOperation::FCommitData CommitData;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	TArray<FMovieSceneEntityID, TInlineAllocator<16>> EntitiesScratch;

	auto MarkChildrenForFree = [this, &EntitiesScratch, &CommitData, BuiltInComponents](FMovieSceneEntityID Entity)
	{
		for (auto ChildIt = this->EntityManager->ParentToChild.CreateKeyIterator(Entity); ChildIt; ++ChildIt)
		{
			EntitiesScratch.Add(ChildIt.Value());
		}

		while (EntitiesScratch.Num() != 0)
		{
			const int32 StartNum = EntitiesScratch.Num();

			for (int32 MarkIndex = 0; MarkIndex < StartNum; ++MarkIndex)
			{
				FMovieSceneEntityID MarkedEntity = EntitiesScratch[MarkIndex];

				FEntityManager::FEntityLocation Location = EntityManager->EntityLocations[MarkedEntity.AsIndex()];
				if (Location.IsValid())
				{
					ensureAlwaysMsgf(EntityManager->EntityAllocationMasks[Location.GetAllocationIndex()].Contains(BuiltInComponents->Tags.NeedsUnlink), TEXT("Attempting to free an entity that has not been unlinked - this might result in stale references"));

					FAllocationMask& Mask = CommitData.AllocationsToEntities.FindOrAdd(Location.GetAllocationIndex());
					if (!Mask.bDestroyAllocation)
					{
						const int32 BitIndex = Location.GetEntryIndexWithinAllocation();

						Mask.Mask.PadToNum(BitIndex + 1, false);
						Mask.Mask[BitIndex] = true;
					}
				}
				else
				{
					CommitData.EmptyEntities.Add(MarkedEntity);
				}

				// Mark all children
				for (auto ChildIt = EntityManager->ParentToChild.CreateKeyIterator(MarkedEntity); ChildIt; ++ChildIt)
				{
					EntitiesScratch.Add(ChildIt.Value());
				}
			}

			// Remove current iteration
			EntitiesScratch.RemoveAtSwap(0, StartNum, EAllowShrinking::No);
		}
	};


	// Populate full allocations
	for (int32 AllocationIndex : AllocationsToDestroy)
	{
		FEntityAllocation* Allocation = EntityManager->EntityAllocations[AllocationIndex];

		CommitData.AllocationsToEntities.Add(AllocationIndex).bDestroyAllocation = true;

		for (FMovieSceneEntityID Entity : Allocation->GetEntityIDs())
		{
			MarkChildrenForFree(Entity);
		}
	}

	for (FMovieSceneEntityID Entity : LooseEntitiesToDestroy)
	{
		FEntityManager::FEntityLocation Location = EntityManager->EntityLocations[Entity.AsIndex()];
		if (Location.IsValid())
		{
			FAllocationMask& Mask = CommitData.AllocationsToEntities.FindOrAdd(Location.GetAllocationIndex());
			if (!Mask.bDestroyAllocation)
			{
				const int32 BitIndex = Location.GetEntryIndexWithinAllocation();

				Mask.Mask.PadToNum(BitIndex + 1, false);
				Mask.Mask[BitIndex] = true;
			}
		}
		else
		{
			CommitData.EmptyEntities.Add(Entity);
		}

		MarkChildrenForFree(Entity);
	}

	return CommitData;
}

FEntityManager::FEntityManager()
{
	IterationCount = 0;
	NextAllocationID = 0;
	CurrentHandleGeneration = 0;
	bHandleGenerationStale = false;
	GatherThread   = ENamedThreads::AnyThread;
	DispatchThread = ENamedThreads::AnyThread;
	ManagerDebugName = TEXT("UE::MovieScene::FEntityManager");
	LockdownState = ELockdownState::Unlocked;
	SystemSerialNumber = 1;
	StructureMutationSystemSerialNumber = 0;
	ThreadingModel = EEntityThreadingModel::NoThreading;

#if UE_MOVIESCENE_ENTITY_DEBUG
	if (!GRichComponentDebuggingInitialized)
	{
		GRichComponentDebugging = FPlatformMisc::IsDebuggerPresent();
		GRichComponentDebuggingInitialized = true;
	}

	RichComponentDebuggingPtr = &GRichComponentDebugging;
#endif
}

FEntityManager::~FEntityManager()
{
	// Call destructors for any allocated component data
	for (FEntityAllocation* Allocation : EntityAllocations)
	{
		DestroyAllocation(Allocation);
	}
}

void FEntityManager::DestroyAllocation(FEntityAllocation* Allocation, bool bDestructComponentData)
{
	CheckCanChangeStructure();

	if (Allocation->Num() > 0 && bDestructComponentData)
	{
		for (const FComponentHeader& Header : Allocation->GetComponentHeaders())
		{
			if (Header.HasData())  // If this is NOT a tag, nor is it a component header whose data buffer was relocated...
			{
				check(Header.Components != nullptr);
				const FComponentTypeInfo& ComponentTypeInfo = ComponentRegistry->GetComponentTypeChecked(Header.ComponentType);
				ComponentTypeInfo.DestructItems(Header.Components, Allocation->Num());
			}
		}
	}

	Allocation->~FEntityAllocation();
	FEntityInitializer::TearDown(Allocation);
}

void FEntityManager::Destroy()
{
	CheckCanChangeStructure();

	bHandleGenerationStale = true;
	EntityGenerationMap.Reset();

	for (FEntityAllocation* Allocation : EntityAllocations)
	{
		DestroyAllocation(Allocation);
	}
	EntityLocations.Reset();
	AllocationsWithCapacity.Reset();
	EntityAllocationMasks.Reset();
	EntityAllocations.Reset();
	ParentToChild.Reset();

	OnStructureChanged();
}

FMovieSceneEntityID FEntityManager::AllocateEntity()
{
	CheckCanChangeStructure();

	const int32 NewEntityIndex = EntityLocations.Add(FEntityLocation());

	return FMovieSceneEntityID::FromIndex(NewEntityIndex);
}

FEntityInfo FEntityManager::AllocateEntity(const FComponentMask& EntityComponentMask)
{
	if (EntityComponentMask.Find(true) == INDEX_NONE)
	{
		FMovieSceneEntityID NewEntityID = AllocateEntity();
		return FEntityInfo{ FEntityDataLocation{ nullptr, INDEX_NONE }, NewEntityID };
	}

	checkf(LockdownState == ELockdownState::Unlocked, TEXT("Structural changes to the entity manager are not permitted while it is locked down"));

	CheckCanChangeStructure();

	const int32 NewEntityIndex = EntityLocations.Add(FEntityLocation{});
	FMovieSceneEntityID NewEntityID = FMovieSceneEntityID::FromIndex(NewEntityIndex);

	int32 AllocationIndex = GetOrCreateAllocationWithSlack(EntityComponentMask);
	int32 EntryIndexWithinAllocation = AddEntityToAllocation(AllocationIndex, NewEntityID);

	EntityLocations[NewEntityIndex].Set(AllocationIndex, EntryIndexWithinAllocation);

	FEntityInfo NewEntity = {
		FEntityDataLocation { EntityAllocations[AllocationIndex], EntryIndexWithinAllocation },
		NewEntityID
	};

	OnStructureChanged();

	return NewEntity;
}

FEntityDataLocation FEntityManager::AllocateContiguousEntities(const FComponentMask& EntityComponentMask, int32* InOutNum)
{
	check(InOutNum && *InOutNum >= 1);

	CheckCanChangeStructure();

	const int32 AllocationIndex = GetOrCreateAllocationWithSlack(EntityComponentMask, InOutNum);
	const int32 NumAllocated = *InOutNum;

	const int32 FirstEntityIndex = EntityLocations.Add(FEntityLocation{});
	FMovieSceneEntityID FirstEntityID = FMovieSceneEntityID::FromIndex(FirstEntityIndex);

	int32 FirstComponentOffset = AddEntityToAllocation(AllocationIndex, FirstEntityID);
	EntityLocations[FirstEntityIndex].Set(AllocationIndex, FirstComponentOffset);

	for (int32 Index = 1; Index < NumAllocated; ++Index)
	{
		const int32 NewEntityIndex = EntityLocations.Add(FEntityLocation{});
		FMovieSceneEntityID NewEntityID = FMovieSceneEntityID::FromIndex(NewEntityIndex);

		int32 EntryIndexWithinAllocation = AddEntityToAllocation(AllocationIndex, NewEntityID);
		EntityLocations[NewEntityIndex].Set(AllocationIndex, EntryIndexWithinAllocation);
	}

	FEntityDataLocation DataLocation = {
		EntityAllocations[AllocationIndex],
		FirstComponentOffset
	};

	CheckInvariants();

	OnStructureChanged();

	return DataLocation;
}

FEntityInfo FEntityManager::GetEntity(FMovieSceneEntityID EntityID) const
{
	const int32 Index = EntityID.AsIndex();
	check(EntityLocations.IsValidIndex(Index));

	FEntityLocation Location = EntityLocations[Index];
	if (Location.IsValid())
	{
		return FEntityInfo { FEntityDataLocation{ EntityAllocations[Location.GetAllocationIndex()], Location.GetEntryIndexWithinAllocation() }, EntityID };
	}

	return FEntityInfo { FEntityDataLocation{ nullptr, INDEX_NONE }, EntityID };
}

FEntityHandle FEntityManager::GetEntityHandle(FMovieSceneEntityID EntityID)
{
	if (!EntityID)
	{
		return FEntityHandle{};
	}

	checkSlow(EntityLocations.IsValidIndex(EntityID.AsIndex()));

	// Does a handle already exist for this entityID?
	const uint32* ExistingGeneration = EntityGenerationMap.Find(EntityID);
	if (ExistingGeneration)
	{
		return FEntityHandle(EntityID, *ExistingGeneration);
	}

	const uint32 NewHandleGeneration = GetHandleGeneration();
	EntityGenerationMap.Add(EntityID, NewHandleGeneration);
	return FEntityHandle(EntityID, NewHandleGeneration);
}

bool FEntityManager::IsHandleValid(FEntityHandle InEntityHandle) const
{
	if (!InEntityHandle.ID)
	{
		return false;
	}

	const uint32* Generation = EntityGenerationMap.Find(InEntityHandle.ID);
	return Generation && *Generation == InEntityHandle.HandleGeneration;
}

EEntityThreadingModel FEntityManager::ComputeThreadingModel() const
{
	const bool bCanThread = FPlatformProcess::SupportsMultithreading();

	const bool bShouldThread = bCanThread &&
		(EntityAllocations.Num() >= GThreadedEvaluationAllocationThreshold ||
		EntityLocations.Num() >= GThreadedEvaluationEntityThreshold);

	return bShouldThread ? EEntityThreadingModel::TaskGraph : EEntityThreadingModel::NoThreading;
}

EEntityThreadingModel FEntityManager::GetThreadingModel() const
{
	return ThreadingModel;
}

void FEntityManager::UpdateThreadingModel()
{
	ThreadingModel = ComputeThreadingModel();
}

const FComponentMask& FEntityManager::GetAccumulatedMask() const
{
	FEntityManager* This = const_cast<FEntityManager*>(this);

	if (This->bAccumulatedMaskStale == true)
	{
		This->AccumulatedMask.Reset();
		for (const FComponentMask& Mask : EntityAllocationMasks)
		{
			This->AccumulatedMask.CombineWithBitwiseOR(Mask, EBitwiseOperatorFlags::MaxSize);
		}
		This->bAccumulatedMaskStale = false;
	}

	return AccumulatedMask;
}

void FEntityManager::FreeEntity(FMovieSceneEntityID EntityID)
{
	check(EntityLocations.IsValidIndex(EntityID.AsIndex()));

	CheckCanChangeStructure();

	FFreeEntityOperation Operation(this);
	Operation.MarkEntityForFree(EntityID);

	FreeEntities(Operation);
}

int32 FEntityManager::FreeEntities(const FEntityComponentFilter& Filter, TSet<FMovieSceneEntityID>* OutFreedEntities)
{
	CheckCanChangeStructure();

	check(Filter.IsValid());

	FFreeEntityOperation Operation(this);

	for (auto AllocationIt = EntityAllocationMasks.CreateIterator(); AllocationIt; ++AllocationIt)
	{
		const int32 AllocationIndex = AllocationIt.GetIndex();
		if (Filter.Match(EntityAllocationMasks[AllocationIndex]))
		{
			Operation.MarkAllocationForFree(AllocationIndex);
		}
	}

	return FreeEntities(Operation, OutFreedEntities);
}

int32 FEntityManager::FreeEntities(const FFreeEntityOperation& Operation, TSet<FMovieSceneEntityID>* OutFreedEntities)
{
	CheckCanChangeStructure();
	if (Operation.LooseEntitiesToDestroy.Num() == 0 && Operation.AllocationsToDestroy.Num() == 0)
	{
		return 0;
	}

	FEntityAllocationWriteContext WriteContext(*this);

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FFreeEntityOperation::FCommitData Committed = Operation.Commit();

	auto ReleaseEntity = [this, OutFreedEntities](FMovieSceneEntityID EntityID)
	{
		const int32 Index = EntityID.AsIndex();

		FEntityLocation Location = EntityLocations[Index];
		if (Location.GetParentID())
		{
			ParentToChild.Remove(Location.GetParentID(), EntityID);
		}
		ParentToChild.Remove(EntityID);

		EntityGenerationMap.Remove(EntityID);
		EntityLocations.RemoveAt(Index);

		if (OutFreedEntities)
		{
			OutFreedEntities->Add(EntityID);
		}
	};

	const int32 StartingHandleCount = EntityGenerationMap.Num();

	int32 NumFreed = Committed.EmptyEntities.Num();

	// First off free any entities that have no data
	for (FMovieSceneEntityID EntityID : Committed.EmptyEntities)
	{
		ReleaseEntity(EntityID);
	}

	// Next go through all the allocations that have entities to be removed

	TBitArray<> ReversedSetBits;
	for (const TTuple<int32, FFreeEntityOperation::FAllocationMask>& Pair : Committed.AllocationsToEntities)
	{
		const int32        AllocationIndex = Pair.Key;
		FEntityAllocation* Allocation      = EntityAllocations[AllocationIndex];

		TArrayView<const FMovieSceneEntityID> EntityIDs = Allocation->GetEntityIDs();

		const int32 Num = Allocation->Num();
		const int32 NumToFree = Pair.Value.bDestroyAllocation ? Num : Pair.Value.Mask.CountSetBits();

		ensureMsgf(Allocation->HasComponent(BuiltInComponents->Tags.NeedsUnlink), TEXT("Attempting to free an entity that has not been unlinked - this might result in stale references"));

		NumFreed += NumToFree;

		// If we're freeing everything, just destroy the whole allocation
		if (NumToFree == Num)
		{
			for (FMovieSceneEntityID Entity : EntityIDs)
			{
				ReleaseEntity(Entity);
			}

			DestroyAllocation(Allocation);

			EntityAllocations.RemoveAt(AllocationIndex);
			EntityAllocationMasks.RemoveAt(AllocationIndex);

			AllocationsWithCapacity[AllocationIndex] = false;
			continue;
		}

		const bool bHadCapacity = Allocation->Num() != Allocation->GetMaxCapacity();
		int32 LastEntityIndex = Num - 1;

		// Modify allocation and headers
		Allocation->PostModifyStructure(WriteContext);

		// Reverse the set bits so we can iterate backwards
		const int32 MaskSize = Pair.Value.Mask.Num();
		ReversedSetBits.Init(false, MaskSize+1);
		for (TConstSetBitIterator<> SetBit(Pair.Value.Mask); SetBit; ++SetBit)
		{
			const int32 ReversedIndex = MaskSize - SetBit.GetIndex();
			ReversedSetBits[ReversedIndex] = true;
		}

		// Iterate the entities to free starting from the last
		for (TConstSetBitIterator<> SetBit(ReversedSetBits); SetBit; ++SetBit)
		{
			const int32         EntityOffset = MaskSize - SetBit.GetIndex();
			FMovieSceneEntityID Entity       = EntityIDs[EntityOffset];

			// Destruct its components
			for (FComponentHeader& Header : Allocation->GetComponentHeaders())
			{
				if (!Header.IsTag())
				{
					const FComponentTypeInfo& ComponentTypeInfo = ComponentRegistry->GetComponentTypeChecked(Header.ComponentType);

					void* Value = Header.GetValuePtr(EntityOffset);
					ComponentTypeInfo.DestructItems(Value, 1);

					if (LastEntityIndex != EntityOffset)
					{
						void* SwapSource = Header.GetValuePtr(LastEntityIndex);
						ComponentTypeInfo.RelocateConstructItems(Value, SwapSource, 1);
					}
				}
			}

			// When removing we just swap the tail element with the element to remove, and fix up the indices
			if (LastEntityIndex != EntityOffset)
			{
				FEntityInitializer::MoveEntryIndex(Allocation, LastEntityIndex, EntityOffset);

				// Fixup entry offset for the changed entry
				FMovieSceneEntityID SwappedEntityID = Allocation->GetEntityIDs()[EntityOffset];
				EntityLocations[SwappedEntityID.AsIndex()].Set(AllocationIndex, EntityOffset);
			}

			// Free this entity index without needing to fix up any other indices
			FEntityInitializer::FreeEntryIndex(Allocation, LastEntityIndex);

			ReleaseEntity(Entity);

			--LastEntityIndex;
		}

		if (!bHadCapacity)
		{
			AllocationsWithCapacity[AllocationIndex] = true;
		}
	}

	if (StartingHandleCount != EntityGenerationMap.Num())
	{
		bHandleGenerationStale = true;
	}

	CheckInvariants();

	OnStructureChanged();

	return NumFreed;
}

void FEntityManager::Compact()
{
	// Step 1: Combine equal allocations
	for (int32 AllocationIndex = 0; AllocationIndex < EntityAllocations.GetMaxIndex(); ++AllocationIndex)
	{
		if (!EntityAllocations.IsAllocated(AllocationIndex))
		{
			continue;
		}

		int32 CombineWithIndex = INDEX_NONE;
		int32 RequiredSlack    = EntityAllocations[AllocationIndex]->Num();
		
		// Find an allocation we can combine wtih
		for (TConstSetBitIterator<> It(AllocationsWithCapacity); It && It.GetIndex() < AllocationIndex; ++It)
		{
			const int32 PotentialCombinationIndex = It.GetIndex();
			if (EntityAllocationMasks[PotentialCombinationIndex].CompareSetBits(EntityAllocationMasks[AllocationIndex]) == true)
			{
				FEntityAllocation* CombineWithAllocation = EntityAllocations[PotentialCombinationIndex];
				if (CombineWithAllocation->GetMaxCapacity() - CombineWithAllocation->Num() >= RequiredSlack)
				{
					CombineWithIndex = PotentialCombinationIndex;
					break;
				}
			}
		}

		if (CombineWithIndex != INDEX_NONE)
		{
			// Combine allocations will destroy the entry at AllocationReadIndex, so we do not increment AllocationWriteIndex
			// So this allocation entry can be reused
			CombineAllocations(CombineWithIndex, AllocationIndex);
		}
	}


	// Step 2: Remove empty allocations
	int32 AllocationReadIndex = 0;
	int32 AllocationWriteIndex = 0;

	TMap<int32, int32> OldToNewIndex;
	for (; AllocationReadIndex < EntityAllocations.GetMaxIndex(); ++AllocationReadIndex)
	{
		if (!EntityAllocations.IsAllocated(AllocationReadIndex))
		{
			continue;
		}

		FEntityAllocation* Allocation = EntityAllocations[AllocationReadIndex];
		if (Allocation->Num() == 0)
		{
			DestroyAllocation(Allocation);

			AllocationsWithCapacity[AllocationReadIndex] = false;
			continue;
		}

		if (AllocationReadIndex != AllocationWriteIndex)
		{
			EntityAllocations.Insert(AllocationWriteIndex, Allocation);
			EntityAllocations.RemoveAt(AllocationReadIndex);

			EntityAllocationMasks.Insert(AllocationWriteIndex, EntityAllocationMasks[AllocationReadIndex]);
			EntityAllocationMasks.RemoveAt(AllocationReadIndex);

			AllocationsWithCapacity[AllocationWriteIndex] = AllocationsWithCapacity[AllocationReadIndex];
			AllocationsWithCapacity[AllocationReadIndex] = false;

			OldToNewIndex.Add(AllocationReadIndex, AllocationWriteIndex);
		}

		++AllocationWriteIndex;
	}

	if (OldToNewIndex.Num() > 0)
	{
		for (FEntityLocation& TypeInfo : EntityLocations)
		{
			if (TypeInfo.IsValid())
			{
				if (const int32* NewIndex = OldToNewIndex.Find(TypeInfo.GetAllocationIndex()))
				{
					TypeInfo.Set(*NewIndex, TypeInfo.GetEntryIndexWithinAllocation());
				}
			}
		}
	}

	if (AllocationWriteIndex != AllocationReadIndex)
	{
		EntityAllocations.Shrink();
		EntityAllocationMasks.Shrink();

		// TBitArray does not have a Shrink method, so we copy it to a new one with the smallest number of bytes then swap it back
		AllocationsWithCapacity.RemoveAt(AllocationWriteIndex, AllocationsWithCapacity.Num() - AllocationWriteIndex - 1);
		TBitArray<> Temp = AllocationsWithCapacity;
		Swap(AllocationsWithCapacity, Temp);
	}

	EntityLocations.Shrink();

	CheckInvariants();

	OnStructureChanged();
}

void FEntityManager::ReplaceEntityID(FMovieSceneEntityID& InOutEntity, FMovieSceneEntityID NewEntityID)
{
	if (!NewEntityID)
	{
		if (InOutEntity)
		{
			FreeEntity(InOutEntity);
			InOutEntity = FMovieSceneEntityID();
		}
	}
	else
	{
		// At this point we know that we have a loaded entity - if we're loading over the top of another, handle that now
		if (InOutEntity)
		{
			check(EntityLocations.IsValidIndex(InOutEntity.AsIndex()));

			if (ComponentRegistry->GetPreservationMask().Find(true) != INDEX_NONE)
			{
				CombineComponents(NewEntityID, InOutEntity, &ComponentRegistry->GetPreservationMask());
			}

			FEntityLocation NewLocation = EntityLocations[NewEntityID.AsIndex()];
			FEntityLocation OldLocation = EntityLocations[InOutEntity.AsIndex()];

			if (NewLocation.IsValid())
			{
				FEntityInitializer::SetEntryIndex(EntityAllocations[NewLocation.GetAllocationIndex()], NewLocation.GetEntryIndexWithinAllocation(), InOutEntity);
			}
			EntityLocations[InOutEntity.AsIndex()] = NewLocation;

			if (OldLocation.IsValid())
			{
				RemoveEntityFromAllocation(OldLocation.GetAllocationIndex(), OldLocation.GetEntryIndexWithinAllocation());
			}

			EntityLocations.RemoveAt(NewEntityID.AsIndex());
		}
		else
		{
			// Not loading over any entity, so just assign the new ID
			InOutEntity = NewEntityID;
		}
	}

	CheckInvariants();

	OnStructureChanged();
}

FMovieSceneEntityID FEntityManager::DuplicateEntity(FMovieSceneEntityID InOther)
{
	if (!InOther)
	{
		return FMovieSceneEntityID::Invalid();
	}

	check(EntityLocations.IsValidIndex(InOther.AsIndex()));

	const FEntityLocation SourceEntry = EntityLocations[InOther.AsIndex()];
	if (!SourceEntry.IsValid())
	{
		return AllocateEntity();
	}

	const int32 NewEntityIndex = EntityLocations.Add(FEntityLocation{});
	FMovieSceneEntityID NewEntityID = FMovieSceneEntityID::FromIndex(NewEntityIndex);

	const int32 DestAllocationIndex = GetOrCreateAllocationWithSlack(EntityAllocationMasks[SourceEntry.GetAllocationIndex()]);
	const int32 DestEntryIndexWithinAllocation = AddEntityToAllocation(DestAllocationIndex, NewEntityID);

	EntityLocations[NewEntityIndex].Set(DestAllocationIndex, DestEntryIndexWithinAllocation);

	CopyComponents(DestAllocationIndex, DestEntryIndexWithinAllocation, SourceEntry.GetAllocationIndex(), SourceEntry.GetEntryIndexWithinAllocation());

	CheckInvariants();

	OnStructureChanged();

	return NewEntityID;
}


void FEntityManager::OverwriteEntityWithDuplicate(FMovieSceneEntityID& InOutEntity, FMovieSceneEntityID InEntityToDuplicate)
{
	FMovieSceneEntityID NewEntityID;

	if (InEntityToDuplicate)
	{
		NewEntityID = DuplicateEntity(InEntityToDuplicate);
	}

	ReplaceEntityID(InOutEntity, NewEntityID);

	CheckInvariants();

	OnStructureChanged();
}


FEntityAllocationIteratorProxy FEntityManager::Iterate(const FEntityComponentFilter* InFilter) const
{
	check(InFilter);
	return FEntityAllocationIteratorProxy(this, InFilter);
}

bool FEntityManager::Contains(const FEntityComponentFilter& InFilter) const
{
	FEntityAllocationIteratorProxy It = Iterate(&InFilter);
	return It.begin() != It.end();
}

void FEntityManager::AccumulateMask(const FEntityComponentFilter& InFilter, FComponentMask& OutMask) const
{
	for (const FComponentMask& Mask : EntityAllocationMasks)
	{
		if (InFilter.Match(Mask))
		{
			OutMask.CombineWithBitwiseOR(Mask, EBitwiseOperatorFlags::MaxSize);
		}
	}
}

void FEntityManager::EnterIteration() const
{
	IterationCount.Increment(ThreadingModel);
}

void FEntityManager::ExitIteration() const
{
	checkSlow(IterationCount.Load(ThreadingModel) > 0);
	IterationCount.Decrement(ThreadingModel);
}

FEntityAllocation* FEntityManager::CreateEntityAllocation(const FComponentMask& EntityComponentMask, uint16 InitialCapacity, uint16 MaxCapacity, FEntityAllocation* MigrateComponentDataFrom)
{
	CheckCanChangeStructure();

	check(!(EntityComponentMask.Contains(FBuiltInComponentTypes::Get()->Tags.NeedsLink) && EntityComponentMask.Contains(FBuiltInComponentTypes::Get()->Tags.NeedsUnlink)));
	check(InitialCapacity <= MaxCapacity && MaxCapacity < MAX_uint16);

	const int32 NumComponents = EntityComponentMask.NumComponents();

	check(EntityComponentMask.Num() < TNumericLimits<uint16>::Max());
	check(NumComponents > 0 && NumComponents < TNumericLimits<uint16>::Max());

	const SIZE_T SizeofComponentHeaders = NumComponents*sizeof(FComponentHeader);
	SIZE_T ComponentDataSize = PLATFORM_CACHE_LINE_SIZE;
	for (FComponentMaskIterator It = EntityComponentMask.Iterate(); It; ++It)
	{
		const FComponentTypeInfo& TypeInfo = ComponentRegistry->GetComponentTypeChecked(FComponentTypeID::FromBitIndex(It.GetIndex()));
		if (!TypeInfo.IsTag())   // No need to reserve any memory if the component is a tag.
		{
			uint8 Alignment = FMath::Max<uint8>(PLATFORM_CACHE_LINE_SIZE, TypeInfo.Alignment);
			ComponentDataSize += Alignment + TypeInfo.Sizeof*InitialCapacity;
		}
	}

	const SIZE_T SizeofEntityIDs = InitialCapacity*sizeof(FMovieSceneEntityID);

	// Create the memory layout for the new entity allocation.
	FEntityInitializer::FEntityAllocationInitializationInfo InitInfo;
	InitInfo.AllocationID = NextAllocationID;
	InitInfo.NumComponents = NumComponents;
	InitInfo.InitialCapacity = InitialCapacity;
	InitInfo.MaxCapacity = MaxCapacity;
	InitInfo.SizeofComponentHeaders = SizeofComponentHeaders;
	InitInfo.SizeofEntityIDs = SizeofEntityIDs;
	InitInfo.SizeofComponentData = (MigrateComponentDataFrom == nullptr) ? ComponentDataSize : 0;
	InitInfo.MigrateComponentDataFrom = MigrateComponentDataFrom;
	FEntityAllocation* Structure = FEntityInitializer::Initialize(*this, EntityComponentMask, InitInfo);

	++NextAllocationID;

	return Structure;
}

int32 FEntityManager::CreateEntityAllocationEntry(const FComponentMask& EntityComponentMask, uint16 InitialCapacity, uint16 MaxCapacity)
{
	CheckCanChangeStructure();

	check(EntityAllocations.Num() < TNumericLimits<uint16>::Max());
	check(EntityAllocationMasks.Num() < TNumericLimits<uint16>::Max());

	FEntityAllocation* NewAllocation = CreateEntityAllocation(EntityComponentMask, InitialCapacity, MaxCapacity);

	const int32 MaskIndex = EntityAllocationMasks.Add(EntityComponentMask);
	const int32 AllocationIndex = EntityAllocations.Add(NewAllocation);

	check(MaskIndex == AllocationIndex);

	AllocationsWithCapacity.PadToNum(MaskIndex+1, false);
	AllocationsWithCapacity[MaskIndex] = true;

	return AllocationIndex;
}

int32 FEntityManager::GetOrCreateAllocationWithSlack(const FComponentMask& EntityComponentMask, int32* InOutDesiredSlack)
{
	check(!InOutDesiredSlack || IsValidUint16(*InOutDesiredSlack));

	for (TConstSetBitIterator<> It(AllocationsWithCapacity); It; ++It)
	{
		const int32 AllocationIndex = It.GetIndex();
		if (EntityAllocationMasks[AllocationIndex].CompareSetBits(EntityComponentMask) == true)
		{
			if (InOutDesiredSlack)
			{
				*InOutDesiredSlack = ReserveAllocation(AllocationIndex, *InOutDesiredSlack);
			}
			return AllocationIndex;
		}
	}
	return CreateAllocationWithSlack(EntityComponentMask, InOutDesiredSlack);
}

int32 FEntityManager::CreateAllocationWithSlack(const FComponentMask& EntityComponentMask, int32* InOutDesiredSlack)
{
	static const uint16 MaxCapacity = 64;

	uint16 DefaultCapacity = 4;
	if (InOutDesiredSlack)
	{
		*InOutDesiredSlack = FMath::Min((uint16)*InOutDesiredSlack, MaxCapacity);
		DefaultCapacity = *InOutDesiredSlack;
	}
	return CreateEntityAllocationEntry(EntityComponentMask, DefaultCapacity, MaxCapacity);
}

void FEntityManager::AddComponent(FMovieSceneEntityID EntityID, FComponentTypeID ComponentType)
{
	check(EntityLocations.IsValidIndex(EntityID.AsIndex()));

	CheckCanChangeStructure();

	FComponentMask NewEntityComponentMask;
	NewEntityComponentMask.Set(ComponentType);

	FEntityLocation& Location = EntityLocations[EntityID.AsIndex()];
	if (Location.IsValid())
	{
		const FComponentMask& ExistingMask = EntityAllocationMasks[Location.GetAllocationIndex()];

		if (ExistingMask.Contains(ComponentType))
		{
			// Entity already has this component type
			return;
		}

		NewEntityComponentMask.CombineWithBitwiseOR(EntityAllocationMasks[Location.GetAllocationIndex()], EBitwiseOperatorFlags::MaxSize);
	}

	int32 NewAllocationIndex = GetOrCreateAllocationWithSlack(NewEntityComponentMask);

	int32 NewEntityIndex;
	if (!Location.IsValid())
	{
		NewEntityIndex = AddEntityToAllocation(NewAllocationIndex, EntityID);
	}
	else
	{
		NewEntityIndex = MigrateEntity(NewAllocationIndex, Location.GetAllocationIndex(), Location.GetEntryIndexWithinAllocation());
	}

	Location.Set(NewAllocationIndex, NewEntityIndex);

	CheckInvariants();

	OnStructureChanged();
}

void FEntityManager::AddComponents(FMovieSceneEntityID EntityID, const FComponentMask& ComponentMask)
{
	if (ComponentMask.Find(true) == INDEX_NONE)
	{
		return;
	}

	CheckCanChangeStructure();

	FComponentMask NewEntityComponentMask = ComponentMask;

	FEntityLocation& Location = EntityLocations[EntityID.AsIndex()];
	if (Location.IsValid())
	{
		const FComponentMask& ExistingMask = EntityAllocationMasks[Location.GetAllocationIndex()];
		NewEntityComponentMask.CombineWithBitwiseOR(ExistingMask, EBitwiseOperatorFlags::MaxSize);

		if (NewEntityComponentMask.CompareSetBits(ExistingMask))
		{
			// Entity already has all the component types
			return;
		}
	}

	int32 NewAllocationIndex = GetOrCreateAllocationWithSlack(NewEntityComponentMask);

	int32 NewEntityIndex;
	if (!Location.IsValid())
	{
		NewEntityIndex = AddEntityToAllocation(NewAllocationIndex, EntityID);
	}
	else
	{
		NewEntityIndex = MigrateEntity(NewAllocationIndex, Location.GetAllocationIndex(), Location.GetEntryIndexWithinAllocation());
	}

	Location.Set(NewAllocationIndex, NewEntityIndex);

	CheckInvariants();

	OnStructureChanged();
}

void FEntityManager::AddComponent(FMovieSceneEntityID ThisEntityID, FComponentTypeID ComponentTypeID, EEntityRecursion Recursion)
{
	if (EnumHasAnyFlags(Recursion, EEntityRecursion::This))
	{
		AddComponent(ThisEntityID, ComponentTypeID);
	}
	if (EnumHasAnyFlags(Recursion, EEntityRecursion::Children))
	{
		IterateChildren_ParentFirst(ThisEntityID, [this, ComponentTypeID](FMovieSceneEntityID EntityID)
		{
			this->AddComponent(EntityID, ComponentTypeID);
		});
	}
}

void FEntityManager::AddComponents(FMovieSceneEntityID ThisEntityID, const FComponentMask& EntityComponentMask, EEntityRecursion Recursion)
{
	if (EnumHasAnyFlags(Recursion, EEntityRecursion::This))
	{
		AddComponents(ThisEntityID, EntityComponentMask);
	}
	if (EnumHasAnyFlags(Recursion, EEntityRecursion::Children))
	{
		IterateChildren_ParentFirst(ThisEntityID, [this, &EntityComponentMask](FMovieSceneEntityID EntityID)
		{
			this->AddComponents(EntityID, EntityComponentMask);
		});
	}
}

bool FEntityManager::HasComponent(FMovieSceneEntityID EntityID, FComponentTypeID ComponentType) const
{
	FEntityLocation Location = EntityLocations[EntityID.AsIndex()];
	return Location.IsValid() && EntityAllocationMasks[Location.GetAllocationIndex()].Contains(ComponentType);
}

const FComponentMask& FEntityManager::GetEntityType(FMovieSceneEntityID EntityID) const
{
	FEntityLocation Location = EntityLocations[EntityID.AsIndex()];
	return Location.IsValid() ? EntityAllocationMasks[Location.GetAllocationIndex()] : GEntityManagerEmptyMask;
}

void FEntityManager::ChangeEntityType(FMovieSceneEntityID EntityID, const FComponentMask& InNewMask)
{
	CheckCanChangeStructure();

	const bool bNewTypeIsEmpty = (InNewMask.Find(true) == INDEX_NONE);

	FEntityLocation& Location = EntityLocations[EntityID.AsIndex()];
	if (Location.IsValid())
	{
		const FComponentMask& ExistingMask = EntityAllocationMasks[Location.GetAllocationIndex()];
		if (ExistingMask.CompareSetBits(InNewMask))
		{
			// Already exactly this type
			return;
		}

		if (bNewTypeIsEmpty)
		{
			// Remove all components
			RemoveEntityFromAllocation(Location.GetAllocationIndex(), Location.GetEntryIndexWithinAllocation());
			Location.Reset();
			return;
		}
	}
	else if (bNewTypeIsEmpty)
	{
		return;
	}

	int32 NewAllocationIndex = GetOrCreateAllocationWithSlack(InNewMask);

	int32 NewEntityIndex;
	if (!Location.IsValid())
	{
		NewEntityIndex = AddEntityToAllocation(NewAllocationIndex, EntityID);
	}
	else
	{
		NewEntityIndex = MigrateEntity(NewAllocationIndex, Location.GetAllocationIndex(), Location.GetEntryIndexWithinAllocation());
	}

	Location.Set(NewAllocationIndex, NewEntityIndex);

	CheckInvariants();

	OnStructureChanged();
}

void FEntityManager::RemoveComponent(FMovieSceneEntityID EntityID, FComponentTypeID ComponentType)
{
	CheckCanChangeStructure();

	FEntityLocation& Location = EntityLocations[EntityID.AsIndex()];
	if (!Location.IsValid())
	{
		return;
	}
	else if (!EntityAllocationMasks[Location.GetAllocationIndex()].Contains(ComponentType))
	{
		return;
	}

	bool bNewEntityHasComponents = false;
	FComponentMask NewEntityComponentMask;

	FEntityAllocation* Allocation = EntityAllocations[Location.GetAllocationIndex()];
	for (const FComponentHeader& Header : Allocation->GetComponentHeaders())
	{
		// @todo: Should this be an error?
		if (Header.ComponentType != ComponentType)
		{
			NewEntityComponentMask.Set(Header.ComponentType);
			bNewEntityHasComponents = true;
		}
	}

	FEntityLocation OldLocation = Location;
	if (!bNewEntityHasComponents)
	{
		Location.Reset();
		RemoveEntityFromAllocation(OldLocation.GetAllocationIndex(), OldLocation.GetEntryIndexWithinAllocation());
	}
	else
	{
		int32 AllocationIndex = GetOrCreateAllocationWithSlack(MoveTemp(NewEntityComponentMask));
		int32 EntityIndex     = MigrateEntity(AllocationIndex, OldLocation.GetAllocationIndex(), OldLocation.GetEntryIndexWithinAllocation());

		Location.Set(AllocationIndex, EntityIndex);
	}

	CheckInvariants();

	OnStructureChanged();
}

void FEntityManager::RemoveComponents(FMovieSceneEntityID EntityID, const FComponentMask& EntitiesToRemove)
{
	CheckCanChangeStructure();

	if (EntitiesToRemove.Find(true) == INDEX_NONE)
	{
		return;
	}

	FEntityLocation& Location = EntityLocations[EntityID.AsIndex()];
	if (!Location.IsValid())
	{
		return;
	}

	const FComponentMask& CurrentMask = EntityAllocationMasks[Location.GetAllocationIndex()];

	// Initialize a new mask with all bits set
	FComponentMask NewEntityComponentMask(true, CurrentMask.Num());
	// Unset any bits that we need to remove
	NewEntityComponentMask.CombineWithBitwiseXOR(EntitiesToRemove, EBitwiseOperatorFlags::MaintainSize);
	// Copy over bits that still match
	NewEntityComponentMask.CombineWithBitwiseAND(CurrentMask, EBitwiseOperatorFlags::MaintainSize);

	if (EntityAllocationMasks[Location.GetAllocationIndex()].CompareSetBits(NewEntityComponentMask))
	{
		return;
	}

	const bool bNewEntityHasComponents = NewEntityComponentMask.Find(true) != INDEX_NONE;

	FEntityLocation OldLocation = Location;
	if (!bNewEntityHasComponents)
	{
		Location.Reset();
		RemoveEntityFromAllocation(OldLocation.GetAllocationIndex(), OldLocation.GetEntryIndexWithinAllocation());
	}
	else
	{
		int32 AllocationIndex = GetOrCreateAllocationWithSlack(MoveTemp(NewEntityComponentMask));
		int32 EntityIndex     = MigrateEntity(AllocationIndex, OldLocation.GetAllocationIndex(), OldLocation.GetEntryIndexWithinAllocation());

		Location.Set(AllocationIndex, EntityIndex);
	}

	CheckInvariants();

	OnStructureChanged();
}

void FEntityManager::RemoveComponent(FMovieSceneEntityID ThisEntityID, FComponentTypeID ComponentTypeID, EEntityRecursion Recursion)
{
	if (EnumHasAnyFlags(Recursion, EEntityRecursion::This))
	{
		RemoveComponent(ThisEntityID, ComponentTypeID);
	}
	if (EnumHasAnyFlags(Recursion, EEntityRecursion::Children))
	{
		IterateChildren_ParentFirst(ThisEntityID, [this, ComponentTypeID](FMovieSceneEntityID EntityID)
		{
			this->RemoveComponent(EntityID, ComponentTypeID);
		});
	}
}

void FEntityManager::RemoveComponents(FMovieSceneEntityID ThisEntityID, const FComponentMask& EntitiesToRemove, EEntityRecursion Recursion)
{
	if (EnumHasAnyFlags(Recursion, EEntityRecursion::This))
	{
		RemoveComponents(ThisEntityID, EntitiesToRemove);
	}
	if (EnumHasAnyFlags(Recursion, EEntityRecursion::Children))
	{
		IterateChildren_ParentFirst(ThisEntityID, [this, &EntitiesToRemove](FMovieSceneEntityID EntityID)
		{
			this->RemoveComponents(EntityID, EntitiesToRemove);
		});
	}
}

bool FEntityManager::CopyComponent(FMovieSceneEntityID SrcEntityID, FMovieSceneEntityID DstEntityID, FComponentTypeID ComponentTypeID)
{
	CheckCanChangeStructure();

	FEntityLocation SrcLocation = EntityLocations[SrcEntityID.AsIndex()];
	if (!SrcLocation.IsValid())
	{
		return false;
	}
	else if (!EntityAllocationMasks[SrcLocation.GetAllocationIndex()].Contains(ComponentTypeID))
	{
		return false;
	}

	FEntityLocation DstLocation = EntityLocations[DstEntityID.AsIndex()];

	bool bDidChangeStructure = false;
	if (!HasComponent(DstEntityID, ComponentTypeID))
	{
		FComponentMask NewEntityComponentMask;
		if (DstLocation.IsValid())
		{
			NewEntityComponentMask = EntityAllocationMasks[DstLocation.GetAllocationIndex()];
		}
		NewEntityComponentMask.Set(ComponentTypeID);

		// Need to reallocate the destination entity because it doesn't contain this component
		const int32 NewAllocationIndex = GetOrCreateAllocationWithSlack(NewEntityComponentMask);

		int32 NewEntityIndex;
		if (!DstLocation.IsValid())
		{
			NewEntityIndex = AddEntityToAllocation(NewAllocationIndex, DstEntityID);
		}
		else
		{
			NewEntityIndex = MigrateEntity(NewAllocationIndex, DstLocation.GetAllocationIndex(), DstLocation.GetEntryIndexWithinAllocation());
		}

		DstLocation.Set(NewAllocationIndex, NewEntityIndex);
		EntityLocations[DstEntityID.AsIndex()] = DstLocation;

		bDidChangeStructure = true;
	}

	FComponentHeader& SrcHeader = EntityAllocations[SrcLocation.GetAllocationIndex()]->GetComponentHeaderChecked(ComponentTypeID);
	FComponentHeader& DstHeader = EntityAllocations[DstLocation.GetAllocationIndex()]->GetComponentHeaderChecked(ComponentTypeID);

	// Copy the component value to the destination allocation
	const FComponentTypeInfo& ComponentTypeInfo = ComponentRegistry->GetComponentTypeChecked(ComponentTypeID);

	void* DstValue = DstHeader.GetValuePtr(DstLocation.GetEntryIndexWithinAllocation());
	const void* SrcValue = SrcHeader.GetValuePtr(SrcLocation.GetEntryIndexWithinAllocation());

	FEntityAllocationWriteContext WriteContext(*this);

	ComponentTypeInfo.CopyItems(DstValue, SrcValue, 1);
	DstHeader.PostWriteComponents(WriteContext);

	if (bDidChangeStructure)
	{
		CheckInvariants();
		OnStructureChanged();
	}

	return true;
}

void FEntityManager::CopyComponents(FMovieSceneEntityID SrcEntityID, FMovieSceneEntityID DstEntityID, const FComponentMask& ComponentsToCopy)
{
	CheckCanChangeStructure();

	FEntityLocation SrcLocation = EntityLocations[SrcEntityID.AsIndex()];
	FEntityLocation DstLocation = EntityLocations[DstEntityID.AsIndex()];

	if (!SrcLocation.IsValid())
	{
		return;
	}

	bool bDidChangeStructure = false;

	FComponentMask ComponentsOnSource = FComponentMask::BitwiseAND(ComponentsToCopy, EntityAllocationMasks[SrcLocation.GetAllocationIndex()], EBitwiseOperatorFlags::MinSize);
	if (ComponentsOnSource.Find(true) == INDEX_NONE)
	{
		return;
	}

	if (!DstLocation.IsValid())
	{
		const int32 NewAllocationIndex = GetOrCreateAllocationWithSlack(ComponentsOnSource);
		const int32 NewEntityIndex = AddEntityToAllocation(NewAllocationIndex, DstEntityID);

		DstLocation.Set(NewAllocationIndex, NewEntityIndex);
		EntityLocations[DstEntityID.AsIndex()] = DstLocation;

		bDidChangeStructure = true;
	}
	else
	{
		FComponentMask ExistingType  = EntityAllocationMasks[DstLocation.GetAllocationIndex()];
		FComponentMask NewEntityType = FComponentMask::BitwiseOR(ExistingType, ComponentsOnSource, EBitwiseOperatorFlags::MaxSize);
		if (!NewEntityType.CompareSetBits(ExistingType))
		{
			const int32 NewAllocationIndex = GetOrCreateAllocationWithSlack(NewEntityType);
			const int32 NewEntityIndex = MigrateEntity(NewAllocationIndex, DstLocation.GetAllocationIndex(), DstLocation.GetEntryIndexWithinAllocation());

			DstLocation.Set(NewAllocationIndex, NewEntityIndex);
			EntityLocations[DstEntityID.AsIndex()] = DstLocation;

			bDidChangeStructure = true;
		}
	}

	CopyComponents(
		DstLocation.GetAllocationIndex(),
		DstLocation.GetEntryIndexWithinAllocation(),
		SrcLocation.GetAllocationIndex(),
		SrcLocation.GetEntryIndexWithinAllocation(),
		&ComponentsOnSource);

	if (bDidChangeStructure)
	{
		CheckInvariants();
		OnStructureChanged();
	}
}

void FEntityManager::FilterComponents(FMovieSceneEntityID EntityID, const FComponentMask& EntitiesToKeep)
{
	CheckCanChangeStructure();

	FEntityLocation& Location = EntityLocations[EntityID.AsIndex()];
	if (!Location.IsValid())
	{
		return;
	}

	FComponentMask NewEntityComponentMask = FComponentMask::BitwiseAND(EntityAllocationMasks[Location.GetAllocationIndex()], EntitiesToKeep, EBitwiseOperatorFlags::MaxSize);
	if (EntityAllocationMasks[Location.GetAllocationIndex()].CompareSetBits(NewEntityComponentMask))
	{
		return;
	}

	const bool bNewEntityHasComponents = NewEntityComponentMask.Find(true) != INDEX_NONE;

	FEntityLocation OldLocation = Location;
	if (!bNewEntityHasComponents)
	{
		Location.Reset();
		RemoveEntityFromAllocation(OldLocation.GetAllocationIndex(), OldLocation.GetEntryIndexWithinAllocation());
	}
	else
	{
		int32 AllocationIndex = GetOrCreateAllocationWithSlack(MoveTemp(NewEntityComponentMask));
		int32 EntityIndex     = MigrateEntity(AllocationIndex, OldLocation.GetAllocationIndex(), OldLocation.GetEntryIndexWithinAllocation());

		Location.Set(AllocationIndex, EntityIndex);
	}

	CheckInvariants();

	OnStructureChanged();
}

void FEntityManager::CombineComponents(FMovieSceneEntityID DestinationEntityID, FMovieSceneEntityID SourceEntityID, const FComponentMask* OptionalMask)
{
	CheckCanChangeStructure();

	FEntityLocation SourceLocation      = EntityLocations[SourceEntityID.AsIndex()];
	FEntityLocation DestinationLocation = EntityLocations[DestinationEntityID.AsIndex()];

	if (!SourceLocation.IsValid())
	{
		return;
	}

	const int32 OldAllocationIndex = DestinationLocation.IsValid() ? DestinationLocation.GetAllocationIndex() : INDEX_NONE;

	FComponentMask NewEntityComponentMask = EntityAllocationMasks[SourceLocation.GetAllocationIndex()];
	if (OptionalMask)
	{
		NewEntityComponentMask.CombineWithBitwiseAND(*OptionalMask, EBitwiseOperatorFlags::MaxSize);
	}

	if (OldAllocationIndex != INDEX_NONE)
	{
		NewEntityComponentMask.CombineWithBitwiseOR(EntityAllocationMasks[OldAllocationIndex], EBitwiseOperatorFlags::MaxSize);

		int32 NewEntityIndex     = DestinationLocation.GetEntryIndexWithinAllocation();
		int32 NewAllocationIndex = OldAllocationIndex;

		if (!NewEntityComponentMask.CompareSetBits(EntityAllocationMasks[OldAllocationIndex]))
		{
			NewAllocationIndex = GetOrCreateAllocationWithSlack(NewEntityComponentMask);
			NewEntityIndex = MigrateEntity(NewAllocationIndex, OldAllocationIndex, DestinationLocation.GetEntryIndexWithinAllocation());

			EntityLocations[DestinationEntityID.AsIndex()].Set(NewAllocationIndex, NewEntityIndex);
		}

		CopyComponents(NewAllocationIndex, NewEntityIndex, SourceLocation.GetAllocationIndex(), SourceLocation.GetEntryIndexWithinAllocation(), OptionalMask);
	}
	else
	{
		const int32 NewAllocationIndex = GetOrCreateAllocationWithSlack(NewEntityComponentMask);
		int32 NewEntityIndex = AddEntityToAllocation(NewAllocationIndex, DestinationEntityID);

		CopyComponents(NewAllocationIndex, NewEntityIndex, SourceLocation.GetAllocationIndex(), SourceLocation.GetEntryIndexWithinAllocation());

		EntityLocations[DestinationEntityID.AsIndex()].Set(NewAllocationIndex, NewEntityIndex);
	}

	CheckInvariants();

	OnStructureChanged();
}

int32 FEntityManager::MutateAll(const FEntityComponentFilter& Filter, const IMovieSceneEntityMutation& Mutation, EMutuallyInclusiveComponentType MutualTypes)
{
	CheckCanChangeStructure();

	FMutualComponentInitializers MutualInitializers;
	FEntityAllocationWriteContext WriteContext(*this);

	int32 TotalNumMutations = 0;

	for (int32 AllocationIndex = 0; AllocationIndex < EntityAllocationMasks.GetMaxIndex(); ++AllocationIndex)
	{
		if (!EntityAllocationMasks.IsAllocated(AllocationIndex) || !Filter.Match(EntityAllocationMasks[AllocationIndex]))
		{
			continue;
		}

		// Process the mutation
		FComponentMask NewAllocationType = EntityAllocationMasks[AllocationIndex];
		Mutation.CreateMutation(this, &NewAllocationType);

		// Add mutual components
		MutualInitializers.Reset();
		ComponentRegistry->Factories.ComputeMutuallyInclusiveComponents(MutualTypes, NewAllocationType, MutualInitializers);

		FEntityAllocation* SourceAllocation = EntityAllocations[AllocationIndex];

		// If the type hasn't changed at all, we can just run the unmodified initializer
		if (NewAllocationType.CompareSetBits(EntityAllocationMasks[AllocationIndex]))
		{
			Mutation.InitializeUnmodifiedAllocation(SourceAllocation, NewAllocationType);
			continue;
		}

		// The type has changed so we need to migrate the allocation - we just reallocate within
		// the same allocation index to avoid having to fix up specific entity entry indices
		FEntityAllocation* NewAllocation = MigrateAllocation(AllocationIndex, NewAllocationType);
		TotalNumMutations += NewAllocation->Num();

		FComponentMask OldAllocationType = EntityAllocationMasks[AllocationIndex];
		EntityAllocationMasks[AllocationIndex] = NewAllocationType;
		EntityAllocations[AllocationIndex] = NewAllocation;

		FEntityAllocationMutexGuard LockGuard(NewAllocation, EComponentHeaderLockMode::LockFree);

		// Default construct all the new components in the allocation, and then allow the mutation to further initialize this data if needed
		for (FComponentMaskIterator Component = NewAllocationType.Iterate(); Component; ++Component)
		{
			FComponentTypeID ComponentTypeID = FComponentTypeID::FromBitIndex(Component.GetIndex());
			if (OldAllocationType.Contains(ComponentTypeID))
			{
				continue;
			}

			const FComponentHeader& ComponentHeader = NewAllocation->GetComponentHeaderChecked(ComponentTypeID);
			if (!ComponentHeader.IsTag())
			{
				const FComponentTypeInfo& ComponentTypeInfo = ComponentRegistry->GetComponentTypeChecked(ComponentTypeID);

				void* Components = ComponentHeader.GetValuePtr(0);
				ComponentTypeInfo.ConstructItems(Components, NewAllocation->Num());
			}
		}

		// Run custom initializers
		MutualInitializers.Execute(FEntityRange{ NewAllocation, 0, NewAllocation->Num() }, WriteContext);

		// Run mutation initializer
		Mutation.InitializeAllocation(NewAllocation, NewAllocationType);

		// Destroy the old data
		DestroyAllocation(SourceAllocation);
	}

	if (TotalNumMutations != 0)
	{
		CheckInvariants();
		OnStructureChanged();
	}

	return TotalNumMutations;
}

int32 FEntityManager::MutateConditional(const FEntityComponentFilter& Filter, const IMovieSceneConditionalEntityMutation& Mutation, EMutuallyInclusiveComponentType MutualTypes)
{
	CheckCanChangeStructure();

	struct FConditionalMutationState
	{
		FComponentMask NewType;
		TBitArray<> MarkedEntities;
		FMutualComponentInitializers MutualInitializers;
	};
	TMap<int32, FConditionalMutationState> AllocationMutations;

	FMutualComponentInitializers MutualInitializersScratch;
	for (int32 AllocationIndex = 0; AllocationIndex < EntityAllocationMasks.GetMaxIndex(); ++AllocationIndex)
	{
		if (EntityAllocationMasks.IsAllocated(AllocationIndex) && Filter.Match(EntityAllocationMasks[AllocationIndex]))
		{
			FEntityAllocation* SourceAllocation = EntityAllocations[AllocationIndex];

			TBitArray<> MarkedEntities;
			Mutation.MarkAllocation(SourceAllocation, MarkedEntities);

			if (MarkedEntities.Num() == 0)
			{
				continue;
			}

			const int32 NumIrrelevantBits = MarkedEntities.Num() - SourceAllocation->Num();
			if (NumIrrelevantBits > 1)
			{
				MarkedEntities.RemoveAt(SourceAllocation->Num(), NumIrrelevantBits);
			}

			FComponentMask NewMutation = EntityAllocationMasks[AllocationIndex];
			Mutation.CreateMutation(this, &NewMutation);

			// Add mutual components
			MutualInitializersScratch.Reset();
			ComponentRegistry->Factories.ComputeMutuallyInclusiveComponents(MutualTypes, NewMutation, MutualInitializersScratch);

			if (!NewMutation.CompareSetBits(EntityAllocationMasks[AllocationIndex]))
			{
				AllocationMutations.Add(AllocationIndex, FConditionalMutationState{
					MoveTemp(NewMutation),
					MoveTemp(MarkedEntities),
					MoveTemp(MutualInitializersScratch)
				});
			}
		}
	}

	if (AllocationMutations.Num() == 0)
	{
		return 0;
	}

	int32 TotalNumMutations = 0;

	FMutualComponentInitializers MutualInitializers;
	FEntityAllocationWriteContext WriteContext(*this);

	for (TTuple<int32, FConditionalMutationState>& Pair : AllocationMutations)
	{
		const int32 SourceAllocationIndex = Pair.Key;
		FEntityAllocation* SourceAllocation = EntityAllocations[SourceAllocationIndex];

		const int32 NumEntities = Pair.Value.MarkedEntities.CountSetBits();
		const bool bAllEntitiesMarked = NumEntities == SourceAllocation->Num();

		TotalNumMutations += NumEntities;

		const FComponentMask& DesiredType = Pair.Value.NewType;
		if (bAllEntitiesMarked)
		{
			// When adding a component to an entire allocation we just reallocate within the same allocation entry to avoid having to fix up 
			// Specific entity entry indices
			FEntityAllocation* NewAllocation = MigrateAllocation(SourceAllocationIndex, DesiredType);

			// Default construct all the new components in the allocation, and then allow the mutation to further initialize this data if needed
			for (FComponentMaskIterator Component = DesiredType.Iterate(); Component; ++Component)
			{
				FComponentTypeID ComponentTypeID = FComponentTypeID::FromBitIndex(Component.GetIndex());
				if (EntityAllocationMasks[SourceAllocationIndex].Contains(ComponentTypeID))
				{
					continue;
				}

				const FComponentHeader& ComponentHeader = NewAllocation->GetComponentHeaderChecked(ComponentTypeID);
				if (!ComponentHeader.IsTag())
				{
					const FComponentTypeInfo& ComponentTypeInfo = ComponentRegistry->GetComponentTypeChecked(ComponentTypeID);

					void* Components = ComponentHeader.GetValuePtr(0);
					ComponentTypeInfo.ConstructItems(Components, NewAllocation->Num());
				}
			}

			EntityAllocationMasks[SourceAllocationIndex] = DesiredType;
			EntityAllocations[SourceAllocationIndex] = NewAllocation;

			FEntityRange Range { NewAllocation, 0, NewAllocation->Num() };

			// Run custom initializers
			Pair.Value.MutualInitializers.Execute(Range, WriteContext);

			Mutation.InitializeEntities(Range, DesiredType);

			DestroyAllocation(SourceAllocation);
		}
		else
		{
			int32 DesiredSlack = NumEntities;
			const int32 NewAllocationIndex = CreateAllocationWithSlack(DesiredType, &DesiredSlack);

			const FEntityAllocation* DestAllocation = EntityAllocations[NewAllocationIndex];
			FEntityRange Range { DestAllocation, DestAllocation->Num(), NumEntities };

			const FMovieSceneEntityID* EntityIDs = SourceAllocation->GetRawEntityIDs();

			// Migrate entities to the new allocation - care is taken to iterate the allocation backwards
			// since entities can shift around in the allocation as they are migrated
			for (TBitArray<>::FConstReverseIterator It(Pair.Value.MarkedEntities); It; ++It)
			{
				if (It.GetValue())
				{
					FEntityLocation& Location = EntityLocations[EntityIDs[It.GetIndex()].AsIndex()];

					int32 NewEntityIndex = MigrateEntity(NewAllocationIndex, Location.GetAllocationIndex(), Location.GetEntryIndexWithinAllocation());
					Location.Set(NewAllocationIndex, NewEntityIndex);
				}
			}

			check(Range.ComponentStartOffset + Range.Num == Range.Allocation->Num());

			// Run custom initializers
			Pair.Value.MutualInitializers.Execute(Range, WriteContext);

			Mutation.InitializeEntities(Range, DesiredType);
		}
	}

	CheckInvariants();

	OnStructureChanged();

	return TotalNumMutations;
}

void FEntityManager::TouchEntity(FMovieSceneEntityID EntityID)
{
	FEntityLocation Location = EntityLocations[EntityID.AsIndex()];
	if (Location.IsValid())
	{
		FEntityAllocation* Allocation = GetAllocation(Location.GetAllocationIndex());
		check(Allocation);

		FEntityAllocationWriteContext WriteContext(*this);

		for (FComponentHeader& Header : Allocation->GetComponentHeaders())
		{
			if (!Header.IsTag())
			{
				Header.PostWriteComponents(WriteContext);
			}
		}
	}
}

void FEntityManager::AddChild(FMovieSceneEntityID ParentID, FMovieSceneEntityID ChildID)
{
	FEntityLocation& Location = EntityLocations[ChildID.AsIndex()];
	if (Location.GetParentID())
	{
		ParentToChild.Remove(Location.GetParentID(), ChildID);
	}

	ParentToChild.Add(ParentID, ChildID);
	Location.SetParentID(ParentID);
}

void FEntityManager::GetImmediateChildren(FMovieSceneEntityID ParentID, TArray<FMovieSceneEntityID>& OutChildren) const
{
	for (auto ChildIt = ParentToChild.CreateConstKeyIterator(ParentID); ChildIt; ++ChildIt)
	{
		OutChildren.Add(ChildIt.Value());
	}
}

void FEntityManager::GetChildren_ParentFirst(FMovieSceneEntityID ParentID, TArray<FMovieSceneEntityID>& OutChildren) const
{
	for (auto ChildIt = ParentToChild.CreateConstKeyIterator(ParentID); ChildIt; ++ChildIt)
	{
		OutChildren.Add(ChildIt.Value());
		GetChildren_ParentFirst(ChildIt.Value(), OutChildren);
	}
}

void FEntityManager::InitializeChildAllocation(const FComponentMask& ParentType, const FComponentMask& ChildType, const FEntityAllocation* ParentAllocation, TArrayView<const int32> ParentAllocationOffsets, const FEntityRange& InChildEntityRange)
{
	ComponentRegistry->Factories.RunInitializers(ParentType, ChildType, ParentAllocation, ParentAllocationOffsets, InChildEntityRange);

	for (TInlineValue<FChildEntityInitializer>& ChildInit : InstancedChildInitializers)
	{
		if (ChildInit->IsRelevant(ParentType, ChildType))
		{
			ChildInit->Run(InChildEntityRange, ParentAllocation, ParentAllocationOffsets);
		}
	}
}

void FEntityManager::AddMutualComponents()
{
	AddMutualComponents(FEntityComponentFilter());
}

void FEntityManager::AddMutualComponents(const FEntityComponentFilter& InFilter)
{
	CheckCanChangeStructure();

	struct FBenignMutation : IMovieSceneEntityMutation
	{
		void CreateMutation(FEntityManager* EntityManager, FComponentMask* InOutEntityComponentTypes) const override
		{}
	};

	FBenignMutation BenignMutation;
	MutateAll(InFilter, BenignMutation, EMutuallyInclusiveComponentType::All);
}

FEntityAllocation* FEntityManager::MigrateAllocation(int32 AllocationIndex, const FComponentMask& NewComponentMask)
{
	CheckCanChangeStructure();

	check(EntityAllocations.IsValidIndex(AllocationIndex));
	FEntityAllocation* Source = EntityAllocations[AllocationIndex];

	// Check if the migration only adds/removes tags, in which case we can do that in-place without re-allocating
	// the components' data buffer.
	const FComponentMask OldComponentMask = EntityAllocationMasks[AllocationIndex];
	const FComponentMask ChangedComponentsMask = FComponentMask::BitwiseXOR(OldComponentMask, NewComponentMask, EBitwiseOperatorFlags::MaxSize);
	const FComponentMask ChangedNonTagComponentsMask = FComponentMask::BitwiseAND(ComponentRegistry->GetDataComponentTypes(), ChangedComponentsMask, EBitwiseOperatorFlags::MaxSize);
	const bool bOnlyTagComponentsChanged = (ChangedNonTagComponentsMask.Find(true) == INDEX_NONE);

	// Create a new allocation of the necessary capacity with the correct new mask
	FEntityAllocation* Dest = nullptr;
	if (bOnlyTagComponentsChanged)
	{
		// ~~~~~~~~~~~ WARNING: after this, the source allocation has null component pointers! ~~~~~~~~~~~~
		Dest = CreateEntityAllocation(NewComponentMask, Source->GetCapacity(), Source->GetMaxCapacity(), Source);
	}
	else
	{
		Dest = CreateEntityAllocation(NewComponentMask, Source->GetCapacity(), Source->GetMaxCapacity());
	}
	check(Dest != nullptr);

	FEntityInitializer::Duplicate(Dest, Source);

	// Relocate component data component by component if the migration has changed the memory layout.
	// If it hasn't (i.e. if we only added/removed tags), then we don't have anything to do, because the memory
	// layout is the same and we "stole" the data buffer directly from the source allocation (see above).
	if (!bOnlyTagComponentsChanged)
	{
		// Iterate all destination component types
		FComponentHeader* SrcComponentHeader     = Source->ComponentHeaders;
		FComponentHeader* SrcLastComponentHeader = Source->ComponentHeaders + Source->GetNumComponentTypes() - 1;

		for (int32 DstHeaderOffset = 0; DstHeaderOffset < Dest->GetNumComponentTypes(); ++DstHeaderOffset)
		{
			FComponentHeader* DstComponentHeader = &Dest->ComponentHeaders[DstHeaderOffset];

			// Try to find a matching source component type
			while (SrcComponentHeader != SrcLastComponentHeader && SrcComponentHeader->ComponentType.BitIndex() < DstComponentHeader->ComponentType.BitIndex())
			{
				++SrcComponentHeader;
			}

			// Copy the component value to the new allocation
			if (DstComponentHeader->ComponentType == SrcComponentHeader->ComponentType && !SrcComponentHeader->IsTag())
			{
				const FComponentTypeInfo& ComponentTypeInfo = ComponentRegistry->GetComponentTypeChecked(SrcComponentHeader->ComponentType);

				void* DstValue = DstComponentHeader->GetValuePtr(0);
				void* SrcValue = SrcComponentHeader->GetValuePtr(0);

				ComponentTypeInfo.RelocateConstructItems(DstValue, SrcValue, Source->Num());

				// Mark these components as having be relocated, so they don't need to be destructed later.
				SrcComponentHeader->Components = nullptr;
			}
		}
	}

	CheckInvariants();

	OnStructureChanged();

	return Dest;
}

void FEntityManager::CombineAllocations(int32 DestinationIndex, int32 SourceIndex)
{
	CheckCanChangeStructure();

	FEntityAllocation* DstAllocation = EntityAllocations[DestinationIndex];
	FEntityAllocation* SrcAllocation = EntityAllocations[SourceIndex];

	check(DstAllocation->GetMaxCapacity() - DstAllocation->Num() >= SrcAllocation->Num())
	if (DstAllocation->GetCapacity() - DstAllocation->Num() < SrcAllocation->Num())
	{
		// Need to grow the allocation
		DstAllocation = GrowAllocation(DestinationIndex, SrcAllocation->Num());
	}

	// Write new entity locations
	{
		const int32 NewEntityStart = DstAllocation->Num();

		TArrayView<const FMovieSceneEntityID> EntityIDs = SrcAllocation->GetEntityIDs();
		for (int32 EntityIndex = 0; EntityIndex < EntityIDs.Num(); ++EntityIndex)
		{
			FMovieSceneEntityID EntityID = EntityIDs[EntityIndex];
			EntityLocations[EntityID.AsIndex()].Set(DestinationIndex, NewEntityStart + EntityIndex);
		}
	}

	// Copy the entitiy IDs and component data over to the new allocation
	FEntityInitializer::MigrateAllocation(DstAllocation, SrcAllocation, ComponentRegistry);

	// Destroy the old allocation
	const bool bDestructComponentData = false;  // We have relocated all the component data, there's nothing left to destroy.
	DestroyAllocation(SrcAllocation, bDestructComponentData);

	// Reset allocation entries
	EntityAllocationMasks.RemoveAt(SourceIndex);
	EntityAllocations.RemoveAt(SourceIndex);

	AllocationsWithCapacity[SourceIndex] = false;
	AllocationsWithCapacity[DestinationIndex] = DstAllocation->GetMaxCapacity() > DstAllocation->Num();

	CheckInvariants();
	OnStructureChanged();
}


void FEntityManager::AddReferencedObjects(FReferenceCollector& ReferenceCollector)
{
	for (FEntityAllocation* Allocation : EntityAllocations)
	{
		const int32 AllocationSize = Allocation->Num();
		for (const FComponentHeader& Header : Allocation->GetComponentHeaders())
		{
			const FComponentTypeInfo& ComponentTypeInfo = ComponentRegistry->GetComponentTypeChecked(Header.ComponentType);

			if (!Header.IsTag() && ComponentTypeInfo.bHasReferencedObjects)
			{
				ComponentTypeInfo.AddReferencedObjects(ReferenceCollector, Header.Components, AllocationSize);
			}
		}
	}
}

FString FEntityManager::GetReferencerName() const
{
	return ManagerDebugName;
}

void FEntityManager::NotifyUObjectDeleted(const UObjectBase* Object, int32 Index)
{
}

void FEntityManager::OnUObjectArrayShutdown()
{
}

int32 FEntityManager::MigrateEntity(int32 DestAllocationIndex, int32 SourceAllocationIndex, int32 SourceEntityIndex)
{
	check(DestAllocationIndex != SourceAllocationIndex);

	CheckCanChangeStructure();

	FEntityAllocation* Dst = EntityAllocations[DestAllocationIndex];
	FEntityAllocation* Src = EntityAllocations[SourceAllocationIndex];

	check(SourceEntityIndex < Src->Num());

	if (Dst->GetCapacity() == Dst->Num())
	{
		Dst = GrowAllocation(DestAllocationIndex);
	}

	FMovieSceneEntityID EntityID = Src->GetEntityIDs()[SourceEntityIndex];

	// Make space for the new entity in the destination without initializing the memory
	// This is important because we either default construct or relocate construct into this new allocation ourselves
	const int32 DestEntityIndex = AddEntityToAllocation(DestAllocationIndex, EntityID, EMemoryType::DefaultConstructed);

	const int32 SrcOffset = SourceEntityIndex;
	const int32 DstOffset = DestEntityIndex;

	const int32 LastEntityIndex = Src->Num() - 1;

	check(SrcOffset < Src->Num() && DstOffset < Dst->Num());

	FEntityAllocationWriteContext WriteContext(*this);
	Src->PostModifyStructureExcludingHeaders(WriteContext);
	Dst->PostModifyStructureExcludingHeaders(WriteContext);

	const int32 NumSrcHeaders = Src->GetNumComponentTypes();
	int32 SrcHeaderIndex = 0;

	// This function takes a component header and address from the original allocation and removes it from the allocation
	// by destructing the component, and relocating the last element in the component array into its place, similar to TArray::RemoveAtSwap
	// This allows minimal shuffling of component data when moving entities between allocations.
	auto RemoveAtSwapComponent = [this, LastEntityIndex, WriteContext](const FComponentHeader& SrcComponentHeader, int32 RemoveAtIndex)
	{
		SrcComponentHeader.PostWriteComponents(WriteContext);
		if (!SrcComponentHeader.IsTag())
		{
			const FComponentTypeInfo& ComponentTypeInfo = this->ComponentRegistry->GetComponentTypeChecked(SrcComponentHeader.ComponentType);

			void* RemovedValueAddress = SrcComponentHeader.GetValuePtr(RemoveAtIndex);

			// Destroy the component at the address
			ComponentTypeInfo.DestructItems(RemovedValueAddress, 1);

			// Swap the last entity's components in this allocation with the migrated one
			if (RemoveAtIndex != LastEntityIndex)
			{
				void* LastItem = SrcComponentHeader.GetValuePtr(LastEntityIndex);
				ComponentTypeInfo.RelocateConstructItems(RemovedValueAddress, LastItem, 1);
			}
		}
	};

	// Iterate all destination component types and either default construct, or relocate the existing component
	for (int32 DstHeaderOffset = 0; DstHeaderOffset < Dst->GetNumComponentTypes(); ++DstHeaderOffset)
	{
		FComponentHeader* DstComponentHeader = &Dst->ComponentHeaders[DstHeaderOffset];
		if (DstComponentHeader->IsTag())
		{
			continue;
		}

		// Try to locate a matching source component header for this component type by walking through the source headers
		// Component headers are sorted by component type, so if we encounter any with a type ID less than the current destination type
		// it must not exist in the new allocation, and so should be destroyed from this one
		while (SrcHeaderIndex < NumSrcHeaders && Src->ComponentHeaders[SrcHeaderIndex].ComponentType.BitIndex() < DstComponentHeader->ComponentType.BitIndex())
		{
			// Destination does not have this component type so it needs destroying
			RemoveAtSwapComponent(Src->ComponentHeaders[SrcHeaderIndex], SrcOffset);
			++SrcHeaderIndex;
		}

		const FComponentTypeInfo& ComponentTypeInfo = ComponentRegistry->GetComponentTypeChecked(DstComponentHeader->ComponentType);
		void* DstValue = DstComponentHeader->GetValuePtr(DstOffset);

		// Relocate the component value to the new allocation if it is the same type as our current source header
		if (SrcHeaderIndex < NumSrcHeaders && Src->ComponentHeaders[SrcHeaderIndex].ComponentType.BitIndex() == DstComponentHeader->ComponentType.BitIndex())
		{
			void* MigratedValueAddress = Src->ComponentHeaders[SrcHeaderIndex].GetValuePtr(SrcOffset);
			ComponentTypeInfo.RelocateConstructItems(DstValue, MigratedValueAddress, 1);

			// We do not use RemoveAtSwapComponent here because the SrcValue is now already considered destructed
			// Manually swap the last entity's components in this allocation with the migrated one
			if (LastEntityIndex != SrcOffset)
			{
				void* LastItem = Src->ComponentHeaders[SrcHeaderIndex].GetValuePtr(LastEntityIndex);
				ComponentTypeInfo.RelocateConstructItems(MigratedValueAddress, LastItem, 1);
			}

			// This source header is now dealt with so skip over it
			++SrcHeaderIndex;
		}
		// or default construct it if it's a new component that didn't exist before
		else
		{
			ComponentTypeInfo.ConstructItems(DstValue, 1);
		}
	}

	// Process any remaining components in the source allocation that were not in the destination
	// by destructing the components and potentially relocating the RemoveAtSwap candidate
	for ( ; SrcHeaderIndex < NumSrcHeaders; ++SrcHeaderIndex)
	{
		RemoveAtSwapComponent(Src->ComponentHeaders[SrcHeaderIndex], SrcOffset);
	}

	// When removing we just swap the tail element with the element to remove, and fix up the indices
	if (LastEntityIndex != SrcOffset)
	{
		FEntityInitializer::MoveEntryIndex(Src, LastEntityIndex, SrcOffset);

		// Fixup entry offset for the changed entry
		FMovieSceneEntityID SwappedEntityID = Src->GetEntityIDs()[SrcOffset];
		EntityLocations[SwappedEntityID.AsIndex()].Set(SourceAllocationIndex, SrcOffset);
	}

	// Free this entity index without needing to fix up any other indices
	FEntityInitializer::FreeEntryIndex(Src, LastEntityIndex);

	// Does the one we just added to no longer have any capacity?
	if (Dst->Num() == Dst->GetMaxCapacity())
	{
		AllocationsWithCapacity[DestAllocationIndex] = false;
	}
	
	// Does the one we just removed from now have capacity when it didn't before?
	if (Src->Num() == 0)
	{
		DestroyAllocation(Src);

		EntityAllocationMasks.RemoveAt(SourceAllocationIndex);
		EntityAllocations.RemoveAt(SourceAllocationIndex);

		AllocationsWithCapacity[SourceAllocationIndex] = false;
	}
	else if (Src->Num() == Src->GetMaxCapacity()-1)
	{
		AllocationsWithCapacity[SourceAllocationIndex] = true;
	}

	OnStructureChanged();

	return DestEntityIndex;
}

void FEntityManager::CopyComponents(int32 DestAllocationIndex, int32 DestEntityIndex, int32 SourceAllocationIndex, int32 SourceEntityIndex, const FComponentMask* OptionalMask)
{
	CheckCanChangeStructure();

	FEntityAllocation* Dst = EntityAllocations[DestAllocationIndex];
	FEntityAllocation* Src = EntityAllocations[SourceAllocationIndex];

	FComponentHeader* SrcComponentHeader = Src->ComponentHeaders;
	FComponentHeader* SrcLastComponentHeader = Src->ComponentHeaders + Src->GetNumComponentTypes() - 1;

	const int32 SrcOffset = SourceEntityIndex;
	const int32 DstOffset = DestEntityIndex;

	check(SrcOffset < Src->Num() && DstOffset < Dst->Num());

	FEntityAllocationWriteContext WriteContext(*this);
	Dst->PostModifyStructureExcludingHeaders(WriteContext);

	// Iterate all source component types
	for (int32 DstHeaderOffset = 0;  DstHeaderOffset < Dst->GetNumComponentTypes(); ++DstHeaderOffset)
	{
		FComponentHeader* DstComponentHeader = &Dst->ComponentHeaders[DstHeaderOffset];
		if (OptionalMask && !OptionalMask->Contains(DstComponentHeader->ComponentType))
		{
			continue;
		}

		DstComponentHeader->PostWriteComponents(WriteContext);

		// Try to find a matching source component type
		while (SrcComponentHeader != SrcLastComponentHeader && SrcComponentHeader->ComponentType.BitIndex() < DstComponentHeader->ComponentType.BitIndex())
		{
			++SrcComponentHeader;
		}

		// Copy the component value to the destination allocation
		if (DstComponentHeader->ComponentType == SrcComponentHeader->ComponentType && !SrcComponentHeader->IsTag())
		{
			const FComponentTypeInfo& ComponentTypeInfo = ComponentRegistry->GetComponentTypeChecked(SrcComponentHeader->ComponentType);

			void* DstValue = DstComponentHeader->GetValuePtr(DstOffset);
			const void* SrcValue = SrcComponentHeader->GetValuePtr(SrcOffset);

			ComponentTypeInfo.CopyItems(DstValue, SrcValue, 1);
		}
	}

	OnStructureChanged();
}

int32 FEntityManager::ReserveAllocation(int32 AllocationIndex, int32 NumToReserve)
{
	CheckCanChangeStructure();

	FEntityAllocation* Allocation = EntityAllocations[AllocationIndex];

	// Max out this allocation if possible
	const int32 MaxAvailableSlack = Allocation->GetMaxCapacity() - Allocation->Num();

	NumToReserve = FMath::Min(MaxAvailableSlack, NumToReserve);
	if (Allocation->Num() + NumToReserve > Allocation->GetCapacity())
	{
		GrowAllocation(AllocationIndex, NumToReserve);
	}

	return NumToReserve;
}

FEntityAllocation* FEntityManager::GrowAllocation(int32 AllocationIndex, int32 MinNumToGrowBy)
{
	CheckCanChangeStructure();

	FEntityAllocation* Allocation = EntityAllocations[AllocationIndex];

	check(MinNumToGrowBy > 0 && Allocation->Num() + MinNumToGrowBy <= Allocation->GetMaxCapacity());

	const int32 EntityCount = Allocation->Num();

	const bool bAllowQuantize = true;
	// Increase the size of this allocation to house the new entity
	int32 NewCapacity = DefaultCalculateSlackGrow(EntityCount + MinNumToGrowBy, Allocation->GetCapacity(), 1, bAllowQuantize);

	NewCapacity = FMath::Max(EntityCount + MinNumToGrowBy, NewCapacity);
	NewCapacity = FMath::Min(NewCapacity, Allocation->GetMaxCapacity());

	// Create a new allocation with the same mask
	FEntityAllocation* NewAllocation = CreateEntityAllocation(EntityAllocationMasks[AllocationIndex], NewCapacity, Allocation->GetMaxCapacity());
	FEntityInitializer::Duplicate(NewAllocation, Allocation);

	check(Allocation->GetNumComponentTypes() == NewAllocation->GetNumComponentTypes());

	for (int32 Index = 0; Index < Allocation->GetNumComponentTypes(); ++Index)
	{
		const FComponentHeader* SrcHeader = Allocation->ComponentHeaders + Index;
		FComponentHeader* DstHeader = NewAllocation->ComponentHeaders + Index;

		check(SrcHeader->ComponentType == DstHeader->ComponentType);
		if (!SrcHeader->IsTag())
		{
			const FComponentTypeInfo& ComponentTypeInfo = ComponentRegistry->GetComponentTypeChecked(SrcHeader->ComponentType);

			void* SrcValue = SrcHeader->GetValuePtr(0);
			void* DstValue = DstHeader->GetValuePtr(0);

			ComponentTypeInfo.RelocateConstructItems(DstValue, SrcValue, Allocation->Num());
		}
	}

	const bool bDestructComponentData = false;  // We have relocated all the component data, there's nothing left to destroy.
	DestroyAllocation(Allocation, bDestructComponentData);

	EntityAllocations[AllocationIndex] = NewAllocation;

	return NewAllocation;
}

int32 FEntityManager::AddEntityToAllocation(int32 AllocationIndex, FMovieSceneEntityID ID, EMemoryType MemoryType)
{
	CheckCanChangeStructure();

	FEntityAllocation* Allocation = EntityAllocations[AllocationIndex];

	check(Allocation->Num() < Allocation->GetMaxCapacity());
	if (Allocation->Num() == Allocation->GetCapacity())
	{
		Allocation = GrowAllocation(AllocationIndex);
	}

	const int32 ActualEntityOffset   = Allocation->Num();
	FEntityInitializer::AddEntity(Allocation, ActualEntityOffset, ID);

	FEntityAllocationWriteContext WriteContext(*this);
	Allocation->PostModifyStructureExcludingHeaders(WriteContext);

	for (FComponentHeader& Header : Allocation->GetComponentHeaders())
	{
		Header.PostWriteComponents(WriteContext);

		if (!Header.IsTag() && MemoryType == EMemoryType::DefaultConstructed)
		{
			const FComponentTypeInfo& ComponentTypeInfo = ComponentRegistry->GetComponentTypeChecked(Header.ComponentType);
			void* Value = Header.GetValuePtr(ActualEntityOffset);
			ComponentTypeInfo.ConstructItems(Value, 1);
		}
	}

	if (Allocation->Num() == Allocation->GetMaxCapacity())
	{
		AllocationsWithCapacity[AllocationIndex] = false;
	}

	OnStructureChanged();

	return ActualEntityOffset;
}

void FEntityManager::RemoveEntityFromAllocation(int32 AllocationIndex, int32 EntryIndexWithinAllocation)
{
	CheckCanChangeStructure();

	FEntityAllocation* Allocation = EntityAllocations[AllocationIndex];

	const int32 ActualEntityOffset = EntryIndexWithinAllocation;
	const int32 LastEntityIndex    = Allocation->Num() - 1;

	const bool bHadCapacity = Allocation->Num() != Allocation->GetMaxCapacity();

	FEntityAllocationWriteContext WriteContext(*this);
	Allocation->PostModifyStructureExcludingHeaders(WriteContext);

	// Destruct its components
	for (FComponentHeader& Header : Allocation->GetComponentHeaders())
	{
		Header.PostWriteComponents(WriteContext);

		if (Header.IsTag())
		{
			continue;
		}

		const FComponentTypeInfo& ComponentTypeInfo = ComponentRegistry->GetComponentTypeChecked(Header.ComponentType);

		void* Value = Header.GetValuePtr(ActualEntityOffset);
		ComponentTypeInfo.DestructItems(Value, 1);

		if (LastEntityIndex != ActualEntityOffset)
		{
			void* SwapSource = Header.GetValuePtr(LastEntityIndex);
			ComponentTypeInfo.RelocateConstructItems(Value, SwapSource, 1);
		}
	}

	// When removing we just swap the tail element with the element to remove, and fix up the indices
	if (LastEntityIndex != ActualEntityOffset)
	{
		FEntityInitializer::MoveEntryIndex(Allocation, LastEntityIndex, ActualEntityOffset);

		// Fixup entry offset for the changed entry
		FMovieSceneEntityID SwappedEntityID = Allocation->GetEntityIDs()[ActualEntityOffset];
		EntityLocations[SwappedEntityID.AsIndex()].Set(AllocationIndex, ActualEntityOffset);
	}

	// Free this entity index without needing to fix up any other indices
	FEntityInitializer::FreeEntryIndex(Allocation, LastEntityIndex);

	// Does the one we just removed from now have capacity when it didn't before?
	if (Allocation->Num() == 0)
	{
		DestroyAllocation(Allocation);

		EntityAllocationMasks.RemoveAt(AllocationIndex);
		EntityAllocations.RemoveAt(AllocationIndex);

		AllocationsWithCapacity[AllocationIndex] = false;
	}
	else if (!bHadCapacity)
	{
		AllocationsWithCapacity[AllocationIndex] = true;
	}

	OnStructureChanged();
}

void FEntityManager::OnStructureChanged()
{
	StructureMutationSystemSerialNumber = SystemSerialNumber;
	bAccumulatedMaskStale = true;
}

void FEntityManager::CheckInvariants()
{
#if DO_GUARD_SLOW
	if (!GCheckMovieSceneEntityManagerInvariants)
	{
		return;
	}

	for (int32 AllocationIndex = 0; AllocationIndex < EntityAllocations.GetMaxIndex(); ++AllocationIndex)
	{
		if (EntityAllocations.IsAllocated(AllocationIndex))
		{
			TArrayView<const FMovieSceneEntityID> EntityIDs = EntityAllocations[AllocationIndex]->GetEntityIDs();

			for (int32 EntityIndex = 0; EntityIndex < EntityIDs.Num(); ++EntityIndex)
			{
				FEntityLocation Location = EntityLocations[EntityIDs[EntityIndex].AsIndex()];
				check(Location.GetAllocationIndex() == AllocationIndex && Location.GetEntryIndexWithinAllocation() == EntityIndex);
			}
		}
	}

	for (int32 EntityIndex = 0; EntityIndex < EntityLocations.GetMaxIndex(); ++EntityIndex)
	{
		if (EntityLocations.IsAllocated(EntityIndex))
		{
			FEntityLocation Location = EntityLocations[EntityIndex];
			if (Location.IsValid())
			{
				const FEntityAllocation* Allocation = EntityAllocations[Location.GetAllocationIndex()];
				check(Allocation->GetEntityIDs()[Location.GetEntryIndexWithinAllocation()].AsIndex() == EntityIndex);
			}
		}
	}

#endif
}

} // namespace MovieScene
} // namespace UE
