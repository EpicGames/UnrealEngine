// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneEntityMutations.h"

namespace UE
{
namespace MovieScene
{


void FAddSingleMutation::CreateMutation(FEntityManager* EntityManager, FComponentMask* InOutEntityComponentTypes) const
{
	InOutEntityComponentTypes->Set(ComponentToAdd);
}

void FRemoveSingleMutation::CreateMutation(FEntityManager* EntityManager, FComponentMask* InOutEntityComponentTypes) const
{
	InOutEntityComponentTypes->Remove(ComponentToRemove);
}

void FAddMultipleMutation::CreateMutation(FEntityManager* EntityManager, FComponentMask* InOutEntityComponentTypes) const
{
	InOutEntityComponentTypes->CombineWithBitwiseOR(MaskToAdd, EBitwiseOperatorFlags::MaintainSize);
}

void FRemoveMultipleMutation::RemoveComponent(FComponentTypeID InComponentType)
{
	checkSlow(InComponentType);
	MaskToRemove.PadToNum(InComponentType.BitIndex() + 1, true);
	MaskToRemove.Remove(InComponentType);
}

void FRemoveMultipleMutation::CreateMutation(FEntityManager* EntityManager, FComponentMask* InOutEntityComponentTypes) const
{
	InOutEntityComponentTypes->CombineWithBitwiseAND(MaskToRemove, EBitwiseOperatorFlags::MaintainSize | EBitwiseOperatorFlags::OneFillMissingBits);
}


} // namespace MovieScene
} // namespace UE