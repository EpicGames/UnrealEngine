// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneComponentRegistry.h"
#include "EntitySystem/MovieSceneTaskScheduler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneEntitySystem)

namespace UE
{
namespace MovieScene
{

struct FSystemDependencyGraph
{
	uint16 GetGraphID(UClass* Class)
	{
		const uint16* Existing = GraphIDsByClass.Find(Class->GetFName());
		if (Existing)
		{
			return *Existing;
		}

		const int32 NewIndex = Nodes.Add(FNode::FromClass(Class));
		check(NewIndex >= 0 && NewIndex < TNumericLimits<uint16>::Max());

		const uint16 NewGraphID = static_cast<uint16>(NewIndex);

		ImplicitPrerequisites.AllocateNode(NewGraphID);
		ImplicitSubsequents.AllocateNode(NewGraphID);

		GraphIDsByClass.Add(Class->GetFName(), NewGraphID);

		FlowGraph.AllocateNode(NewGraphID);

		++SerialNumber;

		return NewGraphID;
	}

	uint16 GetGraphID(FComponentTypeID InComponentType)
	{
		const uint16* Existing = GraphIDsByComponent.Find(InComponentType);
		if (Existing)
		{
			return *Existing;
		}

		const int32 NewIndex = Nodes.Add(FNode::FromComponent(InComponentType));
		check(NewIndex >= 0 && NewIndex < TNumericLimits<uint16>::Max());

		const uint16 NewGraphID = static_cast<uint16>(NewIndex);

		ImplicitPrerequisites.AllocateNode(NewGraphID);
		ImplicitSubsequents.AllocateNode(NewGraphID);

		GraphIDsByComponent.Add(InComponentType, NewGraphID);

		FlowGraph.AllocateNode(NewGraphID);

		++SerialNumber;

		return NewGraphID;
	}

	void MakeRelationship(uint16 UpstreamGraphID, uint16 DownstreamGraphID)
	{
		const bool bSubsequentEdgeAdded = ImplicitSubsequents.MakeEdge(UpstreamGraphID, DownstreamGraphID);
		const bool bPrerequisiteEdgeAdded = ImplicitPrerequisites.MakeEdge(DownstreamGraphID, UpstreamGraphID);
		if (bSubsequentEdgeAdded || bPrerequisiteEdgeAdded)
		{
			++SerialNumber;
		}
	}

	UClass* ClassFromGraphID(uint16 GraphID) const
	{
		check(Nodes.IsValidIndex(GraphID));
		return Nodes[GraphID].Class.Get();
	}

	uint16 NumGraphIDs() const
	{
		return Nodes.Num();
	}

	void SortByFlowOrder(TArray<uint16>& InOutNodeIDs)
	{
		struct FSortableID
		{
			uint16 NodeID;
			uint16 FlowOrder;
		};

		UpdateCache();

		TArray<FSortableID, TInlineAllocator<32>> SortableNodeIDs;
		for (uint16 NodeID : InOutNodeIDs)
		{
			SortableNodeIDs.Add(FSortableID{ NodeID, ReverseLookupFlowOrderNodes[NodeID] });
		}
		Algo::SortBy(SortableNodeIDs, &FSortableID::FlowOrder);

		InOutNodeIDs.Reset();
		for (const FSortableID& SortableNodeID : SortableNodeIDs)
		{
			InOutNodeIDs.Add(SortableNodeID.NodeID);
		}
	}

	void GetSubsequents(uint16 FromNodeID, TArray<uint16>& OutSubsequentNodeIDs)
	{
		using FDirectionalEdge = FDirectedGraph::FDirectionalEdge;

		UpdateCache();

		for (FDirectionalEdge Edge : FlowGraph.GetEdgesFrom(FromNodeID))
		{
			OutSubsequentNodeIDs.Add(Edge.ToNode);
		}
	}

	void PrintFilteredFlowGraph(TArrayView<const TBitArray<>> GroupFilters, FStringBuilderBase& OutString)
	{
		using FDirectionalEdge = FDirectedGraph::FDirectionalEdge;

		FDirectedGraph::FDiscoverCyclicEdges CyclicEdges(&FlowGraph);
		CyclicEdges.Search();

		TArrayView<const FDirectionalEdge> FlowEdges = FlowGraph.GetEdges();
		for (int32 EdgeIndex = 0; EdgeIndex < FlowEdges.Num(); ++EdgeIndex)
		{
			FDirectionalEdge Edge = FlowEdges[EdgeIndex];

			for (int32 GroupIndex = 0; GroupIndex < GroupFilters.Num(); ++GroupIndex)
			{
				if (GroupFilters[GroupIndex].IsValidIndex(Edge.FromNode) && GroupFilters[GroupIndex][Edge.FromNode] && 
					GroupFilters[GroupIndex].IsValidIndex(Edge.ToNode) && GroupFilters[GroupIndex][Edge.ToNode])
				{
					const bool bIsCyclic = CyclicEdges.IsCyclic(EdgeIndex);

					OutString.Appendf(TEXT("\tflow_node%d_%d -> flow_node%d_%d [color=\"%s\"];\n"),
						(int32)Edge.FromNode, GroupIndex,
						(int32)Edge.ToNode, GroupIndex,
						bIsCyclic ? TEXT("#FF0000") : TEXT("#3992ad")
					);
				}
			}
		}
	}

	void DebugPrint(bool bUpdateCache)
	{
		static constexpr TCHAR FormatString[] =
			TEXT("----------------------------------------------------------------------------------\n")
			TEXT("%s\n")
			TEXT("----------------------------------------------------------------------------------\n");

		if (bUpdateCache)
		{
			UpdateCache();
		}

		GLog->Log(TEXT("Printing debug graph for Entity System Graph (in standard graphviz syntax):"));
		GLog->Log(FString::Printf(FormatString, *ToString()));
	}

	MOVIESCENE_API void DebugPrint()
	{
		DebugPrint(true);
	}

	FString ToString() const
	{
		using FDirectionalEdge = FDirectedGraph::FDirectionalEdge;

		FComponentRegistry* ComponentRegistry = UMovieSceneEntitySystemLinker::GetComponents();

		FString String;
		String += TEXT("\ndigraph FMovieSceneEntitySystemGraph {\n");
		String += TEXT("\tnode [shape=record,height=.1];\n");

		FString FlowStrings[] = 
		{
			TEXT("\tsubgraph cluster_flow_0 { label=\"Spawn\"; color=\"#0e868c\";\n"),
			TEXT("\tsubgraph cluster_flow_1 { label=\"Instantiation\"; color=\"#96c74c\";\n"),
			TEXT("\tsubgraph cluster_flow_2 { label=\"Evaluation\"; color=\"#6dc74c\";\n"),
			TEXT("\tsubgraph cluster_flow_3 { label=\"Finalization\"; color=\"#aa42f5\";\n"),
		};

		TMap<uint16, ESystemPhase> SystemPhasesConnectedToWriteComponents;
		for (FDirectionalEdge Edge : FlowGraph.GetEdges())
		{
			if (Nodes[Edge.FromNode].WriteComponentType)
			{
				if (UClass* SystemClass = Nodes[Edge.ToNode].Class.Get())
				{
					UMovieSceneEntitySystem* System = SystemClass->GetDefaultObject<UMovieSceneEntitySystem>();
					SystemPhasesConnectedToWriteComponents.FindOrAdd(Edge.FromNode) |= System->GetPhase();
				}
			}
			else if (Nodes[Edge.ToNode].WriteComponentType)
			{
				if (UClass* SystemClass = Nodes[Edge.FromNode].Class.Get())
				{
					UMovieSceneEntitySystem* System = SystemClass->GetDefaultObject<UMovieSceneEntitySystem>();
					SystemPhasesConnectedToWriteComponents.FindOrAdd(Edge.ToNode) |= System->GetPhase();
				}
			}
		}

		for (int32 NodeID = 0; NodeID < Nodes.Num(); ++NodeID)
		{
			const FNode& Node(Nodes[NodeID]);
			if (UClass* SystemClass = Node.Class.Get())
			{
				UMovieSceneEntitySystem* DefaultSystem = SystemClass->GetDefaultObject<UMovieSceneEntitySystem>();

				ESystemPhase SystemPhase = DefaultSystem->GetPhase();

				if (EnumHasAnyFlags(SystemPhase, ESystemPhase::Spawn))
				{
					FlowStrings[0] += FString::Printf(TEXT("\t\tflow_node%d_0[label=\"%s\"];\n"), NodeID, *SystemClass->GetName());
				}
				if (EnumHasAnyFlags(SystemPhase, ESystemPhase::Instantiation))
				{
					FlowStrings[1] += FString::Printf(TEXT("\t\tflow_node%d_1[label=\"%s\"];\n"), NodeID, *SystemClass->GetName());
				}
				if (EnumHasAnyFlags(SystemPhase, ESystemPhase::Evaluation | ESystemPhase::Scheduling))
				{
					FlowStrings[2] += FString::Printf(TEXT("\t\tflow_node%d_2[label=\"%s\"];\n"), NodeID, *SystemClass->GetName());
				}
				if (EnumHasAnyFlags(SystemPhase, ESystemPhase::Finalization))
				{
					FlowStrings[3] += FString::Printf(TEXT("\t\tflow_node%d_3[label=\"%s\"];\n"), NodeID, *SystemClass->GetName());
				}
			}
			else if(Node.WriteComponentType)
			{
				const FComponentTypeInfo& WriteComponentInfo = ComponentRegistry->GetComponentTypeChecked(Node.WriteComponentType);

				ESystemPhase* ConnectedSystemPhase = SystemPhasesConnectedToWriteComponents.Find(NodeID);
				if (ConnectedSystemPhase)
				{
					if (EnumHasAnyFlags(*ConnectedSystemPhase, ESystemPhase::Spawn))
					{
#if UE_MOVIESCENE_ENTITY_DEBUG
						FlowStrings[0] += FString::Printf(TEXT("\t\tflow_node%d_0[label=\"%s\"];\n"), NodeID, *WriteComponentInfo.DebugInfo->DebugName);
#else
						FlowStrings[0] += FString::Printf(TEXT("\t\tflow_node%d_0[label=\"Component ID=%d\"];\n"), NodeID, Node.WriteComponentType.BitIndex());
#endif
					}
					if (EnumHasAnyFlags(*ConnectedSystemPhase, ESystemPhase::Instantiation))
					{
#if UE_MOVIESCENE_ENTITY_DEBUG
						FlowStrings[1] += FString::Printf(TEXT("\t\tflow_node%d_1[label=\"%s\"];\n"), NodeID, *WriteComponentInfo.DebugInfo->DebugName);
#else
						FlowStrings[1] += FString::Printf(TEXT("\t\tflow_node%d_1[label=\"Component ID=%d\"];\n"), NodeID, Node.WriteComponentType.BitIndex());
#endif
					}
					if (EnumHasAnyFlags(*ConnectedSystemPhase, ESystemPhase::Evaluation | ESystemPhase::Scheduling))
					{
#if UE_MOVIESCENE_ENTITY_DEBUG
						FlowStrings[2] += FString::Printf(TEXT("\t\tflow_node%d_2[label=\"%s\"];\n"), NodeID, *WriteComponentInfo.DebugInfo->DebugName);
#else
						FlowStrings[2] += FString::Printf(TEXT("\t\tflow_node%d_2[label=\"Component ID=%d\"];\n"), NodeID, Node.WriteComponentType.BitIndex());
#endif
					}
					if (EnumHasAnyFlags(*ConnectedSystemPhase, ESystemPhase::Finalization))
					{
#if UE_MOVIESCENE_ENTITY_DEBUG
						FlowStrings[3] += FString::Printf(TEXT("\t\tflow_node%d_3[label=\"%s\"];\n"), NodeID, *WriteComponentInfo.DebugInfo->DebugName);
#else
						FlowStrings[3] += FString::Printf(TEXT("\t\tflow_node%d_3[label=\"Component ID=%d\"];\n"), NodeID, Node.WriteComponentType.BitIndex());
#endif
					}
				}
			}
		}

		for (FString& FlowString : FlowStrings)
		{
			String += FlowString;
			String += TEXT("\t}\n");
		}

		{
			FDirectedGraph::FDiscoverCyclicEdges CyclicEdges(&FlowGraph);
			CyclicEdges.Search();

			TArrayView<const FDirectionalEdge> FlowEdges = FlowGraph.GetEdges();
			for (int32 EdgeIndex = 0; EdgeIndex < FlowEdges.Num(); ++EdgeIndex)
			{
				FDirectionalEdge Edge = FlowEdges[EdgeIndex];
				const bool bIsCyclic = CyclicEdges.IsCyclic(EdgeIndex);
				const FString EdgeColor = bIsCyclic ? TEXT("#FF0000") : TEXT("#39ad3b");

				ESystemPhase FromPhase = ESystemPhase::None;
				if (UClass* FromClass = Nodes[Edge.FromNode].Class.Get())
				{
					FromPhase = FromClass->GetDefaultObject<UMovieSceneEntitySystem>()->GetPhase();
				}

				ESystemPhase ToPhase = ESystemPhase::None;
				if (UClass* ToClass = Nodes[Edge.ToNode].Class.Get())
				{
					ToPhase = ToClass->GetDefaultObject<UMovieSceneEntitySystem>()->GetPhase();
				}

				if ((FromPhase == ESystemPhase::None || EnumHasAnyFlags(FromPhase, ESystemPhase::Spawn)) && 
						(ToPhase == ESystemPhase::None || EnumHasAnyFlags(ToPhase, ESystemPhase::Spawn)))
				{
					String += FString::Printf(TEXT("\tflow_node%d_0 -> flow_node%d_0 [color=\"%s\"];\n"), (int32)Edge.FromNode, (int32)Edge.ToNode, *EdgeColor);
				}
				if ((FromPhase == ESystemPhase::None || EnumHasAnyFlags(FromPhase, ESystemPhase::Instantiation)) && 
						(ToPhase == ESystemPhase::None || EnumHasAnyFlags(ToPhase, ESystemPhase::Instantiation)))
				{
					String += FString::Printf(TEXT("\tflow_node%d_1 -> flow_node%d_1 [color=\"%s\"];\n"), (int32)Edge.FromNode, (int32)Edge.ToNode, *EdgeColor);
				}
				if ((FromPhase == ESystemPhase::None || EnumHasAnyFlags(FromPhase, ESystemPhase::Evaluation | ESystemPhase::Scheduling)) && 
						(ToPhase == ESystemPhase::None || EnumHasAnyFlags(ToPhase, ESystemPhase::Evaluation | ESystemPhase::Scheduling)))
				{
					String += FString::Printf(TEXT("\tflow_node%d_2 -> flow_node%d_2 [color=\"%s\"];\n"), (int32)Edge.FromNode, (int32)Edge.ToNode, *EdgeColor);
				}
				if ((FromPhase == ESystemPhase::None || EnumHasAnyFlags(FromPhase, ESystemPhase::Finalization)) && 
						(ToPhase == ESystemPhase::None || EnumHasAnyFlags(ToPhase, ESystemPhase::Finalization)))
				{
					String += FString::Printf(TEXT("\tflow_node%d_3 -> flow_node%d_3 [color=\"%s\"];\n"), (int32)Edge.FromNode, (int32)Edge.ToNode, *EdgeColor);
				}
			}
		}

		String += TEXT("}");
		return String;
	}

	void UpdateCache()
	{
		if (PreviousSerialNumber == SerialNumber)
		{
			return;
		}

		BuildFlowGraph();

		FDirectedGraph::FDepthFirstSearch DepthFirstSearch(&FlowGraph);

		TBitArray<> EdgeNodes = FlowGraph.FindEdgeUpstreamNodes();
		for (TConstSetBitIterator<> EdgeNodeIt(EdgeNodes); EdgeNodeIt; ++EdgeNodeIt)
		{
			const uint16 NodeID = static_cast<uint16>(EdgeNodeIt.GetIndex());
			check(Nodes.IsValidIndex(NodeID));

			DepthFirstSearch.Search(NodeID);
		}

		Algo::Reverse(DepthFirstSearch.PostNodes);

		// FlowOrderNodes has the NodeIDs sorted by dependencies.
		FlowOrderNodes = DepthFirstSearch.PostNodes;

		// Build a reverse lookup array for the flow order, i.e. an array that gives the flow order index
		// of a system given that system's NodeID.
		ReverseLookupFlowOrderNodes.SetNum(Nodes.Num());
		for (int32 Index = 0; Index < FlowOrderNodes.Num(); ++Index)
		{
			ReverseLookupFlowOrderNodes[FlowOrderNodes[Index]] = Index;
		}

		PreviousSerialNumber = SerialNumber;
	}

private:

	void BuildFlowGraph()
	{
		FlowGraph.DestroyAllEdges();

		for (int32 NodeIndex = 0, NodeCount = Nodes.Num(); NodeIndex < NodeCount; ++NodeIndex)
		{
			SetupFlowDependencies(NodeIndex);
		}

		FlowGraph.CleanUpDanglingEdges();

		checkf(!FlowGraph.IsCyclic(), TEXT("Cycle detected in system flow graph! \n\n %s"), *ToString());
	}

	void SetupFlowDependencies(int32 NodeIndex)
	{
		using FDirectionalEdge = FDirectedGraph::FDirectionalEdge;

		const uint16 CurrentNodeID(static_cast<uint16>(NodeIndex));

		// Set up prerequisites
		for (FDirectionalEdge Edge : ImplicitPrerequisites.GetEdgesFrom(CurrentNodeID))
		{
			const FNode& Node = Nodes[Edge.ToNode];

			// Follow edges from components
			if (Node.WriteComponentType)
			{
				for (FDirectionalEdge ComponentEdge : ImplicitPrerequisites.GetEdgesFrom(Edge.ToNode))
				{
					// Components shouldn't be connected to other components
					SetupFlowDependency(ComponentEdge.ToNode, CurrentNodeID);
				}
			}
			else
			{
				SetupFlowDependency(Edge.ToNode, CurrentNodeID);
			}
		}

		// Set up subsequents
		for (FDirectionalEdge Edge : ImplicitSubsequents.GetEdgesFrom(CurrentNodeID))
		{
			const FNode& Node = Nodes[Edge.ToNode];

			// Follow edges from components
			if (Node.WriteComponentType)
			{
				for (FDirectionalEdge ComponentEdge : ImplicitSubsequents.GetEdgesFrom(Edge.ToNode))
				{
					// Components shouldn't be connected to other components
					SetupFlowDependency(CurrentNodeID, ComponentEdge.ToNode);
				}
			}
			else
			{
				SetupFlowDependency(CurrentNodeID, Edge.ToNode);
			}
		}
	}

	void SetupFlowDependency(uint16 UpstreamID, uint16 DownstreamID)
	{
		check(UpstreamID != TNumericLimits<uint16>::Max() && DownstreamID != TNumericLimits<uint16>::Max());

		FlowGraph.MakeEdge(UpstreamID, DownstreamID);
	}

private:

	TMap<FName, uint16> GraphIDsByClass;
	TMap<FComponentTypeID, uint16> GraphIDsByComponent;

	struct FNode
	{
		static FNode FromClass(UClass* InClass)
		{
			return FNode{ InClass, FComponentTypeID::Invalid() };
		}
		static FNode FromComponent(FComponentTypeID InComponent)
		{
			return FNode{ nullptr, InComponent };
		}

		TWeakObjectPtr<UClass> Class;
		FComponentTypeID WriteComponentType;
	};

	TArray<FNode> Nodes;

	FDirectedGraph ImplicitPrerequisites;
	FDirectedGraph ImplicitSubsequents;

	FDirectedGraph FlowGraph;
	TArray<uint16> FlowOrderNodes;
	TArray<uint16> ReverseLookupFlowOrderNodes;

	uint32 SerialNumber = 0;
	uint32 PreviousSerialNumber = 0;
};

FSystemDependencyGraph GlobalDependencyGraph;

TMap<FName, TStatId> SystemStats;

} // namespace MovieScene
} // namespace UE

UMovieSceneEntitySystem::UMovieSceneEntitySystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	bSystemIsEnabled = true;
	SystemCategories = EEntitySystemCategory::Unspecified;

	Phase = ESystemPhase::Evaluation;
	GraphID = TNumericLimits<uint16>::Max();

	if (!GetClass()->HasAnyClassFlags(CLASS_Abstract))
	{
		GlobalDependencyGraphID = GlobalDependencyGraph.GetGraphID(GetClass());
	}
	else
	{
		GlobalDependencyGraphID = MAX_uint16;
	}

#if STATS || ENABLE_STATNAMEDEVENTS
	const TStatId* ExistingStat = SystemStats.Find(GetClass()->GetFName());

	if (ExistingStat)
	{
		StatID = *ExistingStat;
	}
	else
	{
#if STATS
		TStatId NewStatID = FDynamicStats::CreateStatId<STAT_GROUP_TO_FStatGroup(STATGROUP_MovieSceneECS)>(GetClass()->GetName());
#else
		// Just use the base UObject stat ID if we only have named events
		TStatId NewStatID = GetStatID(true /* bForDeferredUse */);
#endif

		StatID = NewStatID;
		SystemStats.Add(GetClass()->GetFName(), NewStatID);
	}
#endif
}

UMovieSceneEntitySystem::~UMovieSceneEntitySystem()
{
}

void UMovieSceneEntitySystem::StaticInitialize()
{
	using namespace UE::MovieScene;

	// Pre-emptively call UpdateCache() on the global dependency graph to prevent doing it at a critical time.
	// In the event that a plugin/module that mutates this graph loads after this initialization, 
	// the cache will be invalidated and will be updated when the first sequence is played.
	GlobalDependencyGraph.UpdateCache();
}

void UMovieSceneEntitySystem::DefineImplicitPrerequisite(TSubclassOf<UMovieSceneEntitySystem> UpstreamSystemType, TSubclassOf<UMovieSceneEntitySystem> DownstreamSystemType)
{
	using namespace UE::MovieScene;

	const uint16 UpstreamGlobalDependencyGraphID   = GlobalDependencyGraph.GetGraphID(UpstreamSystemType.Get());
	const uint16 DownstreamGlobalDependencyGraphID = GlobalDependencyGraph.GetGraphID(DownstreamSystemType.Get());

	GlobalDependencyGraph.MakeRelationship(UpstreamGlobalDependencyGraphID, DownstreamGlobalDependencyGraphID);
}

void UMovieSceneEntitySystem::DefineComponentProducer(TSubclassOf<UMovieSceneEntitySystem> ThisClassType, FComponentTypeID ComponentType)
{
	using namespace UE::MovieScene;

	const uint16 UpstreamGlobalDependencyGraphID   = GlobalDependencyGraph.GetGraphID(ThisClassType.Get());
	const uint16 DownstreamGlobalDependencyGraphID = GlobalDependencyGraph.GetGraphID(ComponentType);

	GlobalDependencyGraph.MakeRelationship(UpstreamGlobalDependencyGraphID, DownstreamGlobalDependencyGraphID);
}

void UMovieSceneEntitySystem::DefineComponentConsumer(TSubclassOf<UMovieSceneEntitySystem> ThisClassType, FComponentTypeID ComponentType)
{
	using namespace UE::MovieScene;

	const uint16 UpstreamGlobalDependencyGraphID   = GlobalDependencyGraph.GetGraphID(ComponentType);
	const uint16 DownstreamGlobalDependencyGraphID = GlobalDependencyGraph.GetGraphID(ThisClassType.Get());

	GlobalDependencyGraph.MakeRelationship(UpstreamGlobalDependencyGraphID, DownstreamGlobalDependencyGraphID);
}

void UMovieSceneEntitySystem::LinkRelevantSystems(UMovieSceneEntitySystemLinker* InLinker)
{
	using namespace UE::MovieScene;

	for (uint16 GraphID = 0; GraphID < GlobalDependencyGraph.NumGraphIDs(); ++GraphID)
	{
		if (InLinker->HasLinkedSystem(GraphID))
		{
			continue;
		}

		UClass* Class = GlobalDependencyGraph.ClassFromGraphID(GraphID);
		UMovieSceneEntitySystem* SystemCDO = Class ? Cast<UMovieSceneEntitySystem>(Class->GetDefaultObject()) : nullptr;

		if (SystemCDO && InLinker->GetSystemFilter().CheckSystem(SystemCDO))
		{
			SystemCDO->ConditionalLinkSystem(InLinker);
		}
	}
}

void UMovieSceneEntitySystem::LinkCategorySystems(UMovieSceneEntitySystemLinker* InLinker, UE::MovieScene::EEntitySystemCategory InCategory)
{
	using namespace UE::MovieScene;

	for (uint16 GraphID = 0; GraphID < GlobalDependencyGraph.NumGraphIDs(); ++GraphID)
	{
		if (InLinker->HasLinkedSystem(GraphID))
		{
			continue;
		}

		UClass* Class = GlobalDependencyGraph.ClassFromGraphID(GraphID);
		if (Class == UMovieSceneEntitySystem::StaticClass())
		{
			// Ignore the base class
			continue;
		}

		UMovieSceneEntitySystem* SystemCDO = Class ? Cast<UMovieSceneEntitySystem>(Class->GetDefaultObject()) : nullptr;
		if (SystemCDO && EnumHasAnyFlags(SystemCDO->SystemCategories, InCategory))
		{
			InLinker->LinkSystemIfAllowed(Class);
		}
	}
}

void UMovieSceneEntitySystem::LinkAllSystems(UMovieSceneEntitySystemLinker* InLinker)
{
	using namespace UE::MovieScene;

	for (uint16 GraphID = 0; GraphID < GlobalDependencyGraph.NumGraphIDs(); ++GraphID)
	{
		if (InLinker->HasLinkedSystem(GraphID))
		{
			continue;
		}

		UClass* Class = GlobalDependencyGraph.ClassFromGraphID(GraphID);
		// Ignore the base class
		if (Class && Class != UMovieSceneEntitySystem::StaticClass())
		{
			InLinker->LinkSystemIfAllowed(Class);
		}
	}
}

UE::MovieScene::EEntitySystemCategory UMovieSceneEntitySystem::RegisterCustomSystemCategory()
{
	using namespace UE::MovieScene;

	static EEntitySystemCategory NextCustomCategory = EEntitySystemCategory::Custom;

	EEntitySystemCategory Result = NextCustomCategory;
	NextCustomCategory = (EEntitySystemCategory)((uint32)NextCustomCategory << 1);
	check(NextCustomCategory != EEntitySystemCategory::Last);
	return Result;
}

bool UMovieSceneEntitySystem::IsCustomSchedulingEnabled()
{
	return UE::MovieScene::FEntitySystemScheduler::IsCustomSchedulingEnabled();
}

void UMovieSceneEntitySystem::SortByFlowOrder(TArray<uint16>& InOutGlobalNodeIDs)
{
	using namespace UE::MovieScene;

	GlobalDependencyGraph.SortByFlowOrder(InOutGlobalNodeIDs);
}

void UMovieSceneEntitySystem::GetSubsequentSystems(uint16 FromGlobalNodeID, TArray<uint16>& OutSubsequentGlobalNodeIDs)
{
	using namespace UE::MovieScene;

	GlobalDependencyGraph.GetSubsequents(FromGlobalNodeID, OutSubsequentGlobalNodeIDs);
}

void UMovieSceneEntitySystem::PrintFilteredFlowGraph(TArrayView<const TBitArray<>> GroupFilters, FStringBuilderBase& OutString)
{
	using namespace UE::MovieScene;

	GlobalDependencyGraph.PrintFilteredFlowGraph(GroupFilters, OutString);
}

void UMovieSceneEntitySystem::DebugPrintGlobalDependencyGraph(bool bUpdateCache)
{
	using namespace UE::MovieScene;

	GlobalDependencyGraph.DebugPrint(bUpdateCache);
}

bool UMovieSceneEntitySystem::IsRelevant(UMovieSceneEntitySystemLinker* InLinker) const
{
	if (RelevantComponent && InLinker->EntityManager.ContainsComponent(RelevantComponent))
	{
		return true;
	}

	return IsRelevantImpl(InLinker);
}

bool UMovieSceneEntitySystem::IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	return false;
}

void UMovieSceneEntitySystem::ConditionalLinkSystem(UMovieSceneEntitySystemLinker* InLinker) const
{
	check(HasAnyFlags(RF_ClassDefaultObject));
	ConditionalLinkSystemImpl(InLinker);
}

void UMovieSceneEntitySystem::ConditionalLinkSystemImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	if (IsRelevant(InLinker))
	{
		InLinker->LinkSystem(GetClass());
	}
}

void UMovieSceneEntitySystem::Enable()
{
	bSystemIsEnabled = true;
}

void UMovieSceneEntitySystem::Disable()
{
	bSystemIsEnabled = false;
}

void UMovieSceneEntitySystem::TagGarbage()
{
	OnTagGarbage();
}

void UMovieSceneEntitySystem::CleanTaggedGarbage()
{
	OnCleanTaggedGarbage();
}

bool UMovieSceneEntitySystem::IsReadyForFinishDestroy()
{
	return Linker == nullptr;
}

void UMovieSceneEntitySystem::Abandon()
{
	Linker = nullptr;
	GraphID = TNumericLimits<uint16>::Max();
}

void UMovieSceneEntitySystem::FinishDestroy()
{
	checkf(!Linker, TEXT("System being destroyed without Unlink being called"));

	Super::FinishDestroy();
}

void UMovieSceneEntitySystem::SchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	if (!bSystemIsEnabled)
	{
		return;
	}

	checkf(Linker != nullptr, TEXT("Attempting to evaluate a system that has been unlinked!"));

	// We may have erroneously linked a system we should have done, but we must not run it in this case
	if (!Linker->GetSystemFilter().CheckSystem(this))
	{
		return;
	}

	Linker->EntityManager.IncrementSystemSerial();
	OnSchedulePersistentTasks(TaskScheduler);
}

void UMovieSceneEntitySystem::Run(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	if (!bSystemIsEnabled)
	{
		return;
	}

#if STATS || ENABLE_STATNAMEDEVENTS
	FScopeCycleCounter Scope(StatID);
#endif

	checkf(Linker != nullptr, TEXT("Attempting to evaluate a system that has been unlinked!"));

	// We may have erroneously linked a system we should have done, but we must not run it in this case
	if (!Linker->GetSystemFilter().CheckSystem(this))
	{
		return;
	}

	Linker->EntityManager.IncrementSystemSerial();

	UE_LOGF(LogMovieSceneECS, Verbose, "Running moviescene system for phase %d: %ls", (int32)Phase, *GetName());
	OnRun(InPrerequisites, Subsequents);
}

void UMovieSceneEntitySystem::Link(UMovieSceneEntitySystemLinker* InLinker)
{
	using namespace UE::MovieScene;

	check(GraphID != TNumericLimits<uint16>::Max());

	Linker = InLinker;
	Linker->SystemLinked(this);

	OnLink();
}

void UMovieSceneEntitySystem::Unlink()
{
	if (GraphID != TNumericLimits<uint16>::Max())
	{
		Linker->SystemGraph.RemoveSystem(this);
	}

	OnUnlink();

	Linker->SystemUnlinked(this);
	Linker = nullptr;
}

