// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneEntitySystemDirectedGraph.h"
#include "Containers/UnrealString.h"
#include "Misc/StringBuilder.h"

//#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneEntitySystemDirectedGraph)

namespace UE::MovieScene
{

FDirectedGraph::FDepthFirstSearch::FDepthFirstSearch(const FDirectedGraph* InGraph)
	: Visited(false, InGraph->Nodes.Num())
	, IsVisiting(false, InGraph->Nodes.Num())
{
	Graph = InGraph;
	PostNodes.Reserve(InGraph->Nodes.CountSetBits());
	check(!Graph->bHasDanglingEdges);
}

void FDirectedGraph::FDepthFirstSearch::Search(uint16 InNodeID)
{
	IsVisiting[InNodeID] = true;

	for (FDirectedGraph::FDirectionalEdge Edge : Graph->GetEdgesFrom(InNodeID))
	{
		if (Visited[Edge.ToNode] == false)
		{
			if (!ensureMsgf(IsVisiting[Edge.ToNode] == false, TEXT("Cycle found in graph.")))
			{
				return;
			}
			Visited[Edge.ToNode] = true;
			Search(Edge.ToNode);
		}
	}

	PostNodes.Add(InNodeID);

	IsVisiting[InNodeID] = false;
}

FDirectedGraph::FBreadthFirstSearch::FBreadthFirstSearch(const FDirectedGraph* InGraph)
	: Visited(false, InGraph->Nodes.Num())
	, StackIndex(0)
{
	Graph = InGraph;
	Nodes.Reserve(InGraph->Nodes.CountSetBits());
	check(!InGraph->bHasDanglingEdges);
}

void FDirectedGraph::FBreadthFirstSearch::Search(uint16 InNodeID)
{
	if (Visited[InNodeID] == true)
	{
		return;
	}

	Nodes.Reset();
	StackIndex = 0;

	Visited[InNodeID] = true;
	Nodes.Add(InNodeID);

	while (StackIndex < Nodes.Num())
	{
		const int32 StackEnd = Nodes.Num();
		for ( ; StackIndex < StackEnd; ++StackIndex)
		{
			const uint16 NodeID = Nodes[StackIndex];

			// Visit all nodes this points to
			for (FDirectedGraph::FDirectionalEdge Edge : Graph->GetEdgesFrom(NodeID))
			{
				if (Visited[Edge.ToNode] == false)
				{
					Visited[Edge.ToNode] = true;
					Nodes.Add(Edge.ToNode);
				}
			}
		}
	}
}

FDirectedGraph::FDiscoverCyclicEdges::FDiscoverCyclicEdges(const FDirectedGraph* InGraph)
	: CyclicEdges(false, InGraph->SortedEdges.Num())
	, VisitedEdges(false, InGraph->SortedEdges.Num())
{
	Graph = InGraph;
	check(!Graph->bHasDanglingEdges);
}

void FDirectedGraph::FDiscoverCyclicEdges::Search()
{
	for (uint16 EdgeIndex = 0; EdgeIndex < Graph->SortedEdges.Num(); ++EdgeIndex)
	{
		if (VisitedEdges[EdgeIndex] == false)
		{
			EdgeChain.Reset();
			EdgeChain.Add(EdgeIndex);

			SearchFrom(Graph->SortedEdges[EdgeIndex].FromNode);

			VisitedEdges[EdgeIndex] = true;
		}
	}
}

void FDirectedGraph::FDiscoverCyclicEdges::SearchFrom(uint16 NodeID)
{
	TBitArray<> VisitedNodes(false, Graph->Nodes.Num());
	VisitedNodes[NodeID] = true;
	DiscoverCycles(NodeID, VisitedNodes);
}

void FDirectedGraph::FDiscoverCyclicEdges::DiscoverCycles(uint16 NodeID, TBitArray<>& VisitedNodes)
{
	// Iterate all edges from this node
	for (int32 SubsequentEdge = Graph->FindEdgeStart(NodeID); SubsequentEdge < Graph->SortedEdges.Num() && Graph->SortedEdges[SubsequentEdge].FromNode == NodeID; ++SubsequentEdge)
	{
		if (VisitedEdges[SubsequentEdge] == true)
		{
			continue;
		}

		VisitedEdges[SubsequentEdge] = true;
		EdgeChain.Add(SubsequentEdge);

		const uint16 SubsequentNode = Graph->SortedEdges[SubsequentEdge].ToNode;
		if (VisitedNodes[SubsequentNode] == true)
		{
			TagCyclicChain(SubsequentNode);
		}
		else
		{
			VisitedNodes[SubsequentNode] = true;
			DiscoverCycles(SubsequentNode, VisitedNodes);
			VisitedNodes[SubsequentNode] = false;
		}

		EdgeChain.Pop();
	}
}

void FDirectedGraph::FDiscoverCyclicEdges::TagCyclicChain(uint16 CyclicNodeID)
{
	// Found a cycle
	for (int32 EdgeChainIndex = EdgeChain.Num() - 1; EdgeChainIndex >= 0; --EdgeChainIndex)
	{
		const uint16 UpstreamEdgeIndex = EdgeChain[EdgeChainIndex];
		CyclicEdges.PadToNum(UpstreamEdgeIndex + 1, false);
		CyclicEdges[UpstreamEdgeIndex] = true;

		if (Graph->SortedEdges[UpstreamEdgeIndex].FromNode == CyclicNodeID)
		{
			return;
		}
	}
}

void FDirectedGraph::AllocateNode(uint16 NodeID)
{
	CleanUpDanglingEdges();
	Nodes.PadToNum(NodeID + 1, false);
	Nodes[NodeID] = true;
}

bool FDirectedGraph::IsNodeAllocated(uint16 NodeID) const
{
	return Nodes.IsValidIndex(NodeID) && Nodes[NodeID] == true;
}

void FDirectedGraph::RemoveNode(uint16 NodeID)
{
	check(NodeID != TNumericLimits<uint16>::Max() && IsNodeAllocated(NodeID));

	// Remove the node from the graph
	Nodes[NodeID] = false;

	bHasDanglingEdges = true;
}

void FDirectedGraph::CleanUpDanglingEdges()
{
	if (!bHasDanglingEdges)
	{
		return;
	}

	bHasDanglingEdges = false;
	for (int32 Index = 0; Index < SortedEdges.Num(); )
	{
		FDirectionalEdge Edge = SortedEdges[Index];
		if (!IsNodeAllocated(Edge.ToNode) || !IsNodeAllocated(Edge.FromNode))
		{
			SortedEdges.RemoveAt(Index, EAllowShrinking::No);
		}
		else
		{
			++Index;
		}
	}
}

bool FDirectedGraph::IsCyclic() const
{
	TBitArray<> Visited(false, Nodes.Num());

	for (FDirectionalEdge Edge : SortedEdges)
	{
		if (Visited[Edge.ToNode] == true)
		{
			continue;
		}

		TBitArray<> Visiting(false, Nodes.Num());
		if (IsCyclicImpl(Edge.ToNode, Visiting))
		{
			return true;
		}

		Visited.CombineWithBitwiseOR(Visiting, EBitwiseOperatorFlags::MaxSize);
	}

	return false;
}

bool FDirectedGraph::IsCyclicImpl(uint16 NodeID, TBitArray<>& Visiting) const
{
	if (Visiting[NodeID] == true)
	{
		return true;
	}

	Visiting[NodeID] = true;

	for (FDirectionalEdge Edge : GetEdgesFrom(NodeID))
	{
		if (IsCyclicImpl(Edge.ToNode, Visiting))
		{
			return true;
		}
	}

	Visiting[NodeID] = false;
	return false;
}

bool FDirectedGraph::MakeEdge(uint16 FromNode, uint16 ToNode)
{
	FDirectionalEdge NewEdge(FromNode, ToNode);

	const int32 InsertIndex = FindEdgeIndex(NewEdge);
	if (!SortedEdges.IsValidIndex(InsertIndex) || SortedEdges[InsertIndex] != NewEdge)
	{
		SortedEdges.Insert(NewEdge, InsertIndex);
		return true;
	}
	return false;
}

void FDirectedGraph::DestroyEdge(uint16 FromNode, uint16 ToNode)
{
	FDirectionalEdge Edge(FromNode, ToNode);

	const int32 RemoveIndex = Algo::BinarySearch(SortedEdges, Edge);
	if (RemoveIndex != INDEX_NONE)
	{
		SortedEdges.RemoveAt(RemoveIndex, EAllowShrinking::No);
	}
}

void FDirectedGraph::DestroyAllEdges()
{
	SortedEdges.Reset();
	bHasDanglingEdges = false;
}

int32 FDirectedGraph::FindEdgeStart(uint16 FromNode) const
{
	return Algo::LowerBoundBy(SortedEdges, FromNode, &FDirectionalEdge::FromNode);
}

TArrayView<const FDirectedGraph::FDirectionalEdge> FDirectedGraph::GetEdges() const
{
	check(!bHasDanglingEdges);
	return SortedEdges;
}

bool FDirectedGraph::HasEdgeFrom(uint16 InNode) const
{
	const int32 ExpectedIndex = FindEdgeStart(InNode);
	return SortedEdges.IsValidIndex(ExpectedIndex) && SortedEdges[ExpectedIndex].FromNode == InNode;
}

bool FDirectedGraph::HasEdgeTo(uint16 InNode) const
{
	check(InNode != TNumericLimits<uint16>::Max());
	return Algo::FindBy(SortedEdges, InNode, &FDirectionalEdge::ToNode) != nullptr;
}

TArrayView<const FDirectedGraph::FDirectionalEdge> FDirectedGraph::GetEdgesFrom(uint16 InNodeID) const
{
	check(!bHasDanglingEdges);

	const int32 EdgeIndex = FindEdgeStart(InNodeID);

	int32 Num = 0;
	while (EdgeIndex + Num < SortedEdges.Num() && SortedEdges[EdgeIndex + Num].FromNode == InNodeID)
	{
		++Num;
	}

	if (Num > 0)
	{
		return MakeArrayView(SortedEdges.GetData() + EdgeIndex, Num);
	}
	return TArrayView<const FDirectionalEdge>();
}

TBitArray<> FDirectedGraph::FindEdgeUpstreamNodes() const
{
	check(!bHasDanglingEdges);

	TBitArray<> EdgeNodes(true, Nodes.Num());

	// Unmark nodes that have edges pointing towards them
	for (uint16 EdgeIndex = 0; EdgeIndex < SortedEdges.Num(); ++EdgeIndex)
	{
		const uint16 ToNode = SortedEdges[EdgeIndex].ToNode;
		EdgeNodes[ToNode] = false;
	}

	// Mask with nodes that are actually allocated
	return TBitArray<>::BitwiseAND(EdgeNodes, Nodes, EBitwiseOperatorFlags::MaxSize);
}

int32 FDirectedGraph::FindEdgeIndex(const FDirectionalEdge& Edge) const
{
	check(!bHasDanglingEdges);
	return Algo::LowerBound(SortedEdges, Edge);
}

bool FDirectedGraph::EdgeExists(const FDirectionalEdge& Edge) const
{
	check(!bHasDanglingEdges);

	const int32 EdgeIndex = FindEdgeIndex(Edge);
	return EdgeIndex < SortedEdges.Num() && SortedEdges[EdgeIndex] == Edge;
}

FString FDirectedGraph::ToString(const FDirectedGraphStringParameters& Parameters, TFunctionRef<void(uint16, FStringBuilderBase&)> EmitLabel) const
{
	TStringBuilder<256> String;

	String += TEXT("\ndigraph FDirectedGraph {\n");
	String += TEXT("\tnode [shape=record,height=.1];\n");
	String += TEXT("\tsubgraph cluster_flow_0\n\t{\n");
	if (Parameters.ClusterName.Len() != 0)
	{
		String.Append(TEXT("\tlabel=\""));
		String.Append(Parameters.ClusterName);
		String.Append(TEXT("\";\n"));
	}
	if (Parameters.Color != FColor::White)
	{
		String.Appendf(TEXT("\tcolor=\"%s\";\n"), *Parameters.Color.ToHex());
	}
	String += TEXT("\t}\n");

	// Add the nodes to the graph
	for (TConstSetBitIterator<> SetBitIt(Nodes); SetBitIt; ++SetBitIt)
	{
		const uint16 NodeID = static_cast<uint16>(SetBitIt.GetIndex());

		String += TEXT("\t\tnode%d[label=\"");
		EmitLabel(NodeID, String);
		String += TEXT("\"];\n");
	}

	String += TEXT("\t}\n\n");

	// Make edges
	{
		FDirectedGraph::FDiscoverCyclicEdges CyclicEdges(this);
		CyclicEdges.Search();

		TArrayView<const FDirectedGraph::FDirectionalEdge> Edges = GetEdges();
		for (int32 EdgeIndex = 0; EdgeIndex < Edges.Num(); ++EdgeIndex)
		{
			FDirectedGraph::FDirectionalEdge Edge = Edges[EdgeIndex];
			const bool bIsCyclic = CyclicEdges.IsCyclic(EdgeIndex);

			String += FString::Printf(TEXT("\tnode%d -> node%d [color=\"%s\"];\n"), (int32)Edge.FromNode, (int32)Edge.ToNode, bIsCyclic ? TEXT("#FF0000") : TEXT("#3992ad"));
		}
	}

	String += TEXT("}\n\n");
	return String.ToString();
}

FString FDirectedGraph::ToString(const FDirectedGraphStringParameters& Parameters) const
{
	return ToString(Parameters, [](uint16 NodeID, FStringBuilderBase& OutStringBuilder){
		OutStringBuilder.Appendf(TEXT("Node %d"), NodeID);
	});
}

} // namespace UE::MovieScene