// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneEntitySystemGraphs.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneSystemTaskDependencies.h"
#include "EntitySystem/MovieSceneTaskScheduler.h"

#include "Templates/SubclassOf.h"
#include "Algo/IndexOf.h"
#include "Algo/Reverse.h"
#include "Algo/RemoveIf.h"
#include "Algo/BinarySearch.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneEntitySystemGraphs)

void FMovieSceneEntitySystemGraphNodes::AddStructReferencedObjects(FReferenceCollector& Collector) const
{
	for (FMovieSceneEntitySystemGraphNode& Node : const_cast<TSparseArray<FMovieSceneEntitySystemGraphNode>&>(Array))
	{
		Collector.AddReferencedObject(Node.System);
	}
}

FMovieSceneEntitySystemGraph::FMovieSceneEntitySystemGraph() = default;
FMovieSceneEntitySystemGraph::~FMovieSceneEntitySystemGraph() = default;

FMovieSceneEntitySystemGraph::FMovieSceneEntitySystemGraph(FMovieSceneEntitySystemGraph&&) = default;
FMovieSceneEntitySystemGraph& FMovieSceneEntitySystemGraph::operator=(FMovieSceneEntitySystemGraph&&) = default;

void FMovieSceneEntitySystemGraph::AddSystem(UMovieSceneEntitySystem* InSystem)
{
	checkf(ReentrancyGuard == 0, TEXT("Attempting to add a system to the graph recursively."));

	checkSlow(InSystem->GetGraphID() == TNumericLimits<uint16>::Max() && Nodes.Array.GetMaxIndex() < TNumericLimits<uint16>::Max() - 1);

	const int32 NewIndex = Nodes.Array.Add(FMovieSceneEntitySystemGraphNode(InSystem));
	check(NewIndex >= 0 && NewIndex < TNumericLimits<uint16>::Max());

	const uint16 NewNodeID = static_cast<uint16>(NewIndex);

	ReferenceGraph.AllocateNode(NewNodeID);

	InSystem->SetGraphID(NewNodeID);

	checkf(!GlobalToLocalNodeIDs.Contains(InSystem->GetGlobalDependencyGraphID()), TEXT("Got more than one instance of a given system class"));
	GlobalToLocalNodeIDs.Add(InSystem->GetGlobalDependencyGraphID(), NewNodeID);

	++SerialNumber;
}

void FMovieSceneEntitySystemGraph::AddReference(UMovieSceneEntitySystem* FromReference, UMovieSceneEntitySystem* ToReference)
{
	const uint16 FromID = FromReference->GetGraphID();
	const uint16 ToID   = ToReference->GetGraphID();

	check(FromID != TNumericLimits<uint16>::Max() && ToID != TNumericLimits<uint16>::Max());

	ReferenceGraph.MakeEdge(FromID, ToID);
}

void FMovieSceneEntitySystemGraph::RemoveReference(UMovieSceneEntitySystem* FromReference, UMovieSceneEntitySystem* ToReference)
{
	const uint16 FromID = FromReference->GetGraphID();
	const uint16 ToID   = ToReference->GetGraphID();

	check(FromID != TNumericLimits<uint16>::Max() && ToID != TNumericLimits<uint16>::Max());

	ReferenceGraph.DestroyEdge(FromID, ToID);
}

void FMovieSceneEntitySystemGraph::RemoveSystem(UMovieSceneEntitySystem* InSystem)
{
	checkf(ReentrancyGuard == 0, TEXT("Attempting to remove a system from the graph recursively."));

	++ReentrancyGuard;

	const uint16 NodeID = InSystem->GetGraphID();
	check(NodeID != TNumericLimits<uint16>::Max());

	ReferenceGraph.RemoveNode(NodeID);

	Nodes.Array.RemoveAt(NodeID);

	InSystem->SetGraphID(TNumericLimits<uint16>::Max());

	GlobalToLocalNodeIDs.Remove(InSystem->GetGlobalDependencyGraphID());

	++SerialNumber;
	--ReentrancyGuard;

	ReferenceGraph.CleanUpDanglingEdges();
}

int32 FMovieSceneEntitySystemGraph::RemoveIrrelevantSystems(UMovieSceneEntitySystemLinker* Linker)
{
	checkf(ReentrancyGuard == 0, TEXT("Attempting to remove a system from the graph recursively."));
	++ReentrancyGuard;

	check(PreviousSerialNumber == SerialNumber);

	int32 NumRemoved = 0;

	UE::MovieScene::FDirectedGraph::FBreadthFirstSearch Search(&ReferenceGraph);

	// Search from all non-intermediate systems and mark systems that are still referenced
	for (const FMovieSceneEntitySystemGraphNode& Node : Nodes.Array)
	{
		const uint16 GraphID = Node.System->GetGraphID();
		if (Search.GetVisited()[GraphID] == false && Node.System->IsRelevant(Linker))
		{
			Search.Search(GraphID);
		}
	}

	const bool bAnyNotVisited = Search.GetVisited().Num() < Nodes.Array.GetMaxIndex() || Search.GetVisited().Find(false) != INDEX_NONE;
	if (bAnyNotVisited)
	{
		for (int32 Index = 0; Index < Nodes.Array.GetMaxIndex(); ++Index)
		{
			if (Nodes.Array.IsAllocated(Index) && Search.GetVisited()[Index] == false)
			{
				const uint16 NodeID = Index;

				ReferenceGraph.RemoveNode(NodeID);

				UMovieSceneEntitySystem* System = Nodes.Array[NodeID].System;
				Nodes.Array.RemoveAt(NodeID);

				// Remove this system from the graph to ensure we are not re-entrant when calling Unlink() on it
				System->SetGraphID(TNumericLimits<uint16>::Max());

				GlobalToLocalNodeIDs.Remove(System->GetGlobalDependencyGraphID());

				System->Unlink();
				++NumRemoved;
			}
		}
	}

	if (NumRemoved > 0)
	{
		++SerialNumber;

		ReferenceGraph.CleanUpDanglingEdges();
	}

	--ReentrancyGuard;
	return NumRemoved;
}

void FMovieSceneEntitySystemGraph::UpdateCache()
{
	using namespace UE::MovieScene;

	if (PreviousSerialNumber == SerialNumber)
	{
		return;
	}

	ReferenceGraph.CleanUpDanglingEdges();

	checkf(!ReferenceGraph.IsCyclic(), TEXT("Cycle detected in system reference graph.\n")
		TEXT("----------------------------------------------------------------------------------\n")
		TEXT("%s\n")
		TEXT("----------------------------------------------------------------------------------\n"),
		*ToString());

	SpawnPhase.Empty();
	InstantiationPhase.Empty();
	SchedulingPhase.Empty();
	EvaluationPhase.Empty();
	FinalizationPhase.Empty();

	TArray<uint16> SortedGlobalNodeIDs;
	for (const FMovieSceneEntitySystemGraphNode& Node : Nodes.Array)
	{
		SortedGlobalNodeIDs.Add(Node.System->GetGlobalDependencyGraphID());
	}
	UMovieSceneEntitySystem::SortByFlowOrder(SortedGlobalNodeIDs);

	TArray<uint16> SortedNodeIDs;
	for (uint16 GlobalNodeID : SortedGlobalNodeIDs)
	{
		SortedNodeIDs.Add(GlobalToLocalNodeIDs[GlobalNodeID]);
	}

	const bool bCombineSchedulingAndEvaluation = !FEntitySystemScheduler::IsCustomSchedulingEnabled();

	for (uint16 NodeID : SortedNodeIDs)
	{
		ESystemPhase SystemPhase = Nodes.Array[NodeID].System->GetPhase();

		if (EnumHasAnyFlags(SystemPhase, ESystemPhase::Spawn))
		{
			SpawnPhase.Emplace(NodeID);
		}
		if (EnumHasAnyFlags(SystemPhase, ESystemPhase::Instantiation))
		{
			InstantiationPhase.Emplace(NodeID);
		}
		if (EnumHasAnyFlags(SystemPhase, ESystemPhase::Scheduling))
		{
			if (bCombineSchedulingAndEvaluation)
			{
				EvaluationPhase.Emplace(NodeID);
			}
			else
			{
				SchedulingPhase.Emplace(NodeID);
			}
		}
		if (EnumHasAnyFlags(SystemPhase, ESystemPhase::Evaluation))
		{
			EvaluationPhase.Emplace(NodeID);
		}
		if (EnumHasAnyFlags(SystemPhase, ESystemPhase::Finalization))
		{
			FinalizationPhase.Emplace(NodeID);
		}
	}

	PreviousSerialNumber = SerialNumber;
}

void FMovieSceneEntitySystemGraph::DebugPrint() const
{
	static constexpr TCHAR FormatString[] =
		TEXT("----------------------------------------------------------------------------------\n")
		TEXT("%s\n")
		TEXT("----------------------------------------------------------------------------------\n");

	GLog->Log(TEXT("Printing debug graph for Entity System Graph (in standard graphviz syntax):"));
	GLog->Log(FString::Printf(FormatString, *ToString()));
}

FString FMovieSceneEntitySystemGraph::ToString() const
{
	using namespace UE::MovieScene;

	TStringBuilder<1024> String;

	String += TEXT("\ndigraph FMovieSceneEntitySystemGraph {\n");
	String += TEXT("\tnode [shape=record,height=.1];\n");

	FString FlowStrings[] = 
	{
		TEXT("\tsubgraph cluster_flow_0 { label=\"Spawn\"; color=\"#0e868c\";\n"),
		TEXT("\tsubgraph cluster_flow_1 { label=\"Instantiation\"; color=\"#96c74c\";\n"),
		TEXT("\tsubgraph cluster_flow_2 { label=\"Evaluation\"; color=\"#6dc74c\";\n"),
		TEXT("\tsubgraph cluster_flow_3 { label=\"Finalization\"; color=\"#aa42f5\";\n"),
	};

	FString ReferenceGraphString = "\tsubgraph cluster_references { label=\"Explicit Reference Graph (connections imply ownership)\"; color=\"#bfc74c\";\n";

	TBitArray<> FlowFilter[4];
	TBitArray<> ReferencedSystems;

	for (int32 SystemIndex = 0; SystemIndex < this->Nodes.Array.GetMaxIndex(); ++SystemIndex)
	{
		if (Nodes.Array.IsAllocated(SystemIndex))
		{
			UMovieSceneEntitySystem* System = Nodes.Array[SystemIndex].System;

			ESystemPhase SystemPhase = System->GetPhase();
			const uint16 GlobalIndex = System->GetGlobalDependencyGraphID();

			if (EnumHasAnyFlags(SystemPhase, ESystemPhase::Spawn))
			{
				FlowStrings[0] += FString::Printf(TEXT("\t\tflow_node%d_0[label=\"%s\"];\n"), GlobalIndex, *System->GetName());
				FlowFilter[0].PadToNum(GlobalIndex+1, false);
				FlowFilter[0][GlobalIndex] = true;
			}
			if (EnumHasAnyFlags(SystemPhase, ESystemPhase::Instantiation))
			{
				FlowStrings[1] += FString::Printf(TEXT("\t\tflow_node%d_1[label=\"%s\"];\n"), GlobalIndex, *System->GetName());
				FlowFilter[1].PadToNum(GlobalIndex+1, false);
				FlowFilter[1][GlobalIndex] = true;
			}
			if (EnumHasAnyFlags(SystemPhase, ESystemPhase::Evaluation | ESystemPhase::Scheduling))
			{
				FlowStrings[2] += FString::Printf(TEXT("\t\tflow_node%d_2[label=\"%s\"];\n"), GlobalIndex, *System->GetName());
				FlowFilter[2].PadToNum(GlobalIndex+1, false);
				FlowFilter[2][GlobalIndex] = true;
			}
			if (EnumHasAnyFlags(SystemPhase, ESystemPhase::Finalization))
			{
				FlowStrings[3] += FString::Printf(TEXT("\t\tflow_node%d_3[label=\"%s\"];\n"), GlobalIndex, *System->GetName());
				FlowFilter[3].PadToNum(GlobalIndex+1, false);
				FlowFilter[3][GlobalIndex] = true;
			}

			TArrayView<const FDirectionalEdge> ReferenceEdges = ReferenceGraph.GetEdgesFrom(SystemIndex);
			if (ReferenceEdges.Num() > 0)
			{
				ReferencedSystems.PadToNum(SystemIndex+1, false);
				ReferencedSystems[SystemIndex] = true;

				for (FDirectionalEdge Edge : ReferenceEdges)
				{
					ReferencedSystems.PadToNum(Edge.ToNode+1, false);
					ReferencedSystems[Edge.ToNode] = true;
				}
			}
		}
	}

	for (TConstSetBitIterator<> ReferencedSystemIt(ReferencedSystems); ReferencedSystemIt; ++ReferencedSystemIt)
	{
		UMovieSceneEntitySystem* System = Nodes.Array[ReferencedSystemIt.GetIndex()].System;
		const uint16 GlobalIndex = System->GetGlobalDependencyGraphID();

		ReferenceGraphString += FString::Printf(TEXT("\t\treference_node%d[label=\"%s\"];\n"), GlobalIndex, *System->GetName());
	}

	for (FString& FlowString : FlowStrings)
	{
		String += FlowString;
		String += TEXT("\t}\n");
	}
	String += ReferenceGraphString;
	String += TEXT("\t}\n");

	UMovieSceneEntitySystem::PrintFilteredFlowGraph(FlowFilter, String);

	{
		FDirectedGraph::FDiscoverCyclicEdges CyclicEdges(&ReferenceGraph);
		CyclicEdges.Search();

		TArrayView<const FDirectionalEdge> ReferenceEdges = ReferenceGraph.GetEdges();
		for (int32 EdgeIndex = 0; EdgeIndex < ReferenceEdges.Num(); ++EdgeIndex)
		{
			FDirectionalEdge Edge = ReferenceEdges[EdgeIndex];
			const bool bIsCyclic = CyclicEdges.IsCyclic(EdgeIndex);

			const uint16 FromGlobalIndex = Nodes.Array[Edge.FromNode].System->GetGlobalDependencyGraphID();
			const uint16 ToGlobalIndex   = Nodes.Array[Edge.ToNode].System->GetGlobalDependencyGraphID();
			String += FString::Printf(TEXT("\treference_node%d -> reference_node%d [color=\"%s\"];\n"), (int32)FromGlobalIndex, (int32)ToGlobalIndex, bIsCyclic ? TEXT("#FF0000") : TEXT("#3992ad"));
		}
	}

	String += TEXT("}");
	return String.ToString();
}

TArray<UMovieSceneEntitySystem*> FMovieSceneEntitySystemGraph::GetSystems() const
{
	TArray<UMovieSceneEntitySystem*> Systems;
	for (const FMovieSceneEntitySystemGraphNode& Node : Nodes.Array)
	{
		Systems.Add(Node.System);
	}
	return Systems;
}

UMovieSceneEntitySystem* FMovieSceneEntitySystemGraph::FindSystemOfType(TSubclassOf<UMovieSceneEntitySystem> InClassType) const
{
	UClass* ClassType = InClassType.Get();
	for (const FMovieSceneEntitySystemGraphNode& Node : Nodes.Array)
	{
		if (Node.System->GetClass() == ClassType)
		{
			return Node.System;
		}
	}
	return nullptr;
}

int32 FMovieSceneEntitySystemGraph::NumInPhase(UE::MovieScene::ESystemPhase Phase) const
{
	switch (Phase)
	{
	case UE::MovieScene::ESystemPhase::Spawn:         return SpawnPhase.Num();
	case UE::MovieScene::ESystemPhase::Instantiation: return InstantiationPhase.Num();
	case UE::MovieScene::ESystemPhase::Evaluation:    return EvaluationPhase.Num();
	case UE::MovieScene::ESystemPhase::Finalization:  return FinalizationPhase.Num();
	default: ensureMsgf(false, TEXT("Invalid phase specified for execution.")); return 0;
	}
}

void FMovieSceneEntitySystemGraph::ExecutePhase(UE::MovieScene::ESystemPhase Phase, UMovieSceneEntitySystemLinker* Linker, FGraphEventArray& OutTasks)
{
	UpdateCache();

	switch (Phase)
	{
	case UE::MovieScene::ESystemPhase::Spawn:         ExecutePhase(Phase, SpawnPhase,         Linker, OutTasks); break;
	case UE::MovieScene::ESystemPhase::Instantiation: ExecutePhase(Phase, InstantiationPhase, Linker, OutTasks); break;
	case UE::MovieScene::ESystemPhase::Evaluation:    ExecutePhase(Phase, EvaluationPhase,    Linker, OutTasks); break;
	case UE::MovieScene::ESystemPhase::Finalization:  ExecutePhase(Phase, FinalizationPhase,  Linker, OutTasks); break;
	default: ensureMsgf(false, TEXT("Invalid phase specified for execution.")); break;
	}
}

void FMovieSceneEntitySystemGraph::IteratePhase(UE::MovieScene::ESystemPhase Phase, TFunctionRef<void(UMovieSceneEntitySystem*)> InIter)
{
	UpdateCache();

	TArrayView<const uint16> Array;
	switch (Phase)
	{
	case UE::MovieScene::ESystemPhase::Spawn:         Array = SpawnPhase;         break;
	case UE::MovieScene::ESystemPhase::Instantiation: Array = InstantiationPhase; break;
	case UE::MovieScene::ESystemPhase::Scheduling:    Array = SchedulingPhase;    break;
	case UE::MovieScene::ESystemPhase::Evaluation:    Array = EvaluationPhase;    break;
	case UE::MovieScene::ESystemPhase::Finalization:  Array = FinalizationPhase;  break;
	default: ensureMsgf(false, TEXT("Invalid phase specified for iteration."));   return;
	}

	for (uint16 NodeID : Array)
	{
		InIter(Nodes.Array[NodeID].System);
	}
}

template<typename ArrayType>
void FMovieSceneEntitySystemGraph::ExecutePhase(UE::MovieScene::ESystemPhase Phase, const ArrayType& SortedEntries, UMovieSceneEntitySystemLinker* Linker, FGraphEventArray& OutTasks)
{
	using namespace UE::MovieScene;

	const bool bCustomSchedulingEnabled = FEntitySystemScheduler::IsCustomSchedulingEnabled();

	const EEntityThreadingModel ThreadingModel = Linker->EntityManager.GetThreadingModel();

	// In a transaction we can only support the no-threading mode.
	check(!AutoRTFM::IsTransactional() || (EEntityThreadingModel::NoThreading == ThreadingModel));

	FSystemSubsequentTasks DownstreamTasks(this, &OutTasks, ThreadingModel);

	FSystemTaskPrerequisites NoPrerequisites;

	const bool bStructureCanChange = !Linker->EntityManager.IsLockedDown();

	for (int32 CurrentIndex = 0; CurrentIndex < SortedEntries.Num(); ++CurrentIndex)
	{
		const uint16 NodeID = SortedEntries[CurrentIndex];

		UMovieSceneEntitySystem* System = Nodes.Array[NodeID].System;
		checkSlow(System);

		// Initialize downstream task structure for this system
		DownstreamTasks.ResetNode(NodeID);

		TSharedPtr<FSystemTaskPrerequisites> Prerequisites = Nodes.Array[NodeID].Prerequisites;
		System->Run(Prerequisites ? *Prerequisites : NoPrerequisites, DownstreamTasks);

		if (bStructureCanChange)
		{
			Linker->AutoLinkRelevantSystems();
		}

		// If we linked any new systems, we may have to move our current offset
		if (SerialNumber != PreviousSerialNumber)
		{
#if DO_CHECK
			// Cache the systems we've already run
			TArray<uint16> HeadList(SortedEntries.GetData(), CurrentIndex+1);
#endif

			// This may actually change the SortedEntries array
			UpdateCache();

			CurrentIndex = Algo::IndexOf(SortedEntries, NodeID);
			checkf(CurrentIndex != INDEX_NONE, TEXT("System has removed itself while being Run. This is not supported."));

#if DO_CHECK
			for (int32 NewIndex = 0; NewIndex < CurrentIndex; ++NewIndex)
			{
				if (!HeadList.Contains(SortedEntries[NewIndex]))
				{
					const uint16 NewNodeIndex     = SortedEntries[NewIndex];
					const uint16 CurrentNodeIndex = SortedEntries[CurrentIndex];
					ensureAlwaysMsgf(false, 
						TEXT("System %s has been inserted upstream of %s in the same execution phase that is currently in-flight, and will not be run this frame. "
							 "This can be either because this system has been newly linked, or because it has been re-ordered due to other newly linked systems."),
						*this->Nodes.Array[NewNodeIndex].System->GetName(),
						*this->Nodes.Array[CurrentNodeIndex].System->GetName()
					);
				}
			}
#endif
		}

		// Don't need prerequisites now
		if (Prerequisites)
		{
			Prerequisites->Empty();
		}

		// Propagate subsequent tasks
		if (DownstreamTasks.Subsequents && DownstreamTasks.Subsequents->Num() != 0)
		{
			SCOPE_CYCLE_COUNTER(MovieSceneEval_SystemDependencyCost)

			TArray<uint16> ToGlobalNodeIDs;
			UMovieSceneEntitySystem::GetSubsequentSystems(System->GetGlobalDependencyGraphID(), ToGlobalNodeIDs);
			for (uint16 ToGlobalNodeID : ToGlobalNodeIDs)
			{
				uint16* ToNodeID = GlobalToLocalNodeIDs.Find(ToGlobalNodeID);
				if (ToNodeID)
				{
					FMovieSceneEntitySystemGraphNode& ToNode = Nodes.Array[*ToNodeID];
					if (EnumHasAnyFlags(ToNode.System->GetPhase(), Phase)
						// If custom scheduling is disabled, allow propagation between evaluation/scheduling phase as well
						|| (!bCustomSchedulingEnabled && Phase == ESystemPhase::Evaluation && EnumHasAnyFlags(ToNode.System->GetPhase(), ESystemPhase::Scheduling))
						)
					{
						if (!ToNode.Prerequisites)
						{
							ToNode.Prerequisites = MakeShared<FSystemTaskPrerequisites>();
						}
						ToNode.Prerequisites->Consume(*DownstreamTasks.Subsequents);
					}
				}
			}

			// Done with subsequents now
			DownstreamTasks.Subsequents->Empty();
		}
	}
}

void FMovieSceneEntitySystemGraph::ReconstructTaskSchedule(UE::MovieScene::FEntityManager* EntityManager)
{
	using namespace UE::MovieScene;

	if (!FEntitySystemScheduler::IsCustomSchedulingEnabled())
	{
		// When disabled, we combine the scheduling phase with the evaluation phase and
		// execute them using the legacy procedural task prerequisites API through OnRun

		return;
	}

	UpdateCache();

	if (!TaskScheduler)
	{
		TaskScheduler = MakeUnique<FEntitySystemScheduler>(EntityManager);
	}

	TaskScheduler->BeginConstruction();

	for (int32 CurrentIndex = 0; CurrentIndex < SchedulingPhase.Num(); ++CurrentIndex)
	{
		const uint16 NodeID = SchedulingPhase[CurrentIndex];

		UMovieSceneEntitySystem* System = Nodes.Array[NodeID].System;
		checkSlow(System);

		// Initilaize downstream task structure for this system
		TaskScheduler->BeginSystem(NodeID);

		System->SchedulePersistentTasks(TaskScheduler.Get());

		// Propagate subsequent tasks
		if (TaskScheduler->HasAnyTasksToPropagateDownstream())
		{
			TArray<uint16> ToGlobalNodeIDs;
			UMovieSceneEntitySystem::GetSubsequentSystems(System->GetGlobalDependencyGraphID(), ToGlobalNodeIDs);
			for (uint16 ToGlobalNodeID : ToGlobalNodeIDs)
			{
				uint16* ToNodeID = GlobalToLocalNodeIDs.Find(ToGlobalNodeID);
				if (ToNodeID)
				{
					TaskScheduler->PropagatePrerequisite(*ToNodeID);
				}
			}
		}

		TaskScheduler->EndSystem(NodeID);
	}

	TaskScheduler->EndConstruction();

	SchedulerSerialNumber = EntityManager->GetSystemSerial();
}

void FMovieSceneEntitySystemGraph::ScheduleTasks(UE::MovieScene::FEntityManager* EntityManager)
{
	// @todo: First off go through and increase the WriteContexts?
	if (TaskScheduler)
	{
		EntityManager->IncrementSystemSerial();

		if (EntityManager->HasStructureChangedSince(SchedulerSerialNumber))
		{
			ReconstructTaskSchedule(EntityManager);
		}

		TaskScheduler->ExecuteTasks();
	}
}

void FMovieSceneEntitySystemGraph::Shutdown()
{
	checkf(ReentrancyGuard == 0, TEXT("Attempting to shutdown the system graph while it is in use."));

	++ReentrancyGuard;

	for (int32 Index = 0; Index < Nodes.Array.GetMaxIndex(); ++Index)
	{
		// When reference elimination is enabled, it's possible for the pointers in this array
		//    to become null during a GC pass which is also likely to be when Shutdown is called,
		//    so we have to check for null here, even though that is not necessary if reference
		//    elimination is disabled
		UMovieSceneEntitySystem* System = Nodes.Array.IsAllocated(Index) ? Nodes.Array[Index].System : nullptr;
		if (System)
		{
			System->Abandon();
		}
	}

	--ReentrancyGuard;

	*this = FMovieSceneEntitySystemGraph();
}

uint16 FMovieSceneEntitySystemGraph::GetGraphID(const UMovieSceneEntitySystem* InSystem)
{
	return InSystem->GetGraphID();
}

