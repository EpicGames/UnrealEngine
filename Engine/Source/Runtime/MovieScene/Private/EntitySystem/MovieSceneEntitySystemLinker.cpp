// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntityMutations.h"
#include "EntitySystem/MovieSceneComponentRegistry.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieScenePreAnimatedStateSystem.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "MovieSceneFwd.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "Templates/SubclassOf.h"
#include "Containers/Ticker.h"
#include "UObject/ObjectKey.h"
#include "Engine/World.h"
#include "Algo/Find.h"
#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"
#include "ProfilingDebugging/CountersTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneEntitySystemLinker)

DECLARE_CYCLE_STAT(TEXT("Link Relevant Systems"),		MovieSceneEval_LinkRelevantSystems,		STATGROUP_MovieSceneECS);

namespace UE
{
namespace MovieScene
{

struct FCustomEventDeleter
{
	void operator()(FEvent* Event)
	{
		FPlatformProcess::ReturnSynchEventToPool(Event);
	}
};

static FComponentRegistry GComponentRegistry;

EEntitySystemLinkerRole RegisterCustomEntitySystemLinkerRole()
{
	static EEntitySystemLinkerRole NextCustom = EEntitySystemLinkerRole::Custom;

	EEntitySystemLinkerRole Result = NextCustom;
	NextCustom = (EEntitySystemLinkerRole)((uint32)NextCustom + 1);
	check((uint32)NextCustom != TNumericLimits<uint32>::Max());
	return Result;
}

FSystemFilter::FSystemFilter()
{
	CategoriesAllowed = EEntitySystemCategory::All;
	CategoriesDisallowed = EEntitySystemCategory::None;
}

bool FSystemFilter::CheckSystem(TSubclassOf<UMovieSceneEntitySystem> InClass) const
{
	const UMovieSceneEntitySystem* SystemCDO = InClass.GetDefaultObject();
	if (SystemCDO)
	{
		return CheckSystem(SystemCDO);
	}
	// Allow any systems without a CDO.
	return true;
}

bool FSystemFilter::CheckSystem(const UMovieSceneEntitySystem* InSystem) const
{
	// Specific allow/disallow rules for systems take precedence.
	const uint16 SystemClassID = InSystem->GetGlobalDependencyGraphID();
	if (SystemsDisallowed.IsValidIndex(SystemClassID) && SystemsDisallowed[SystemClassID])
	{
		return false;
	}
	if (SystemsAllowed.IsValidIndex(SystemClassID) && SystemsAllowed[SystemClassID])
	{
		return true;
	}

	// If any given category is disallowed, the entire thing is refused.
	if (EnumHasAnyFlags(InSystem->GetCategories(), CategoriesDisallowed))
	{
		return false;
	}
	// If the given categories contain at least one allowed category, it's good.
	if (EnumHasAnyFlags(InSystem->GetCategories(), CategoriesAllowed))
	{
		return true;
	}
	// Nothing explicitly disallowed, but nothing allowed either, so we don't want it.
	return false;
}

void FSystemFilter::SetAllowedCategories(EEntitySystemCategory InCategory)
{
	CategoriesAllowed = InCategory;
}

void FSystemFilter::AllowCategory(EEntitySystemCategory InCategory)
{
	CategoriesAllowed |= InCategory;
}

void FSystemFilter::SetDisallowedCategories(EEntitySystemCategory InCategory)
{
	CategoriesDisallowed = InCategory;
}

void FSystemFilter::DisallowCategory(EEntitySystemCategory InCategory)
{
	CategoriesDisallowed |= InCategory;
}

void FSystemFilter::AllowSystem(TSubclassOf<UMovieSceneEntitySystem> InClass)
{
	const UMovieSceneEntitySystem* SystemCDO = InClass.GetDefaultObject();
	if (ensure(SystemCDO))
	{
		const uint16 SystemClassID = SystemCDO->GetGlobalDependencyGraphID();
		SystemsAllowed.PadToNum(SystemClassID + 1, false);
		SystemsAllowed[SystemClassID] = true;
	}
}

void FSystemFilter::DisallowSystem(TSubclassOf<UMovieSceneEntitySystem> InClass)
{
	const UMovieSceneEntitySystem* SystemCDO = InClass.GetDefaultObject();
	if (ensure(SystemCDO))
	{
		const uint16 SystemClassID = SystemCDO->GetGlobalDependencyGraphID();
		SystemsDisallowed.PadToNum(SystemClassID + 1, false);
		SystemsDisallowed[SystemClassID] = true;
	}
}

} // namespace MovieScene
} // namespace UE


UMovieSceneEntitySystemLinker::UMovieSceneEntitySystemLinker(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	Role = EEntitySystemLinkerRole::Unknown;
	LastSystemLinkVersion = 0;
	LastSystemUnlinkVersion = 0;
	LastInstantiationVersion = 0;
	AutoLinkMode = EAutoLinkRelevantSystems::Enabled;

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		PreAnimatedState.Initialize(this);

		FCoreUObjectDelegates::GetPreGarbageCollectDelegate().AddUObject(this, &UMovieSceneEntitySystemLinker::HandlePreGarbageCollection);
		FCoreUObjectDelegates::GetPostGarbageCollect().AddUObject(this, &UMovieSceneEntitySystemLinker::HandlePostGarbageCollection);

		EntityManager.SetDebugName(GetName() + TEXT("[Entity Manager]"));
		EntityManager.SetComponentRegistry(&GComponentRegistry);

		InstanceRegistry.Reset(new FInstanceRegistry(this));

		Runner = MakeShared<FMovieSceneEntitySystemRunner>(this);

#if WITH_EDITOR
		FCoreUObjectDelegates::OnObjectsReplaced.AddUObject(this, &UMovieSceneEntitySystemLinker::OnObjectsReplaced);
#endif
	}
}

UE::MovieScene::FEntitySystemLinkerExtensionID UMovieSceneEntitySystemLinker::RegisterExtension()
{
	static int32 StaticID = 0;
	return UE::MovieScene::FEntitySystemLinkerExtensionID{ StaticID++ };
}

void UMovieSceneEntitySystemLinker::Reset()
{
	Events.AbandonLinker.Broadcast(this);

	Events.TagGarbage.Clear();
	Events.CleanTaggedGarbage.Clear();
	Events.AddReferencedObjects.Clear();
	Events.AbandonLinker.Clear();

	SystemGraph.Shutdown();
	EntitySystemsByGlobalGraphID.Reset();

	EntityManager.Destroy();
}

UMovieSceneEntitySystemLinker* UMovieSceneEntitySystemLinker::FindOrCreateLinker(UObject* PreferredOuter, UE::MovieScene::EEntitySystemLinkerRole LinkerRole, const TCHAR* Name)
{
	if (!PreferredOuter)
	{
		PreferredOuter = GetTransientPackage();
	}

	UMovieSceneEntitySystemLinker* Existing = FindObject<UMovieSceneEntitySystemLinker>(PreferredOuter, Name);
	if (!Existing)
	{
		Existing = NewObject<UMovieSceneEntitySystemLinker>(PreferredOuter, Name);
		Existing->SetLinkerRole(LinkerRole);
	}
	ensure(Existing->Role == LinkerRole);
	return Existing;
}

UMovieSceneEntitySystemLinker* UMovieSceneEntitySystemLinker::CreateLinker(UObject* PreferredOuter, UE::MovieScene::EEntitySystemLinkerRole LinkerRole)
{
	if (!PreferredOuter)
	{
		PreferredOuter = GetTransientPackage();
	}

	UMovieSceneEntitySystemLinker* NewLinker = NewObject<UMovieSceneEntitySystemLinker>(PreferredOuter);
	NewLinker->Role = LinkerRole;
	return NewLinker;
}

UE::MovieScene::FComponentRegistry* UMovieSceneEntitySystemLinker::GetComponents()
{
	return &UE::MovieScene::GComponentRegistry;
}

void UMovieSceneEntitySystemLinker::InvalidateObjectBinding(const FGuid& ObjectBindingID, FInstanceHandle InInstanceHandle)
{
	if (InstanceRegistry->IsHandleValid(InInstanceHandle))
	{
		InstanceRegistry->InvalidateObjectBinding(ObjectBindingID, InInstanceHandle);
	}
}

void UMovieSceneEntitySystemLinker::SystemLinked(UMovieSceneEntitySystem* InSystem)
{
	const uint16 GlobalID = InSystem->GetGlobalDependencyGraphID();

	EntitySystemsByGlobalGraphID.Insert(GlobalID, InSystem);
}

void UMovieSceneEntitySystemLinker::SystemUnlinked(UMovieSceneEntitySystem* InSystem)
{
	const uint16 GlobalID = InSystem->GetGlobalDependencyGraphID();

	check(EntitySystemsByGlobalGraphID[GlobalID] == InSystem);
	EntitySystemsByGlobalGraphID.RemoveAt(GlobalID);

	Events.PostSpawnEvent.RemoveAll(InSystem);
	Events.TagGarbage.RemoveAll(InSystem);
	Events.CleanTaggedGarbage.RemoveAll(InSystem);
	Events.AddReferencedObjects.RemoveAll(InSystem);
	Events.AbandonLinker.RemoveAll(InSystem);

	// Add the system to the recycling pool.
	ensure(EntitySystemsRecyclingPool.Contains(InSystem->GetClass()) == false);
	EntitySystemsRecyclingPool.Add(InSystem->GetClass(), InSystem);
}

bool UMovieSceneEntitySystemLinker::HasLinkedSystem(const uint16 GlobalDependencyGraphID)
{
	return EntitySystemsByGlobalGraphID.IsValidIndex(GlobalDependencyGraphID);
}

void UMovieSceneEntitySystemLinker::BeginDestroy()
{
	Events.AbandonLinker.Broadcast(this);

	SystemGraph.Shutdown();

	FCoreUObjectDelegates::GetPostGarbageCollect().RemoveAll(this);

	Super::BeginDestroy();
}

void UMovieSceneEntitySystemLinker::CleanupInvalidBoundObjects()
{
	TagInvalidBoundObjects();
	Events.TagGarbage.Broadcast(this);
	CleanGarbage();
}

void UMovieSceneEntitySystemLinker::TagInvalidBoundObjects()
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	// Tag any bound objects that are now invalid
	TArray<FMovieSceneEntityID> ExpiredBoundObjects;

	auto Iter = [&ExpiredBoundObjects](FMovieSceneEntityID EntityID, const FObjectKey& BoundObjectKey)
	{
		UObject* BoundObject = BoundObjectKey.ResolveObjectPtr();
		if (FBuiltInComponentTypes::IsBoundObjectGarbage(BoundObject))
		{
			ExpiredBoundObjects.Add(EntityID);
		}
	};
	FEntityTaskBuilder()
	.ReadEntityIDs()
	.Read(BuiltInComponents->BoundObjectKey)
	.Iterate_PerEntity(&EntityManager, Iter);

	for (FMovieSceneEntityID Entity : ExpiredBoundObjects)
	{
		EntityManager.RemoveComponent(Entity, BuiltInComponents->Tags.NeedsLink, EEntityRecursion::Full);
		EntityManager.AddComponent(Entity, BuiltInComponents->Tags.NeedsUnlink, EEntityRecursion::Full);
	}
}

bool UMovieSceneEntitySystemLinker::HasStructureChangedSinceLastRun() const
{
	return EntityManager.HasStructureChangedSince(LastInstantiationVersion);
}

bool UMovieSceneEntitySystemLinker::StartEvaluation()
{
	if (RunnerReentrancyFlags.Num() == 0 || RunnerReentrancyFlags[RunnerReentrancyFlags.Num() - 1])
	{
		// Default to re-entrancy being forbidden. The runner will allow re-entrancy at specific spots
		// in the evaluation loop, via a "re-entrancy window".
		RunnerReentrancyFlags.Add(false);
		return true;
	}
		
	UE_LOGF(LogMovieSceneECS, Warning, "Can't start a new evaluation: the active runner is not in a re-entrancy window.");
	return false;
}

TSharedRef<FMovieSceneEntitySystemRunner> UMovieSceneEntitySystemLinker::GetRunner() const
{
	// The runner might lose us if we are GC'ed while someone else is keeping the runner's ref-count up,
	// but we can never lose our runner, since we have at least one reference.
	return Runner.ToSharedRef();
}

void UMovieSceneEntitySystemLinker::PostInstantation()
{
	LastInstantiationVersion = EntityManager.GetSystemSerial();

	GetInstanceRegistry()->PostInstantation();
}

void UMovieSceneEntitySystemLinker::EndEvaluation()
{
	const int32 LastIndex = RunnerReentrancyFlags.Num()-1;
	ensureAlways(RunnerReentrancyFlags[LastIndex] == false);
	RunnerReentrancyFlags.RemoveAt(LastIndex);
}

void UMovieSceneEntitySystemLinker::HandlePreGarbageCollection()
{
}

void UMovieSceneEntitySystemLinker::HandlePostGarbageCollection()
{
	using namespace UE::MovieScene;

	// Increment the system serial number to ensure that any structural mutation that occurs in this function does so under a unique serial
	EntityManager.IncrementSystemSerial();

	// All the instance registry to unlink garbage first
	InstanceRegistry->TagGarbage();

	// Clean any garbage bound objects
	TagInvalidBoundObjects();

	// Allow any other system to tag garbage
	Events.TagGarbage.Broadcast(this);

	TSet<UMovieSceneEntitySystem*> SystemsToTag;
	auto GatherSystemsToTag = [&SystemsToTag](UMovieSceneEntitySystem* System){ SystemsToTag.Add(System); };
	SystemGraph.IteratePhase(ESystemPhase::Spawn, GatherSystemsToTag);
	SystemGraph.IteratePhase(ESystemPhase::Instantiation, GatherSystemsToTag);
	for (UMovieSceneEntitySystem* System : SystemsToTag)
	{
		System->TagGarbage();
	}

	CleanGarbage();
}

void UMovieSceneEntitySystemLinker::CleanGarbage()
{
	using namespace UE::MovieScene;

	// Cleanup any stale preanimated state. This can exist even without Entities being marked as NeedsUnlink.
	PreAnimatedState.DiscardStaleObjectState();

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FComponentTypeID NeedsUnlink = BuiltInComponents->Tags.NeedsUnlink;
	if (!EntityManager.ContainsComponent(NeedsUnlink))
	{
		return;
	}

	// Clear the instantiation serial to indicate that we probably need to re-run the instantiation systems
	// the next time a runner gets flushed
	LastInstantiationVersion = 0;

	// Since some systems might belong to both Spawn and Instantiation phase, we need to gather them in a Set
	// to prevent calling CleanTaggedGarbage twice on those.
	TSet<UMovieSceneEntitySystem*> SystemsToClean;
	auto GatherSystemsToClean = [&SystemsToClean](UMovieSceneEntitySystem* System){ SystemsToClean.Add(System); };
	SystemGraph.IteratePhase(ESystemPhase::Spawn, GatherSystemsToClean);
	SystemGraph.IteratePhase(ESystemPhase::Instantiation, GatherSystemsToClean);
	for (UMovieSceneEntitySystem* System : SystemsToClean)
	{
		System->CleanTaggedGarbage();
	}

	// Allow any other system to cleanup garbage
	// NOTE: Order is important here - we need to broadcast this after systems have CleanTaggedGarbage called
	//       since systems are able to (and more likely to) cause additional entities to be unlinked in response
	//       to finding entities tagged NeedsUnlink.
	Events.CleanTaggedGarbage.Broadcast(this);


	TArray<FMovieSceneEntityID> UnresolvedEntities;

	FEntityTaskBuilder()
	.Read(BuiltInComponents->BoundObject)
	.Read(BuiltInComponents->ParentEntity)
	.FilterNone({ BuiltInComponents->Tags.NeedsUnlink, BuiltInComponents->Tags.Ignored, BuiltInComponents->Tags.Finished })
	.Iterate_PerEntity(&EntityManager, [&UnresolvedEntities](UObject* Object, FMovieSceneEntityID ParentEntityID)
	{
		if (!Object)
		{
			UnresolvedEntities.Add(ParentEntityID);
		}
	});

	for (FMovieSceneEntityID EntityID : UnresolvedEntities)
	{
		EntityManager.AddComponent(EntityID, BuiltInComponents->Tags.HasUnresolvedBinding);
	}

	// Remove NeedsLink off any NeedsUnLink entities - these tags are incompatible. Any such entities should get cleaned up
	// through a binding to Events.CleanTaggedGarbage rather than through the usual NeedsUnlink methods
	FRemoveSingleMutation RemoveNeedsLink(BuiltInComponents->Tags.NeedsLink);
	EntityManager.MutateAll(FEntityComponentFilter().All({ BuiltInComponents->Tags.NeedsLink, NeedsUnlink }), RemoveNeedsLink);

	// Free the entities
	TSet<FMovieSceneEntityID> FreedEntities;
	EntityManager.FreeEntities(FEntityComponentFilter().All({ NeedsUnlink }), &FreedEntities);

	InstanceRegistry->CleanupLinkerEntities(FreedEntities);

	// If we have any runners part-way through an evaluation, we need to reset them so that they re-evaluate from the start
	ResetRunner();
}

void UMovieSceneEntitySystemLinker::ResetRunner()
{
	// If we have any runners part-way through an evaluation, we need to reset them so that they re-evaluate from the start
	Runner->ResetFlushState();
}

void UMovieSceneEntitySystemLinker::DestroyInstanceImmediately(UE::MovieScene::FRootInstanceHandle Instance)
{
	// Ensure that any changes are under a new serial
	EntityManager.IncrementSystemSerial();

	// Destroy the instance and any sub sequences. Any pre-existing NeedsLink entities will be forcibly made NeedsUnlink and cleaned as garbage
	GetInstanceRegistry()->DestroyInstance(Instance);
	CleanGarbage();
	ResetRunner();
}

void UMovieSceneEntitySystemLinker::OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap)
{
#if WITH_EDITOR
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	FEntityTaskBuilder()
	.Write(BuiltInComponents->BoundObjectKey)
	.Write(BuiltInComponents->BoundObject)
	.Iterate_PerEntity(&EntityManager, [&ReplacementMap](FObjectKey& ObjectKey, UObject*& Object)
	{
		if (UObject* Replacement = ReplacementMap.FindRef(Object))
		{
			Object = Replacement;
			ObjectKey = Replacement;
		}
	});
#endif

	PreAnimatedState.OnObjectsReplaced(ReplacementMap);
}

void UMovieSceneEntitySystemLinker::AddReferencedObjects(UObject* Object, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(Object, Collector);

	UMovieSceneEntitySystemLinker* This = CastChecked<UMovieSceneEntitySystemLinker>(Object);

	This->EntityManager.AddReferencedObjects(Collector);
	This->Events.AddReferencedObjects.Broadcast(This, Collector);
	Collector.AddReferencedObjects(This->EntitySystemsRecyclingPool);
}

UMovieSceneEntitySystem* UMovieSceneEntitySystemLinker::LinkSystem(TSubclassOf<UMovieSceneEntitySystem> InClassType)
{
	UMovieSceneEntitySystem* Existing = FindSystem(InClassType);
	if (Existing)
	{
		return Existing;
	}

	return LinkSystemImpl(InClassType);
}

UMovieSceneEntitySystem* UMovieSceneEntitySystemLinker::LinkSystemIfAllowed(TSubclassOf<UMovieSceneEntitySystem> InClassType)
{
	UMovieSceneEntitySystem* Existing = FindSystem(InClassType);
	if (Existing)
	{
		return Existing;
	}

	if (!SystemFilter.CheckSystem(InClassType))
	{
		return nullptr;
	}

	return LinkSystemImpl(InClassType);
}

UMovieSceneEntitySystem* UMovieSceneEntitySystemLinker::LinkSystemImpl(TSubclassOf<UMovieSceneEntitySystem> InClassType)
{
	// We always create systems with a fixed name (since there should only ever be one of that name)
	// This means we can do our own recycling within the scope of this linker, to save on the cost of re-creating
	// systems when the first instantiation phase kicks in after a period without any sequence playing.
	UMovieSceneEntitySystem* NewSystem = nullptr;
	auto* Recycled = EntitySystemsRecyclingPool.Find(InClassType);
	if (Recycled)
	{
		// Revive a recycled system.
		NewSystem = *Recycled;
		check(NewSystem);
		EntitySystemsRecyclingPool.Remove(InClassType);
		UE_LOGF(LogMovieSceneECS, Verbose, "Recycling system: ");
	}
	else
	{
		// Unique names also mean we will recycle systems if they previously existed but are no longer used to avoid thrashing the GC
		// Recycling will destruct + memzero + construct the object so we can be sure that previous state doesn't roll over
		UClass* SystemClass = InClassType.Get();
		FName   SystemName  = SystemClass->GetFName();
		NewSystem = NewObject<UMovieSceneEntitySystem>(this, SystemClass, SystemName);
	}

	// If a system implements a hard depdency on another (through direct use of LinkSystem<>), we can't break the client code by returning null, but we can still warn that it should have checked whether it can call LinkSystem first
	ensureMsgf(SystemFilter.CheckSystem(NewSystem), TEXT("Attempting to link a system that should have been excluded - this is probably an explicit call to Link a system that should have been excluded."));

	SystemGraph.AddSystem(NewSystem);
	NewSystem->Link(this);
	return NewSystem;
}

UMovieSceneEntitySystem* UMovieSceneEntitySystemLinker::FindSystem(TSubclassOf<UMovieSceneEntitySystem> InClassType) const
{
	UClass* Class = InClassType.Get();
	UMovieSceneEntitySystem* SystemCDO = Class ? Cast<UMovieSceneEntitySystem>(Class->GetDefaultObject()) : nullptr;
	if (SystemCDO)
	{
		const uint16 GlobalID = SystemCDO->GetGlobalDependencyGraphID();
		if (EntitySystemsByGlobalGraphID.IsValidIndex(GlobalID))
		{
			return EntitySystemsByGlobalGraphID[GlobalID];
		}
	}

	return nullptr;
}

void UMovieSceneEntitySystemLinker::LinkRelevantSystems()
{
	MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_LinkRelevantSystems);

	// If the structure has not changed there's no way that there are any other relevant systems still
	if (EntityManager.HasStructureChangedSince(LastSystemLinkVersion))
	{
		UMovieSceneEntitySystem::LinkRelevantSystems(this);

		LastSystemLinkVersion = EntityManager.GetSystemSerial();
	}
}

void UMovieSceneEntitySystemLinker::UnlinkIrrelevantSystems()
{
	if (EntityManager.HasStructureChangedSince(LastSystemUnlinkVersion))
	{
		SystemGraph.RemoveIrrelevantSystems(this);

		LastSystemUnlinkVersion = EntityManager.GetSystemSerial();
	}
}

void UMovieSceneEntitySystemLinker::AutoLinkRelevantSystems()
{
	if (AutoLinkMode == UE::MovieScene::EAutoLinkRelevantSystems::Enabled)
	{
		LinkRelevantSystems();
	}
}
void UMovieSceneEntitySystemLinker::AutoUnlinkIrrelevantSystems()
{
	if (AutoLinkMode == UE::MovieScene::EAutoLinkRelevantSystems::Enabled)
	{
		UnlinkIrrelevantSystems();
	}
}

