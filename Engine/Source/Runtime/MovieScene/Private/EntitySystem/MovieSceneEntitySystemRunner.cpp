// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntityMutations.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "EntitySystem/MovieSceneTaskScheduler.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedCaptureSource.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "IMovieScenePlayer.h"
#include "Logging/LogVerbosity.h"
#include "MovieSceneSequence.h"
#include "MovieSceneTimeHelpers.h"
#include "Algo/Reverse.h"
#include "Algo/Sort.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "UObject/ObjectMacros.h"

DECLARE_CYCLE_STAT(TEXT("Runner Flush"), 				MovieSceneEval_RunnerFlush, 				STATGROUP_MovieSceneEval);

DECLARE_CYCLE_STAT(TEXT("Spawn Phase"),                 MovieSceneEval_SpawnPhase,              	STATGROUP_MovieSceneECS);
DECLARE_CYCLE_STAT(TEXT("Post Spawn Event"),            MovieSceneEval_PostSpawnEvent,            	STATGROUP_MovieSceneECS);
DECLARE_CYCLE_STAT(TEXT("Instantiation Phase"), 		MovieSceneEval_InstantiationPhase, 			STATGROUP_MovieSceneECS);
DECLARE_CYCLE_STAT(TEXT("Instantiation Async Tasks"), 	MovieSceneEval_AsyncInstantiationTasks,		STATGROUP_MovieSceneECS);
DECLARE_CYCLE_STAT(TEXT("Post Instantiation"), 			MovieSceneEval_PostInstantiation, 			STATGROUP_MovieSceneECS);

DECLARE_CYCLE_STAT(TEXT("Evaluation Phase"), 			MovieSceneEval_EvaluationPhase, 			STATGROUP_MovieSceneECS);
DECLARE_CYCLE_STAT(TEXT("Finalization Phase"),          MovieSceneEval_FinalizationPhase,       	STATGROUP_MovieSceneECS);
DECLARE_CYCLE_STAT(TEXT("Post Evaluation Phase"),       MovieSceneEval_PostEvaluationPhase,     	STATGROUP_MovieSceneECS);

namespace UE::MovieScene::FlushState
{
	// Signifies that, during the Finalization task, there were still outstanding tasks and we need to perform another iteration
	ERunnerFlushState LoopEval = ERunnerFlushState::ConditionalRecompile
		| ERunnerFlushState::Import
		| ERunnerFlushState::Spawn
		| ERunnerFlushState::Instantiation
		| ERunnerFlushState::Evaluation
		| ERunnerFlushState::Finalization
		| ERunnerFlushState::EventTriggers
		| ERunnerFlushState::PostEvaluation
		| ERunnerFlushState::End;

	ERunnerFlushState Everything = ERunnerFlushState::Start | LoopEval;
}


/**
 * Structure for making it possible to make re-entrant evaluation on a linker.
 */
struct FMovieSceneEntitySystemEvaluationReentrancyWindow
{
	FMovieSceneEntitySystemRunner* Runner;
	UMovieSceneEntitySystemLinker* Linker;
	int32 ReentrancyLevel;
	UE::MovieScene::ERunnerFlushState CachedFlushState;
	UE::MovieScene::ERunnerFlushState CachedCurrentFlushState;

	FMovieSceneEntitySystemEvaluationReentrancyWindow(FMovieSceneEntitySystemRunner* InRunner, UMovieSceneEntitySystemLinker* InLinker)
		: Runner(InRunner)
		, Linker(InLinker)
		, ReentrancyLevel(Linker->RunnerReentrancyFlags.Num() - 1)
		, CachedFlushState(Runner->FlushState)
		, CachedCurrentFlushState(Runner->CurrentFlushState)
	{
		check(ReentrancyLevel >= 0);
		checkf(Linker->RunnerReentrancyFlags[ReentrancyLevel] == false, TEXT("Nested FMovieSceneEntitySystemEvaluationReentrancyWindows are not supported for the same active runner"));

		Linker->RunnerReentrancyFlags[ReentrancyLevel] = true;

		// Clear the current flush state to prevent inner scopes from flushing
		//      our pending states and set the flag that allows us to be re-entrant
		Runner->FlushState = UE::MovieScene::ERunnerFlushState::None;
		Runner->CurrentFlushState = UE::MovieScene::ERunnerFlushState::None;
	}

	~FMovieSceneEntitySystemEvaluationReentrancyWindow()
	{
		// If a re-entrant call left some evaluation still outstanding, we have to flush that before we can continue
		if (Runner->FlushState != UE::MovieScene::ERunnerFlushState::None)
		{
			Runner->FlushOutstanding();
		}

		Runner->FlushState = CachedFlushState;
		Runner->CurrentFlushState = CachedCurrentFlushState;
		if (ensure(Linker->RunnerReentrancyFlags.IsValidIndex(ReentrancyLevel)))
		{
			Linker->RunnerReentrancyFlags[ReentrancyLevel] = false;
		}
	}
};

FMovieSceneEntitySystemRunner::FMovieSceneEntitySystemRunner()
	: GameThread(ENamedThreads::GameThread_Local)
	, CurrentPhase(UE::MovieScene::ESystemPhase::None)
	, FlushState(UE::MovieScene::ERunnerFlushState::None)
	, CurrentFlushState(UE::MovieScene::ERunnerFlushState::None)
	, bRequireFullFlush(false)
	, bIsUpdatingSequence(false)
{
}

FMovieSceneEntitySystemRunner::FMovieSceneEntitySystemRunner(UMovieSceneEntitySystemLinker* InLinker)
	: WeakLinker(InLinker)
	, GameThread(ENamedThreads::GameThread_Local)
	, CurrentPhase(UE::MovieScene::ESystemPhase::None)
	, FlushState(UE::MovieScene::ERunnerFlushState::None)
	, CurrentFlushState(UE::MovieScene::ERunnerFlushState::None)
	, bRequireFullFlush(false)
	, bIsUpdatingSequence(false)
{
}

FMovieSceneEntitySystemRunner::~FMovieSceneEntitySystemRunner()
{
}

UE::MovieScene::FEntityManager* FMovieSceneEntitySystemRunner::GetEntityManager() const
{
	if (UMovieSceneEntitySystemLinker* Linker = GetLinker())
	{
		return &Linker->EntityManager;
	}
	return nullptr;
}

UE::MovieScene::FInstanceRegistry* FMovieSceneEntitySystemRunner::GetInstanceRegistry() const
{
	if (UMovieSceneEntitySystemLinker* Linker = GetLinker())
	{
		return Linker->GetInstanceRegistry();
	}
	return nullptr;
}

bool FMovieSceneEntitySystemRunner::IsCurrentlyEvaluating() const
{
	return FlushState != UE::MovieScene::ERunnerFlushState::None;
}

bool FMovieSceneEntitySystemRunner::IsUpdatingSequence() const
{
	return bIsUpdatingSequence;
}

int32 FMovieSceneEntitySystemRunner::GetQueuedUpdateCount() const
{
	return UpdateQueue.Num() + DissectedUpdates.Num();
}

bool FMovieSceneEntitySystemRunner::HasQueuedUpdates() const
{
	if (UpdateQueue.Num() != 0 || DissectedUpdates.Num() != 0)
	{
		return true;
	}
	if (const UMovieSceneEntitySystemLinker* Linker = GetLinker())
	{
		return Linker->HasStructureChangedSinceLastRun();
	}
	return false;
}

bool FMovieSceneEntitySystemRunner::HasQueuedUpdates(FInstanceHandle InInstanceHandle) const
{
	using namespace UE::MovieScene;

	return Algo::FindBy(UpdateQueue, InInstanceHandle, [](const FUpdateParamsAndContext& In){ return In.Params.InstanceHandle; }) != nullptr ||
		Algo::FindBy(DissectedUpdates, InInstanceHandle, &FDissectedUpdate::InstanceHandle) != nullptr;
}

void FMovieSceneEntitySystemRunner::QueueUpdate(const FMovieSceneContext& Context, FInstanceHandle Instance, UE::MovieScene::ERunnerUpdateFlags Flags)
{
	if (EnumHasAnyFlags(Flags, UE::MovieScene::ERunnerUpdateFlags::Flush))
	{
		bRequireFullFlush = true;
	}
	UpdateQueue.Add(FUpdateParamsAndContext{ FSimpleDelegate(), Context, { Instance, Flags } });

#if !NO_LOGGING
	if (LogMovieSceneECS.GetVerbosity() >= ELogVerbosity::VeryVerbose)
	{
		using namespace UE::MovieScene;

		const FInstanceRegistry* InstanceRegistry = GetLinker()->GetInstanceRegistry();
		const FSequenceInstance& SequenceInstance = InstanceRegistry->GetInstance(Instance);
		const TSharedRef<FSharedPlaybackState> SharedPlaybackState = SequenceInstance.GetSharedPlaybackState();
		const UMovieSceneSequence* Sequence = SharedPlaybackState->GetSequence(SequenceInstance.GetSequenceID());

		UE_LOGF(LogMovieSceneECS, VeryVerbose, "Queue update: Instance(%d,%d); Status='%ls'; Range=%ls; Sequence='%ls'",
				Instance.InstanceID, Instance.InstanceSerial,
				*UEnum::GetValueAsString(Context.GetStatus()),
				*LexToString(Context.GetRange()),
				*GetNameSafe(Sequence));
	}
#endif
}

void FMovieSceneEntitySystemRunner::QueueUpdate(const FMovieSceneContext& Context, FInstanceHandle Instance, FSimpleDelegate&& OnFlushedDelegate, UE::MovieScene::ERunnerUpdateFlags Flags)
{
	if (EnumHasAnyFlags(Flags, UE::MovieScene::ERunnerUpdateFlags::Flush))
	{
		bRequireFullFlush = true;
	}

	UpdateQueue.Add(FUpdateParamsAndContext{ OnFlushedDelegate, Context, { Instance, Flags } });

#if !NO_LOGGING
	if (LogMovieSceneECS.GetVerbosity() >= ELogVerbosity::VeryVerbose)
	{
		using namespace UE::MovieScene;

		const FInstanceRegistry* InstanceRegistry = GetLinker()->GetInstanceRegistry();
		const FSequenceInstance& SequenceInstance = InstanceRegistry->GetInstance(Instance);
		const TSharedRef<FSharedPlaybackState> SharedPlaybackState = SequenceInstance.GetSharedPlaybackState();
		const UMovieSceneSequence* Sequence = SharedPlaybackState->GetSequence(SequenceInstance.GetSequenceID());

		UE_LOGF(LogMovieSceneECS, VeryVerbose, "Queue update: Instance(%d,%d); Status='%ls'; Range=%ls; Sequence='%ls'",
				Instance.InstanceID, Instance.InstanceSerial,
				*UEnum::GetValueAsString(Context.GetStatus()),
				*LexToString(Context.GetRange()),
				*GetNameSafe(Sequence));
	}
#endif
}

bool FMovieSceneEntitySystemRunner::QueueFinalUpdate(FInstanceHandle InInstanceHandle)
{
	return QueueFinalUpdateImpl(InInstanceHandle, FSimpleDelegate(), false);
}

bool FMovieSceneEntitySystemRunner::QueueFinalUpdate(FInstanceHandle InInstanceHandle, FSimpleDelegate&& InOnLastFlushDelegate)
{
	return QueueFinalUpdateImpl(InInstanceHandle, MoveTemp(InOnLastFlushDelegate), false);
}

bool FMovieSceneEntitySystemRunner::QueueFinalUpdateAndDestroy(FInstanceHandle InInstanceHandle)
{
	return QueueFinalUpdateImpl(InInstanceHandle, FSimpleDelegate(), true);
}

bool FMovieSceneEntitySystemRunner::QueueFinalUpdateAndDestroy(FInstanceHandle InInstanceHandle, FSimpleDelegate&& InOnLastFlushDelegate)
{
	return QueueFinalUpdateImpl(InInstanceHandle, MoveTemp(InOnLastFlushDelegate), true);
}

bool FMovieSceneEntitySystemRunner::QueueFinalUpdateImpl(FInstanceHandle InInstanceHandle, FSimpleDelegate&& InOnLastFlushDelegate, bool bDestroyInstance)
{
	using namespace UE::MovieScene;

	UMovieSceneEntitySystemLinker* Linker = GetLinker();
	if (!Linker)
	{
		return false;
	}

	FInstanceRegistry* InstanceRegistry = GetInstanceRegistry();
	if (!InstanceRegistry->IsHandleValid(InInstanceHandle))
	{
		return false;
	}

	FSequenceInstance& Instance = InstanceRegistry->MutateInstance(InInstanceHandle);

	// If the following conditions are met, we can destroy this instance right away:
	// 
	// 1. the instance has no entities left to unlink
	// 2. we're not in the middle of an update loop 
	// 3. the instance has no current updates
	//
	const bool bCanFinishImmediately = Instance.CanFinishImmediately();
	const ERunnerFlushState UnsafeDestroyMask = FlushState::Everything & ~(ERunnerFlushState::PostEvaluation | ERunnerFlushState::End);
	const bool bSafeToDestroyNow = !EnumHasAnyFlags(FlushState, UnsafeDestroyMask);
	if (bCanFinishImmediately && bSafeToDestroyNow && !HasQueuedUpdates(InInstanceHandle))
	{
		Instance.Finish();
		Instance.PostEvaluation();

		InOnLastFlushDelegate.ExecuteIfBound();

		// PostEvaluation could have prompted a bunch of other logic that may have destroyed our instance handle so we have to check it for validity again
		if (bDestroyInstance && InstanceRegistry->IsHandleValid(InInstanceHandle))
		{
			InstanceRegistry->DestroyInstance(InInstanceHandle);
		}

		return false;
	}

	// We queue up one last update that unlinks all of this instance's entities and lets systems tear down
	// anything that has notable side-effects.
	// It's possible that the instance already had an update request queued. We leave it in so it gets honored,
	// and this second update request will be fulfilled in a second flush pass.
	FUpdateParamsAndContext LastUpdate;
	LastUpdate.Params.InstanceHandle = InInstanceHandle;
	LastUpdate.Params.UpdateFlags = (bDestroyInstance ? (ERunnerUpdateFlags::Finish | ERunnerUpdateFlags::Destroy) : ERunnerUpdateFlags::Finish);
	LastUpdate.OnFlushed = InOnLastFlushDelegate;
	UpdateQueue.Add(LastUpdate);

#if !NO_LOGGING
	if (LogMovieSceneECS.GetVerbosity() >= ELogVerbosity::VeryVerbose)
	{
		using namespace UE::MovieScene;

		const TSharedRef<FSharedPlaybackState> SharedPlaybackState = Instance.GetSharedPlaybackState();
		const UMovieSceneSequence* Sequence = SharedPlaybackState->GetSequence(Instance.GetSequenceID());

		UE_LOGF(LogMovieSceneECS, VeryVerbose, "Queue update: Instance(%d,%d); %ls; Sequence='%ls'",
				InInstanceHandle.InstanceID, InInstanceHandle.InstanceSerial,
				bDestroyInstance ? TEXT("Destroy Instance") : TEXT("Finish Instance"),
				*GetNameSafe(Sequence));
	}
#endif

	return true;
}

void FMovieSceneEntitySystemRunner::AbandonAndDestroyInstance(FInstanceHandle Instance)
{
	if (QueueFinalUpdateAndDestroy(Instance))
	{
		Flush();
	}
}

void FMovieSceneEntitySystemRunner::AbandonAndDestroyInstance(FInstanceHandle Instance, FSimpleDelegate&& InOnLastFlushDelegate)
{
	if (QueueFinalUpdateAndDestroy(Instance, MoveTemp(InOnLastFlushDelegate)))
	{
		Flush();
	}
}

UE::MovieScene::ERunnerFlushResult FMovieSceneEntitySystemRunner::StartEvaluation(UMovieSceneEntitySystemLinker* Linker)
{
	using namespace UE::MovieScene;

	// We specifically only check whether the entity manager has changed since the last instantiation once
	// to ensure that we are not vulnerable to infinite loops where components are added/removed in post-evaluation
	const bool bStructureHadChanged = Linker->HasStructureChangedSinceLastRun();

	if (!bStructureHadChanged && UpdateQueue.Num() == 0 && DissectedUpdates.Num() == 0)
	{
		// If nothing has changed, and we have no pending updates, we don't run any evaluation
		FlushState = ERunnerFlushState::None;
		return ERunnerFlushResult::Break;
	}

	if (!Linker->StartEvaluation())
	{
		return ERunnerFlushResult::Break;
	}

	EnterFlushState(ERunnerFlushState::Start);

	// Our entity manager cannot be locked down for us to continue. Something must have left it locked if 
	// this check fails
	FEntityManager& EntityManager = Linker->EntityManager;
	check(!EntityManager.IsLockedDown());

	EntityManager.SetDispatchThread(ENamedThreads::GameThread_Local);
	EntityManager.SetGatherThread(ENamedThreads::GameThread_Local);

	return ERunnerFlushResult::ContinueAllowBudget;
}

void FMovieSceneEntitySystemRunner::EndEvaluation(UMovieSceneEntitySystemLinker* Linker)
{
	using namespace UE::MovieScene;
	Linker->EndEvaluation();
}

UE::MovieScene::ERunnerFlushResult FMovieSceneEntitySystemRunner::FlushNext(UMovieSceneEntitySystemLinker* Linker)
{
	using namespace UE::MovieScene;

	// The evaluation loop is set up as a crude state machine which allows us to break out of the
	// loop and pick up where we left off next frame. Each bit within the FlushState bitmask defines
	// a particular task that needs to execute. In this way we can loop the evaluation while there
	// are updates pending by simply setting the necessary flags for each task. For convenience, the
	// required tasks to perform another iteration are defined in ERunnerFlushState::LoopEval

	// Guard the value of the CurrentPhase so it always returns back to ::None
	TGuardValue<ESystemPhase> CurrentPhaseGuard(CurrentPhase, ESystemPhase::None);
	TGuardValue<ERunnerFlushState> CurrentFlushStateGuard(CurrentFlushState, ERunnerFlushState::None);

	// Step 1: Initialize all external flags and broadcast 'begin eval' events.
	//         NOTE: This is only ever called once, regardless of how many iterations we perform
	if (EnumHasAnyFlags(FlushState, ERunnerFlushState::Start))
	{
		// EnterFlushState is called by StartEvaluation itself since it can fail
		return StartEvaluation(Linker);
	}

	// Step 2: Execute any conditional recompiles for dirtied sequences.
	if (EnumHasAnyFlags(FlushState, ERunnerFlushState::ConditionalRecompile))
	{
		EnterFlushState(ERunnerFlushState::ConditionalRecompile);
		return GameThread_ConditionalRecompile(Linker);
	}

	// Step 3: Update sequence instances and import entities
	//         This is the entry-point for for each iteration
	if (EnumHasAnyFlags(FlushState, ERunnerFlushState::Import))
	{
		EnterFlushState(ERunnerFlushState::Import);
		return GameThread_UpdateSequenceInstances(Linker);
	}

	// Step 4: Maybe re-update sequence instances if a recompile has ocurred after they were last updated, but before evaluation finished
	if (EnumHasAnyFlags(FlushState, ERunnerFlushState::ReimportAfterCompile))
	{
		EnterFlushState(ERunnerFlushState::ReimportAfterCompile);
		return GameThread_ReimportSequenceInstances(Linker);
	}

	// Step 5: Conditionally run the spawn phase of the system graph
	//
	if (EnumHasAnyFlags(FlushState, ERunnerFlushState::Spawn))
	{
		EnterFlushState(ERunnerFlushState::Spawn);
		return GameThread_SpawnPhase(Linker);
	}

	// Step 6: Conditionally run the instantiation phase of the system graph
	//
	if (EnumHasAnyFlags(FlushState, ERunnerFlushState::Instantiation))
	{
		EnterFlushState(ERunnerFlushState::Instantiation);
		return GameThread_InstantiationPhase(Linker);
	}

	// Step 7: Run the evaluation phase of the system graph
	//
	if (EnumHasAnyFlags(FlushState, ERunnerFlushState::Evaluation))
	{
		EnterFlushState(ERunnerFlushState::Evaluation);
		return GameThread_EvaluationPhase(Linker);
	}

	// Step 8: Run the finalization phase of the system graph, including legacy templates
	//
	if (EnumHasAnyFlags(FlushState, ERunnerFlushState::Finalization))
	{
		EnterFlushState(ERunnerFlushState::Finalization);
		GameThread_EvaluationFinalizationPhase(Linker);
		return ERunnerFlushResult::ContinueAllowBudget;
	}

	// Step 9: Trigger events
	//
	if (EnumHasAnyFlags(FlushState, ERunnerFlushState::EventTriggers))
	{
		EnterFlushState(ERunnerFlushState::EventTriggers);
		GameThread_EventTriggerPhase(Linker);
		return ERunnerFlushResult::ContinueAllowBudget;
	}

	// Step 10: Call PostEvaluation on all current sequence instances
	//
	if (EnumHasAnyFlags(FlushState, ERunnerFlushState::PostEvaluation))
	{
		EnterFlushState(ERunnerFlushState::PostEvaluation);
		GameThread_PostEvaluationPhase(Linker);
		return ERunnerFlushResult::ContinueAllowBudget;
	}

	// Step 11: Perform any clean up and broadcast 'end eval' events
	//          NOTE: Only ever called once regardless of how many iterations we perform
	if (EnumHasAnyFlags(FlushState, ERunnerFlushState::End))
	{
		EnterFlushState(ERunnerFlushState::End);
		EndEvaluation(Linker);
		return ERunnerFlushResult::ContinueAllowBudget;
	}

	return ERunnerFlushResult::Break;
}

void FMovieSceneEntitySystemRunner::Flush(double BudgetMs, UE::MovieScene::ERunnerFlushState TargetState)
{
	using namespace UE::MovieScene;

	// If we're not currently evaluating, start by flushing everything
	if (FlushState == ERunnerFlushState::None)
	{
		FlushState = FlushState::Everything;
	}
	if (TargetState == ERunnerFlushState::None)
	{
		TargetState = FlushState::Everything;
	}

	FlushOutstanding(BudgetMs, TargetState);
}

void FMovieSceneEntitySystemRunner::FlushOutstanding(double BudgetMs, UE::MovieScene::ERunnerFlushState TargetState)
{
	using namespace UE::MovieScene;

	// Simple case is we have nothing outstanding, so just early return
	if (FlushState == ERunnerFlushState::None)
	{
		return;
	}

	// If this runner is already being flushed, early return
	if (CurrentFlushState != ERunnerFlushState::None)
	{
		UE_LOGF(LogMovieSceneECS, Warning, "Cannot flush this runner while it is already being flushed outside of a re-entrancy window");
		return;
	}

	// If this runner's linker has been destroyed, early return
	UMovieSceneEntitySystemLinker* Linker = WeakLinker.Get();
	if (!Linker)
	{
		return;
	}

	// Fire PreFlush before any phases run, allowing systems to perform
	// work that must happen outside the ECS pipeline (e.g. rebuilding
	// caches that require re-entrant evaluation).
	Linker->Events.PreFlush.Broadcast(Linker);

	SCOPE_CYCLE_COUNTER(MovieSceneEval_RunnerFlush);

	// We need to run the system from the game thread so we know we can fire events and callbacks from here.
	check(IsInGameThread());

	// Set the debug vizualizer's entity manager pointer, so all debugging happening here will show
	// relevant information. We need to set it here instead of higher up because we could have, say,
	// a blocking sequence triggering another blocking sequence via an event track. The nested call stack
	// of the second sequence needs to show debug information relevant to its private linker, but when
	// we return back up to the first sequence, which might still have another update round (such as
	// the other side of the dissected update range around the event), we need to set the pointer back
	// again.
	TGuardValue<FEntityManager*> DebugVizGuard(GEntityManagerForDebuggingVisualizers, GetEntityManager());

	// For the purposes of this function, None means that everything must be flushed
	if (TargetState == ERunnerFlushState::None)
	{
		TargetState = FlushState::Everything;
	}

	UE_LOGF(LogMovieSceneECS, VeryVerbose, "Flushing ECS runner state '0x%#08x' with budget %fms", EnumToUnderlyingType(FlushState), BudgetMs);

	const double BudgetSeconds = BudgetMs / 1000.f;
	if (bRequireFullFlush)
	{
		while (EnumHasAnyFlags(FlushState, FlushState::Everything))
		{
			if (FlushNext(Linker) == ERunnerFlushResult::Break)
			{
				break;
			}
		}

		bRequireFullFlush = false;
	}
	else if (BudgetSeconds > 0.0)
	{
		double StartTime = FPlatformTime::Seconds();
		while (EnumHasAnyFlags(FlushState, TargetState))
		{
			ERunnerFlushResult Result = FlushNext(Linker);
			if (Result == ERunnerFlushResult::Break)
			{
				break;
			}

			if (Result == ERunnerFlushResult::ContinueAllowBudget && FPlatformTime::Seconds() - StartTime >= BudgetSeconds)
			{
				break;
			}
		}
	}
	else
	{
		while (EnumHasAnyFlags(FlushState, TargetState))
		{
			if (FlushNext(Linker) == ERunnerFlushResult::Break)
			{
				break;
			}
		}
	}

	// If we are not right at the end of evaluation, we must check for compilation next time
	if (FlushState != ERunnerFlushState::None && FlushState != ERunnerFlushState::End)
	{
		FlushState |= ERunnerFlushState::ConditionalRecompile;
	}
}

void FMovieSceneEntitySystemRunner::ResetFlushState()
{
	using namespace UE::MovieScene;

	constexpr ERunnerFlushState StatesThatTriggerReset = ERunnerFlushState::Instantiation | ERunnerFlushState::Evaluation | ERunnerFlushState::Finalization | ERunnerFlushState::EventTriggers;
	constexpr ERunnerFlushState StatesThatDontTriggerReset = ERunnerFlushState::Import | ERunnerFlushState::Spawn;

	// We only need to reset the flush state if we are far enough through an evaluation
	if (EnumHasAnyFlags(FlushState, StatesThatTriggerReset) && !EnumHasAnyFlags(FlushState, StatesThatDontTriggerReset))
	{
		// When resetting - we don't need to (or want to) re-import anything, we just want to re-run the current
		// frame of updates 
		FlushState = UE::MovieScene::FlushState::LoopEval & ~UE::MovieScene::ERunnerFlushState::Import;
	}
}

void FMovieSceneEntitySystemRunner::DiscardQueuedUpdates(FInstanceHandle Instance)
{
	using namespace UE::MovieScene;

	for (int32 Index = UpdateQueue.Num()-1; Index >= 0; --Index)
	{
		if (UpdateQueue[Index].Params.InstanceHandle == Instance)
		{
			UpdateQueue.RemoveAt(Index, EAllowShrinking::No);
		}
	}
	for (int32 Index = DissectedUpdates.Num()-1; Index >= 0; --Index)
	{
		if (DissectedUpdates[Index].InstanceHandle == Instance)
		{
			DissectedUpdates.RemoveAt(Index, EAllowShrinking::No);
		}
	}

	for (int32 Index = CurrentInstances.Num()-1; Index >= 0; --Index)
	{
		if (CurrentInstances[Index].InstanceHandle == Instance)
		{
			CurrentInstances.RemoveAt(Index, EAllowShrinking::No);
		}
	}
}

void FMovieSceneEntitySystemRunner::EnterFlushState(UE::MovieScene::ERunnerFlushState EnteredFlushState)
{
	// Stop this state from being run again and allow the next one to run
	FlushState &= ~EnteredFlushState;
	// Mark this as the current state
	CurrentFlushState = EnteredFlushState;
}

void FMovieSceneEntitySystemRunner::SkipFlushState(UE::MovieScene::ERunnerFlushState FlushStateToSkip)
{
	FlushState &= ~FlushStateToSkip;
}

UE::MovieScene::ERunnerFlushResult FMovieSceneEntitySystemRunner::GameThread_ConditionalRecompile(UMovieSceneEntitySystemLinker* Linker)
{
	using namespace UE::MovieScene;

	TRACE_CPUPROFILER_EVENT_SCOPE(FMovieSceneEntitySystemRunner::GameThread_ConditionalRecompile);

	FInstanceRegistry* InstanceRegistry = GetInstanceRegistry();

	bool bAnyRecompile = false;

	// If we have currently running instances, use those for the conditional compile
	if (CurrentInstances.Num())
	{
		for (const FQueuedUpdateParams& UpdatedInstance : CurrentInstances)
		{
			if (InstanceRegistry->IsHandleValid(UpdatedInstance.InstanceHandle) && !EnumHasAnyFlags(UpdatedInstance.UpdateFlags, ERunnerUpdateFlags::Finish | ERunnerUpdateFlags::Destroy))
			{
				FSequenceInstance& Instance = InstanceRegistry->MutateInstance(UpdatedInstance.InstanceHandle);
				if (Instance.IsRootSequence() && Instance.ConditionalRecompile())
				{
					bAnyRecompile = true;
				}
			}
		}
	}
	// Otherwise process the update queue
	else for (int32 UpdateIndex = 0; UpdateIndex < UpdateQueue.Num(); ++UpdateIndex)
	{
		const FUpdateParamsAndContext& Request = UpdateQueue[UpdateIndex];
		if (InstanceRegistry->IsHandleValid(Request.Params.InstanceHandle) && !EnumHasAnyFlags(Request.Params.UpdateFlags, ERunnerUpdateFlags::Finish | ERunnerUpdateFlags::Destroy))
		{
			FSequenceInstance& Instance = InstanceRegistry->MutateInstance(Request.Params.InstanceHandle);
			if (Instance.IsRootSequence() && Instance.ConditionalRecompile())
			{
				bAnyRecompile = true;
			}
		}
	}

	if (bAnyRecompile)
	{
		if (!EnumHasAnyFlags(FlushState, ERunnerFlushState::Import))
		{
			// If we are part-way through a phase that depends on NeedsLink/NeedsUnlink to match,
			// we need to flush through those before we allow Reimport to run
			if (Linker->EntityManager.ContainsComponent(FBuiltInComponentTypes::Get()->Tags.NeedsUnlink) || Linker->EntityManager.ContainsComponent(FBuiltInComponentTypes::Get()->Tags.NeedsLink))
			{
				TGuardValue<ERunnerFlushState> FlushStateGuard(CurrentFlushState, ERunnerFlushState::None);
				FlushOutstanding(0.0, ERunnerFlushState::Spawn | ERunnerFlushState::Instantiation);
			}

			// If we have already imported everything, we have to run a reimport to make sure everything is imported correctly
			FlushState |= ERunnerFlushState::ReimportAfterCompile;
		}

		// If a recompile has occurred, we need to make sure we run everything again (except for checking for another recompile!)
		FlushState |= (FlushState::LoopEval & ~(ERunnerFlushState::ConditionalRecompile | ERunnerFlushState::Import));

		return ERunnerFlushResult::ContinueAllowBudget;
	}

	// Do not allow us to get stuck in an infinite loop checking for compilation
	return ERunnerFlushResult::ContinueNoBudgeting;
}

UE::MovieScene::ERunnerFlushResult FMovieSceneEntitySystemRunner::GameThread_UpdateSequenceInstances(UMovieSceneEntitySystemLinker* Linker)
{
	using namespace UE::MovieScene;

	TRACE_CPUPROFILER_EVENT_SCOPE(FMovieSceneEntitySystemRunner::GameThread_UpdateSequenceInstances);

	// Also reset the capture source scope so that each group of sequences tied to a given linker starts
	// with a clean slate.
	TGuardValue<FScopedPreAnimatedCaptureSource*> CaptureSourceGuard(FScopedPreAnimatedCaptureSource::GetCaptureSourcePtr(), nullptr);

	FInstanceRegistry* InstanceRegistry = GetInstanceRegistry();

	if (DissectedUpdates.Num() == 0)
	{
		TArray<TRange<FFrameTime>> Dissections;
		TBitArray<> UpdatedSequenceInstances;

		TArray<FUpdateParamsAndContext> TempUpdateQueue;
		Swap(TempUpdateQueue, UpdateQueue);

		DissectedUpdates.Reserve(TempUpdateQueue.Num());

		for (int32 UpdateIndex = 0; UpdateIndex < TempUpdateQueue.Num(); ++UpdateIndex)
		{
			FUpdateParamsAndContext& Request = TempUpdateQueue[UpdateIndex];
			const int32 InstanceID = Request.Params.InstanceHandle.InstanceID;

			if (!InstanceRegistry->IsHandleValid(Request.Params.InstanceHandle))
			{
				continue;
			}
			else if (UpdatedSequenceInstances.IsValidIndex(InstanceID) && UpdatedSequenceInstances[InstanceID] == true)
			{
				// We already have an update for this instance, so we queue it up again for the next flush
				UpdateQueue.Add(Request);
				continue;
			}

			// Set the bit to indicate this sequence is being updated
			UpdatedSequenceInstances.PadToNum(InstanceID + 1, false);
			UpdatedSequenceInstances[InstanceID] = true;

			// If the request is to destroy the instance, we must assume it is no longer valid and should
			// not be updated
			if (EnumHasAnyFlags(Request.Params.UpdateFlags, ERunnerUpdateFlags::Finish | ERunnerUpdateFlags::Destroy))
			{
				DissectedUpdates.Add(FDissectedUpdate{ MoveTemp(Request.OnFlushed), Request.Context, Request.Params.InstanceHandle, MAX_int32, Request.Params.UpdateFlags });
				MarkForUpdate(Request.Params.InstanceHandle, Request.Params.UpdateFlags);
				continue;
			}

			// Give the instance an opportunity to dissect the range into distinct evaluations
			FSequenceInstance& Instance = InstanceRegistry->MutateInstance(Request.Params.InstanceHandle);
			if (!Instance.IsRootSequence())
			{
				TSharedRef<const FSharedPlaybackState> SharedPlaybackState = Instance.GetSharedPlaybackState();
				UMovieSceneSequence* RootSequence = SharedPlaybackState->GetRootSequence();
				UMovieSceneSequence* SubSequence  = SharedPlaybackState->GetSequence(Instance.GetSequenceID());

				ensureMsgf(Instance.IsRootSequence(), TEXT("Update request received for a non-root sequence ID 0x%08X (%s) in root-sequence %s. This is not supported."),
					Instance.GetSequenceID().GetInternalValue(),
					SubSequence  ? *SubSequence->GetName()  : TEXT("<nullptr>"),
					RootSequence ? *RootSequence->GetName() : TEXT("<nullptr>")
				);
				continue;
			}

			Instance.DissectContext(Request.Context, Dissections);

			if (Dissections.Num() != 0)
			{
				if (Request.Context.GetDirection() == EPlayDirection::Backwards)
				{
					Algo::Reverse(Dissections);
				}

				bool bHasJumped = Request.Context.HasJumped();

				for (int32 Index = 0; Index < Dissections.Num() - 1; ++Index)
				{
					// Never finish or destroy sequence instances until the _last_ dissected update
					ERunnerUpdateFlags Flags = Request.Params.UpdateFlags & ~ERunnerUpdateFlags::FinalDissectionMask;

					FDissectedUpdate Dissection{
						FSimpleDelegate(), // Never trigger the OnFlushed delegate until the final dissection
						FMovieSceneContext(FMovieSceneEvaluationRange(Dissections[Index], Request.Context.GetFrameRate(), Request.Context.GetDirection()), Request.Context.GetStatus()).SetHasJumped(bHasJumped),
						Request.Params.InstanceHandle,
						Index,
						Flags
					};
					DissectedUpdates.Add(Dissection);

					// Only the first dissection needs to know if we've jumped
					bHasJumped = false;

					// Only mark the first dissection for update this round
					if (Index == 0)
					{
						MarkForUpdate(Request.Params.InstanceHandle, Flags);
					}
				}

				// Add the last one with MAX_int32 so it gets evaluated with all the others in this flush
				FDissectedUpdate Dissection{
					MoveTemp(Request.OnFlushed),
					FMovieSceneContext(FMovieSceneEvaluationRange(Dissections.Last(), Request.Context.GetFrameRate(), Request.Context.GetDirection()), Request.Context.GetStatus()).SetHasJumped(bHasJumped),
					Request.Params.InstanceHandle,
					MAX_int32,
					Request.Params.UpdateFlags
				};
				DissectedUpdates.Add(Dissection);

				// Only mark the first dissection for update this round
				if (Dissections.Num() == 1)
				{
					MarkForUpdate(Request.Params.InstanceHandle, Request.Params.UpdateFlags);
				}

				Dissections.Reset();
			}
			else
			{
				DissectedUpdates.Add(FDissectedUpdate{ MoveTemp(Request.OnFlushed), Request.Context, Request.Params.InstanceHandle, MAX_int32, Request.Params.UpdateFlags });
				MarkForUpdate(Request.Params.InstanceHandle, Request.Params.UpdateFlags);
			}
		}

		Algo::SortBy(DissectedUpdates, &FDissectedUpdate::Order);
	}
	else
	{
		// Look for the next batch of updates, and mark the respective sequence instances as currently updating.
		const int32 PredicateOrder = DissectedUpdates[0].Order;
		for (int32 Index = 0; Index < DissectedUpdates.Num() && DissectedUpdates[Index].Order == PredicateOrder; ++Index)
		{
			FDissectedUpdate& Update = DissectedUpdates[Index];
			MarkForUpdate(Update.InstanceHandle, Update.UpdateFlags);
		}
	}

	// If we have no instances marked for update, we are running an evaluation probably because some
	// structural changes have occurred in the entity manager (out of date instantiation serial number
	// in the linker). So we mark everything for update, so that PreEvaluation/PostEvaluation callbacks
	// and legacy templates are correctly executed.
	if (CurrentInstances.Num() == 0)
	{
		for (const FSequenceInstance& Instance : InstanceRegistry->GetSparseInstances())
		{
			MarkForUpdate(Instance.GetInstanceHandle(), ERunnerUpdateFlags::None);
		}
	}

	AccumulatedUpdateFlags = ESequenceInstanceUpdateFlags::None;

	// Let sequence instances do any pre-evaluation work.
	for (const FQueuedUpdateParams& UpdatedInstance : CurrentInstances)
	{
		if (!EnumHasAnyFlags(UpdatedInstance.UpdateFlags, ERunnerUpdateFlags::Destroy))
		{
			FSequenceInstance& SequenceInstance = InstanceRegistry->MutateInstance(UpdatedInstance.InstanceHandle);

			AccumulatedUpdateFlags |= SequenceInstance.GetUpdateFlags();
			uint16 PlayerIndex = FPlayerIndexPlaybackCapability::GetPlayerIndex(SequenceInstance.GetSharedPlaybackState());
			if (PlayerIndex != (uint16)-1)
			{
				IMovieScenePlayer::SetIsEvaluatingFlag(PlayerIndex, true);
			}

			if (EnumHasAnyFlags(SequenceInstance.GetUpdateFlags(), ESequenceInstanceUpdateFlags::NeedsPreEvaluation))
			{
				SequenceInstance.PreEvaluation();
			}
		}
	}

	// NOTE: IncrementSystemSerial must be called before any instance updates are made
	//       to ensure that up-to-date versions are used inside FEntityManager::OnStructureChanged
	Linker->EntityManager.IncrementSystemSerial();

#if !NO_LOGGING
	if (LogMovieSceneECS.GetVerbosity() >= ELogVerbosity::VeryVerbose && DissectedUpdates.Num() > 0)
	{
		UE_LOGF(LogMovieSceneECS, VeryVerbose, "Executing %d dissected updates", DissectedUpdates.Num());

		for (int32 Index = 0; Index < DissectedUpdates.Num(); ++Index)
		{
			const FDissectedUpdate& DissectedUpdate(DissectedUpdates[Index]);
			const FSequenceInstance& Instance = InstanceRegistry->GetInstance(DissectedUpdate.InstanceHandle);
			const TSharedRef<FSharedPlaybackState> SharedPlaybackState = Instance.GetSharedPlaybackState();
			const UMovieSceneSequence* Sequence = SharedPlaybackState->GetSequence(Instance.GetSequenceID());

			UE_LOGF(LogMovieSceneECS, VeryVerbose,
					"[%d] Order='%ls'; Instance(%d,%d); Status='%ls'; Range=%ls; OnFlushed='%ls'; Flags='%#03x', Sequence='%ls'",
					Index,
					DissectedUpdate.Order == MAX_int32 ? TEXT("MAX") : *LexToString(DissectedUpdate.Order),
					DissectedUpdate.InstanceHandle.InstanceID, DissectedUpdate.InstanceHandle.InstanceSerial,
					*UEnum::GetValueAsString(DissectedUpdate.Context.GetStatus()),
					*LexToString(DissectedUpdate.Context.GetRange()),
					DissectedUpdate.OnFlushed.IsBound() ? TEXT("BOUND") : TEXT("unbound"),
					EnumToUnderlyingType(DissectedUpdate.UpdateFlags),
					*GetNameSafe(Sequence));
		}
	}
#endif

	// Update all systems
	if (DissectedUpdates.Num() != 0)
	{
		TGuardValue<bool> IsUpdatingSequenceGuard(bIsUpdatingSequence, true);

		const int32 PredicateOrder = DissectedUpdates[0].Order;
		int32 Index = 0;
		for (; Index < DissectedUpdates.Num() && DissectedUpdates[Index].Order == PredicateOrder; ++Index)
		{
			FDissectedUpdate& Update = DissectedUpdates[Index];

			// Always forward the OnFlushed delegate to be called at the end of the frame, even if the instance is no longer valid
			if (Update.OnFlushed.IsBound())
			{
				OnFlushedDelegates.Add(MoveTemp(Update.OnFlushed));

				// If we have any on-flushed delegates then we have to do a full Post-Eval phase
				AccumulatedUpdateFlags |= ESequenceInstanceUpdateFlags::NeedsPostEvaluation;
			}

			if (ensure(InstanceRegistry->IsHandleValid(Update.InstanceHandle)))
			{
				FSequenceInstance& Instance = InstanceRegistry->MutateInstance(Update.InstanceHandle);

				if (EnumHasAnyFlags(Update.UpdateFlags, ERunnerUpdateFlags::Finish))
				{
					// Context is irrelevant for Finishing sequences
					Instance.Finish();
				}
				else if (EnumHasAnyFlags(Update.UpdateFlags, ERunnerUpdateFlags::Destroy))
				{
					ensure(Instance.HasFinished());
				}
				else
				{
					Instance.Update(Update.Context);
				}
			}
		}
		DissectedUpdates.RemoveAt(0, Index);
	}

#if !NO_LOGGING
	if (LogMovieSceneECS.GetVerbosity() >= ELogVerbosity::VeryVerbose && CurrentInstances.Num() > 0)
	{
		UE_LOGF(LogMovieSceneECS, VeryVerbose, "Running evaluation for %d current instances to update", CurrentInstances.Num());

		for (int32 Index = 0; Index < CurrentInstances.Num(); ++Index)
		{
			const FQueuedUpdateParams& UpdateParams(CurrentInstances[Index]);
			const FSequenceInstance& Instance = InstanceRegistry->GetInstance(UpdateParams.InstanceHandle);
			const TSharedRef<FSharedPlaybackState> SharedPlaybackState = Instance.GetSharedPlaybackState();
			const UMovieSceneSequence* Sequence = SharedPlaybackState->GetSequence(Instance.GetSequenceID());

			UE_LOGF(LogMovieSceneECS, VeryVerbose,
					"[%d] Instance(%d,%d); Status='%ls'; Range=%ls; Flags='%#03x', Sequence='%ls'",
					Index,
					UpdateParams.InstanceHandle.InstanceID, UpdateParams.InstanceHandle.InstanceSerial,
					*UEnum::GetValueAsString(Instance.GetContext().GetStatus()),
					*LexToString(Instance.GetContext().GetRange()),
					EnumToUnderlyingType(UpdateParams.UpdateFlags),
					*GetNameSafe(Sequence));
		}
	}
#endif

#if DO_GUARD_SLOW
	// Check that there are no duplicates in the CurrentInstances array
	{
		TBitArray<> VisitedBits;
		VisitedBits.Reserve(InstanceRegistry->GetSparseInstances().GetMaxIndex());
		for (const FQueuedUpdateParams& Update : CurrentInstances)
		{
			const int32 Index = Update.InstanceHandle.InstanceID;

			checkf(!VisitedBits.IsValidIndex(Index) || VisitedBits[Index] == false, TEXT("Multiple updates exist for the same sequence instance. This indicates a bookkeeping error with the CurrentInstances array."));

			VisitedBits.PadToNum(Index + 1, false);
			VisitedBits[Index] = true;
		}
	}
#endif

	return ERunnerFlushResult::ContinueAllowBudget;
}

UE::MovieScene::ERunnerFlushResult FMovieSceneEntitySystemRunner::GameThread_ReimportSequenceInstances(UMovieSceneEntitySystemLinker* Linker)
{
	using namespace UE::MovieScene;

	// Only called after a sequence has been recompiled after we have already updated the current instances
	// This allows us to re-update all the sequence instances in case anything has changed

	TGuardValue<bool> IsUpdatingSequenceGuard(bIsUpdatingSequence, true);

	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();

	// NOTE: IncrementSystemSerial must be called before any instance updates are made
	//       to ensure that up-to-date versions are used inside FEntityManager::OnStructureChanged
	Linker->EntityManager.IncrementSystemSerial();

	// Operate on a copy of the update params since it is possible CurrentInstances can change if sequence instances are destroyed
	TArray<FQueuedUpdateParams> CurrentInstancesCopy(MoveTemp(CurrentInstances));
	CurrentInstances.Reset(CurrentInstancesCopy.Num());

	for (const FQueuedUpdateParams& UpdatedInstance : CurrentInstancesCopy)
	{
		if (!InstanceRegistry->IsHandleValid(UpdatedInstance.InstanceHandle))
		{
			continue;
		}

		if (EnumHasAnyFlags(UpdatedInstance.UpdateFlags, ERunnerUpdateFlags::Destroy | ERunnerUpdateFlags::Finish))
		{
			FSequenceInstance& SequenceInstance = InstanceRegistry->MutateInstance(UpdatedInstance.InstanceHandle);
			// Need to check whether this has actually finished yet or not
			if (!SequenceInstance.HasFinished())
			{
				SequenceInstance.Finish();
			}

			// If it is being finished or destroyed, just re-add this update back.
			CurrentInstances.Add(UpdatedInstance);
		}
		else
		{
			FSequenceInstance& SequenceInstance = InstanceRegistry->MutateInstance(UpdatedInstance.InstanceHandle);
			// Only update root instances
			if (SequenceInstance.IsRootSequence())
			{
				// Add root sequences back to the CurrentInstances array. When we call Update any active sub-sequences will be re-added via MarkForUpdate
				CurrentInstances.Add(UpdatedInstance);
				SequenceInstance.Update(SequenceInstance.GetContext());
			}
		}
	}

	return ERunnerFlushResult::ContinueAllowBudget;
}

UE::MovieScene::ERunnerFlushResult FMovieSceneEntitySystemRunner::GameThread_SpawnPhase(UMovieSceneEntitySystemLinker* Linker)
{
	using namespace UE::MovieScene;

	check(GameThread == ENamedThreads::GameThread || GameThread == ENamedThreads::GameThread_Local);


	CurrentPhase = ESystemPhase::Spawn;

	FInstanceRegistry* InstanceRegistry = GetInstanceRegistry();

	const bool bInstantiationDirty = Linker->HasStructureChangedSinceLastRun() || InstanceRegistry->HasInvalidatedBindings();

	FGraphEventArray AllTasks;

	Linker->AutoLinkRelevantSystems();

	// --------------------------------------------------------------------------------------------------------------------------------------------
	// Run the spawn phase if there were any changes to the current entity instantiations, and wait on the result
	{
		SCOPE_CYCLE_COUNTER(MovieSceneEval_SpawnPhase);
		if (bInstantiationDirty)
		{
			// The spawn phase can queue events to trigger from the event tracks.
			bCanQueueEventTriggers = true;
			{
				Linker->EntityManager.UpdateThreadingModel();
				Linker->SystemGraph.ExecutePhase(ESystemPhase::Spawn, Linker, AllTasks);
			}
			bCanQueueEventTriggers = false;

			// We don't open a re-entrancy window, however, because there's no way we can recursively evaluate things at this point... too many
			// things are in an intermediate state. So events triggered as PreSpawn/PostSpawn can't be wired to something that starts a sequence.
			if (EventTriggers.IsBound())
			{
				EventTriggers.Broadcast();
				EventTriggers.Clear();
			}
		}

		// If there were any tasks created, we need to wait on them before proceeding. This is rare for the spawn phase.
		if (AllTasks.Num() != 0)
		{
			FGraphEventRef SpawnEvent = TGraphTask<FNullGraphTask>::CreateTask(&AllTasks, ENamedThreads::GameThread)
			.ConstructAndDispatchWhenReady(TStatId(), GameThread);

			FTaskGraphInterface::Get().WaitUntilTaskCompletes(SpawnEvent, ENamedThreads::GameThread_Local);
		}
	}

	{
		SCOPE_CYCLE_COUNTER(MovieSceneEval_PostSpawnEvent);
		Linker->Events.PostSpawnEvent.Broadcast(Linker);
	}

	// --------------------------------------------------------------------------------------------------------------------------------------------
	// Only run the instantiation phase if there is anything to instantiate. This must come after the spawn phase because new instantiations may
	// be created during the spawn phase
	const bool bAnyPending = Linker->EntityManager.ContainsAnyComponent(FBuiltInComponentTypes::Get()->RequiresInstantiationMask) || InstanceRegistry->HasInvalidatedBindings();
	if (bInstantiationDirty == false && bAnyPending == false)
	{
		SkipFlushState(ERunnerFlushState::Instantiation);
	}

	return ERunnerFlushResult::ContinueAllowBudget;
}

UE::MovieScene::ERunnerFlushResult FMovieSceneEntitySystemRunner::GameThread_InstantiationPhase(UMovieSceneEntitySystemLinker* Linker)
{
	using namespace UE::MovieScene;

	SCOPE_CYCLE_COUNTER(MovieSceneEval_InstantiationPhase);

	check(GameThread == ENamedThreads::GameThread || GameThread == ENamedThreads::GameThread_Local);

	CurrentPhase = ESystemPhase::Instantiation;

	Linker->EntityManager.UpdateThreadingModel();

	FGraphEventArray AllTasks;
	Linker->SystemGraph.ExecutePhase(ESystemPhase::Instantiation, Linker, AllTasks);

	// If there were any tasks created, we need to wait on them before proceeding. This is rare for the instantiation phase.
	if (AllTasks.Num() != 0)
	{
		FGraphEventRef InstantiationEvent = TGraphTask<FNullGraphTask>::CreateTask(&AllTasks, ENamedThreads::GameThread)
		.ConstructAndDispatchWhenReady(TStatId(), GameThread);

		FTaskGraphInterface::Get().WaitUntilTaskCompletes(InstantiationEvent, ENamedThreads::GameThread_Local);
	}

	return GameThread_PostInstantiation(Linker);
}

UE::MovieScene::ERunnerFlushResult FMovieSceneEntitySystemRunner::GameThread_PostInstantiation(UMovieSceneEntitySystemLinker* Linker)
{
	using namespace UE::MovieScene;

	SCOPE_CYCLE_COUNTER(MovieSceneEval_PostInstantiation);

	check(GameThread == ENamedThreads::GameThread || GameThread == ENamedThreads::GameThread_Local);

	Linker->PostInstantation();

	FEntityManager& EntityManager = Linker->EntityManager;
	FBuiltInComponentTypes* BuiltInComponentTypes = FBuiltInComponentTypes::Get();

	// Nothing needs linking, caching or restoring any more
	FRemoveMultipleMutation Mutation;
	Mutation.MaskToRemove = BuiltInComponentTypes->RequiresInstantiationMask;
	// Inverse the filter because MaskToRemove is applied as a binary AND against all matching allocations.
	// Therefore, any set bits in RequiresInstantiationMask will be removed from components
	Mutation.MaskToRemove.BitwiseNOT();

	FEntityComponentFilter Filter = FEntityComponentFilter().Any(BuiltInComponentTypes->RequiresInstantiationMask);
	EntityManager.MutateAll(Filter, Mutation);

	// Free anything that has been unlinked
	EntityManager.FreeEntities(FEntityComponentFilter().All({ BuiltInComponentTypes->Tags.NeedsUnlink }));

	Linker->AutoUnlinkIrrelevantSystems();

	EntityManager.Compact();

	if (FEntitySystemScheduler::IsCustomSchedulingEnabled())
	{
		Linker->SystemGraph.ReconstructTaskSchedule(&Linker->EntityManager);
	}
	return ERunnerFlushResult::ContinueAllowBudget;
}

UE::MovieScene::ERunnerFlushResult FMovieSceneEntitySystemRunner::GameThread_EvaluationPhase(UMovieSceneEntitySystemLinker* Linker)
{
	using namespace UE::MovieScene;

	SCOPE_CYCLE_COUNTER(MovieSceneEval_EvaluationPhase);

	CurrentPhase = ESystemPhase::Evaluation;

	// --------------------------------------------------------------------------------------------------------------------------------------------
	// Step 2: Run the evaluation phase. The entity manager is locked down for this phase, meaning no changes to entity-component structure is allowed
	//         This vastly simplifies the concurrent handling of entity component allocations
	Linker->EntityManager.LockDown();

	checkf(!Linker->EntityManager.ContainsComponent(FBuiltInComponentTypes::Get()->Tags.NeedsUnlink), TEXT("Stale entities remain in the entity manager during evaluation - these should have been destroyed during the instantiation phase. Did it run?"));

	Linker->EntityManager.UpdateThreadingModel();

	if (FEntitySystemScheduler::IsCustomSchedulingEnabled())
	{
		Linker->SystemGraph.ScheduleTasks(&Linker->EntityManager);
	}

	FGraphEventArray AllTasks;
	Linker->SystemGraph.ExecutePhase(ESystemPhase::Evaluation, Linker, AllTasks);

	if (AllTasks.Num() != 0)
	{
		FGraphEventRef EvaluationEvent = TGraphTask<FNullGraphTask>::CreateTask(&AllTasks, ENamedThreads::GameThread)
		.ConstructAndDispatchWhenReady(TStatId(), GameThread);

		FTaskGraphInterface::Get().WaitUntilTaskCompletes(EvaluationEvent, ENamedThreads::GameThread_Local);
	}

	Linker->EntityManager.ReleaseLockDown();

	if ( !EnumHasAnyFlags(AccumulatedUpdateFlags, ESequenceInstanceUpdateFlags::HasLegacyTemplates)
		 && Linker->SystemGraph.NumInPhase(ESystemPhase::Finalization) == 0
		 && !EventTriggers.IsBound())
	{
		// Skip Finalization if there's no need for it
		SkipFlushState(ERunnerFlushState::Finalization | ERunnerFlushState::EventTriggers);
	}

	return ERunnerFlushResult::ContinueAllowBudget;
}

void FMovieSceneEntitySystemRunner::GameThread_EvaluationFinalizationPhase(UMovieSceneEntitySystemLinker* Linker)
{
	using namespace UE::MovieScene;

	check(GameThread == ENamedThreads::GameThread || GameThread == ENamedThreads::GameThread_Local);

	CurrentPhase = ESystemPhase::Finalization;

	// Post-eval events can be queued during the finalization phase so let's open that up.
	// The events are actually executed a bit later, in GameThread_EventTriggerPhase.
	bCanQueueEventTriggers = true;
	{
		SCOPE_CYCLE_COUNTER(MovieSceneEval_FinalizationPhase);

		FInstanceRegistry* InstanceRegistry = GetInstanceRegistry();

		// Iterate on a copy of our current instances, since LegacyEvaluator->Evaluate() could change the instance handle, which would affect PostEvaluationPhase
		TArray<FQueuedUpdateParams> CurrentInstancesCopy(CurrentInstances);
		for (const FQueuedUpdateParams& UpdateParams : CurrentInstancesCopy)
		{
			if (InstanceRegistry->IsHandleValid(UpdateParams.InstanceHandle))
			{
				FSequenceInstance& Instance = InstanceRegistry->MutateInstance(UpdateParams.InstanceHandle);
				if (Instance.IsRootSequence())
				{
					Instance.RunLegacyTrackTemplates();
				}
			}
		}

		Linker->EntityManager.UpdateThreadingModel();

		FGraphEventArray Tasks;
		Linker->SystemGraph.ExecutePhase(ESystemPhase::Finalization, Linker, Tasks);
		checkf(Tasks.Num() == 0, TEXT("Cannot dispatch new tasks during finalization"));
	}
	bCanQueueEventTriggers = false;

	if (!EventTriggers.IsBound())
	{
		// Skip event triggers if there are none
		SkipFlushState(ERunnerFlushState::EventTriggers);
	}
}

void FMovieSceneEntitySystemRunner::GameThread_EventTriggerPhase(UMovieSceneEntitySystemLinker* Linker)
{
	using namespace UE::MovieScene;

	// Execute any queued events from the evaluation finalization phase.
	if (EventTriggers.IsBound())
	{
		// Event triggers are allowed to be re-entrant
		FMovieSceneEntitySystemEvaluationReentrancyWindow Window(this, Linker);

		// Trigger from a temporary delegate to ensure that re-entrant evaluations do not re-trigger events
		FMovieSceneEntitySystemEventTriggers TmpEventTriggers;

		Swap(TmpEventTriggers, EventTriggers);
		EventTriggers.Clear();

		TmpEventTriggers.Broadcast();
	}
}

void FMovieSceneEntitySystemRunner::GameThread_PostEvaluationPhase(UMovieSceneEntitySystemLinker* Linker)
{
	using namespace UE::MovieScene;

	SCOPE_CYCLE_COUNTER(MovieSceneEval_PostEvaluationPhase);

	// Now run the post-evaluation logic so that we can safely handle broadcast events (like OnFinished)
	// that trigger some new evaluations (such as connecting it to another sequence's Play in Blueprint).
	//
	// If we are the global linker (and not a "private" linker, as is the case with "blocking" sequences),
	// we may find ourselves in a re-entrant call, which means we need to save our state here and restore
	// it afterwards. We also iterate on a copy of our current instances, since a re-entrant call would
	// modify that array.

	// Temporarily cache the pending updates and dissected updates so we can preserve re-entrant ordering
	//    by appending the arrays afterwards. This ensures that any updates queued during a re-entrant call
	//    get evaluated _before_ the updates that are specified in the outer scope, even if the evaluation was
	//    budgeted and didn't fully flush, whilst simultaneously correctly only flushing inner updates if a
	//    full flush is performed.
	TArray<FQueuedUpdateParams> TmpCurrentInstances;
	TArray<FUpdateParamsAndContext> TmpUpdateQueue;
	TArray<FDissectedUpdate> TmpDissectedUpdates;
	TArray<FInstanceHandle> TmpDestroyInstances;
	TArray<FSimpleDelegate> TmpOnFlushedDelegates;

	Swap(UpdateQueue, TmpUpdateQueue);
	Swap(DissectedUpdates, TmpDissectedUpdates);
	Swap(CurrentInstances, TmpCurrentInstances);
	Swap(OnFlushedDelegates, TmpOnFlushedDelegates);

	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();

	{
		FMovieSceneEntitySystemEvaluationReentrancyWindow Window(this, Linker);

		for (const FQueuedUpdateParams& UpdateParams : TmpCurrentInstances)
		{
			// We must check for validity here because the cache handles may have become invalid
			// during this iteration (since there is a re-entrancy window open)
			if (InstanceRegistry->IsHandleValid(UpdateParams.InstanceHandle))
			{
				FSequenceInstance& Instance = InstanceRegistry->MutateInstance(UpdateParams.InstanceHandle);

				Instance.Ledger.UnlinkOneShots(Linker);
				uint16 PlayerIndex = FPlayerIndexPlaybackCapability::GetPlayerIndex(Instance.GetSharedPlaybackState());
				if (PlayerIndex != (uint16)-1)
				{
					IMovieScenePlayer::SetIsEvaluatingFlag(PlayerIndex, false);
				}

				if (EnumHasAnyFlags(Instance.GetUpdateFlags(), ESequenceInstanceUpdateFlags::NeedsPostEvaluation))
				{
					Instance.PostEvaluation();
				}

				if (EnumHasAnyFlags(UpdateParams.UpdateFlags, ERunnerUpdateFlags::Destroy))
				{
					TmpDestroyInstances.Add(UpdateParams.InstanceHandle);
				}
			}
		}

		for (const FSimpleDelegate& OnFlushed : TmpOnFlushedDelegates)
		{
			OnFlushed.Execute();
		}
	}

	// Destroy any instances that need destroying
	for (FInstanceHandle Handle : TmpDestroyInstances)
	{
		if (InstanceRegistry->IsHandleValid(Handle))
		{
			InstanceRegistry->DestroyInstance(Handle);
		}
	}

	// If we had any updates _before_ we called the external callbacks (which are stored in TmpUpdateQueue), make sure those are maintained first
	Swap(TmpUpdateQueue, UpdateQueue);

	// TmpUpdateQueue now contains any updates that were queued _during_ the external callbacks, so we append those to our old queue (which is now stored back in UpdateQueue)
	if (TmpUpdateQueue.Num() > 0)
	{
		UpdateQueue.Append(TmpUpdateQueue);
	}

	// Do the same with the dissected updates
	Swap(TmpDissectedUpdates, DissectedUpdates);
	if (TmpDissectedUpdates.Num() > 0)
	{
		DissectedUpdates.Append(TmpDissectedUpdates);
		// Important: we do not sort the Dissected updates here to ensure that the previously populated entries are evaluated first
	}

	// If we have any pending updates, we need to run another evaluation.
	// This will cause us to effectively loop back to ERunnerFlushState::Import
	// which will pick up the next updates
	if (UpdateQueue.Num() > 0 || DissectedUpdates.Num() > 0)
	{
		FlushState = FlushState::LoopEval;
	}
}

void FMovieSceneEntitySystemRunner::MarkForUpdate(FInstanceHandle InInstanceHandle, UE::MovieScene::ERunnerUpdateFlags UpdateFlags)
{
	using namespace UE::MovieScene;

	// If we are reimporting after a compile, we might be finishing/destroying a sequence for the first time and so the call to
	//    Finish from GameThread_UpdateSequenceInstances may have been missed. In this case we have to check whether the sequence
	//    needs finishing so we don't have entities left in the ledger when it comes to being destroyed.
	if (bIsUpdatingSequence && EnumHasAnyFlags(UpdateFlags, ERunnerUpdateFlags::Finish | ERunnerUpdateFlags::Destroy))
	{
		UMovieSceneEntitySystemLinker* Linker = WeakLinker.Get();
		FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();
		FSequenceInstance& SequenceInstance = InstanceRegistry->MutateInstance(InInstanceHandle);

		// Need to check whether this has actually finished yet or not
		if (!SequenceInstance.HasFinished())
		{
			SequenceInstance.Finish();
		}
	}

	CurrentInstances.Add(FQueuedUpdateParams{ InInstanceHandle, UpdateFlags });
}

FMovieSceneEntitySystemEventTriggers& FMovieSceneEntitySystemRunner::GetQueuedEventTriggers()
{
	checkf(bCanQueueEventTriggers, TEXT("Can't queue event triggers at this point in the update loop."));
	return EventTriggers;
}

bool FMovieSceneEntitySystemRunner::FlushSingleEvaluationPhase()
{
	using namespace UE::MovieScene;

	if (!ensureMsgf(
			!IsCurrentlyEvaluating() && CurrentPhase == ESystemPhase::None,
			TEXT("Can't run nested flush phase while the runner is evaluating and no re-entrancy window is open")))
	{
		return false;
	}

	UMovieSceneEntitySystemLinker* Linker = WeakLinker.Get();
	if (!ensureMsgf(Linker, TEXT("Runner doesn't have a valid linker")))
	{
		return false;
	}

	TGuardValue<ESystemPhase> PhaseGuard(CurrentPhase, ESystemPhase::Evaluation);

	Linker->EntityManager.LockDown();

	Linker->EntityManager.UpdateThreadingModel();

	if (FEntitySystemScheduler::IsCustomSchedulingEnabled())
	{
		Linker->SystemGraph.ScheduleTasks(&Linker->EntityManager);
	}

	FGraphEventArray AllTasks;
	Linker->SystemGraph.ExecutePhase(ESystemPhase::Evaluation, Linker, AllTasks);

	if (AllTasks.Num() != 0)
	{
		FTaskGraphInterface::Get().WaitUntilTasksComplete(AllTasks, ENamedThreads::GameThread_Local);
	}

	Linker->EntityManager.ReleaseLockDown();

	return true;
}

