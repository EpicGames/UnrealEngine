// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/MovieSceneComponentAccessors.h"
#include "EntitySystem/MovieSceneComponentDebug.h"

namespace UE
{
namespace MovieScene
{

/**
 * Return true if the parameters satisfy the condition (Input & Mask) == Mask
 */
bool InputMatchesAll(const FComponentMask& Input, const FComponentMask& Mask)
{
	FComponentMask Temp = Mask;
	Temp.CombineWithBitwiseAND(Input, EBitwiseOperatorFlags::MaintainSize);
	return Temp == Mask;
}

/**
 * Return true if the parameters satisfy the condition (Input & Mask) != 0
 */
bool InputMatchesAny(const FComponentMask& Input, const FComponentMask& Mask)
{
	FComponentMask Temp = Mask;
	Temp.CombineWithBitwiseAND(Input, EBitwiseOperatorFlags::MaintainSize);
	return Temp.Find(true) != INDEX_NONE;
}

/**
 * Return true if the parameters satisfy the condition countbits(Input & Mask) == 1
 */
bool InputMatchesOne(const FComponentMask& Input, const FComponentMask& Mask)
{
	FComponentMask Temp = Mask;
	Temp.CombineWithBitwiseAND(Input, EBitwiseOperatorFlags::MaintainSize);
	return Temp.NumComponents() == 1;
}

bool FEntityComponentFilter::Match(const FComponentMask& Input) const
{
	if (AllMask.Num() > 0 && !InputMatchesAll(Input, AllMask))
	{
		return false;
	}

	if (NoneMask.Num() > 0 && InputMatchesAny(Input, NoneMask))
	{
		return false;
	}

	for (const FComplexMask& ComplexMask : ComplexMasks)
	{
		bool bSuccess = false;

		if (EnumHasAnyFlags(ComplexMask.Mode, EComplexFilterMode::OneOf))
		{
			bSuccess = InputMatchesOne(Input, ComplexMask.Mask);
		}
		else if (EnumHasAnyFlags(ComplexMask.Mode, EComplexFilterMode::OneOrMoreOf))
		{
			bSuccess = InputMatchesAny(Input, ComplexMask.Mask);
		}
		else if (EnumHasAnyFlags(ComplexMask.Mode, EComplexFilterMode::AllOf))
		{
			bSuccess = InputMatchesAll(Input, ComplexMask.Mask);
		}

		if (EnumHasAnyFlags(ComplexMask.Mode, EComplexFilterMode::Negate))
		{
			bSuccess = !bSuccess;
		}

		if (!bSuccess)
		{
			return false;
		}
	}

	return true;
}

bool FEntityComponentFilter::IsValid() const
{
	if (AllMask.Find(true) != INDEX_NONE)
	{
		return true;
	}
	if (NoneMask.Find(true) != INDEX_NONE)
	{
		return true;
	}

	for (const FComplexMask& ComplexMask : ComplexMasks)
	{
		if (ComplexMask.Mask.Find(true) != INDEX_NONE)
		{
			return true;
		}
	}

	return false;
}

FEntityAllocationWriteContext::FEntityAllocationWriteContext(const FEntityManager& EntityManager)
	: SystemSerial(EntityManager.GetSystemSerial())
{}

FComponentReader FEntityAllocation::ReadComponentsErased(FComponentTypeID ComponentType) const
{
	const FComponentHeader& Header = GetComponentHeaderChecked(ComponentType);
	return FComponentReader(&Header, LockMode);
}

FComponentWriter FEntityAllocation::WriteComponentsErased(FComponentTypeID ComponentType, FEntityAllocationWriteContext InWriteContext) const
{
	const FComponentHeader& Header = GetComponentHeaderChecked(ComponentType);
	return FComponentWriter(&Header, LockMode, InWriteContext);
}

FOptionalComponentReader FEntityAllocation::TryReadComponentsErased(FComponentTypeID ComponentType) const
{
	if (const FComponentHeader* Header = FindComponentHeader(ComponentType))
	{
		return FOptionalComponentReader(Header, LockMode);
	}

	return FOptionalComponentReader();
}

FOptionalComponentWriter FEntityAllocation::TryWriteComponentsErased(FComponentTypeID ComponentType, FEntityAllocationWriteContext InWriteContext) const
{
	if (const FComponentHeader* Header = FindComponentHeader(ComponentType))
	{
		return FOptionalComponentWriter(Header, LockMode, InWriteContext);
	}

	return FOptionalComponentWriter();
}

const FEntityAllocation* FEntityAllocationProxy::GetAllocation() const
{
	return Manager->EntityAllocations[AllocationIndex];
}

FEntityAllocation* FEntityAllocationProxy::GetAllocation()
{
	return Manager->EntityAllocations[AllocationIndex];
}

const FComponentMask& FEntityAllocationProxy::GetAllocationType() const
{
	return Manager->EntityAllocationMasks[AllocationIndex];
}

#if UE_MOVIESCENE_ENTITY_DEBUG
FComponentHeader::~FComponentHeader()
{
	//~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
	// WARNING: This destructor is only compiled in when UE_MOVIESCENE_ENTITY_DEBUG is true
	if (DebugComponents)
	{
		delete[] DebugComponents;
	}
}
#endif // UE_MOVIESCENE_ENTITY_DEBUG

} // namespace MovieScene
} // namespace UE
