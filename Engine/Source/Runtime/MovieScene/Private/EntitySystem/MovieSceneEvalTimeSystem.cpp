// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneEvalTimeSystem.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneInstanceRegistry.h"
#include "Variants/MovieSceneTimeWarpGetter.h"
#include "Stats/Stats.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneEvalTimeSystem)

namespace UE::MovieScene
{

DECLARE_CYCLE_STAT(TEXT("MovieScene: Gather evaluation times"), MovieSceneEval_GatherEvalTimes, STATGROUP_MovieSceneECS);
DECLARE_CYCLE_STAT(TEXT("MovieScene: Assign evaluation times"), MovieSceneEval_AssignEvalTimes, STATGROUP_MovieSceneECS);

struct FGatherTimes
{
	const FInstanceRegistry* InstanceRegistry;
	TArray<FEvaluatedTime, TInlineAllocator<16>>* EvaluatedTimes;

	void Run(FEntityAllocationWriteContext WriteContext) const
	{
		Run();
	}
	void Run() const
	{
		EvaluatedTimes->SetNum(InstanceRegistry->GetSparseInstances().GetMaxIndex());
		for (auto It = InstanceRegistry->GetSparseInstances().CreateConstIterator(); It; ++It)
		{
			const FMovieSceneContext& Context = It->GetContext();

			FEvaluatedTime EvaluatedTime;
			EvaluatedTime.FrameTime = Context.GetTime();
			EvaluatedTime.FrameRate = Context.GetFrameRate();
			EvaluatedTime.Seconds = EvaluatedTime.FrameRate.AsSeconds(EvaluatedTime.FrameTime);

			(*EvaluatedTimes)[It.GetIndex()] = EvaluatedTime;
		}
	}
};

struct FAssignEvalTimesTask
{
	const TArray<FEvaluatedTime, TInlineAllocator<16>>* EvaluatedTimes;

	FAssignEvalTimesTask(const TArray<FEvaluatedTime, TInlineAllocator<16>>* InEvaluatedTimes)
		: EvaluatedTimes(InEvaluatedTimes)
	{}

	void ForEachEntity(FInstanceHandle InstanceHandle, FFrameTime& EvalTime) const
	{
		EvalTime = (*EvaluatedTimes)[InstanceHandle.InstanceID].FrameTime;
	}
};

struct FAssignEvalSecondsTask
{
	const TArray<FEvaluatedTime, TInlineAllocator<16>>* EvaluatedTimes;

	FAssignEvalSecondsTask(const TArray<FEvaluatedTime, TInlineAllocator<16>>* InEvaluatedTimes)
		: EvaluatedTimes(InEvaluatedTimes)
	{}

	void ForEachEntity(FInstanceHandle InstanceHandle, double& EvalSeconds) const
	{
		EvalSeconds = (*EvaluatedTimes)[InstanceHandle.InstanceID].Seconds;
	}
};


struct FAssignTimeWarpedEvalTimesTask
{
	const TArray<FEvaluatedTime, TInlineAllocator<16>>* EvaluatedTimes;

	FAssignTimeWarpedEvalTimesTask(const TArray<FEvaluatedTime, TInlineAllocator<16>>* InEvaluatedTimes)
		: EvaluatedTimes(InEvaluatedTimes)
	{}

	void ForEachEntity(FInstanceHandle InstanceHandle, UMovieSceneTimeWarpGetter* TimeWarp, FFrameTime& EvalTime) const
	{
		FFrameTime UnwarpedEvalTime = (*EvaluatedTimes)[InstanceHandle.InstanceID].FrameTime;
		EvalTime = TimeWarp->RemapTime(UnwarpedEvalTime);
	}
};

struct FAssignTimeWarpedEvalSecondsTask
{
	const TArray<FEvaluatedTime, TInlineAllocator<16>>* EvaluatedTimes;

	FAssignTimeWarpedEvalSecondsTask(const TArray<FEvaluatedTime, TInlineAllocator<16>>* InEvaluatedTimes)
		: EvaluatedTimes(InEvaluatedTimes)
	{}

	void ForEachEntity(FInstanceHandle InstanceHandle, UMovieSceneTimeWarpGetter* TimeWarp, double& EvalSeconds) const
	{
		const FEvaluatedTime EvaluatedTime = (*EvaluatedTimes)[InstanceHandle.InstanceID];
		const FFrameTime     EvalTime      = TimeWarp->RemapTime(EvaluatedTime.FrameTime);

		EvalSeconds = EvaluatedTime.FrameRate.AsSeconds(EvalTime);
	}
};

} // namespace UE::MovieScene

UMovieSceneEvalTimeSystem::UMovieSceneEvalTimeSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	SystemCategories = EEntitySystemCategory::Core;
	RelevantFilter.Any({ BuiltInComponents->EvalTime, BuiltInComponents->EvalSeconds });

	Phase = ESystemPhase::Scheduling;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineComponentProducer(GetClass(), BuiltInComponents->EvalTime);
		DefineComponentProducer(GetClass(), BuiltInComponents->EvalSeconds);
	}
}

bool UMovieSceneEvalTimeSystem::IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	return InLinker->EntityManager.Contains(RelevantFilter);
}

void UMovieSceneEvalTimeSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	FTaskID GatherTask = TaskScheduler->AddTask<FGatherTimes>(
		FTaskParams(TEXT("Gather Evaluation Times"), GET_STATID(MovieSceneEval_GatherEvalTimes)),
		Linker->GetInstanceRegistry(),
		&EvaluatedTimes
	);

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	FTaskID WriteTask1 = FEntityTaskBuilder()
	.Read(BuiltInComponents->InstanceHandle)
	.Write(BuiltInComponents->EvalTime)
	.FilterNone({ BuiltInComponents->Tags.FixedTime, BuiltInComponents->TimeWarp })
	.SetStat(GET_STATID(MovieSceneEval_AssignEvalTimes))
	.Fork_PerEntity<FAssignEvalTimesTask>(&Linker->EntityManager, TaskScheduler, &EvaluatedTimes);

	FTaskID WriteTask2 = FEntityTaskBuilder()
	.Read(BuiltInComponents->InstanceHandle)
	.Write(BuiltInComponents->EvalSeconds)
	.FilterNone({ BuiltInComponents->Tags.FixedTime, BuiltInComponents->TimeWarp })
	.SetStat(GET_STATID(MovieSceneEval_AssignEvalTimes))
	.Fork_PerEntity<FAssignEvalSecondsTask>(&Linker->EntityManager, TaskScheduler, &EvaluatedTimes);

	FTaskID WriteTask3 = FEntityTaskBuilder()
	.Read(BuiltInComponents->InstanceHandle)
	.Read(BuiltInComponents->TimeWarp)
	.Write(BuiltInComponents->EvalTime)
	.FilterNone({ BuiltInComponents->Tags.FixedTime })
	.SetStat(GET_STATID(MovieSceneEval_AssignEvalTimes))
	.Fork_PerEntity<FAssignTimeWarpedEvalTimesTask>(&Linker->EntityManager, TaskScheduler, &EvaluatedTimes);

	FTaskID WriteTask4 = FEntityTaskBuilder()
	.Read(BuiltInComponents->InstanceHandle)
	.Read(BuiltInComponents->TimeWarp)
	.Write(BuiltInComponents->EvalSeconds)
	.FilterNone({ BuiltInComponents->Tags.FixedTime })
	.SetStat(GET_STATID(MovieSceneEval_AssignEvalTimes))
	.Fork_PerEntity<FAssignTimeWarpedEvalSecondsTask>(&Linker->EntityManager, TaskScheduler, &EvaluatedTimes);

	TaskScheduler->AddPrerequisite(GatherTask, WriteTask1);
	TaskScheduler->AddPrerequisite(GatherTask, WriteTask2);
	TaskScheduler->AddPrerequisite(GatherTask, WriteTask3);
	TaskScheduler->AddPrerequisite(GatherTask, WriteTask4);
}

void UMovieSceneEvalTimeSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	FSystemTaskPrerequisites EvalPrereqs;
	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	FGraphEventRef GatherEvalTimesEvent = TEntityTaskComponents<>()
	.SetStat(GET_STATID(MovieSceneEval_GatherEvalTimes))
	.Dispatch<FGatherTimes>(&Linker->EntityManager, InPrerequisites, nullptr, Linker->GetInstanceRegistry(), &EvaluatedTimes);

	if (GatherEvalTimesEvent)
	{
		EvalPrereqs.AddComponentTask(BuiltInComponents->EvalTime, GatherEvalTimesEvent);
		EvalPrereqs.AddComponentTask(BuiltInComponents->EvalSeconds, GatherEvalTimesEvent);
	}

	FEntityTaskBuilder()
	.Read(BuiltInComponents->InstanceHandle)
	.Write(BuiltInComponents->EvalTime)
	.FilterNone({ FBuiltInComponentTypes::Get()->Tags.FixedTime })
	.SetStat(GET_STATID(MovieSceneEval_AssignEvalTimes))
	.Dispatch_PerEntity<FAssignEvalTimesTask>(&Linker->EntityManager, EvalPrereqs, &Subsequents, &EvaluatedTimes);

	FEntityTaskBuilder()
	.Read(BuiltInComponents->InstanceHandle)
	.Write(BuiltInComponents->EvalSeconds)
	.FilterNone({ FBuiltInComponentTypes::Get()->Tags.FixedTime })
	.SetStat(GET_STATID(MovieSceneEval_AssignEvalTimes))
	.Dispatch_PerEntity<FAssignEvalSecondsTask>(&Linker->EntityManager, EvalPrereqs, &Subsequents, &EvaluatedTimes);
}

