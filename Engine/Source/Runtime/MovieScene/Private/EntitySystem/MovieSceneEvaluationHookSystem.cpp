// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneEvaluationHookSystem.h"

#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "EntitySystem/MovieSceneSpawnablesSystem.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedCaptureSource.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedCaptureSources.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Templates/Greater.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneEvaluationHookSystem)

DECLARE_CYCLE_STAT(TEXT("Generic Hooks"),  MovieSceneECS_GenericHooks, STATGROUP_MovieSceneECS);

namespace UE
{
namespace MovieScene
{

struct FEvaluationHookUpdater
{
	UMovieSceneEvaluationHookSystem* HookSystem;
	FInstanceRegistry* InstanceRegistry;

	FEvaluationHookUpdater(UMovieSceneEvaluationHookSystem* InHookSystem, FInstanceRegistry* InInstanceRegistry)
		: HookSystem(InHookSystem), InstanceRegistry(InInstanceRegistry)
	{}

	void ForEachAllocation(FEntityAllocationProxy Item, TRead<FInstanceHandle> InstanceHandles, TRead<FMovieSceneEvaluationHookComponent> Hooks, TRead<FFrameTime> EvalTimes, TWrite<FEvaluationHookFlags> WriteFlags) const
	{
		const int32 Num = Item.GetAllocation()->Num();
		const bool bRestoreState = Item.GetAllocationType().Contains(FBuiltInComponentTypes::Get()->Tags.RestoreState);

		for (int32 Index = 0; Index < Num; ++Index)
		{
			if (WriteFlags[Index].bHasBegun == false)
			{
				WriteFlags[Index].bHasBegun = true;
				continue;
			}

			const FSequenceInstance& SequenceInstance = InstanceRegistry->GetInstance(InstanceHandles[Index]);
			FMovieSceneInverseSequenceTransform SequenceToRootTransform = SequenceInstance.GetContext().GetRootToSequenceTransform().Inverse();

			FMovieSceneEvaluationHookEvent NewEvent;
			NewEvent.Hook          = Hooks[Index];
			NewEvent.Type          = EEvaluationHookEvent::Update;
			NewEvent.RootTime      = SequenceToRootTransform.TryTransformTime(EvalTimes[Index], SequenceInstance.GetContext().GetRootToSequenceWarpCounter()).Get(FFrameTime());
			NewEvent.RootInstanceHandle = SequenceInstance.GetRootInstanceHandle();
			NewEvent.SequenceID    = SequenceInstance.GetSequenceID();
			NewEvent.bRestoreState = bRestoreState;

			HookSystem->AddEvent(SequenceInstance.GetRootInstanceHandle(), NewEvent);
		}
	}

	void PostTask()
	{
		HookSystem->SortEvents();
	}
};

} // namespace MovieScene
} // namespace UE

UMovieSceneEvaluationHookSystem::UMovieSceneEvaluationHookSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	Phase = UE::MovieScene::ESystemPhase::Instantiation | UE::MovieScene::ESystemPhase::Scheduling | UE::MovieScene::ESystemPhase::Finalization;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineComponentConsumer(GetClass(), UE::MovieScene::FBuiltInComponentTypes::Get()->EvalTime);
	}
}

void UMovieSceneEvaluationHookSystem::AddEvent(UE::MovieScene::FInstanceHandle RootInstance, const FMovieSceneEvaluationHookEvent& InEvent)
{
	PendingEventsByRootInstance.FindOrAdd(RootInstance).Events.Add(InEvent);
}

bool UMovieSceneEvaluationHookSystem::HasEvents() const
{
	return PendingEventsByRootInstance.Num() != 0;
}

bool UMovieSceneEvaluationHookSystem::IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	return HasEvents() || InLinker->EntityManager.ContainsComponent(UE::MovieScene::FBuiltInComponentTypes::Get()->EvaluationHook);
}

void UMovieSceneEvaluationHookSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();

	FEntityTaskBuilder()
	.Read(Components->InstanceHandle)
	.Read(Components->EvaluationHook)
	.Read(Components->EvalTime)
	.Write(Components->EvaluationHookFlags)
	.SetDesiredThread(Linker->EntityManager.GetDispatchThread())
	.Schedule_PerAllocation<FEvaluationHookUpdater>(&Linker->EntityManager, TaskScheduler, this, Linker->GetInstanceRegistry());
}

void UMovieSceneEvaluationHookSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	TSharedRef<FMovieSceneEntitySystemRunner> Runner = Linker->GetRunner();

	ESystemPhase CurrentPhase = Runner->GetCurrentPhase();
	if (CurrentPhase == ESystemPhase::Instantiation)
	{
		UpdateHooks();
	}
	else if (CurrentPhase == ESystemPhase::Evaluation)
	{
		FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();

		FGraphEventRef UpdateEvent = FEntityTaskBuilder()
		.Read(Components->InstanceHandle)
		.Read(Components->EvaluationHook)
		.Read(Components->EvalTime)
		.Write(Components->EvaluationHookFlags)
		.SetDesiredThread(Linker->EntityManager.GetDispatchThread())
		.Dispatch_PerAllocation<FEvaluationHookUpdater>(&Linker->EntityManager, InPrerequisites, &Subsequents, this, Linker->GetInstanceRegistry());
	}
	else if (HasEvents())
	{
		ensure(CurrentPhase == ESystemPhase::Finalization);
		Runner->GetQueuedEventTriggers().AddUObject(this, &UMovieSceneEvaluationHookSystem::TriggerAllEvents);
	}
}

void UMovieSceneEvaluationHookSystem::UpdateHooks()
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();

	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();

	auto VisitNew = [this, InstanceRegistry](FEntityAllocationProxy Item, TRead<FInstanceHandle> InstanceHandles, TRead<FFrameTime> EvalTimes, TRead<FMovieSceneEvaluationHookComponent> Hooks)
	{
		const int32 Num = Item.GetAllocation()->Num();
		const bool bRestoreState = Item.GetAllocationType().Contains(FBuiltInComponentTypes::Get()->Tags.RestoreState);

		for (int32 Index = 0; Index < Num; ++Index)
		{
			const FSequenceInstance& SequenceInstance = InstanceRegistry->GetInstance(InstanceHandles[Index]);
			FMovieSceneInverseSequenceTransform SequenceToRootTransform = SequenceInstance.GetContext().GetRootToSequenceTransform().Inverse();

			FMovieSceneEvaluationHookEvent NewEvent;
			NewEvent.Hook          = Hooks[Index];
			NewEvent.Type          = EEvaluationHookEvent::Begin;
			NewEvent.RootTime      = SequenceToRootTransform.TryTransformTime(EvalTimes[Index], SequenceInstance.GetContext().GetRootToSequenceWarpCounter()).Get(FFrameTime());
			NewEvent.RootInstanceHandle = SequenceInstance.GetRootInstanceHandle();
			NewEvent.SequenceID    = SequenceInstance.GetSequenceID();
			NewEvent.bRestoreState = bRestoreState;

			this->AddEvent(SequenceInstance.GetRootInstanceHandle(), NewEvent);
		}
	};

	auto VisitOld = [this, InstanceRegistry](FEntityAllocationProxy Item, TRead<FInstanceHandle> InstanceHandles, TRead<FFrameTime> EvalTimes, TRead<FMovieSceneEvaluationHookComponent> Hooks)
	{
		const int32 Num = Item.GetAllocation()->Num();
		const bool bRestoreState = Item.GetAllocationType().Contains(FBuiltInComponentTypes::Get()->Tags.RestoreState);

		for (int32 Index = 0; Index < Num; ++Index)
		{
			const FSequenceInstance& SequenceInstance = InstanceRegistry->GetInstance(InstanceHandles[Index]);
			FMovieSceneInverseSequenceTransform SequenceToRootTransform = SequenceInstance.GetContext().GetRootToSequenceTransform().Inverse();

			FMovieSceneEvaluationHookEvent NewEvent;
			NewEvent.Hook          = Hooks[Index];
			NewEvent.Type          = EEvaluationHookEvent::End;
			NewEvent.RootTime      = SequenceToRootTransform.TryTransformTime(EvalTimes[Index], SequenceInstance.GetContext().GetRootToSequenceWarpCounter()).Get(FFrameTime());
			NewEvent.RootInstanceHandle = SequenceInstance.GetRootInstanceHandle();
			NewEvent.SequenceID    = SequenceInstance.GetSequenceID();
			NewEvent.bRestoreState = bRestoreState;

			this->AddEvent(SequenceInstance.GetRootInstanceHandle(), NewEvent);
		}
	};

	FEntityTaskBuilder()
	.Read(Components->InstanceHandle)
	.Read(Components->EvalTime)
	.Read(Components->EvaluationHook)
	.FilterAny({ Components->Tags.NeedsLink })
	.Iterate_PerAllocation(&Linker->EntityManager, VisitNew);

	FEntityTaskBuilder()
	.Read(Components->InstanceHandle)
	.Read(Components->EvalTime)
	.Read(Components->EvaluationHook)
	.FilterAny({ Components->Tags.Finished })
	.Iterate_PerAllocation(&Linker->EntityManager, VisitOld);
}

void UMovieSceneEvaluationHookSystem::SortEvents()
{
	using namespace UE::MovieScene;

	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();

	for (TPair<FMovieSceneEvaluationInstanceKey, FMovieSceneEvaluationHookEventContainer>& Pair : PendingEventsByRootInstance)
	{
		const FSequenceInstance& RootInstance = InstanceRegistry->GetInstance(Pair.Key.InstanceHandle);
		if (RootInstance.GetContext().GetDirection() == EPlayDirection::Forwards)
		{
			Algo::SortBy(Pair.Value.Events, &FMovieSceneEvaluationHookEvent::RootTime);
		}
		else
		{
			Algo::SortBy(Pair.Value.Events, &FMovieSceneEvaluationHookEvent::RootTime, TGreater<>());
		}
	}
}

void UMovieSceneEvaluationHookSystem::TriggerAllEvents()
{
	using namespace UE::MovieScene;

	SCOPE_CYCLE_COUNTER(MovieSceneECS_GenericHooks);

	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();
	FPreAnimatedEvaluationHookCaptureSources* EvaluationHookMetaData = Linker->PreAnimatedState.GetEvaluationHookMetaData();

	// We need to clean our state before actually triggering the events because one of those events could
	// call back into an evaluation (for instance, by starting play on another sequence). If we don't clean
	// this before, would would re-enter and re-trigger past events, resulting in an infinite loop!
	TMap<FMovieSceneEvaluationInstanceKey, FMovieSceneEvaluationHookEventContainer> LocalEvents;
	Swap(LocalEvents, PendingEventsByRootInstance);

	for (TPair<FMovieSceneEvaluationInstanceKey, FMovieSceneEvaluationHookEventContainer>& Pair : LocalEvents)
	{
		for (const FMovieSceneEvaluationHookEvent& Event : Pair.Value.Events)
		{
			// Re-fetch the sequence instance in each loop. This is because the hook we execute might run
			// arbitrary code that, for instance, starts other sequences. In that case, the instance registry
			// would grow, and rellocate all sequence instances. If we held onto one of them, we could
			// end up holding deallocated memory.
			const FSequenceInstance& SequenceInstance = InstanceRegistry->GetInstance(Pair.Key.InstanceHandle);

			FMovieSceneContext RootContext = SequenceInstance.GetContext();
			TSharedRef<FSharedPlaybackState> SharedPlaybackState = SequenceInstance.GetSharedPlaybackState();

			FScopedPreAnimatedCaptureSource CaptureSource(SharedPlaybackState, Event.Hook.Interface.GetObject(), Event.SequenceID, Event.bRestoreState);

			FEvaluationHookParams Params = {
				Event.Hook.ObjectBindingID, RootContext, Event.SequenceID, Event.TriggerIndex
			};

			if (Event.SequenceID != MovieSceneSequenceID::Root)
			{
				FInstanceHandle SubInstance = SequenceInstance.FindSubInstance(Event.SequenceID);
				if (SubInstance.IsValid())
				{
					Params.Context = InstanceRegistry->GetInstance(SubInstance).GetContext();
				}
			}

			switch (Event.Type)
			{
				case EEvaluationHookEvent::Begin:
					Event.Hook.Interface->Begin(SharedPlaybackState, Params);
					break;
				case EEvaluationHookEvent::Update:
					Event.Hook.Interface->Update(SharedPlaybackState, Params);
					break;
				case EEvaluationHookEvent::End:
					Event.Hook.Interface->End(SharedPlaybackState, Params);
					if (EvaluationHookMetaData)
					{
						EvaluationHookMetaData->StopTrackingCaptureSource(Event.Hook.Interface.GetObject(), Event.RootInstanceHandle, Event.SequenceID);
					}
					break;

				case EEvaluationHookEvent::Trigger:
					Event.Hook.Interface->Trigger(SharedPlaybackState, Params);
					break;
			}
		}
	}
}

