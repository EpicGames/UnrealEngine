// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneInitialValueCache.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"

namespace UE
{
namespace MovieScene
{


TEntitySystemLinkerExtensionID<FInitialValueCache> FInitialValueCache::GetExtensionID()
{
	static TEntitySystemLinkerExtensionID<FInitialValueCache> ID = UMovieSceneEntitySystemLinker::RegisterExtension<FInitialValueCache>();
	return ID;
}

TSharedPtr<FInitialValueCache> FInitialValueCache::GetGlobalInitialValues()
{
	static TWeakPtr<FInitialValueCache> GSharedInitialValues = MakeShared<FInitialValueCache>();

	TSharedPtr<FInitialValueCache> Ptr = GSharedInitialValues.Pin();
	if (!Ptr)
	{
		Ptr = MakeShared<FInitialValueCache>();
		GSharedInitialValues = Ptr;
	}
	return Ptr;
}

} // namespace MovieScene
} // namespace UE

