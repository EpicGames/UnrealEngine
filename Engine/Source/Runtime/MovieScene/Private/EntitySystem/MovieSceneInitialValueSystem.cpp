// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneInitialValueSystem.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieScenePropertyRegistry.h"
#include "EntitySystem/MovieSceneEntityMutations.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneInitialValueCache.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationExtension.h"

#include "Systems/MovieScenePropertyInstantiator.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogatedPropertyInstantiator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneInitialValueSystem)

namespace UE
{
namespace MovieScene
{

struct FInitialValueProcessorEntry
{
	FComponentTypeID InitialValueType;
	FEntityComponentFilter Filter;
	TSharedPtr<IInitialValueProcessor> Processor;
};

TArray<FInitialValueProcessorEntry> GInitialValueProcessors;

struct FInitialValueMutation : IMovieSceneEntityMutation
{
	TMultiMap<FComponentTypeID, int32> InitialValueTypeToProcessor;
	FInitialValueCache* InitialValueCache;
	FBuiltInComponentTypes* BuiltInComponents;
	FComponentMask AnyInitialValue;

	FInitialValueMutation(UMovieSceneEntitySystemLinker* Linker)
	{
		BuiltInComponents = FBuiltInComponentTypes::Get();

		InitialValueCache = Linker->FindExtension<FInitialValueCache>();

		const int32 NumProcessors = GInitialValueProcessors.Num();
		for (int32 Index = 0; Index < NumProcessors; ++Index)
		{
			const FInitialValueProcessorEntry& Entry = GInitialValueProcessors[Index];

			if (Linker->EntityManager.ContainsComponent(Entry.InitialValueType) &&
				(!Entry.Filter.IsValid() || Linker->EntityManager.Contains(Entry.Filter))
				)
			{
				Entry.Processor->Initialize(Linker, InitialValueCache);
				AnyInitialValue.Set(Entry.InitialValueType);
				InitialValueTypeToProcessor.Add(Entry.InitialValueType, Index);
			}
		}
	}

	~FInitialValueMutation()
	{
		for (TPair<FComponentTypeID, int32> Pair : InitialValueTypeToProcessor)
		{
			GInitialValueProcessors[Pair.Value].Processor->Finalize();
		}
	}

	bool IsCached() const
	{
		return InitialValueCache != nullptr;
	}

	virtual void CreateMutation(FEntityManager* EntityManager, FComponentMask* InOutEntityComponentTypes) const override
	{
		if (IsCached())
		{
			InOutEntityComponentTypes->Set(BuiltInComponents->InitialValueIndex);
		}
		InOutEntityComponentTypes->Set(BuiltInComponents->Tags.HasAssignedInitialValue);
	}

	virtual void InitializeAllocation(FEntityAllocation* Allocation, const FComponentMask& AllocationType) const
	{
		FComponentTypeID InitialValueType = FComponentMask::BitwiseAND(AllocationType, AnyInitialValue, EBitwiseOperatorFlags::MinSize).First();

		for (auto IndexIt = InitialValueTypeToProcessor.CreateConstKeyIterator(InitialValueType); IndexIt; ++IndexIt)
		{
			const FInitialValueProcessorEntry& Entry = GInitialValueProcessors[IndexIt.Value()];
			if (!Entry.Filter.IsValid() || Entry.Filter.Match(AllocationType))
			{
				Entry.Processor->Process(Allocation, AllocationType);
			}
		}
	}
};

} // namespace MovieScene
} // namespace UE

UMovieSceneInitialValueSystem::UMovieSceneInitialValueSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	SystemCategories = UE::MovieScene::EEntitySystemCategory::Core;
}

void UMovieSceneInitialValueSystem::RegisterProcessor(
	UE::MovieScene::FComponentTypeID InitialValueComponent,
	TSharedPtr<UE::MovieScene::IInitialValueProcessor> Processor,
	UE::MovieScene::FEntityComponentFilter&& OptionalFilter)
{
	using namespace UE::MovieScene;

	Processor->PopulateFilter(OptionalFilter);

	GInitialValueProcessors.Add(FInitialValueProcessorEntry{
		InitialValueComponent,
		MoveTemp(OptionalFilter),
		MoveTemp(Processor)
	});
}

bool UMovieSceneInitialValueSystem::IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	using namespace UE::MovieScene;

	for (const FInitialValueProcessorEntry& Entry : GInitialValueProcessors)
	{
		if (InLinker->EntityManager.ContainsComponent(Entry.InitialValueType))
		{
			return true;
		}
	}
	return false;
}

void UMovieSceneInitialValueSystem::OnLink()
{

}

void UMovieSceneInitialValueSystem::OnUnlink()
{

}

void UMovieSceneInitialValueSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	FInitialValueMutation Mutation(Linker);

	// If we don't have any initial value processors, we've no work to do
	if (Mutation.AnyInitialValue.NumComponents() == 0)
	{
		return;
	}

	if (Mutation.IsCached() && Linker->FindExtension<IInterrogationExtension>() == nullptr)
	{
		// When there is an initial value cache extension, we mutate anything with an initial value component on it by
		// adding an additional index that refers to its cached position. This ensures we are able to clean up the cache
		// easily.
		{
			FEntityComponentFilter Filter;
			Filter.Any(Mutation.AnyInitialValue);
			Filter.All({ BuiltInComponents->Tags.NeedsLink });
			Filter.None({ BuiltInComponents->InitialValueIndex });
			Filter.None({ BuiltInComponents->Tags.HasAssignedInitialValue, BuiltInComponents->Tags.Ignored });

			Linker->EntityManager.MutateAll(Filter, Mutation);
		}

		// Clean up any stale cache entries
		{
			FEntityComponentFilter Filter;
			Filter.Any(Mutation.AnyInitialValue);
			Filter.All({ BuiltInComponents->InitialValueIndex, BuiltInComponents->Tags.NeedsUnlink });

			for (FEntityAllocationIteratorItem Item : Linker->EntityManager.Iterate(&Filter))
			{
				const FEntityAllocation* Allocation     = Item.GetAllocation();
				FComponentMask           AllocationType = Item.GetAllocationType();

				FComponentTypeID InitialValueType = FComponentMask::BitwiseAND(AllocationType, Mutation.AnyInitialValue, EBitwiseOperatorFlags::MinSize).First();

				TComponentReader<FInitialValueIndex> Indices = Allocation->ReadComponents(BuiltInComponents->InitialValueIndex);
				Mutation.InitialValueCache->Reset(InitialValueType, Indices.AsArray(Allocation->Num()));
			}
		}
	}
	else
	{
		// When there is no caching extension, or we are interrogating we simply initialize any initial values directly without going through the cache
		FEntityComponentFilter Filter;
		Filter.Any(Mutation.AnyInitialValue);
		//Filter.Any({ BuiltInComponents->Interrogation.OutputKey });
		Filter.All({ BuiltInComponents->Tags.NeedsLink });
		Filter.None({ BuiltInComponents->Tags.HasAssignedInitialValue, BuiltInComponents->Tags.Ignored });

		Linker->EntityManager.MutateAll(Filter, Mutation);
	}
}

