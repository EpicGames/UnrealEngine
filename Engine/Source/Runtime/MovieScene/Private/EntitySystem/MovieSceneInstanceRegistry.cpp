// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneInstanceRegistry.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/MovieSceneEntityMutations.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntityInstantiatorSystem.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "EntitySystem/EntityAllocationIterator.h"
#include "EntitySystem/BuiltInComponentTypes.h"

#include "Compilation/MovieSceneCompiledDataManager.h"
#include "Compilation/MovieSceneCompiledVolatilityManager.h"

#include "Evaluation/MovieSceneSequenceHierarchy.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Evaluation/Instances/MovieSceneTrackEvaluator.h"

#include "Engine/World.h"

#include "IMovieScenePlayer.h"

namespace UE
{
namespace MovieScene
{


FInstanceRegistry::FInstanceRegistry(UMovieSceneEntitySystemLinker* InLinker)
	: Linker(InLinker)
	, InstanceSerialNumber(0)
{
}

FInstanceRegistry::~FInstanceRegistry()
{
	// Remove all sub-instances from the array so that they release their ref-count on their shared playback state.
	// This prevents the root instances from triggering an assert about it.
	for (auto It = Instances.CreateIterator(); It; ++It)
	{
		if (!It->IsRootSequence())
		{
			It.RemoveCurrent();
		}
	}
}

FInstanceHandle FInstanceRegistry::FindRelatedInstanceHandle(FInstanceHandle InstanceHandle, FMovieSceneSequenceID SequenceID) const
{
	checkfSlow(IsHandleValid(InstanceHandle), TEXT("Given instance handle is not valid."));
	checkfSlow(SequenceID.IsValid(), TEXT("Given sequence ID is not valid."));

	const FSequenceInstance* RootInstance = &GetInstance(InstanceHandle);

	if (SequenceID == MovieSceneSequenceID::Root)
	{
		return RootInstance->GetRootInstanceHandle();
	}

	if (!RootInstance->IsRootSequence())
	{
		RootInstance = &GetInstance(RootInstance->GetRootInstanceHandle());
	}
	return RootInstance->FindSubInstance(SequenceID);
}

FRootInstanceHandle FInstanceRegistry::AllocateRootInstance(
		UMovieSceneSequence& InRootSequence,
		UObject* InPlaybackContext,
		UMovieSceneCompiledDataManager* InCompiledDataManager)
{
	check(Instances.Num() < 65535);

	const uint16 InstanceSerial = InstanceSerialNumber++;

	FSparseArrayAllocationInfo NewAllocation = Instances.AddUninitialized();
	FRootInstanceHandle InstanceHandle { (uint16)NewAllocation.Index, InstanceSerial };

	if (!InCompiledDataManager)
	{
		InCompiledDataManager = UMovieSceneCompiledDataManager::GetPrecompiledData();
	}

	FSharedPlaybackStateCreateParams PlaybackStateCreateParams;
	PlaybackStateCreateParams.PlaybackContext = InPlaybackContext;
	PlaybackStateCreateParams.RootInstanceHandle = InstanceHandle;
	PlaybackStateCreateParams.Linker = Linker;
	PlaybackStateCreateParams.CompiledDataManager = InCompiledDataManager;

	TSharedRef<FSharedPlaybackState> NewPlaybackState = MakeShared<FSharedPlaybackState>(InRootSequence, PlaybackStateCreateParams);

	new (NewAllocation) FSequenceInstance(NewPlaybackState);

	return InstanceHandle;
}

FInstanceHandle FInstanceRegistry::AllocateSubInstance(FMovieSceneSequenceID SequenceID, FRootInstanceHandle RootInstanceHandle, FInstanceHandle ParentInstanceHandle)
{
	check(Instances.Num() < 65535 && SequenceID != MovieSceneSequenceID::Root && ParentInstanceHandle.IsValid());

	const uint16 InstanceSerial = InstanceSerialNumber++;
	FSparseArrayAllocationInfo NewAllocation = Instances.AddUninitialized();
	FInstanceHandle InstanceHandle { (uint16)NewAllocation.Index, InstanceSerial };
	
	TSharedRef<FSharedPlaybackState> PlaybackState = GetInstance(RootInstanceHandle).GetSharedPlaybackState();

	new (NewAllocation) FSequenceInstance(PlaybackState, InstanceHandle, ParentInstanceHandle, SequenceID);

	PlaybackState->GetCapabilities().OnSubInstanceCreated(PlaybackState, InstanceHandle);

	return InstanceHandle;
}

void FInstanceRegistry::DestroyInstance(FInstanceHandle InstanceHandle)
{
	if (ensureMsgf(Instances.IsValidIndex(InstanceHandle.InstanceID) && Instances[InstanceHandle.InstanceID].GetSerialNumber() == InstanceHandle.InstanceSerial, TEXT("Attempting to destroy an instance an invalid instance handle.")))
	{
		FSequenceInstance& Instance = Instances[InstanceHandle.InstanceID];
		const bool bHasFinished = (GExitPurge || Instance.HasFinished());
		if (!bHasFinished)
		{
			UE_LOGF(LogMovieSceneECS, Verbose, "Instance being destroyed without finishing evaluation.");
		}
		Instance.DestroyImmediately();
		Instances.RemoveAt(InstanceHandle.InstanceID);
	}
}

void FInstanceRegistry::PostInstantation()
{
	InvalidatedObjectBindings.Empty();
	Instances.Shrink();
}

void FInstanceRegistry::TagGarbage()
{
	for (FSequenceInstance& Instance : Instances)
	{
		Instance.Ledger.TagGarbage(Linker);
	}
}

void FInstanceRegistry::CleanupLinkerEntities(const TSet<FMovieSceneEntityID>& ExpiredBoundObjects)
{
	if (ExpiredBoundObjects.Num() != 0)
	{
		for (FSequenceInstance& Instance : Instances)
		{
			Instance.Ledger.CleanupLinkerEntities(ExpiredBoundObjects);
		}
	}
}

FScopedVolatilityManagerSuppression::FScopedVolatilityManagerSuppression(TSharedPtr<FSharedPlaybackState> PlaybackState)
	: WeakPlaybackState(PlaybackState)
{
	ensure(PlaybackState.IsValid());

	FRootInstanceHandle RootInstanceHandle = PlaybackState->GetRootInstanceHandle();
	FInstanceRegistry* InstanceRegistry = PlaybackState->GetLinker()->GetInstanceRegistry();

	FSequenceInstance& Instance = InstanceRegistry->MutateInstance(RootInstanceHandle);
	PreviousVolatilityManager = MoveTemp(Instance.VolatilityManager);
}

FScopedVolatilityManagerSuppression::~FScopedVolatilityManagerSuppression()
{
	if (!WeakPlaybackState.IsValid())
	{
		return;
	}

	FRootInstanceHandle RootInstanceHandle = WeakPlaybackState.Pin()->GetRootInstanceHandle();
	FInstanceRegistry* InstanceRegistry = WeakPlaybackState.Pin()->GetLinker()->GetInstanceRegistry();

	FSequenceInstance& Instance = InstanceRegistry->MutateInstance(RootInstanceHandle);
	Instance.VolatilityManager = MoveTemp(PreviousVolatilityManager);
	Instance.ConditionalRecompile();
}

} // namespace MovieScene
} // namespace UE
