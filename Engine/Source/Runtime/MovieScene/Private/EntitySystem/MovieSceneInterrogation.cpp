// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneInterrogation.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"

namespace UE::MovieScene
{


TEntitySystemLinkerExtensionID<IInterrogationExtension> IInterrogationExtension::GetExtensionID()
{
	static TEntitySystemLinkerExtensionID<IInterrogationExtension> ID = UMovieSceneEntitySystemLinker::RegisterExtension<IInterrogationExtension>();
	return ID;
}


} // namespace UE::MovieScene
