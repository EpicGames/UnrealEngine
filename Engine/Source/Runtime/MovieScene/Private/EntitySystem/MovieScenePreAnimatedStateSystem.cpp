// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieScenePreAnimatedStateSystem.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "IMovieScenePlayer.h"

#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateExtension.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedCaptureSources.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScenePreAnimatedStateSystem)



UMovieSceneCachePreAnimatedStateSystem::UMovieSceneCachePreAnimatedStateSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		// This system relies upon anything that creates entities
		DefineComponentConsumer(GetClass(), FBuiltInComponentTypes::Get()->SymbolicTags.CreatesEntities);
	}
}

bool UMovieSceneCachePreAnimatedStateSystem::IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	using namespace UE::MovieScene;

	// This function can be called on the CDO and instances, so care is taken to do the right thing
	const bool bHasRestoreStateEntities = InLinker->EntityManager.ContainsComponent(FBuiltInComponentTypes::Get()->Tags.RestoreState);
	const bool bIsCapturingGlobalState  = InLinker->PreAnimatedState.IsCapturingGlobalState();

	return bHasRestoreStateEntities || bIsCapturingGlobalState;
}

void UMovieSceneCachePreAnimatedStateSystem::OnLink()
{
}

void UMovieSceneCachePreAnimatedStateSystem::OnUnlink()
{
}

void UMovieSceneCachePreAnimatedStateSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	TArray<IMovieScenePreAnimatedStateSystemInterface*, TInlineAllocator<16>> Interfaces;
	auto ForEachSystem = [&Interfaces](UMovieSceneEntitySystem* InSystem)
	{
		if (IMovieScenePreAnimatedStateSystemInterface* PreAnimInterface = Cast<IMovieScenePreAnimatedStateSystemInterface>(InSystem))
		{
			Interfaces.Add(PreAnimInterface);
		}
	};
	Linker->SystemGraph.IteratePhase(ESystemPhase::Spawn, ForEachSystem);
	Linker->SystemGraph.IteratePhase(ESystemPhase::Instantiation, ForEachSystem);
	Linker->SystemGraph.IteratePhase(ESystemPhase::Scheduling, ForEachSystem);
	Linker->SystemGraph.IteratePhase(ESystemPhase::Evaluation, ForEachSystem);

	IMovieScenePreAnimatedStateSystemInterface::FPreAnimationParameters Params{ &InPrerequisites, &Subsequents, &Linker->PreAnimatedState };
	for (IMovieScenePreAnimatedStateSystemInterface* Interface : Interfaces)
	{
		Interface->SavePreAnimatedState(Params);
	}
}

UMovieSceneRestorePreAnimatedStateSystem::UMovieSceneRestorePreAnimatedStateSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		// This system relies upon anything that creates entities
		DefineImplicitPrerequisite(UMovieSceneCachePreAnimatedStateSystem::StaticClass(), GetClass());
	}
}

bool UMovieSceneRestorePreAnimatedStateSystem::IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	using namespace UE::MovieScene;

	return InLinker->EntityManager.ContainsComponent(FBuiltInComponentTypes::Get()->Tags.RestoreState);
}

void UMovieSceneRestorePreAnimatedStateSystem::OnLink()
{
	using namespace UE::MovieScene;

	UMovieSceneCachePreAnimatedStateSystem* CacheSystem = Linker->LinkSystem<UMovieSceneCachePreAnimatedStateSystem>();
	Linker->SystemGraph.AddReference(this, CacheSystem);
}

void UMovieSceneRestorePreAnimatedStateSystem::OnUnlink()
{
}

void UMovieSceneRestorePreAnimatedStateSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	TArray<IMovieScenePreAnimatedStateSystemInterface*, TInlineAllocator<16>> Interfaces;

	auto ForEachSystem = [&Interfaces](UMovieSceneEntitySystem* InSystem)
	{
		IMovieScenePreAnimatedStateSystemInterface* PreAnimInterface = Cast<IMovieScenePreAnimatedStateSystemInterface>(InSystem);
		if (PreAnimInterface)
		{
			Interfaces.Add(PreAnimInterface);
		}
	};
	Linker->SystemGraph.IteratePhase(ESystemPhase::Spawn, ForEachSystem);
	Linker->SystemGraph.IteratePhase(ESystemPhase::Instantiation, ForEachSystem);
	Linker->SystemGraph.IteratePhase(ESystemPhase::Scheduling, ForEachSystem);
	Linker->SystemGraph.IteratePhase(ESystemPhase::Evaluation, ForEachSystem);

	IMovieScenePreAnimatedStateSystemInterface::FPreAnimationParameters Params{ &InPrerequisites, &Subsequents, &Linker->PreAnimatedState };

	// Iterate backwards restoring stale state
	for (int32 Index = Interfaces.Num()-1; Index >= 0; --Index)
	{
		Interfaces[Index]->RestorePreAnimatedState(Params);
	}

	FPreAnimatedEntityCaptureSource* EntityMetaData = Linker->PreAnimatedState.GetEntityMetaData();
	if (EntityMetaData)
	{
		auto CleanupExpiredObjects = [EntityMetaData](FMovieSceneEntityID EntityID)
		{
			EntityMetaData->StopTrackingEntity(EntityID);
		};

		FEntityTaskBuilder()
		.ReadEntityIDs()
		.FilterAll({ FBuiltInComponentTypes::Get()->Tags.NeedsUnlink })
		.Iterate_PerEntity(&Linker->EntityManager, CleanupExpiredObjects);
	}

	Params.CacheExtension->ResetEntryInvalidation();
}

