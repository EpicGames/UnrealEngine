// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieScenePropertyComponentHandler.h"


namespace UE::MovieScene
{


FRecompositionResult FPropertyRecomposerImpl::RecomposeBlendOperational(const FPropertyDefinition& PropertyDefinition, const FDecompositionQuery& InQuery, const FIntermediatePropertyValueConstRef& InCurrentValue)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	FRecompositionResult Result;
	for (int32 Index = 0; Index < InQuery.Entities.Num(); ++Index)
	{
		Result.Values.Emplace(InCurrentValue.Copy());
	}

	if (InQuery.Entities.Num() == 0)
	{
		return Result;
	}

	const FPropertyRecomposerPropertyInfo Property = OnGetPropertyInfo.Execute(InQuery.Entities[0], InQuery.Object);

	if (Property.BlendChannel == FPropertyRecomposerPropertyInfo::INVALID_BLEND_CHANNEL)
	{
		return Result;
	}

	UMovieSceneBlenderSystem* Blender = Property.BlenderSystem;
	if (!Blender)
	{
		return Result;
	}

	FValueDecompositionParams Params;
	Params.Query = InQuery;
	Params.PropertyEntityID = Property.PropertyEntityID;
	Params.DecomposeBlendChannel = Property.BlendChannel;
	Params.PropertyTag = PropertyDefinition.PropertyType;

	TArrayView<const FPropertyCompositeDefinition> Composites = BuiltInComponents->PropertyRegistry.GetComposites(PropertyDefinition);

	PropertyDefinition.Handler->RecomposeBlendOperational(PropertyDefinition, Composites, Params, Blender, InCurrentValue, Result.Values);

	return Result;
}


} // namespace UE::MovieScene


