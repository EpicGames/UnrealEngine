// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieScenePropertyRegistry.h"
#include "MovieSceneFwd.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "EntitySystem/MovieScenePropertyBinding.h"
#include "EntitySystem/MovieSceneInitialValueSystem.h"
#include "MovieSceneCommonHelpers.h"

#include "Algo/Find.h"


namespace UE
{
namespace MovieScene
{

struct FPropertyAndAddress
{
	FProperty* Property = nullptr;
	uint8* PropertyAddress = nullptr;
};

/**
 * Find a nested property and address from the specified path of the form StructProp.Struct.Inner.
 * Any symbols other than identifiers and dots are not supported (or even checked for).
 */
template<typename CharType>
FPropertyAndAddress FindPropertyFromNestedPath(UStruct* Struct, void* Container, TStringView<CharType> InPath)
{
	// Find the next dot in the path
	int32 DotIndex = InPath.Len();
	const bool bHasTail = InPath.FindChar('.', DotIndex);

	// Look up our property name using FNAME_Find. If no name exists with this name, a property cannot exist for it.
	TStringView<CharType> Head = bHasTail ? InPath.Left(DotIndex) : InPath;
	FName PropertyName(Head.Len(), Head.GetData(), FNAME_Find);
	if (PropertyName.IsNone())
	{
		return FPropertyAndAddress{};
	}

	// Find a property for the head
	FProperty* Property = Struct->FindPropertyByName(PropertyName);
	if (!Property)
	{
		return FPropertyAndAddress{};
	}

	uint8* PropertyAddress = Property->ContainerPtrToValuePtr<uint8>(Container);

	// If we have more properties to look up, start again from the tail path and the current property address
	if (bHasTail)
	{
		FStructProperty* StructProperty = CastField<FStructProperty>(Property);
		if (!StructProperty || !StructProperty->Struct)
		{
			return FPropertyAndAddress{};
		}

		TStringView<CharType> Tail = InPath.RightChop(DotIndex+1);
		return FindPropertyFromNestedPath(StructProperty->Struct, PropertyAddress, Tail);
	}

	return FPropertyAndAddress{ Property, PropertyAddress };
}

FPropertyAndAddress FindPropertyFromPath(UClass* ObjectClass, const FMovieScenePropertyBinding& PropertyBinding)
{
	if (PropertyBinding.PropertyName == PropertyBinding.PropertyPath)
	{
		// Simple case - just a property name
		FProperty* Property = ObjectClass->FindPropertyByName(PropertyBinding.PropertyName);
		if (Property)
		{
			UObject* DefaultObject   = ObjectClass->GetDefaultObject();
			uint8*   PropertyAddress = Property->ContainerPtrToValuePtr<uint8>(DefaultObject);
			return FPropertyAndAddress{ Property, PropertyAddress };
		}
	}
	else
	{
		// Lookup the property name using a string view to avoid allocation.
		// Most properties will be ansi strings
		const FNameEntry* NameEntry = PropertyBinding.PropertyPath.GetComparisonNameEntry();
		if (NameEntry)
		{
			if (NameEntry->IsWide())
			{
				TWideStringBuilder<128> Path;
				NameEntry->AppendNameToString(Path);
				return FindPropertyFromNestedPath(ObjectClass, ObjectClass->GetDefaultObject(), Path.ToView());
			}
			else
			{
				TAnsiStringBuilder<128> Path;
				NameEntry->AppendAnsiNameToString(Path);
				return FindPropertyFromNestedPath(ObjectClass, ObjectClass->GetDefaultObject(), Path.ToView());
			}
		}
	}

	return FPropertyAndAddress{};
}



TOptional<uint16> ComputeFastPropertyPtrOffset(UClass* ObjectClass, const FMovieScenePropertyBinding& PropertyBinding)
{
	using namespace UE::MovieScene;

	FPropertyAndAddress PropertyAndAddress = FindPropertyFromPath(ObjectClass, PropertyBinding);

	const bool bFoundSetter = (!PropertyAndAddress.Property || PropertyAndAddress.Property->HasSetter());
	if (bFoundSetter)
	{
		return TOptional<uint16>();
	}

	if (!PropertyAndAddress.Property)
	{
		return TOptional<uint16>();
	}

	// For object properties, we can't use the fast path, as reference tracking may be necessary
	if (FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(PropertyAndAddress.Property))
	{
		return TOptional<uint16>();
	}

	FString PropertyName = PropertyBinding.PropertyName.ToString();
	
	// If this is a bool property, strip off the 'b' so that the "Set" functions to be 
	// found are, for example, "SetHidden" instead of "SetbHidden"
	FBoolProperty* BoolProperty = CastField<FBoolProperty>(PropertyAndAddress.Property);
	if (BoolProperty)
	{
		PropertyName.RemoveFromStart("b", ESearchCase::CaseSensitive);
	}

	// @todo: Constructing FNames from strings is _very_ costly and we really shouldn't be doing this at runtime.
	//        This is a little better now we use a string builder and an FNAME_Find, but it's still not ideal
	TStringBuilder<128> SetterName;
	SetterName.Append(TEXT("Set"));
	SetterName = SetterName + PropertyName;

	FName SetterFunctionName(SetterName.ToString(), FNAME_Find);

	if (SetterFunctionName.IsNone() || ObjectClass->FindFunctionByName(SetterFunctionName) == nullptr)
	{
		if (BoolProperty)
		{
			// bitfield booleans potentially have an additional byte offset.
			// In practice this is always 0 because the property internal offset itself is incremented, but this is here for completeness
			PropertyAndAddress.PropertyAddress += BoolProperty->GetByteOffset();
		}

		UObject* DefaultObject   = ObjectClass->GetDefaultObject();
		int32    PropertyOffset  = PropertyAndAddress.PropertyAddress - reinterpret_cast<uint8*>(DefaultObject);

		if (PropertyOffset >= 0 && PropertyOffset < int32(uint16(0xFFFF)))
		{
			return uint16(PropertyOffset);
		}
	}

	return TOptional<uint16>();
}

TOptional<FResolvedFastProperty> FPropertyRegistry::ResolveFastProperty(UObject* Object, const FMovieScenePropertyBinding& PropertyBinding, FCustomAccessorView CustomAccessors)
{
	UClass* Class = Object->GetClass();

	if (CustomAccessors.Num() != 0)
	{
		const int32 CustomPropertyIndex = CustomAccessors.FindCustomAccessorIndex(Class, PropertyBinding.PropertyPath);
		if (CustomPropertyIndex != INDEX_NONE)
		{
			check(CustomPropertyIndex < MAX_uint16);

			// This property has a custom property accessor that can apply properties through a static function ptr.
			// Just add the function ptrs to the property entity so they can be called directly
			return FResolvedFastProperty(TInPlaceType<FCustomPropertyIndex>(), FCustomPropertyIndex{ static_cast<uint16>(CustomPropertyIndex) });
		}
	}

	if (PropertyBinding.CanUseClassLookup())
	{
		TOptional<uint16> FastPtrOffset = ComputeFastPropertyPtrOffset(Class, PropertyBinding);
		if (FastPtrOffset.IsSet())
		{
			// This property/object combination has no custom setter function and a constant property offset from the base ptr for all instances of the object.
			return FResolvedFastProperty(TInPlaceType<uint16>(), FastPtrOffset.GetValue());
		}
	}

	return TOptional<FResolvedFastProperty>();
}

TOptional<FResolvedProperty> FPropertyRegistry::ResolveProperty(UObject* Object, const FMovieScenePropertyBinding& PropertyBinding, FCustomAccessorView CustomAccessors)
{
	TOptional< FResolvedFastProperty > FastProperty = FPropertyRegistry::ResolveFastProperty(Object, PropertyBinding, CustomAccessors);
	if (FastProperty.IsSet())
	{
		if (const FCustomPropertyIndex* CustomIndex = FastProperty->TryGet<FCustomPropertyIndex>())
		{
			return FResolvedProperty(TInPlaceType<FCustomPropertyIndex>(), *CustomIndex);
		}
		return FResolvedProperty(TInPlaceType<uint16>(), FastProperty->Get<uint16>());
	}

	// None of the above optimized paths can apply to this property (probably because it has a setter function or because it is within a compound property), so we must use the slow property bindings
	TSharedPtr<FTrackInstancePropertyBindings> SlowBindings = MakeShared<FTrackInstancePropertyBindings>(PropertyBinding.PropertyName, PropertyBinding.PropertyPath.ToString());
	if (!SlowBindings->HasValidBinding(*Object))
	{
		UE_LOGF(LogMovieSceneECS, Warning, "Unable to resolve property '%ls' from '%ls' instance '%ls'", *PropertyBinding.PropertyPath.ToString(), *Object->GetClass()->GetName(), *Object->GetName());
		return TOptional<FResolvedProperty>();
	}

	return FResolvedProperty(TInPlaceType<TSharedPtr<FTrackInstancePropertyBindings>>(), SlowBindings);
}

const FPropertyDefinition* FPropertyRegistry::FindPropertyDefinition(FComponentTypeID ComponentTypeID) const
{
	return Algo::FindBy(Properties, ComponentTypeID, &FPropertyDefinition::PropertyType);
}


void FPropertyDefinition::SetupInitialValueProcessor() const
{
	if (InitialValueType)
	{
		TSharedPtr<IInitialValueProcessor> InitialValueProcessor = Handler->MakeInitialValueProcessor(*this);
		if (InitialValueProcessor)
		{
			FEntityComponentFilter Filter;
			InitialValueProcessor->PopulateFilter(Filter);
			UMovieSceneInitialValueSystem::RegisterProcessor(InitialValueType, MoveTemp(InitialValueProcessor), MoveTemp(Filter));
		}
	}
}

} // namespace MovieScene
} // namespace UE
