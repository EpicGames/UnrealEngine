// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieScenePropertySystemTypes.h"
#include "UObject/ObjectKey.h"
#include "UObject/NameTypes.h"
#include "UObject/Class.h"
#include "Templates/Tuple.h"

namespace UE::MovieScene
{

#if WITH_EDITOR

TSet<TTuple<FObjectKey, FName>> GRegisteredCustomAccessors;

void AddGlobalCustomAccessor(const UClass* ClassType, FName PropertyPath)
{
	GRegisteredCustomAccessors.Add(TTuple<FObjectKey, FName>(ClassType, PropertyPath));
}
void RemoveGlobalCustomAccessor(const UClass* ClassType, FName PropertyPath)
{
	if (UObjectInitialized())
	{
		GRegisteredCustomAccessors.Remove(TTuple<FObjectKey, FName>(ClassType, PropertyPath));
	}
	// else: during engine shutdown, we can't make object keys anymore
}
bool GlobalCustomAccessorExists(const UClass* ClassType, TStringView<WIDECHAR> PropertyPath)
{
	FName PropertyPathName(PropertyPath.Len(), PropertyPath.GetData(), FNAME_Find);
	if (PropertyPathName != NAME_None)
	{
		return GRegisteredCustomAccessors.Contains(TTuple<FObjectKey, FName>(ClassType, PropertyPath));
	}
	return false;
}
bool GlobalCustomAccessorExists(const UClass* ClassType, TStringView<ANSICHAR> PropertyPath)
{
	FName PropertyPathName(PropertyPath.Len(), PropertyPath.GetData(), FNAME_Find);
	if (PropertyPathName != NAME_None)
	{
		return GRegisteredCustomAccessors.Contains(TTuple<FObjectKey, FName>(ClassType, PropertyPath));
	}
	return false;
}

#endif // WITH_EDITOR

} // namespace namespace UE::MovieScene
