// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneRootInstantiatorSystem.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneInstanceRegistry.h"
#include "EntitySystem/MovieSceneComponentRegistry.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneRootInstantiatorSystem)


UMovieSceneRootInstantiatorSystem::UMovieSceneRootInstantiatorSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	RelevantComponent = UE::MovieScene::FBuiltInComponentTypes::Get()->Tags.Root;
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineComponentProducer(GetClass(), UE::MovieScene::FBuiltInComponentTypes::Get()->Tags.Root);
	}
}

void UMovieSceneRootInstantiatorSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();

	FEntityComponentFilter Filter;
	Filter.All({Components->Tags.Root,Components->Tags.NeedsLink});

	TArray<const FEntityAllocation*> Allocations;
	for (const FEntityAllocation* Allocation : Linker->EntityManager.Iterate(&Filter))
	{
		Allocations.Add(Allocation);
	}

	for (const FEntityAllocation* Allocation : Allocations)
	{
		InstantiateAllocation(Allocation);
	}
}

void UMovieSceneRootInstantiatorSystem::InstantiateAllocation(const UE::MovieScene::FEntityAllocation* ParentAllocation)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();

	const FComponentMask PreservationMask = Linker->EntityManager.GetComponents()->GetPreservationMask();

	FMutualComponentInitializers MutualInitializers;
	FComponentMask DerivedEntityType;

	FComponentMask ParentType;
	for (const FComponentHeader& Header : ParentAllocation->GetComponentHeaders())
	{
		ParentType.Set(Header.ComponentType);
	}
	Linker->EntityManager.GetComponents()->Factories.ComputeChildComponents(ParentType, DerivedEntityType);
	Linker->EntityManager.GetComponents()->Factories.ComputeMutuallyInclusiveComponents(EMutuallyInclusiveComponentType::All, DerivedEntityType, MutualInitializers);

	const bool bHasAnyType = DerivedEntityType.Find(true) != INDEX_NONE;
	if (!bHasAnyType)
	{
		return;
	}

	const int32 NumToAdd = ParentAllocation->Num();

	TArray<int32> ParentOffsets;
	ParentOffsets.Reserve(NumToAdd);
	for (int32 ParentOffset = 0; ParentOffset < NumToAdd; ++ParentOffset)
	{
		ParentOffsets.Emplace(ParentOffset);
	}

	TArrayView<const FMovieSceneEntityID> ParentIDs = ParentAllocation->GetEntityIDs();

	FEntityAllocationWriteContext WriteContext(Linker->EntityManager);
	int32 CurrentParentOffset = 0;

	// We attempt to allocate all the linker entities contiguously in memory for efficient initialization,
	// but we may reach capacity constraints within allocations so we may have to run the factories more than once
	while(CurrentParentOffset < NumToAdd)
	{
		// Ask to allocate as many as possible - we may only manage to allocate a smaller number contiguously this iteration however
		int32 NumAdded = NumToAdd - CurrentParentOffset;

		FEntityDataLocation NewLinkerEntities = Linker->EntityManager.AllocateContiguousEntities(DerivedEntityType, &NumAdded);
		FEntityRange ChildRange{ NewLinkerEntities.Allocation, NewLinkerEntities.ComponentOffset, NumAdded };

		MutualInitializers.Execute(ChildRange, WriteContext);

		Linker->EntityManager.InitializeChildAllocation(ParentType, DerivedEntityType, ParentAllocation, MakeArrayView(ParentOffsets.GetData() + CurrentParentOffset, NumAdded), ChildRange);

		TArrayView<const FMovieSceneEntityID> ChildIDs = NewLinkerEntities.Allocation->GetEntityIDs();
		// Iterate backwards because ChildIDs may be removed as we migrate entities
		for (int32 Index = ChildRange.Num-1; Index >= 0; --Index)
		{
			const FMovieSceneEntityID ParentID = ParentIDs[CurrentParentOffset + Index];
			const FMovieSceneEntityID ChildID = ChildIDs[ChildRange.ComponentStartOffset + Index];

			auto CombineChildren = [this, ChildID, &PreservationMask](FMovieSceneEntityID OldChild)
			{
				this->Linker->EntityManager.CombineComponents(ChildID, OldChild, &PreservationMask);
			};
			Linker->EntityManager.IterateImmediateChildren(ParentID, CombineChildren);

			Linker->EntityManager.AddComponent(ParentID, Components->Tags.NeedsUnlink, EEntityRecursion::Children);
			Linker->EntityManager.AddChild(ParentID, ChildID);
		}

		CurrentParentOffset += NumAdded;
	}
}
