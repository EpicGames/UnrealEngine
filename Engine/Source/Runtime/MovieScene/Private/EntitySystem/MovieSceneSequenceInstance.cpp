// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneSequenceInstance.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieSceneSequenceUpdaters.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"

#include "Compilation/MovieSceneCompiledVolatilityManager.h"
#include "Compilation/MovieSceneCompiledDataManager.h"

#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Evaluation/Instances/MovieSceneTrackEvaluator.h"
#include "Evaluation/MovieSceneRootOverridePath.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateExtension.h"

#include "IMovieScenePlayer.h"
#include "Misc/AssertionMacros.h"
#include "MovieSceneLegacyPlayer.h"
#include "MovieSceneSequence.h"
#include "MovieSceneSequencePlayer.h"
#include "MovieSceneTimeHelpers.h"

#include "Algo/IndexOf.h"

namespace UE
{
namespace MovieScene
{


DECLARE_CYCLE_STAT(TEXT("Sequence Instance Update"), MovieSceneEval_SequenceInstanceUpdate, STATGROUP_MovieSceneEval);
DECLARE_CYCLE_STAT(TEXT("[External] Sequence Instance Post-Update"), MovieSceneEval_SequenceInstancePostUpdate, STATGROUP_MovieSceneEval);


void PurgeStaleTrackTemplates(UMovieSceneCompiledDataManager* CompiledDataManager, FMovieSceneCompiledDataID CompiledDataID)
{
	FMovieSceneEvaluationTemplate* EvalTemplate = const_cast<FMovieSceneEvaluationTemplate*>(CompiledDataManager->FindTrackTemplate(CompiledDataID));
	if (EvalTemplate)
	{
		EvalTemplate->PurgeStaleTracks();
	}

	// Do the same for all subsequences
	const FMovieSceneSequenceHierarchy* Hierarchy = CompiledDataManager->FindHierarchy(CompiledDataID);
	if (Hierarchy)
	{
		for (const TTuple<FMovieSceneSequenceID, FMovieSceneSubSequenceData>& Pair : Hierarchy->AllSubSequenceData())
		{
			UMovieSceneSequence* SubSequence = Pair.Value.GetLoadedSequence();
			if (!SubSequence)
			{
				continue;
			}
			FMovieSceneCompiledDataID SubCompiledDataID = CompiledDataManager->FindDataID(SubSequence);
			if (!SubCompiledDataID.IsValid())
			{
				continue;
			}

			FMovieSceneEvaluationTemplate* SubEvalTemplate = const_cast<FMovieSceneEvaluationTemplate*>(CompiledDataManager->FindTrackTemplate(SubCompiledDataID));
			if (SubEvalTemplate)
			{
				SubEvalTemplate->PurgeStaleTracks();
			}
		}
	}
}




FSequenceInstance::FSequenceInstance(TSharedRef<FSharedPlaybackState> PlaybackState)
	: SharedPlaybackState(PlaybackState)
	, SequenceID(MovieSceneSequenceID::Root)
	, RootOverrideSequenceID(MovieSceneSequenceID::Root)
	, InstanceHandle(PlaybackState->GetRootInstanceHandle())
	, RootInstanceHandle(PlaybackState->GetRootInstanceHandle())
	, bInitialized(false)
{
	UpdateFlags = ESequenceInstanceUpdateFlags::None;

	// Root instances always start in a finished state in order to ensure that 'Start'
	// is called correctly for the top level instance. This is subtly different from
	// bHasEverUpdated since a sequence instance can be Finished and restarted multiple times
	bFinished = true;
	bHasEverUpdated = false;

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
	UMovieSceneSequence* RootSequence = PlaybackState->GetRootSequence();
	RootSequenceName = RootSequence->GetPathName();
#endif
}

FSequenceInstance::FSequenceInstance(TSharedRef<FSharedPlaybackState> PlaybackState, FInstanceHandle InInstanceHandle, FInstanceHandle InParentInstanceHandle, FMovieSceneSequenceID InSequenceID)
	: SharedPlaybackState(PlaybackState)
	, SequenceID(InSequenceID)
	, RootOverrideSequenceID(MovieSceneSequenceID::Invalid)
	, InstanceHandle(InInstanceHandle)
	, ParentInstanceHandle(InParentInstanceHandle)
	, RootInstanceHandle(PlaybackState->GetRootInstanceHandle())
	, bInitialized(false)
{
	UpdateFlags = ESequenceInstanceUpdateFlags::None;

	// Sub Sequence instances always start in a non-finished state because they will only ever
	// be created if they are active, and the Start/Update/Finish loop does not apply to sub-instances
	bFinished = false;
	bHasEverUpdated = false;
}

void FSequenceInstance::Initialize()
{
	ensureMsgf(!bInitialized, TEXT("This instance was already initialized!"));
	bInitialized = true;

	InvalidateCachedData();
}

FSequenceInstance::~FSequenceInstance()
{
	const int32 RefCount = SharedPlaybackState.GetSharedReferenceCount();
	if (!ensureAlwaysMsgf(
			RootInstanceHandle != InstanceHandle || RefCount == 1,
			TEXT("References to SharedPlaybackState should not be held past the lifetime of its root sequence instance. SharedPlaybackState has %d references."),
			RefCount))
	{
#if !UE_BUILD_SHIPPING
		SharedPlaybackState->SetDebugBreakOnDestroy();
#endif
	}
}

FSequenceInstance::FSequenceInstance(FSequenceInstance&&) = default;

FSequenceInstance& FSequenceInstance::operator=(FSequenceInstance&&) = default;

IMovieScenePlayer* FSequenceInstance::GetPlayer() const
{
	return FPlayerIndexPlaybackCapability::GetPlayer(SharedPlaybackState);
}

uint16 FSequenceInstance::GetPlayerIndex() const
{
	return FPlayerIndexPlaybackCapability::GetPlayerIndex(SharedPlaybackState);
}

void FSequenceInstance::InitializeLegacyEvaluator()
{
	UMovieSceneCompiledDataManager* CompiledDataManager = SharedPlaybackState->GetCompiledDataManager();
	const FMovieSceneCompiledDataID RootCompiledDataID = SharedPlaybackState->GetRootCompiledDataID();
	const FMovieSceneCompiledDataEntry& CompiledEntry   = CompiledDataManager->GetEntryRef(RootCompiledDataID);

	if (EnumHasAnyFlags(CompiledEntry.AccumulatedMask, EMovieSceneSequenceCompilerMask::EvaluationTemplate))
	{
		IMovieScenePlayer* Player = FPlayerIndexPlaybackCapability::GetPlayer(SharedPlaybackState);
		if (!Player)
		{
			if (ILegacyPlayerProviderPlaybackCapability* LegacyPlayerProvider = SharedPlaybackState->FindCapability<ILegacyPlayerProviderPlaybackCapability>())
			{
				UE_LOGF(LogMovieScene, Log, "Initializing a legacy player for sequence '%ls'.", *GetNameSafe(SharedPlaybackState->GetRootSequence()));
				Player = LegacyPlayerProvider->CreateLegacyPlayer(SharedPlaybackState);
				if (Player)
				{
					SharedPlaybackState->SetOrAddCapability<FPlayerIndexPlaybackCapability>(Player->GetUniqueIndex());
				}
			}
		}
		if (!Player)
		{
			UE_LOGF(LogMovieScene, Log, "Initializing a default legacy player for sequence '%ls'.", *GetNameSafe(SharedPlaybackState->GetRootSequence()));
			InitializeLegacyPlayer();
			Player = LegacyPlayer.Get();
			if (ensure(Player))
			{
				SharedPlaybackState->SetOrAddCapability<FPlayerIndexPlaybackCapability>(Player->GetUniqueIndex());
			}
		}
		if (ensureMsgf(Player, TEXT("Sequence has legacy templates but no player! Legacy tracks will be disabled!")))
		{
			UpdateFlags |= ESequenceInstanceUpdateFlags::HasLegacyTemplates;

			if (!LegacyEvaluator)
			{
				LegacyEvaluator = MakeUnique<FMovieSceneTrackEvaluator>(CompiledEntry.GetSequence(), RootCompiledDataID, CompiledDataManager);
			}
		}
	}
	else if (LegacyEvaluator)
	{
		IMovieScenePlayer* Player = FPlayerIndexPlaybackCapability::GetPlayer(SharedPlaybackState);
		if (ensureMsgf(Player, TEXT("Sequence needs to teardown legacy evaluator, but has no player! Legacy track evaluators will leak!")))
		{
			LegacyEvaluator->Finish(*Player);
			LegacyEvaluator = nullptr;
		}
		UpdateFlags &= ~ESequenceInstanceUpdateFlags::HasLegacyTemplates;
	}
}

void FSequenceInstance::InitializeLegacyPlayer()
{
	IMovieScenePlayer* Player = FPlayerIndexPlaybackCapability::GetPlayer(SharedPlaybackState);
	if (!Player)
	{
		LegacyPlayer = MakeUnique<FMovieSceneLegacyPlayer>(SharedPlaybackState);
		SharedPlaybackState->AddCapability<FPlayerIndexPlaybackCapability>(LegacyPlayer->GetUniqueIndex());
	}
}

void FSequenceInstance::InvalidateCachedData(ESequenceInstanceInvalidationType InvalidationType)
{
	ensureMsgf(bInitialized, TEXT("Sequence instance hasn't been initialized yet!"));

	UMovieSceneSequence* RootSequence = SharedPlaybackState->GetRootSequence();
	if (!ensureMsgf(RootSequence, TEXT("Sequence instance has a null root sequence!")))
	{
		return;
	}

	UMovieSceneCompiledDataManager* CompiledDataManager = SharedPlaybackState->GetCompiledDataManager();
	if (!ensureMsgf(
				CompiledDataManager, 
				TEXT("Sequence instance (%s) has no compiled data manager! Re-building a default one."),
				*RootSequence->GetPathName()))
	{
		CompiledDataManager = UMovieSceneCompiledDataManager::GetPrecompiledData();
	}

	FMovieSceneCompiledDataID RootCompiledDataID = SharedPlaybackState->GetRootCompiledDataID();
	if (!ensureMsgf(
				RootCompiledDataID.IsValid(), 
				TEXT("Sequence instance (%s) has invalid data ID for root sequence! Re-building it."),
				*RootSequence->GetPathName()))
	{
		RootCompiledDataID = CompiledDataManager->GetDataID(RootSequence);
	}

	if (!ensureMsgf(
				CompiledDataManager->ValidateEntry(RootCompiledDataID, RootSequence),
				TEXT("Sequence instance (%s) has invalid data ID for root sequence! Aborting invalidation of cached data."),
				*RootSequence->GetPathName()))
	{
		return;
	}

	Ledger.Invalidate();

	UpdateFlags = ESequenceInstanceUpdateFlags::None;

	FMovieSceneEvaluationState* State = SharedPlaybackState->FindCapability<FMovieSceneEvaluationState>();

	if (SequenceID == MovieSceneSequenceID::Root)
	{
		SharedPlaybackState->InvalidateCachedData();

		if (State)
		{
			State->AssignSequence(SequenceID, *SharedPlaybackState->GetRootSequence(), SharedPlaybackState);
		}

		// Try and recreate the volatility manager if this sequence is now volatile
		if (!VolatilityManager)
		{
			VolatilityManager = FCompiledDataVolatilityManager::Construct(SharedPlaybackState);
		}

		ISequenceUpdater::FactoryInstance(SequenceUpdater, CompiledDataManager, RootCompiledDataID);

		SequenceUpdater->InvalidateCachedData(SharedPlaybackState, InvalidationType);
		SequenceUpdater->PopulateUpdateFlags(SharedPlaybackState, UpdateFlags);

		if (LegacyEvaluator)
		{
			LegacyEvaluator->InvalidateCachedData();
		}

		InitializeLegacyEvaluator();
	}
	else if (UMovieSceneSequence* SubSequence = SharedPlaybackState->GetSequence(SequenceID))
	{
		if (State)
		{
			State->AssignSequence(SequenceID, *SubSequence, SharedPlaybackState);
		}
	}
}

bool FSequenceInstance::ConditionalRecompile()
{
	ensureMsgf(bInitialized, TEXT("This instance hasn't been initialized yet!"));

	if (VolatilityManager)
	{
		if (VolatilityManager->ConditionalRecompile())
		{
			InvalidateCachedData(ESequenceInstanceInvalidationType::DataChanged);
			return true;
		}
	}

	return false;
}

void FSequenceInstance::DissectContext(const FMovieSceneContext& InContext, TArray<TRange<FFrameTime>>& OutDissections)
{
	ensureMsgf(bInitialized, TEXT("This instance hasn't been initialized yet!"));

	if (EnumHasAnyFlags(UpdateFlags, ESequenceInstanceUpdateFlags::NeedsDissection))
	{
		check(SequenceID == MovieSceneSequenceID::Root);
		SequenceUpdater->DissectContext(SharedPlaybackState, InContext, OutDissections);
	}
}

void FSequenceInstance::Start(const FMovieSceneContext& InContext)
{
	ensureMsgf(bInitialized, TEXT("This instance hasn't been initialized yet!"));
	ensureMsgf(SequenceID == MovieSceneSequenceID::Root, TEXT("Only root sequences should be started"));

	bFinished = false;
	bHasEverUpdated = true;

	check(RootInstanceHandle == InstanceHandle);

	SequenceUpdater->Start(SharedPlaybackState, InContext);
}

void FSequenceInstance::Update(const FMovieSceneContext& InContext)
{
	SCOPE_CYCLE_COUNTER(MovieSceneEval_SequenceInstanceUpdate);
	SCOPE_CYCLE_UOBJECT(ContextScope, SharedPlaybackState->GetRootSequence());

	ensureMsgf(bInitialized, TEXT("This instance hasn't been initialized yet!"));

	bHasEverUpdated = true;

	if (bFinished)
	{
		Start(InContext);
	}

	check(RootInstanceHandle == InstanceHandle);

	Context = InContext;
	SequenceUpdater->Update(SharedPlaybackState, InContext);
}

bool FSequenceInstance::CanFinishImmediately() const
{
	if (SequenceUpdater)
	{
		check(RootInstanceHandle == InstanceHandle);

		return SequenceUpdater->CanFinishImmediately(SharedPlaybackState);
	}

	return true;
}

void FSequenceInstance::Finish()
{
	if (IsRootSequence() && !bHasEverUpdated)
	{
		return;
	}

	UMovieSceneEntitySystemLinker* Linker = SharedPlaybackState->GetLinker();
	Linker->EntityManager.IncrementSystemSerial();
	bFinished = true;
	Ledger.UnlinkEverything(Linker);

	Ledger = FEntityLedger();

	if (SequenceUpdater)
	{
		check(RootInstanceHandle == InstanceHandle);
		SequenceUpdater->Finish(SharedPlaybackState);
	}

	IMovieScenePlayer* Player = FPlayerIndexPlaybackCapability::GetPlayer(SharedPlaybackState);
	if (LegacyEvaluator && ensure(Player))
	{
		LegacyEvaluator->Finish(*Player);
	}

	if (IsRootSequence())
	{
		if (FMovieSceneSpawnRegister* SpawnRegister = SharedPlaybackState->FindCapability<FMovieSceneSpawnRegister>())
		{
			SpawnRegister->ForgetExternallyOwnedSpawnedObjects(SharedPlaybackState);
			SpawnRegister->CleanUp(SharedPlaybackState);
		}

		if (Player && Player->PreAnimatedState.IsCapturingGlobalPreAnimatedState())
		{
			Linker->PreAnimatedState.RestoreGlobalState(FRestoreStateParams{ Linker, RootInstanceHandle });
		}
	}
}

void FSequenceInstance::PreEvaluation()
{
	if (!EnumHasAnyFlags(UpdateFlags, ESequenceInstanceUpdateFlags::NeedsPreEvaluation))
	{
		return;
	}

	if (IsRootSequence())
	{
		IMovieScenePlayer* Player = FPlayerIndexPlaybackCapability::GetPlayer(SharedPlaybackState);
		if (Player)
		{
			Player->PreEvaluation(Context);
		}
	}
}

void FSequenceInstance::RunLegacyTrackTemplates()
{
	if (LegacyEvaluator)
	{
		IMovieScenePlayer* Player = FPlayerIndexPlaybackCapability::GetPlayer(SharedPlaybackState);
		if (Player)
		{
			if (bFinished)
			{
				LegacyEvaluator->Finish(*Player);
			}
			else
			{
				LegacyEvaluator->Evaluate(Context, *Player, RootOverrideSequenceID);
			}
		}
	}
}

void FSequenceInstance::PostEvaluation()
{
	if (IsRootSequence() && EnumHasAnyFlags(UpdateFlags, ESequenceInstanceUpdateFlags::NeedsPostEvaluation))
	{
		IMovieScenePlayer* Player = FPlayerIndexPlaybackCapability::GetPlayer(SharedPlaybackState);
		if (Player)
		{
			SCOPE_CYCLE_COUNTER(MovieSceneEval_SequenceInstancePostUpdate);


			// DANGER: This function is highly fragile due to the nature of IMovieScenePlayer::PostEvaluation
			//         being able to re-evaluate sequences. Ultimately this can lead to FSequenceInstances being
			//         created, destroyed, or reallocated. As such
			//
			//                  ***** the current this ptr can become invalid at any point ***** 
			//
			//         Any code which needs to run after PostEvaluate must cache any member variables it needs on
			//         the stack _before_ Player->PostEvaluation is called.


			// If this sequence is volatile and has legacy track templates, purge any stale track templates from the compiled data after evaluation
			const bool bShouldPurgeTemplates = VolatilityManager && LegacyEvaluator;

			UMovieSceneCompiledDataManager* LocalCompiledDataManager = bShouldPurgeTemplates ? Player->GetEvaluationTemplate().GetCompiledDataManager() : nullptr;
			FMovieSceneCompiledDataID       LocalCompiledDataID      = bShouldPurgeTemplates ? Player->GetEvaluationTemplate().GetCompiledDataID()      : FMovieSceneCompiledDataID();

			Player->PostEvaluation(Context);

			if (LocalCompiledDataManager)
			{
				PurgeStaleTrackTemplates(LocalCompiledDataManager, LocalCompiledDataID);
			}
		}
	}
}

void FSequenceInstance::DestroyImmediately()
{
	UMovieSceneEntitySystemLinker* Linker = SharedPlaybackState->GetLinker();
	
	if (!Ledger.IsEmpty() && ensure(Linker))
	{
		UE_LOGF(LogMovieSceneECS, Verbose, "Instance being destroyed without first having been finished by calling Finish()");
		Ledger.UnlinkEverything(Linker, EUnlinkEverythingMode::CleanGarbage);
	}

	if (SequenceUpdater)
	{
		SequenceUpdater->Destroy(SharedPlaybackState);
	}
}

void FSequenceInstance::OverrideRootSequence(FMovieSceneSequenceID NewRootSequenceID)
{
	if (SequenceUpdater)
	{
		check(RootInstanceHandle == InstanceHandle);
		SequenceUpdater->OverrideRootSequence(SharedPlaybackState, NewRootSequenceID);
	}

	RootOverrideSequenceID = NewRootSequenceID;
}

bool FSequenceInstance::EvaluateCondition(const FGuid& BindingID, const FMovieSceneSequenceID& InSequenceID, const UMovieSceneCondition* Condition, UObject* ConditionOwnerObject) const
{
	if (SequenceUpdater)
	{
		return SequenceUpdater->EvaluateCondition(BindingID, InSequenceID, Condition, ConditionOwnerObject, SharedPlaybackState);
	}
	return true;
}

FInstanceHandle FSequenceInstance::FindSubInstance(FMovieSceneSequenceID SubSequenceID) const
{
	return SequenceUpdater ? SequenceUpdater->FindSubInstance(SubSequenceID) : FInstanceHandle();
}

FMovieSceneEntityID FSequenceInstance::FindEntity(UObject* Owner, uint32 EntityID) const
{
	return Ledger.FindImportedEntity(FMovieSceneEvaluationFieldEntityKey{ decltype(FMovieSceneEvaluationFieldEntityKey::EntityOwner)(Owner), EntityID });
}

void FSequenceInstance::FindEntities(UObject* Owner, TArray<FMovieSceneEntityID>& OutEntityIDs) const
{
	Ledger.FindImportedEntities(Owner, OutEntityIDs);
}

FSubSequencePath FSequenceInstance::GetSubSequencePath() const
{
	return FSubSequencePath(SequenceID, SharedPlaybackState);
}

bool FSequenceInstance::ConditionalRecompile(UMovieSceneEntitySystemLinker* Linker)
{
	return ConditionalRecompile();
}

void FSequenceInstance::DissectContext(UMovieSceneEntitySystemLinker* Linker, const FMovieSceneContext& InContext, TArray<TRange<FFrameTime>>& OutDissections)
{
	DissectContext(InContext, OutDissections);
}

void FSequenceInstance::Start(UMovieSceneEntitySystemLinker* Linker, const FMovieSceneContext& InContext)
{
	Start(InContext);
}

void FSequenceInstance::PreEvaluation(UMovieSceneEntitySystemLinker* Linker)
{
	PreEvaluation();
}

void FSequenceInstance::Update(UMovieSceneEntitySystemLinker* Linker, const FMovieSceneContext& InContext)
{
	Update(InContext);
}

bool FSequenceInstance::CanFinishImmediately(UMovieSceneEntitySystemLinker* Linker) const
{
	return CanFinishImmediately();
}

void FSequenceInstance::Finish(UMovieSceneEntitySystemLinker* Linker)
{
	Finish();
}

void FSequenceInstance::PostEvaluation(UMovieSceneEntitySystemLinker* Linker)
{
	PostEvaluation();
}

void FSequenceInstance::InvalidateCachedData(UMovieSceneEntitySystemLinker* Linker)
{
	InvalidateCachedData();
}

void FSequenceInstance::DestroyImmediately(UMovieSceneEntitySystemLinker* Linker)
{
	DestroyImmediately();
}

void FSequenceInstance::OverrideRootSequence(UMovieSceneEntitySystemLinker* Linker, FMovieSceneSequenceID NewRootSequenceID)
{
	OverrideRootSequence(NewRootSequenceID);
}

} // namespace MovieScene
} // namespace UE
