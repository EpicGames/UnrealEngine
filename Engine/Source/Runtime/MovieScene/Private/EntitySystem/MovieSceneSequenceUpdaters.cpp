// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneSequenceUpdaters.h"
#include "EntitySystem/MovieSceneSequenceInstance.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneEntitySystem.h"

#include "Templates/UniquePtr.h"
#include "Containers/BitArray.h"
#include "Containers/SortedMap.h"

#include "MovieSceneTransformTypes.h"
#include "MovieSceneSequence.h"
#include "MovieSceneSequenceID.h"
#include "Evaluation/MovieScenePlayback.h"
#include "Compilation/MovieSceneCompiledDataManager.h"

#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Evaluation/MovieSceneRootOverridePath.h"

#include "MovieSceneTimeHelpers.h"
#include "Channels/MovieSceneTimeWarpChannel.h"

#include "Algo/IndexOf.h"
#include "Algo/Transform.h"
#include "Algo/Unique.h"
#include "Sections/MovieSceneSubSection.h"

namespace UE
{
namespace MovieScene
{

struct FMovieSceneDeterminismFenceWithSubframe
{
	FFrameTime FrameTime;
	uint8 bInclusive : 1;
};

/** Flat sequence updater (ie, no hierarchy) */
struct FSequenceUpdater_Flat : ISequenceUpdater
{
	explicit FSequenceUpdater_Flat(FMovieSceneCompiledDataID InCompiledDataID);
	~FSequenceUpdater_Flat();

	virtual void PopulateUpdateFlags(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, ESequenceInstanceUpdateFlags& OutUpdateFlags) override;
	virtual void DissectContext(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, const FMovieSceneContext& Context, TArray<TRange<FFrameTime>>& OutDissections) override;
	virtual void Start(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, const FMovieSceneContext& InContext) override;
	virtual void Update(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, const FMovieSceneContext& Context) override;
	virtual bool CanFinishImmediately(TSharedRef<const FSharedPlaybackState> SharedPlaybackState) const override;
	virtual void Finish(TSharedRef<const FSharedPlaybackState> SharedPlaybackState) override;
	virtual void InvalidateCachedData(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, ESequenceInstanceInvalidationType InvalidationType) override;
	virtual void Destroy(TSharedRef<const FSharedPlaybackState> SharedPlaybackState) override;
	virtual TUniquePtr<ISequenceUpdater> MigrateToHierarchical() override;
	virtual FInstanceHandle FindSubInstance(FMovieSceneSequenceID SubSequenceID) const override { return FInstanceHandle(); }
	virtual void OverrideRootSequence(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, FMovieSceneSequenceID NewRootOverrideSequenceID) override {}
	virtual bool EvaluateCondition(const FGuid& BindingID, const FMovieSceneSequenceID& SequenceID, const UMovieSceneCondition* Condition, UObject* ConditionOwnerObject, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const override;

private:

	TRange<FFrameNumber> CachedEntityRange;

	TOptional<TArray<FMovieSceneDeterminismFence>> CachedDeterminismFences;
	FMovieSceneCompiledDataID CompiledDataID;

	TOptional<bool> bDynamicWeighting;

	// Conditional entities that need to be re-checked in between entity ranges.
	FMovieSceneEvaluationFieldEntitySet CachedPerTickConditionalEntities;

	// Cached results for conditions that only need to be checked once, stored by the cache key returned by the condition itself.
	mutable TMap<uint32, bool> CachedConditionResults;
};

/** Hierarchical sequence updater */
struct FSequenceUpdater_Hierarchical : ISequenceUpdater
{
	explicit FSequenceUpdater_Hierarchical(FMovieSceneCompiledDataID InCompiledDataID);

	~FSequenceUpdater_Hierarchical();

	virtual void PopulateUpdateFlags(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, ESequenceInstanceUpdateFlags& OutUpdateFlags) override;
	virtual void DissectContext(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, const FMovieSceneContext& Context, TArray<TRange<FFrameTime>>& OutDissections) override;
	virtual void Start(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, const FMovieSceneContext& InContext) override;
	virtual void Update(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, const FMovieSceneContext& Context) override;
	virtual bool CanFinishImmediately(TSharedRef<const FSharedPlaybackState> SharedPlaybackState) const override;
	virtual void Finish(TSharedRef<const FSharedPlaybackState> SharedPlaybackState) override;
	virtual void InvalidateCachedData(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, ESequenceInstanceInvalidationType InvalidationType) override;
	virtual void Destroy(TSharedRef<const FSharedPlaybackState> SharedPlaybackState) override;
	virtual TUniquePtr<ISequenceUpdater> MigrateToHierarchical() override { return nullptr; }
	virtual FInstanceHandle FindSubInstance(FMovieSceneSequenceID SubSequenceID) const override { return SequenceInstances.FindRef(SubSequenceID).Handle; }
	virtual void OverrideRootSequence(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, FMovieSceneSequenceID NewRootOverrideSequenceID) override;
	virtual bool EvaluateCondition(const FGuid& BindingID, const FMovieSceneSequenceID& SequenceID, const UMovieSceneCondition* Condition, UObject* ConditionOwnerObject, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const override;

private:

	TRange<FFrameNumber> UpdateEntitiesForSequence(const FMovieSceneEntityComponentField* ComponentField, FFrameTime SequenceTime, FMovieSceneEvaluationFieldEntitySet& OutEntities);

	FInstanceHandle GetOrCreateSequenceInstance(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, UMovieSceneSequence* SubSequence, const FMovieSceneSequenceHierarchy* Hierarchy, FInstanceRegistry* InstanceRegistry, FMovieSceneSequenceID SequenceID);

private:

	TRange<FFrameNumber> CachedEntityRange;

	struct FSubInstanceData
	{
		FGuid SequenceSignature;
		FInstanceHandle Handle;
	};
	TSortedMap<FMovieSceneSequenceID, FSubInstanceData, TInlineAllocator<8>> SequenceInstances;

	FMovieSceneCompiledDataID CompiledDataID;

	FMovieSceneSequenceID RootOverrideSequenceID;

	TOptional<bool> bDynamicWeighting;

	// Conditional entities per sequence ID in the hierarchy that need to be re-checked in between entity ranges.
	TMap<FMovieSceneSequenceID, FMovieSceneEvaluationFieldEntitySet> CachedPerTickConditionalEntities;

	// Cached results for conditions that only need to be checked once, stored by the cache key returned by the condition itself.
	mutable TMap<uint32, bool> CachedConditionResults;
};

void DissectRange(TArrayView<const FMovieSceneDeterminismFenceWithSubframe> InDissectionTimes, const TRange<FFrameTime>& Bounds, TArray<TRange<FFrameTime>>& OutDissections)
{
	if (InDissectionTimes.Num() == 0)
	{
		return;
	}

	TRangeBound<FFrameTime> LowerBound = Bounds.GetLowerBound();

	for (int32 Index = 0; Index < InDissectionTimes.Num(); ++Index)
	{
		FMovieSceneDeterminismFenceWithSubframe DissectionFence = InDissectionTimes[Index];

		TRange<FFrameTime> Dissection = DissectionFence.bInclusive
			? TRange<FFrameTime>(LowerBound, TRangeBound<FFrameTime>::Inclusive(DissectionFence.FrameTime))
			: TRange<FFrameTime>(LowerBound, TRangeBound<FFrameTime>::Exclusive(DissectionFence.FrameTime));

		if (!Dissection.IsEmpty())
		{
			ensureAlwaysMsgf(Bounds.Contains(Dissection), TEXT("Dissection specified for a range outside of the current bounds"));

			OutDissections.Add(Dissection);

			LowerBound = TRangeBound<FFrameTime>::FlipInclusion(Dissection.GetUpperBound());
		}
	}

	TRange<FFrameTime> TailRange(LowerBound, Bounds.GetUpperBound());
	if (!TailRange.IsEmpty())
	{
		OutDissections.Add(TailRange);
	}
}

void DissectRange(TArrayView<const FMovieSceneDeterminismFence> InDissectionTimes, const TRange<FFrameTime>& Bounds, TArray<TRange<FFrameTime>>& OutDissections)
{
	if (InDissectionTimes.Num() == 0)
	{
		return;
	}

	TRangeBound<FFrameTime> LowerBound = Bounds.GetLowerBound();

	for (int32 Index = 0; Index < InDissectionTimes.Num(); ++Index)
	{
		FMovieSceneDeterminismFence DissectionFence = InDissectionTimes[Index];

		TRange<FFrameTime> Dissection = DissectionFence.bInclusive
			? TRange<FFrameTime>(LowerBound, TRangeBound<FFrameTime>::Inclusive(DissectionFence.FrameNumber))
			: TRange<FFrameTime>(LowerBound, TRangeBound<FFrameTime>::Exclusive(DissectionFence.FrameNumber));

		if (!Dissection.IsEmpty())
		{
			ensureAlwaysMsgf(Bounds.Contains(Dissection), TEXT("Dissection specified for a range outside of the current bounds"));

			OutDissections.Add(Dissection);

			LowerBound = TRangeBound<FFrameTime>::FlipInclusion(Dissection.GetUpperBound());
		}
	}

	TRange<FFrameTime> TailRange(LowerBound, Bounds.GetUpperBound());
	if (!TailRange.IsEmpty())
	{
		OutDissections.Add(TailRange);
	}
}

TArrayView<const FMovieSceneDeterminismFence> GetFencesWithinRange(TArrayView<const FMovieSceneDeterminismFence> Fences, const TRange<FFrameTime>& Boundary)
{
	if (Fences.Num() == 0 || Boundary.IsEmpty())
	{
		return TArrayView<const FMovieSceneDeterminismFence>();
	}

	// Take care to include or exclude the lower bound of the range if it's on a whole frame number
	const int32 StartFence = Boundary.GetLowerBound().IsOpen()
		? 0
		: Boundary.GetLowerBound().IsInclusive() && Boundary.GetLowerBoundValue().GetSubFrame() == 0.0
			? Algo::LowerBoundBy(Fences, Boundary.GetLowerBoundValue().FrameNumber, &FMovieSceneDeterminismFence::FrameNumber)
			: Algo::UpperBoundBy(Fences, Boundary.GetLowerBoundValue().FrameNumber, &FMovieSceneDeterminismFence::FrameNumber);

	if (StartFence >= Fences.Num())
	{
		return TArrayView<const FMovieSceneDeterminismFence>();
	}

	const int32 EndFence = Boundary.GetUpperBound().IsOpen()
		? 0
		: Boundary.GetUpperBound().IsInclusive() && Boundary.GetUpperBoundValue().GetSubFrame() == 0.0
			? Algo::LowerBoundBy(Fences, Boundary.GetUpperBoundValue().FrameNumber, &FMovieSceneDeterminismFence::FrameNumber)
			: Algo::UpperBoundBy(Fences, Boundary.GetUpperBoundValue().FrameNumber, &FMovieSceneDeterminismFence::FrameNumber);

	const int32 NumFences = FMath::Max(0, EndFence - StartFence);
	if (NumFences == 0)
	{
		return TArrayView<const FMovieSceneDeterminismFence>();
	}

	return MakeArrayView(Fences.GetData() + StartFence, NumFences);
}


void ISequenceUpdater::FactoryInstance(TUniquePtr<ISequenceUpdater>& OutPtr, UMovieSceneCompiledDataManager* CompiledDataManager, FMovieSceneCompiledDataID CompiledDataID)
{
	const bool bHierarchical = CompiledDataManager->FindHierarchy(CompiledDataID) != nullptr;

	if (!OutPtr)
	{
		if (!bHierarchical)
		{
			OutPtr = MakeUnique<FSequenceUpdater_Flat>(CompiledDataID);
		}
		else
		{
			OutPtr = MakeUnique<FSequenceUpdater_Hierarchical>(CompiledDataID);
		}
	}
	else if (bHierarchical)
	{ 
		TUniquePtr<ISequenceUpdater> NewHierarchical = OutPtr->MigrateToHierarchical();
		if (NewHierarchical)
		{
			OutPtr = MoveTemp(NewHierarchical);
		}
	}
}

FSequenceUpdater_Flat::FSequenceUpdater_Flat(FMovieSceneCompiledDataID InCompiledDataID)
	: CompiledDataID(InCompiledDataID)
{
	CachedEntityRange = TRange<FFrameNumber>::Empty();
}

FSequenceUpdater_Flat::~FSequenceUpdater_Flat()
{
}

TUniquePtr<ISequenceUpdater> FSequenceUpdater_Flat::MigrateToHierarchical()
{
	return MakeUnique<FSequenceUpdater_Hierarchical>(CompiledDataID);
}

void FSequenceUpdater_Flat::PopulateUpdateFlags(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, ESequenceInstanceUpdateFlags& OutUpdateFlags)
{
	if (!CachedDeterminismFences.IsSet())
	{
		UMovieSceneCompiledDataManager*               CompiledDataManager = SharedPlaybackState->GetCompiledDataManager();
		TArrayView<const FMovieSceneDeterminismFence> DeterminismFences   = CompiledDataManager->GetEntryRef(CompiledDataID).DeterminismFences;

		if (DeterminismFences.Num() != 0)
		{
			CachedDeterminismFences = TArray<FMovieSceneDeterminismFence>(DeterminismFences.GetData(), DeterminismFences.Num());
		}
		else
		{
			CachedDeterminismFences.Emplace();
		}
	}

	if (IMovieScenePlayer* Player = FPlayerIndexPlaybackCapability::GetPlayer(SharedPlaybackState))
	{
		Player->PopulateUpdateFlags(OutUpdateFlags);
	}

	if (CachedDeterminismFences.IsSet() && CachedDeterminismFences->Num() > 0)
	{
		OutUpdateFlags |= ESequenceInstanceUpdateFlags::NeedsDissection;
	}

	const FMovieSceneSequenceHierarchy* Hierarchy = SharedPlaybackState->GetCompiledDataManager()->FindHierarchy(CompiledDataID);
	if (Hierarchy && Hierarchy->GetRootTransform().FindFirstWarpDomain() == ETimeWarpChannelDomain::Time)
	{
		// Time-warped root transforms require dissection to manipulate the evaluation range
		OutUpdateFlags |= ESequenceInstanceUpdateFlags::NeedsDissection;
	}
}

void FSequenceUpdater_Flat::DissectContext(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, const FMovieSceneContext& Context, TArray<TRange<FFrameTime>>& OutDissections)
{
	if (!CachedDeterminismFences.IsSet())
	{
		UMovieSceneCompiledDataManager*               CompiledDataManager = SharedPlaybackState->GetCompiledDataManager();
		TArrayView<const FMovieSceneDeterminismFence> DeterminismFences   = CompiledDataManager->GetEntryRef(CompiledDataID).DeterminismFences;

		if (DeterminismFences.Num() != 0)
		{
			CachedDeterminismFences = TArray<FMovieSceneDeterminismFence>(DeterminismFences.GetData(), DeterminismFences.Num());
		}
		else
		{
			CachedDeterminismFences.Emplace();
		}
	}

	if (CachedDeterminismFences->Num() != 0)
	{
		TArrayView<const FMovieSceneDeterminismFence> TraversedFences = GetFencesWithinRange(CachedDeterminismFences.GetValue(), Context.GetRange());
		UE::MovieScene::DissectRange(TraversedFences, Context.GetRange(), OutDissections);
	}
}

void FSequenceUpdater_Flat::Start(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, const FMovieSceneContext& InContext)
{
}

void FSequenceUpdater_Flat::Update(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, const FMovieSceneContext& Context)
{
	UMovieSceneEntitySystemLinker* Linker = SharedPlaybackState->GetLinker();
	const FRootInstanceHandle& InstanceHandle = SharedPlaybackState->GetRootInstanceHandle();
	FSequenceInstance& SequenceInstance = Linker->GetInstanceRegistry()->MutateInstance(InstanceHandle);
	SequenceInstance.SetContext(Context);

	UMovieSceneCompiledDataManager* CompiledDataManager = SharedPlaybackState->GetCompiledDataManager();
	const FMovieSceneEntityComponentField* ComponentField = CompiledDataManager->FindEntityComponentField(CompiledDataID);
	UMovieSceneSequence* Sequence = SharedPlaybackState->GetRootSequence();
	if (Sequence == nullptr)
	{
		SequenceInstance.Ledger.UnlinkEverything(Linker);
		return;
	}

	if (!bDynamicWeighting.IsSet())
	{
		bDynamicWeighting = EnumHasAnyFlags(Sequence->GetFlags(), EMovieSceneSequenceFlags::DynamicWeighting);
		if (IMovieScenePlayer* Player = FPlayerIndexPlaybackCapability::GetPlayer(SharedPlaybackState))
		{
			bDynamicWeighting = bDynamicWeighting || Player->HasDynamicWeighting();
		}
	}

	FMovieSceneEvaluationFieldEntitySet EntitiesScratch;

	FFrameNumber ImportTime = Context.GetEvaluationFieldTime();
		
	const bool bOutsideCachedRange = !CachedEntityRange.Contains(ImportTime);
	if (bOutsideCachedRange)
	{
		CachedPerTickConditionalEntities.Reset();

		if (ComponentField)
		{
			ComponentField->QueryPersistentEntities(ImportTime, CachedEntityRange, EntitiesScratch);
		}
		else
		{
			CachedEntityRange = TRange<FFrameNumber>::All();
		}

		FEntityImportSequenceParams Params;
		Params.SequenceID = MovieSceneSequenceID::Root;
		Params.InstanceHandle = InstanceHandle;
		Params.RootInstanceHandle = InstanceHandle;
		Params.DefaultCompletionMode = Sequence->DefaultCompletionMode;
		Params.HierarchicalBias = 0;
		Params.bDynamicWeighting = bDynamicWeighting.Get(false);

		SequenceInstance.Ledger.UpdateEntities(Linker, Params, ComponentField, EntitiesScratch, CachedPerTickConditionalEntities, CachedConditionResults);
	}
	else if (CachedPerTickConditionalEntities.Num() != 0)
	{
		FEntityImportSequenceParams Params;
		Params.SequenceID = MovieSceneSequenceID::Root;
		Params.InstanceHandle = InstanceHandle;
		Params.RootInstanceHandle = InstanceHandle;
		Params.DefaultCompletionMode = Sequence->DefaultCompletionMode;
		Params.HierarchicalBias = 0;
		Params.bDynamicWeighting = bDynamicWeighting.Get(false);

		SequenceInstance.Ledger.UpdateConditionalEntities(Linker, Params, ComponentField, CachedPerTickConditionalEntities);
	}

	// Update any one-shot entities for the current frame
	if (ComponentField && ComponentField->HasAnyOneShotEntities())
	{
		EntitiesScratch.Reset();
		ComponentField->QueryOneShotEntities(Context.GetFrameNumberRange(), EntitiesScratch);

		if (EntitiesScratch.Num() != 0)
		{
			FEntityImportSequenceParams Params;
			Params.SequenceID = MovieSceneSequenceID::Root;
			Params.InstanceHandle = InstanceHandle;
			Params.RootInstanceHandle = InstanceHandle;
			Params.DefaultCompletionMode = Sequence->DefaultCompletionMode;
			Params.HierarchicalBias = 0;
			Params.bDynamicWeighting = bDynamicWeighting.Get(false);

			SequenceInstance.Ledger.UpdateOneShotEntities(Linker, Params, ComponentField, EntitiesScratch, CachedConditionResults);
		}
	}
}

bool FSequenceUpdater_Flat::CanFinishImmediately(TSharedRef<const FSharedPlaybackState> SharedPlaybackState) const
{
	UMovieSceneEntitySystemLinker* Linker = SharedPlaybackState->GetLinker();
	const FRootInstanceHandle RootInstanceHandle = SharedPlaybackState->GetRootInstanceHandle();
	const FSequenceInstance& SequenceInstance = Linker->GetInstanceRegistry()->GetInstance(RootInstanceHandle);
	return SequenceInstance.Ledger.IsEmpty();
}

void FSequenceUpdater_Flat::Finish(TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	InvalidateCachedData(SharedPlaybackState, ESequenceInstanceInvalidationType::All);
}

void FSequenceUpdater_Flat::Destroy(TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
}

void FSequenceUpdater_Flat::InvalidateCachedData(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, ESequenceInstanceInvalidationType InvalidationType)
{
	CachedEntityRange = TRange<FFrameNumber>::Empty();
	CachedDeterminismFences.Reset();
	CachedPerTickConditionalEntities.Reset();
	CachedConditionResults.Reset();
	bDynamicWeighting.Reset();
}


bool FSequenceUpdater_Flat::EvaluateCondition(const FGuid& BindingID, const FMovieSceneSequenceID& SequenceID, const UMovieSceneCondition* Condition, UObject* ConditionOwnerObject, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const
{
	if (Condition)
	{
		if (Condition->CanCacheResult(SharedPlaybackState))
		{
			if (bool* ConditionResult = CachedConditionResults.Find(Condition->ComputeCacheKey(BindingID, SequenceID, SharedPlaybackState, ConditionOwnerObject)))
			{
				return *ConditionResult;
			}

			// We specifically don't cache the result of a condition check in this path, since this path is called by UI contexts.
			// The main evaluation path in MovieSceneEntityLedger caches its results.
		}

		return Condition->EvaluateCondition(BindingID, SequenceID, SharedPlaybackState);
	}
	return true;
}


FSequenceUpdater_Hierarchical::FSequenceUpdater_Hierarchical(FMovieSceneCompiledDataID InCompiledDataID)
	: CompiledDataID(InCompiledDataID)
{
	CachedEntityRange = TRange<FFrameNumber>::Empty();
	RootOverrideSequenceID = MovieSceneSequenceID::Root;
}

FSequenceUpdater_Hierarchical::~FSequenceUpdater_Hierarchical()
{
}

void FSequenceUpdater_Hierarchical::PopulateUpdateFlags(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, ESequenceInstanceUpdateFlags& OutUpdateFlags)
{
	if (IMovieScenePlayer* Player = FPlayerIndexPlaybackCapability::GetPlayer(SharedPlaybackState))
	{
		Player->PopulateUpdateFlags(OutUpdateFlags);
	}

	// If we've already been told we need dissection there's nothing left to do
	if (EnumHasAnyFlags(OutUpdateFlags, ESequenceInstanceUpdateFlags::NeedsDissection))
	{
		return;
	}

	UMovieSceneCompiledDataManager* CompiledDataManager = SharedPlaybackState->GetCompiledDataManager();

	{
		const FMovieSceneCompiledDataEntry& RootDataEntry = CompiledDataManager->GetEntryRef(CompiledDataID);
		if (RootDataEntry.DeterminismFences.Num() > 0)
		{
			OutUpdateFlags |= ESequenceInstanceUpdateFlags::NeedsDissection;
		}
	}

	if (const FMovieSceneSequenceHierarchy* Hierarchy = CompiledDataManager->FindHierarchy(CompiledDataID))
	{
		if (Hierarchy->GetRootTransform().FindFirstWarpDomain() == ETimeWarpChannelDomain::Time)
		{
			OutUpdateFlags |= ESequenceInstanceUpdateFlags::NeedsDissection;
		}
		else for (const TPair<FMovieSceneSequenceID, FMovieSceneSubSequenceData>& Pair : Hierarchy->AllSubSequenceData())
		{
			UMovieSceneSequence*      SubSequence = Pair.Value.GetSequence();
			FMovieSceneCompiledDataID SubDataID   = SubSequence ? CompiledDataManager->GetDataID(SubSequence) : FMovieSceneCompiledDataID();

			if (SubDataID.IsValid() && CompiledDataManager->GetEntryRef(SubDataID).DeterminismFences.Num() > 0)
			{
				OutUpdateFlags |= ESequenceInstanceUpdateFlags::NeedsDissection;
				break;
			}
		}
	}
}

void FSequenceUpdater_Hierarchical::DissectContext(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, const FMovieSceneContext& Context, TArray<TRange<FFrameTime>>& OutDissections)
{
	UMovieSceneCompiledDataManager* CompiledDataManager = SharedPlaybackState->GetCompiledDataManager();

	FMovieSceneCompiledDataID   RootCompiledDataID = CompiledDataID;
	FMovieSceneContext          RootContext        = Context;

	const FMovieSceneSequenceHierarchy* RootHierarchy = CompiledDataManager->FindHierarchy(CompiledDataID);

	if (!RootHierarchy)
	{
		return;
	}

	if (RootOverrideSequenceID != MovieSceneSequenceID::Root)
	{
		const FMovieSceneSubSequenceData* SubData = RootHierarchy->FindSubData(RootOverrideSequenceID);
		if (SubData)
		{
			RootCompiledDataID = CompiledDataManager->GetDataID(SubData->GetSequence());
			RootContext = Context.Transform(SubData->RootToSequenceTransform, SubData->TickResolution);
		}
	}
	else if (RootHierarchy->GetRootTransform().FindFirstWarpDomain() == ETimeWarpChannelDomain::Time)
	{
		RootContext = Context.Transform(RootHierarchy->GetRootTransform(), Context.GetFrameRate());
	}

	TRange<FFrameNumber> TraversedRange = RootContext.GetFrameNumberRange();
	TArray<FMovieSceneDeterminismFenceWithSubframe> RootDissectionTimes;

	{
		const FMovieSceneCompiledDataEntry&           DataEntry       = CompiledDataManager->GetEntryRef(RootCompiledDataID);
		TArrayView<const FMovieSceneDeterminismFence> TraversedFences = GetFencesWithinRange(DataEntry.DeterminismFences, RootContext.GetRange());

		for (const FMovieSceneDeterminismFence& Fence : TraversedFences)
		{
			RootDissectionTimes.Add(FMovieSceneDeterminismFenceWithSubframe{ Fence.FrameNumber, Fence.bInclusive });
		}
	}

	// @todo: should this all just be compiled into the root hierarchy?
	if (const FMovieSceneSequenceHierarchy* Hierarchy = CompiledDataManager->FindHierarchy(RootCompiledDataID))
	{
		FMovieSceneEvaluationTreeRangeIterator SubSequenceIt = Hierarchy->GetTree().IterateFromLowerBound(TraversedRange.GetLowerBound());
		for ( ; SubSequenceIt && SubSequenceIt.Range().Overlaps(TraversedRange); ++SubSequenceIt)
		{
			TRange<FFrameTime> RootClampRange = TRange<FFrameTime>::Intersection(ConvertToFrameTimeRange(SubSequenceIt.Range()), RootContext.GetRange());

			// When RootContext.GetRange() does not fall on whole frame boundaries, we can sometimes end up with a range that clamps to being empty, even though the range overlapped
			// the traversed range. ie if we evaluated range (1.5, 10], our traversed range would be [2, 11). If we have a sub sequence range of (10, 20), it would still be iterated here
			// because [2, 11) overlaps (10, 20), but when clamped to the evaluated range, the range is (10, 10], which is empty.
			if (RootClampRange.IsEmpty())
			{
				continue;
			}

			for (const FMovieSceneSubSequenceTreeEntry& Entry : Hierarchy->GetTree().GetAllData(SubSequenceIt.Node()))
			{
				const FMovieSceneSubSequenceData* SubData = Hierarchy->FindSubData(Entry.SequenceID);
				checkf(SubData, TEXT("Sub data does not exist for a SequenceID that exists in the hierarchical tree - this indicates a corrupt compilation product."));

				UMovieSceneSequence*      SubSequence = SubData     ? SubData->GetSequence()                      : nullptr;
				FMovieSceneCompiledDataID SubDataID   = SubSequence ? CompiledDataManager->GetDataID(SubSequence) : FMovieSceneCompiledDataID();
				if (!SubDataID.IsValid())
				{
					continue;
				}

				TArrayView<const FMovieSceneDeterminismFence> SubDeterminismFences = CompiledDataManager->GetEntryRef(SubDataID).DeterminismFences;
				if (SubDeterminismFences.Num() > 0)
				{
					TRange<FFrameTime> InnerRange = SubData->RootToSequenceTransform.ComputeTraversedHull(RootClampRange);

					// Time-warp can result in inside-out ranges
					if (InnerRange.GetLowerBound().IsClosed() && InnerRange.GetUpperBound().IsClosed() && InnerRange.GetLowerBoundValue() > InnerRange.GetUpperBoundValue())
					{
						TRangeBound<FFrameTime> OldLower = InnerRange.GetLowerBound();
						TRangeBound<FFrameTime> OldUpper = InnerRange.GetUpperBound();
						InnerRange.SetLowerBound(OldUpper);
						InnerRange.SetUpperBound(OldLower);
					}

					TArrayView<const FMovieSceneDeterminismFence> TraversedFences = GetFencesWithinRange(SubDeterminismFences, InnerRange);
					if (TraversedFences.Num() > 0)
					{
						// Find the breadcrumbs for this range
						FMovieSceneTransformBreadcrumbs Breadcrumbs;
						SubData->RootToSequenceTransform.TransformTime(RootClampRange.GetLowerBoundValue(), FTransformTimeParams().HarvestBreadcrumbs(Breadcrumbs));

						FMovieSceneInverseSequenceTransform InverseTransform = SubData->RootToSequenceTransform.Inverse();

						for (FMovieSceneDeterminismFence Fence : TraversedFences)
						{
							TOptional<FFrameTime> RootTime = InverseTransform.TryTransformTime(Fence.FrameNumber, Breadcrumbs);
							if (RootTime && TraversedRange.Contains(RootTime->FrameNumber))
							{
								RootDissectionTimes.Emplace(FMovieSceneDeterminismFenceWithSubframe{ RootTime.GetValue(), Fence.bInclusive });
							}
						}
					}
				}
			}
		}
	}

	if (RootDissectionTimes.Num() > 0)
	{
		Algo::SortBy(RootDissectionTimes, &FMovieSceneDeterminismFenceWithSubframe::FrameTime);
		int32 Index = Algo::UniqueBy(RootDissectionTimes, &FMovieSceneDeterminismFenceWithSubframe::FrameTime);
		if (Index < RootDissectionTimes.Num())
		{
			RootDissectionTimes.SetNum(Index);
		}
		UE::MovieScene::DissectRange(RootDissectionTimes, RootContext.GetRange(), OutDissections);
	}
	else if (RootHierarchy->GetRootTransform().FindFirstWarpDomain() == ETimeWarpChannelDomain::Time)
	{
		OutDissections.Add(RootContext.GetRange());
	}
}

FInstanceHandle FSequenceUpdater_Hierarchical::GetOrCreateSequenceInstance(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, UMovieSceneSequence* SubSequence, const FMovieSceneSequenceHierarchy* Hierarchy, FInstanceRegistry* InstanceRegistry, FMovieSceneSequenceID SequenceID)
{
	check(SequenceID != MovieSceneSequenceID::Root);

	if (FSubInstanceData* Existing = SequenceInstances.Find(SequenceID))
	{
		return Existing->Handle;
	}

	const FMovieSceneSequenceHierarchyNode* Node = Hierarchy->FindNode(SequenceID);
	checkf(Node, TEXT("Attempting to construct a new sub sequence instance with a sub sequence ID that does not exist in the hierarchy"));
	checkf(Node->ParentID != MovieSceneSequenceID::Invalid, TEXT("Parent should never be invalid for a non-root SequenceID"));

	const FRootInstanceHandle& RootInstanceHandle = SharedPlaybackState->GetRootInstanceHandle();

	FInstanceHandle ParentInstance;
	if (Node->ParentID == MovieSceneSequenceID::Root)
	{
		ParentInstance = RootInstanceHandle;
	}
	else if (UMovieSceneSequence* ParentSequence = Hierarchy->FindSubData(Node->ParentID)->GetSequence())
	{
		ParentInstance = GetOrCreateSequenceInstance(SharedPlaybackState, ParentSequence, Hierarchy, InstanceRegistry, Node->ParentID);
	}

	FInstanceHandle InstanceHandle = InstanceRegistry->AllocateSubInstance(SequenceID, RootInstanceHandle, ParentInstance);
	SequenceInstances.Add(SequenceID, FSubInstanceData { SubSequence->GetSignature(), InstanceHandle });
	InstanceRegistry->MutateInstance(InstanceHandle).Initialize();

	return InstanceHandle;
}

void FSequenceUpdater_Hierarchical::OverrideRootSequence(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, FMovieSceneSequenceID NewRootOverrideSequenceID)
{
	if (RootOverrideSequenceID != NewRootOverrideSequenceID)
	{
		if (RootOverrideSequenceID == MovieSceneSequenceID::Root)
		{
			// When specifying a new root override where there was none before (ie, when we were previously evaluating from the root)
			// We unlink everything from the root sequence since we know they won't be necessary.
			// This is because the root sequence instance is handled separately in FSequenceUpdater_Hierarchical::Update, and it wouldn't
			// get automatically unlinked like other sub sequences would (by way of not being present in the ActiveSequences map)
			UMovieSceneEntitySystemLinker* Linker = SharedPlaybackState->GetLinker();
			FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();
			const FRootInstanceHandle& RootInstanceHandle = SharedPlaybackState->GetRootInstanceHandle();
			InstanceRegistry->MutateInstance(RootInstanceHandle).Ledger.UnlinkEverything(Linker);
		}

		InvalidateCachedData(SharedPlaybackState, ESequenceInstanceInvalidationType::All);
		RootOverrideSequenceID = NewRootOverrideSequenceID;
	}
}

void FSequenceUpdater_Hierarchical::Start(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, const FMovieSceneContext& InContext)
{
}

void FSequenceUpdater_Hierarchical::Update(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, const FMovieSceneContext& Context)
{
	UMovieSceneEntitySystemLinker* Linker = SharedPlaybackState->GetLinker();
	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();
	UMovieSceneCompiledDataManager* CompiledDataManager = SharedPlaybackState->GetCompiledDataManager();
	IMovieScenePlayer* Player = FPlayerIndexPlaybackCapability::GetPlayer(SharedPlaybackState);

	FMovieSceneEvaluationFieldEntitySet EntitiesScratch;

	FRootInstanceHandle         RootInstanceHandle = SharedPlaybackState->GetRootInstanceHandle();
	FMovieSceneCompiledDataID   RootCompiledDataID = CompiledDataID;
	FSubSequencePath            RootOverridePath;
	FMovieSceneContext          RootContext = Context;

	TArray<FMovieSceneSequenceID, TInlineAllocator<16>> ActiveSequences;

	const FMovieSceneSequenceHierarchy* RootHierarchy = CompiledDataManager->FindHierarchy(CompiledDataID);

	if (RootOverrideSequenceID != MovieSceneSequenceID::Root)
	{
		const FMovieSceneSubSequenceData*   SubData         = RootHierarchy ? RootHierarchy->FindSubData(RootOverrideSequenceID) : nullptr;
		UMovieSceneSequence*                RootSequence    = SubData ? SubData->GetSequence() : nullptr;
		if (ensure(RootSequence))
		{
			FInstanceHandle NewRootInstanceHandle = GetOrCreateSequenceInstance(SharedPlaybackState, RootSequence, RootHierarchy, InstanceRegistry, RootOverrideSequenceID);
			RootInstanceHandle = FRootInstanceHandle(NewRootInstanceHandle.InstanceID, NewRootInstanceHandle.InstanceSerial);
			RootCompiledDataID = CompiledDataManager->GetDataID(RootSequence);
			RootContext        = Context.Transform(SubData->RootToSequenceTransform, SubData->TickResolution);

			RootOverridePath.Reset(RootOverrideSequenceID, RootHierarchy);

			ActiveSequences.Add(RootOverrideSequenceID);
		}
	}

	FFrameNumber ImportTime = RootContext.GetEvaluationFieldTime();
	const bool bGatherEntities = !CachedEntityRange.Contains(ImportTime);

	// ------------------------------------------------------------------------------------------------
	// Handle the root sequence entities first
	{
		// Set the context for the root sequence instance
		FSequenceInstance& RootInstance = InstanceRegistry->MutateInstance(RootInstanceHandle);
		RootInstance.SetContext(RootContext);

		const FMovieSceneEntityComponentField* RootComponentField = CompiledDataManager->FindEntityComponentField(RootCompiledDataID);
		UMovieSceneSequence* RootSequence = CompiledDataManager->GetEntryRef(RootCompiledDataID).GetSequence();

		if (RootSequence == nullptr)
		{
			RootInstance.Ledger.UnlinkEverything(Linker);
		}
		else
		{
			if (!bDynamicWeighting.IsSet())
			{
				bDynamicWeighting = EnumHasAnyFlags(CompiledDataManager->GetEntryRef(RootCompiledDataID).AccumulatedFlags, EMovieSceneSequenceFlags::DynamicWeighting);
				if (Player)
				{
					bDynamicWeighting = bDynamicWeighting || Player->HasDynamicWeighting();
				}
			}

			// Update entities if necessary
			if (bGatherEntities)
			{
				CachedPerTickConditionalEntities.Reset();

				CachedEntityRange = UpdateEntitiesForSequence(RootComponentField, ImportTime, EntitiesScratch);

				FEntityImportSequenceParams Params;
				Params.SequenceID = MovieSceneSequenceID::Root;
				Params.InstanceHandle = RootInstanceHandle;
				Params.RootInstanceHandle = RootInstanceHandle;
				Params.DefaultCompletionMode = RootSequence->DefaultCompletionMode;
				Params.HierarchicalBias = 0;
				Params.bDynamicWeighting = bDynamicWeighting.Get(false);

				FMovieSceneEvaluationFieldEntitySet& RootSequenceCachedConditionalEntries = CachedPerTickConditionalEntities.Add(MovieSceneSequenceID::Root);

				RootInstance.Ledger.UpdateEntities(Linker, Params, RootComponentField, EntitiesScratch, RootSequenceCachedConditionalEntries, CachedConditionResults);
			}
			else if (FMovieSceneEvaluationFieldEntitySet* RootSequenceCachedConditionalEntries = CachedPerTickConditionalEntities.Find(MovieSceneSequenceID::Root))
			{
				if (RootSequenceCachedConditionalEntries->Num() != 0)
				{
					FEntityImportSequenceParams Params;
					Params.SequenceID = MovieSceneSequenceID::Root;
					Params.InstanceHandle = RootInstanceHandle;
					Params.RootInstanceHandle = RootInstanceHandle;
					Params.DefaultCompletionMode = RootSequence->DefaultCompletionMode;
					Params.HierarchicalBias = 0;
					Params.bDynamicWeighting = bDynamicWeighting.Get(false);

					RootInstance.Ledger.UpdateConditionalEntities(Linker, Params, RootComponentField, *RootSequenceCachedConditionalEntries);
				}
			}

			// Update any one-shot entities for the current root frame
			if (RootComponentField && RootComponentField->HasAnyOneShotEntities())
			{
				EntitiesScratch.Reset();
				RootComponentField->QueryOneShotEntities(RootContext.GetFrameNumberRange(), EntitiesScratch);

				if (EntitiesScratch.Num() != 0)
				{
					FEntityImportSequenceParams Params;
					Params.SequenceID = MovieSceneSequenceID::Root;
					Params.InstanceHandle = RootInstanceHandle;
					Params.RootInstanceHandle = RootInstanceHandle;
					Params.DefaultCompletionMode = RootSequence->DefaultCompletionMode;
					Params.HierarchicalBias = 0;
					Params.bDynamicWeighting = bDynamicWeighting.Get(false);

					RootInstance.Ledger.UpdateOneShotEntities(Linker, Params, RootComponentField, EntitiesScratch, CachedConditionResults);
				}
			}
		}
	}

	// ------------------------------------------------------------------------------------------------
	// Handle sub sequence entities next
	const FMovieSceneSequenceHierarchy* RootOverrideHierarchy = CompiledDataManager->FindHierarchy(RootCompiledDataID);
	if (RootOverrideHierarchy)
	{
		FMovieSceneEvaluationTreeRangeIterator SubSequenceIt = RootOverrideHierarchy->GetTree().IterateFromTime(ImportTime);
		
		if (bGatherEntities)
		{
			CachedEntityRange = TRange<FFrameNumber>::Intersection(CachedEntityRange, SubSequenceIt.Range());
		}

		// Track sequences whose own condition has failed so that their descendants are also skipped.
		// The compiled tree is populated depth-first (parent AddRange -> recurse -> child AddRange),
		// so parent entries always appear before their children in the iteration.
		TSet<FMovieSceneSequenceID> ConditionFailedSequences;

		for (const FMovieSceneSubSequenceTreeEntry& Entry : RootOverrideHierarchy->GetTree().GetAllData(SubSequenceIt.Node()))
		{
			// When a root override path is specified, we always remap the 'local' sequence IDs to their equivalents from the root sequence.
			FMovieSceneSequenceID SequenceIDFromRoot = RootOverridePath.ResolveChildSequenceID(Entry.SequenceID);

			const FMovieSceneSubSequenceData* SubData = RootOverrideHierarchy->FindSubData(Entry.SequenceID);

			ActiveSequences.Add(SequenceIDFromRoot);
			checkf(SubData, TEXT("Sub data does not exist for a SequenceID that exists in the hierarchical tree - this indicates a corrupt compilation product."));

			bool bSubSequenceConditionFailed = false;

			// If a parent sequence's condition failed, propagate that failure to this subsequence.
			if (const FMovieSceneSequenceHierarchyNode* SubNode = RootOverrideHierarchy->FindNode(Entry.SequenceID))
			{
				if (SubNode->ParentID != MovieSceneSequenceID::Root)
				{
					const FMovieSceneSequenceID ParentIDFromRoot = RootOverridePath.ResolveChildSequenceID(SubNode->ParentID);
					if (ConditionFailedSequences.Contains(ParentIDFromRoot))
					{
						bSubSequenceConditionFailed = true;
					}
				}
			}

			const UMovieSceneCondition* Condition = SubData->WeakCondition.Get();
			if (!bSubSequenceConditionFailed && Condition)
			{
				// The condition's director BP belongs to the parent sequence (the one
				// containing the sub-section), so resolve the parent's root-relative ID
				// for use in director instance lookup during condition evaluation.
				FMovieSceneSequenceID ConditionOwnerSequenceID = RootOverrideSequenceID;
				if (const FMovieSceneSequenceHierarchyNode* SubNode = RootOverrideHierarchy->FindNode(Entry.SequenceID))
				{
					if (SubNode->ParentID != MovieSceneSequenceID::Root)
					{
						ConditionOwnerSequenceID = RootOverridePath.ResolveChildSequenceID(SubNode->ParentID);
					}
				}

				// If we're able to cache the condition result, then it should be cached above when its entity got processed- retrieve that value.
				// Otherwise, test it again.
				if (Condition->CanCacheResult(SharedPlaybackState))
				{
					if (bool* ConditionResult = CachedConditionResults.Find(Condition->ComputeCacheKey(FGuid(), ConditionOwnerSequenceID, SharedPlaybackState, FindObject<UMovieSceneSubSection>(CompiledDataManager->GetEntryRef(RootCompiledDataID).GetSequence(), *SubData->SectionPath.ToString()))))
					{
						if (*ConditionResult == false)
						{
							bSubSequenceConditionFailed = true;
						}
					}
					else if (!Condition->EvaluateCondition(FGuid(), ConditionOwnerSequenceID, SharedPlaybackState))
					{
						bSubSequenceConditionFailed = true;
					}
				}
				else if (!Condition->EvaluateCondition(FGuid(), ConditionOwnerSequenceID, SharedPlaybackState))
				{
					bSubSequenceConditionFailed = true;
				}
			}
			
			UMovieSceneSequence* SubSequence = SubData->GetSequence();
			if (SubSequence == nullptr || bSubSequenceConditionFailed)
			{
				// Record this sequence so its descendants are also skipped
				ConditionFailedSequences.Add(SequenceIDFromRoot);

				FInstanceHandle SubSequenceHandle = SequenceInstances.FindRef(SequenceIDFromRoot).Handle;
				if (SubSequenceHandle.IsValid())
				{
					FSequenceInstance& SubSequenceInstance = InstanceRegistry->MutateInstance(SubSequenceHandle);
					SubSequenceInstance.Ledger.UnlinkEverything(Linker);
					// Also invalidate the ledge to ensure that if the condition changes, we can detect it and force gather entities
					SubSequenceInstance.Ledger.Invalidate();
				}
			}
			else
			{
				FMovieSceneCompiledDataID SubDataID = CompiledDataManager->GetDataID(SubSequence);

				// Set the context for the root sequence instance
				FInstanceHandle    SubSequenceHandle = GetOrCreateSequenceInstance(SharedPlaybackState, SubSequence, RootHierarchy, InstanceRegistry, SequenceIDFromRoot);
				FSequenceInstance& SubSequenceInstance = InstanceRegistry->MutateInstance(SubSequenceHandle);

				// Update the sub sequence's context
				FMovieSceneContext SubContext = RootContext.Transform(SubData->RootToSequenceTransform, SubData->TickResolution);
				SubContext.ReportOuterSectionRanges(SubData->PreRollRange.Value, SubData->PostRollRange.Value);
				SubContext.SetHierarchicalBias(SubData->HierarchicalBias);

				// Handle crossing a pre/postroll boundary
				const bool bWasPreRoll  = SubSequenceInstance.GetContext().IsPreRoll();
				const bool bWasPostRoll = SubSequenceInstance.GetContext().IsPostRoll();
				const bool bIsPreRoll   = SubContext.IsPreRoll();
				const bool bIsPostRoll  = SubContext.IsPostRoll();

				if (bWasPreRoll != bIsPreRoll || bWasPostRoll != bIsPostRoll)
				{
					// When crossing a pre/postroll boundary, we invalidate all entities currently imported, which results in them being re-imported 
					// with the same EntityID. This ensures that the state is maintained for such entities across prerolls (ie entities with a
					// spawnable binding component on them will not cause the spawnable to be destroyed and recreated again).
					// The one edge case that this could open up is where a preroll entity has meaningfully different components from its 'normal' entity,
					// and there are systems that track the link/unlink lifetime for such components. Under this circumstance, the unlink for the entity
					// will not be seen until the whole entity goes away, not just the preroll region. This is a very nuanced edge-case however, and can
					// be solved by giving the entities unique IDs (FMovieSceneEvaluationFieldEntityKey::EntityID) in the evaluation field.
					SubSequenceInstance.Ledger.Invalidate();
				}

				SubSequenceInstance.SetContext(SubContext);
				SubSequenceInstance.SetFinished(false);

				const FMovieSceneEntityComponentField* SubComponentField = CompiledDataManager->FindEntityComponentField(SubDataID);

				// Update entities if necessary
				const FFrameTime SubSequenceTime = SubContext.GetEvaluationFieldTime();

				FEntityImportSequenceParams Params;
				Params.SequenceID = SequenceIDFromRoot;
				Params.InstanceHandle = SubSequenceHandle;
				Params.RootInstanceHandle = RootInstanceHandle;
				Params.DefaultCompletionMode = SubSequence->DefaultCompletionMode;
				Params.HierarchicalBias = SubData->HierarchicalBias;
				Params.SubSectionFlags = SubData->AccumulatedFlags;
				Params.bPreRoll  = bIsPreRoll;
				Params.bPostRoll = bIsPostRoll;
				Params.bDynamicWeighting = bDynamicWeighting.Get(false); // Always inherit dynamic weighting flags

				if (bGatherEntities || SubSequenceInstance.Ledger.IsInvalidated())
				{
					EntitiesScratch.Reset();

					TRange<FFrameNumber> SubEntityRange = UpdateEntitiesForSequence(SubComponentField, SubSequenceTime, EntitiesScratch);
					SubEntityRange = TRange<FFrameNumber>::Intersection(SubEntityRange, SubData->PlayRange.Value);

					FMovieSceneEvaluationFieldEntitySet& SubSequenceCachedConditionalEntries = CachedPerTickConditionalEntities.Add(SequenceIDFromRoot);
					SubSequenceInstance.Ledger.UpdateEntities(Linker, Params, SubComponentField, EntitiesScratch, SubSequenceCachedConditionalEntries, CachedConditionResults);

					// Convert sub entity range into root space
					// 
					// Sometimes the bounds can be unset if the lower bound does not map to any valid time in the root sequence.
					//   If this happens, we rely in the intersection with SubSequenceIt.Range() to clamp to the bounds of the current sub sequence range
					FMovieSceneInverseSequenceTransform Inv = SubContext.GetSequenceToRootSequenceTransform();

					TRange<FFrameNumber> SubCachedRange = TRange<FFrameNumber>::All();
					if (!SubEntityRange.GetLowerBound().IsOpen())
					{
						TOptional<FFrameTime> LowerBoundRootSpace = Inv.TryTransformTime(SubEntityRange.GetLowerBoundValue(), SubContext.GetRootToSequenceWarpCounter());
						if (LowerBoundRootSpace)
						{
							SubCachedRange.SetLowerBound(TRangeBound<FFrameNumber>::Inclusive(LowerBoundRootSpace.GetValue().CeilToFrame()));
						}
					}

					if (!SubEntityRange.GetUpperBound().IsOpen())
					{
						TOptional<FFrameTime> UpperBoundRootSpace = Inv.TryTransformTime(SubEntityRange.GetUpperBoundValue(), SubContext.GetRootToSequenceWarpCounter());
						if (UpperBoundRootSpace)
						{
							SubCachedRange.SetUpperBound(TRangeBound<FFrameNumber>::Exclusive(UpperBoundRootSpace.GetValue().FloorToFrame()));
						}
					}

					// Time-warp can result in inside-out ranges
					if (SubCachedRange.GetLowerBound().IsClosed() && SubCachedRange.GetUpperBound().IsClosed() && SubCachedRange.GetLowerBoundValue() > SubCachedRange.GetUpperBoundValue())
					{
						TRangeBound<FFrameNumber> OldLower = SubCachedRange.GetLowerBound();
						TRangeBound<FFrameNumber> OldUpper = SubCachedRange.GetUpperBound();
						SubCachedRange.SetLowerBound(OldUpper);
						SubCachedRange.SetUpperBound(OldLower);
					}

					CachedEntityRange = TRange<FFrameNumber>::Intersection(CachedEntityRange, SubCachedRange);
				}
				else if (FMovieSceneEvaluationFieldEntitySet* SubSequenceCachedConditionalEntries = CachedPerTickConditionalEntities.Find(SequenceIDFromRoot))
				{
					if (SubSequenceCachedConditionalEntries->Num() != 0)
					{
						SubSequenceInstance.Ledger.UpdateConditionalEntities(Linker, Params, SubComponentField, *SubSequenceCachedConditionalEntries);
					}
				}

				// Update any one-shot entities for the sub sequence
				if (SubComponentField && SubComponentField->HasAnyOneShotEntities())
				{
					EntitiesScratch.Reset();
					SubComponentField->QueryOneShotEntities(SubContext.GetFrameNumberRange(), EntitiesScratch);

					if (EntitiesScratch.Num() != 0)
					{
						SubSequenceInstance.Ledger.UpdateOneShotEntities(Linker, Params, SubComponentField, EntitiesScratch, CachedConditionResults);
					}
				}
			}
		}
	}

	TSharedRef<FMovieSceneEntitySystemRunner> Runner = Linker->GetRunner();

	for (auto InstanceIt = SequenceInstances.CreateIterator(); InstanceIt; ++InstanceIt)
	{
		FSubInstanceData SubData = InstanceIt.Value();

		ERunnerUpdateFlags Flags = ERunnerUpdateFlags::None;
		if (!ActiveSequences.Contains(InstanceIt.Key()))
		{
			Flags = ERunnerUpdateFlags::Finish | ERunnerUpdateFlags::Destroy;
			InstanceIt.RemoveCurrent();
		}

		Runner->MarkForUpdate(SubData.Handle, Flags);
	}
}

bool FSequenceUpdater_Hierarchical::CanFinishImmediately(TSharedRef<const FSharedPlaybackState> SharedPlaybackState) const
{
	UMovieSceneEntitySystemLinker* Linker = SharedPlaybackState->GetLinker();
	const FRootInstanceHandle RootInstanceHandle = SharedPlaybackState->GetRootInstanceHandle();

	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();

	const FSequenceInstance& RootInstance = InstanceRegistry->GetInstance(RootInstanceHandle);
	if (!RootInstance.Ledger.IsEmpty())
	{
		return false;
	}

	for (TPair<FMovieSceneSequenceID, FSubInstanceData> Pair : SequenceInstances)
	{
		const FSequenceInstance& SubInstance = InstanceRegistry->GetInstance(Pair.Value.Handle);
		if (!SubInstance.Ledger.IsEmpty())
		{
			return false;
		}
	}

	return true;
}

void FSequenceUpdater_Hierarchical::Finish(TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	UMovieSceneEntitySystemLinker* Linker = SharedPlaybackState->GetLinker();
	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();

	// Finish all sub sequences as well
	for (TPair<FMovieSceneSequenceID, FSubInstanceData> Pair : SequenceInstances)
	{
		InstanceRegistry->MutateInstance(Pair.Value.Handle).Finish();
	}

	InvalidateCachedData(SharedPlaybackState, ESequenceInstanceInvalidationType::All);
}

void FSequenceUpdater_Hierarchical::Destroy(TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	UMovieSceneEntitySystemLinker* Linker = SharedPlaybackState->GetLinker();
	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();

	for (TPair<FMovieSceneSequenceID, FSubInstanceData> Pair : SequenceInstances)
	{
		InstanceRegistry->DestroyInstance(Pair.Value.Handle);
	}
}

void FSequenceUpdater_Hierarchical::InvalidateCachedData(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, ESequenceInstanceInvalidationType InvalidationType)
{
	bDynamicWeighting.Reset();
	CachedEntityRange = TRange<FFrameNumber>::Empty();
	CachedPerTickConditionalEntities.Reset();
	CachedConditionResults.Reset();

	UMovieSceneEntitySystemLinker* Linker = SharedPlaybackState->GetLinker();
	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();

	for (TPair<FMovieSceneSequenceID, FSubInstanceData>& Pair : SequenceInstances)
	{
		FSequenceInstance& SubInstance = InstanceRegistry->MutateInstance(Pair.Value.Handle);

		switch (InvalidationType)
		{
			case ESequenceInstanceInvalidationType::All:
				{
					SubInstance.Ledger.Invalidate();
				}
				break;
			case ESequenceInstanceInvalidationType::DataChanged:
				{
					UMovieSceneSequence* Sequence = SharedPlaybackState->GetSequence(Pair.Key);
					if (!Sequence)
					{
						Pair.Value.SequenceSignature = FGuid();
						SubInstance.Ledger.Invalidate();
					}
					else if (Pair.Value.SequenceSignature != Sequence->GetSignature())
					{
						Pair.Value.SequenceSignature = Sequence->GetSignature();
						SubInstance.Ledger.Invalidate();
					}
				}
				break;
		}
	}
}

TRange<FFrameNumber> FSequenceUpdater_Hierarchical::UpdateEntitiesForSequence(const FMovieSceneEntityComponentField* ComponentField, FFrameTime SequenceTime, FMovieSceneEvaluationFieldEntitySet& OutEntities)
{
	TRange<FFrameNumber> CachedRange = TRange<FFrameNumber>::All();

	if (ComponentField)
	{
		// Extract all the entities for the current time
		ComponentField->QueryPersistentEntities(SequenceTime.FrameNumber, CachedRange, OutEntities);
	}

	return CachedRange;
}

bool FSequenceUpdater_Hierarchical::EvaluateCondition(const FGuid& BindingID, const FMovieSceneSequenceID& SequenceID, const UMovieSceneCondition* Condition, UObject* ConditionOwnerObject, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const
{
	if (Condition)
	{
		if (Condition->CanCacheResult(SharedPlaybackState))
		{
			if (bool* ConditionResult = CachedConditionResults.Find(Condition->ComputeCacheKey(BindingID, SequenceID, SharedPlaybackState, ConditionOwnerObject)))
			{
				return *ConditionResult;
			}

			// We specifically don't cache the result of a condition check in this path, since this path is called by UI contexts.
			// The main evaluation path in MovieSceneEntityLedger caches its results.
		}

		return Condition->EvaluateCondition(BindingID, SequenceID, SharedPlaybackState);
	}
	return true;
}

void ISequenceUpdater::InvalidateCachedData(TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	InvalidateCachedData(SharedPlaybackState, ESequenceInstanceInvalidationType::All);
}

} // namespace MovieScene
} // namespace UE
