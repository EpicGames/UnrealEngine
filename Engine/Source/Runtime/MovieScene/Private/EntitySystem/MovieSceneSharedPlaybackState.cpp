// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneSharedPlaybackState.h"

#include "Compilation/MovieSceneCompiledDataManager.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "MovieSceneSequenceID.h"

namespace UE::MovieScene
{
	UE_DEFINE_MOVIESCENE_PLAYBACK_CAPABILITY(FMovieSceneAudioRecordingCapability)

FSharedPlaybackState::FSharedPlaybackState(UMovieSceneEntitySystemLinker* InLinker)
	: WeakLinker(InLinker)
	, PreAnimatedState(InLinker, FRootInstanceHandle())
{
}

FSharedPlaybackState::FSharedPlaybackState(
		UMovieSceneSequence& InRootSequence,
		const FSharedPlaybackStateCreateParams& CreateParams)
	: WeakRootSequence(&InRootSequence)
	, WeakPlaybackContext(CreateParams.PlaybackContext)
	, WeakLinker(CreateParams.Linker)
	, CompiledDataManager(CreateParams.CompiledDataManager)
	, RootInstanceHandle(CreateParams.RootInstanceHandle)
	, PreAnimatedState(CreateParams.Linker, CreateParams.RootInstanceHandle)
{
	if (CompiledDataManager)
	{
		RootCompiledDataID = CompiledDataManager->GetDataID(&InRootSequence);
	}
}

FSharedPlaybackState::~FSharedPlaybackState()
{
#if !UE_BUILD_SHIPPING
	if (bDebugBreakOnDestroy)
	{
		ensureAlwaysMsgf(false, TEXT("Debug break was requested upon destruction of this state."));
	}
#endif
}

TSharedPtr<FMovieSceneEntitySystemRunner> FSharedPlaybackState::GetRunner() const
{
	if (UMovieSceneEntitySystemLinker* Linker = WeakLinker.Get())
	{
		return Linker->GetRunner();
	}
	return nullptr;
}

const FMovieSceneSequenceHierarchy* FSharedPlaybackState::GetHierarchy() const
{
	if (CompiledDataManager && RootCompiledDataID.IsValid())
	{
		return CompiledDataManager->FindHierarchy(RootCompiledDataID);
	}
	return nullptr;
}

UMovieSceneSequence* FSharedPlaybackState::GetSequence(FMovieSceneSequenceIDRef InSequenceID) const
{
	if (InSequenceID == MovieSceneSequenceID::Root)
	{
		return WeakRootSequence.Get();
	}
	else
	{
		const FMovieSceneSequenceHierarchy* Hierarchy = GetHierarchy();
		const FMovieSceneSubSequenceData*   SubData   = Hierarchy ? Hierarchy->FindSubData(InSequenceID) : nullptr;
		return SubData ? SubData->GetSequence() : nullptr;
	}
}

TArrayView<TWeakObjectPtr<>> FSharedPlaybackState::FindBoundObjects(const FGuid& ObjectBindingID, FMovieSceneSequenceIDRef SequenceID) const
{
	if (FMovieSceneEvaluationState* EvaluationState = FindCapability<FMovieSceneEvaluationState>())
	{
		return EvaluationState->FindBoundObjects(ObjectBindingID, SequenceID, SharedThis(this));
	}
	return TArrayView<TWeakObjectPtr<>>();
}

void FSharedPlaybackState::ClearObjectCaches()
{
	if (FMovieSceneEvaluationState* EvaluationState = FindCapability<FMovieSceneEvaluationState>())
	{
		EvaluationState->ClearObjectCaches(SharedThis(this));
	}
}

void FSharedPlaybackState::InvalidateCachedData()
{
	UMovieSceneEntitySystemLinker* Linker = GetLinker();
	Capabilities.InvalidateCachedData(Linker);
}

} // namespace UE::MovieScene

