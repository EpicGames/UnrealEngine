// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneSpawnablesSystem.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntityRange.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntityInstantiatorSystem.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "Evaluation/MovieSceneEvaluationOperand.h"

#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "MovieSceneExecutionToken.h"
#include "MovieSceneSpawnRegister.h"

#include "IMovieScenePlayer.h"
#include "IMovieScenePlaybackClient.h"
#include "MovieSceneBindingReferences.h"
#include "Bindings/MovieSceneSpawnableBinding.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSpawnablesSystem)

namespace UE
{
namespace MovieScene
{

DECLARE_CYCLE_STAT(TEXT("Spawnables System"), MovieSceneEval_SpawnablesSystem, STATGROUP_MovieSceneECS);
DECLARE_CYCLE_STAT(TEXT("Spawnables: Spawn"), MovieSceneEval_SpawnSpawnables, STATGROUP_MovieSceneEval);
DECLARE_CYCLE_STAT(TEXT("Spawnables: Destroy"), MovieSceneEval_DestroySpawnables, STATGROUP_MovieSceneEval);

struct FSpawnTrackPreAnimatedTokenProducer : IMovieScenePreAnimatedTokenProducer
{
	FMovieSceneEvaluationOperand Operand;
	int32 BindingIndex;
	FSpawnTrackPreAnimatedTokenProducer(FMovieSceneEvaluationOperand InOperand, int32 InBindingIndex) : Operand(InOperand), BindingIndex(InBindingIndex) {}

	virtual IMovieScenePreAnimatedTokenPtr CacheExistingState(UObject& Object) const
	{
		struct FToken : IMovieScenePreAnimatedToken
		{
			FMovieSceneEvaluationOperand OperandToDestroy;
			int32 BindingIndexToDestroy;
			FToken(FMovieSceneEvaluationOperand InOperand, int32 InBindingIndex) : OperandToDestroy(InOperand), BindingIndexToDestroy(InBindingIndex) {}

			virtual void RestoreState(UObject& Object, const UE::MovieScene::FRestoreStateParams& Params) override
			{
				TSharedPtr<const FSharedPlaybackState> PlaybackState = Params.GetTerminalPlaybackState();
				if (!ensure(PlaybackState))
				{
					return;
				}

				FMovieSceneSpawnRegister* SpawnRegister = PlaybackState->FindCapability<FMovieSceneSpawnRegister>();
				if (SpawnRegister && !SpawnRegister->DestroySpawnedObject(OperandToDestroy.ObjectBindingID, OperandToDestroy.SequenceID, PlaybackState.ToSharedRef(), BindingIndexToDestroy))
				{
					// This branch should only be taken for Externally owned spawnables that have been 'forgotten',
					// but still had RestoreState tokens generated for them (ie, in FSequencer, or if bRestoreState is enabled)
					// on a UMovieSceneSequencePlayer
					SpawnRegister->DestroyObjectDirectly(Object);
				}
			}
		};
		
		return FToken(Operand, BindingIndex);
	}
};

const FMovieSceneAnimTypeID SpawnableAnimTypeID = FMovieSceneAnimTypeID::Unique();

} // namespace MovieScene
} // namespace UE




UMovieSceneSpawnablesSystem::UMovieSceneSpawnablesSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	Phase = UE::MovieScene::ESystemPhase::Spawn;
	RelevantComponent = UE::MovieScene::FBuiltInComponentTypes::Get()->SpawnableBinding;
}

FMovieSceneAnimTypeID UMovieSceneSpawnablesSystem::GetAnimTypeID()
{
	return UE::MovieScene::SpawnableAnimTypeID;
}

void UMovieSceneSpawnablesSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	SCOPE_CYCLE_COUNTER(MovieSceneEval_SpawnablesSystem)

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FInstanceRegistry*      InstanceRegistry  = Linker->GetInstanceRegistry();

	// Re-link any spawnables that were invalidated
	if (InstanceRegistry->HasInvalidatedBindings())
	{
		TArray<FMovieSceneEntityID> StaleSpawnables;

		auto GatherStaleSpawnables = [InstanceRegistry, &StaleSpawnables](FMovieSceneEntityID EntityID, FInstanceHandle InstanceHandle, const FGuid& InObjectBindingID)
		{
			if (InstanceRegistry->IsBindingInvalidated(InObjectBindingID, InstanceHandle))
			{
				StaleSpawnables.Add(EntityID);
			}
		};

		// Invalidate any spawnables that have been invalidated or destroyed
		FEntityTaskBuilder()
		.ReadEntityIDs()
		.Read(BuiltInComponents->InstanceHandle)
		.Read(BuiltInComponents->SpawnableBinding)
		.FilterNone({ BuiltInComponents->Tags.NeedsUnlink })
		.Iterate_PerEntity(&Linker->EntityManager, GatherStaleSpawnables);

		for (FMovieSceneEntityID Entity : StaleSpawnables)
		{
			// Tag this as needs link, and all children as needs unlink
			Linker->EntityManager.AddComponent(Entity, BuiltInComponents->Tags.NeedsLink);
			Linker->EntityManager.AddComponent(Entity, BuiltInComponents->Tags.NeedsUnlink, EEntityRecursion::Children);
		}
	}

	// Used below.
	TArray<TTuple<FGuid, int32, FMovieSceneSequenceID, FInstanceHandle>> DestroyedObjects;
	auto DestroyOldSpawnables = [this, &DestroyedObjects, InstanceRegistry](FMovieSceneEntityID EntityID, FInstanceHandle InstanceHandle, const FGuid& SpawnableObjectID)
	{
		SCOPE_CYCLE_COUNTER(MovieSceneEval_DestroySpawnables)

		if (ensure(InstanceRegistry->IsHandleValid(InstanceHandle)))
		{
			const FSequenceInstance& Instance = InstanceRegistry->GetInstance(InstanceHandle);
			TSharedRef<FSharedPlaybackState> SharedPlaybackState = Instance.GetSharedPlaybackState();
			
			const UMovieSceneSequence* Sequence = nullptr;
			// Prefer getting the Sequence from the evaluation state if possible.
			// This is because if a sub section has just been destroyed, the sequence will be recompiled and the hierarchy (used by SharedPlaybackState) is not reliable for finding the Sequence asset.
			if (FMovieSceneEvaluationState* EvaluationState = SharedPlaybackState->FindCapability<FMovieSceneEvaluationState>())
			{
				Sequence = EvaluationState->FindSequence(Instance.GetSequenceID());
			}
			else
			{
				SharedPlaybackState->GetSequence(Instance.GetSequenceID());
			}
			if (Sequence)
			{ 
				if (Linker->EntityManager.HasComponent(EntityID, FBuiltInComponentTypes::Get()->Tags.OldStyleSpawnable))
				{
					if (FMovieSceneSpawnable* Spawnable = Sequence->GetMovieScene()->FindSpawnable(SpawnableObjectID))
					{
						// If the sequence instance has finished and it is a sub sequence, we do not destroy the spawnable
						// if it is owned by the root sequence or externally. These will get destroyed or forgotten by the player when it ends
						if (Instance.HasFinished() && Instance.IsSubSequence())
						{
							if (Spawnable->GetSpawnOwnership() != ESpawnOwnership::InnerSequence)
							{
								return;
							}
						}

						DestroyedObjects.Emplace(SpawnableObjectID, 0, Instance.GetSequenceID(), InstanceHandle);
					}
				}
				else if (const FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences())
				{
					for (int32 Index = 0; Index < BindingReferences->GetReferences(SpawnableObjectID).Num(); ++Index)
					{
						if (const UMovieSceneCustomBinding* CustomBinding = BindingReferences->GetCustomBinding(SpawnableObjectID, Index))
						{
							if (const UMovieSceneSpawnableBindingBase* SpawnableBinding = CustomBinding->AsSpawnable(SharedPlaybackState))
							{
								// If the sequence instance has finished and it is a sub sequence, we do not destroy the spawnable
								// if it is owned by the root sequence or externally. These will get destroyed or forgotten by the player when it ends
								if (Instance.HasFinished() && Instance.IsSubSequence())
								{
									if (SpawnableBinding->SpawnOwnership != ESpawnOwnership::InnerSequence)
									{
										return;
									}
								}
								DestroyedObjects.Emplace(SpawnableObjectID, Index, Instance.GetSequenceID(), InstanceHandle);
							}
						}
					}
				}
			}
		}
	};

	// ----------------------------------------------------------------------------------------------------------------------------------------
	// Step 1 - destroy any spawnable objects that are no longer relevant
	//          NOTE: We gather the objects into an array because destroying an object can potentially cause a garbage collection to run (ie
	//                if the spawnable is a level instance), and that could assert because we are currently iterating the ECS

	FEntityTaskBuilder()
		.ReadEntityIDs()
		.Read(BuiltInComponents->InstanceHandle)
		.Read(BuiltInComponents->SpawnableBinding)
		.FilterAll({ BuiltInComponents->Tags.NeedsUnlink })
		.Iterate_PerEntity(&Linker->EntityManager, DestroyOldSpawnables);

	auto DoDestroy = [this, &DestroyedObjects, InstanceRegistry]() {

		for (TTuple<FGuid, int32, FMovieSceneSequenceID, FInstanceHandle> Tuple : DestroyedObjects)
		{
			// Have to check whether the player is still valid because there is a possibility it got cleaned up
			if (InstanceRegistry->IsHandleValid(Tuple.Get<3>()))
			{
				TSharedRef<const FSharedPlaybackState> SharedPlaybackState = InstanceRegistry->GetInstance(Tuple.Get<3>()).GetSharedPlaybackState();
				if (FMovieSceneSpawnRegister* SpawnRegister = SharedPlaybackState->FindCapability<FMovieSceneSpawnRegister>())
				{
					SpawnRegister->DestroySpawnedObject(Tuple.Get<0>(), Tuple.Get<2>(), SharedPlaybackState, Tuple.Get<1>());
				}
			}
		}
		DestroyedObjects.Empty();
	};

	DoDestroy();

	// ----------------------------------------------------------------------------------------------------------------------------------------
	// Step 2 - iterate all pending spawnables and spawn their objects if necessary
	auto SpawnNewObjects = [this, &DestroyOldSpawnables, InstanceRegistry](FMovieSceneEntityID EntityID, FInstanceHandle InstanceHandle, const FGuid& SpawnableBindingID)
	{
		SCOPE_CYCLE_COUNTER(MovieSceneEval_SpawnSpawnables)

		const FSequenceInstance& SequenceInstance = InstanceRegistry->GetInstance(InstanceHandle);

		TSharedRef<FSharedPlaybackState> SharedPlaybackState = SequenceInstance.GetSharedPlaybackState();
		FMovieSceneSpawnRegister* SpawnRegister = SharedPlaybackState->FindCapability<FMovieSceneSpawnRegister>();
		if (!SpawnRegister)
		{
			return;
		}

		FMovieSceneSequenceID SequenceID = SequenceInstance.GetSequenceID();
		const FMovieSceneEvaluationOperand SpawnableOperand(SequenceID, SpawnableBindingID);
		IStaticBindingOverridesPlaybackCapability* StaticOverrides = SharedPlaybackState->FindCapability<IStaticBindingOverridesPlaybackCapability>();
		if (StaticOverrides && StaticOverrides->GetBindingOverride(SpawnableOperand))
		{
			// Don't do anything if this operand was overriden... someone else will take care of it (either another spawn track, or
			// some possessable).
			return;
		}

		UObject* ExistingSpawnedObject = SpawnRegister->FindSpawnedObject(SpawnableBindingID, SequenceID).Get();

		FMovieSceneEvaluationState* State = SharedPlaybackState->FindCapability<FMovieSceneEvaluationState>();
		if (State && !State->GetBindingActivation(SpawnableBindingID, SequenceID))
		{
			// If the binding is currently inactive, don't spawn the object.

			// We may have existing spawned objects, in which case we need to destory them
			DestroyOldSpawnables(EntityID, InstanceHandle, SpawnableBindingID);
			return;
		}

		// Check whether the binding is overridden - if it is we cannot spawn a new object
		if (IMovieScenePlaybackClient* DynamicOverrides = SharedPlaybackState->FindCapability<IMovieScenePlaybackClient>())
		{
			TArray<UObject*, TInlineAllocator<1>> FoundObjects;
			bool bUseDefaultBinding = DynamicOverrides->RetrieveBindingOverrides(SpawnableBindingID, SequenceID, FoundObjects);
			if (!bUseDefaultBinding)
			{
				// If the binding has been overridden, then the binding is new and we need to destroy any spawned objects that may exist
				DestroyOldSpawnables(EntityID, InstanceHandle, SpawnableBindingID);

				// This spawnable is overridden so don't try and spawn anything
				return;
			}
		}

		// At this point we've decided that we should have a spawned object
		const UMovieSceneSequence* Sequence = SharedPlaybackState->GetSequence(SequenceID);
		if (!Sequence)
		{
			return;
		}

		auto SpawnObject = [&SpawnRegister, &SpawnableBindingID, &Sequence, &SequenceID, &SharedPlaybackState](int BindingIndex)
		{
			UObject* SpawnedObject = SpawnRegister->SpawnObject(SpawnableBindingID, *Sequence->GetMovieScene(), SequenceID, SharedPlaybackState, BindingIndex);
			IMovieScenePlayer* Player = FPlayerIndexPlaybackCapability::GetPlayer(SharedPlaybackState);
			if (SpawnedObject)
			{
				FMovieSceneEvaluationOperand Operand(SequenceID, SpawnableBindingID);

				if (Player)
				{
					Player->OnObjectSpawned(SpawnedObject, Operand);
				}

				FMovieSceneInstancePreAnimatedState& PreAnimatedState = SharedPlaybackState->GetPreAnimatedState();
				PreAnimatedState.SavePreAnimatedState(*SpawnedObject, SpawnableAnimTypeID, FSpawnTrackPreAnimatedTokenProducer(Operand, BindingIndex));
			}
			return SpawnedObject;
		};

		// If we have an old-style spawnable, we can just spawn a single object, otherwise we need to check our binding references to see if we need to spawn multiple
		if (Linker->EntityManager.HasComponent(EntityID, FBuiltInComponentTypes::Get()->Tags.OldStyleSpawnable))
		{
			SpawnObject(0);
		}
		else if (const FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences())
		{
			for (int32 Index = 0; Index < BindingReferences->GetReferences(SpawnableBindingID).Num(); ++Index)
			{
				SpawnObject(Index);
			}
		}

		InstanceRegistry->InvalidateObjectBinding(SpawnableBindingID, InstanceHandle);
	};

	FEntityTaskBuilder()
	.ReadEntityIDs()
	.Read(BuiltInComponents->InstanceHandle)
	.Read(BuiltInComponents->SpawnableBinding)
	.FilterAll({ BuiltInComponents->Tags.NeedsLink })
	.Iterate_PerEntity(&Linker->EntityManager, SpawnNewObjects);

	// Destroy any spawnables that have since been added to the DestroyList
	DoDestroy();
}

