// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneSystemTaskDependencies.h"
#include "EntitySystem/MovieSceneEntitySystemGraphs.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "Algo/Find.h"


DEFINE_STAT(MovieSceneEval_SystemDependencyCost);

namespace UE
{
namespace MovieScene
{


bool GIgnoreDependenciesWhenNotThreading = true;
FAutoConsoleVariableRef CVarIgnoreDependenciesWhenNotThreading(
	TEXT("Sequencer.IgnoreDependenciesWhenNotThreading"),
	GIgnoreDependenciesWhenNotThreading,
	TEXT("(Default: true) Whether to ignore task dependencies when there is no threading.")
);


void FSystemTaskPrerequisites::FilterByComponent(FGraphEventArray& OutArray, std::initializer_list<FComponentTypeID> ComponentTypes) const
{
	for (const FPrerequisite& Prereq : Prereqs)
	{
		if (Algo::Find(ComponentTypes, Prereq.ComponentType) != nullptr)
		{
			OutArray.Add(Prereq.GraphEvent);
		}
	}
}

void FSystemTaskPrerequisites::AddComponentTask(FComponentTypeID ComponentType, const FGraphEventRef& InNewTask)
{
	Prereqs.Add(FPrerequisite{ InNewTask, ComponentType });
}

void FSystemTaskPrerequisites::Consume(const FSystemTaskPrerequisites& Other)
{
	Prereqs.Append(Other.Prereqs);
	AllTasks.Reset();
}



FSystemSubsequentTasks::FSystemSubsequentTasks(FMovieSceneEntitySystemGraph* InGraph, FGraphEventArray* InAllTasks, EEntityThreadingModel InThreadingModel)
	: Graph(InGraph)
	, AllTasks(InAllTasks)
	, ThreadingModel(InThreadingModel)
{}

void FSystemSubsequentTasks::ResetNode(uint16 InNodeID)
{
	if (ThreadingModel == EEntityThreadingModel::NoThreading && GIgnoreDependenciesWhenNotThreading)
	{
		return;
	}


	Subsequents = Graph->Nodes.Array[InNodeID].SubsequentTasks;
	NodeID = InNodeID;

	if (Subsequents)
	{
		Subsequents->Empty();
	}
}

void FSystemSubsequentTasks::AddRootTask(FGraphEventRef RootTask)
{
	if (ThreadingModel == EEntityThreadingModel::NoThreading && GIgnoreDependenciesWhenNotThreading)
	{
		return;
	}

	SCOPE_CYCLE_COUNTER(MovieSceneEval_SystemDependencyCost)
	if (RootTask)
	{
		if (!Subsequents)
		{
			Subsequents = MakeShared<FSystemTaskPrerequisites>();
			Graph->Nodes.Array[NodeID].SubsequentTasks = Subsequents;
		}
		Subsequents->AddRootTask(RootTask);

		AllTasks->Add(RootTask);
	}
}

void FSystemSubsequentTasks::AddComponentTask(UE::MovieScene::FComponentTypeID ComponentType, FGraphEventRef ComponentTask)
{
	if (ThreadingModel == EEntityThreadingModel::NoThreading && GIgnoreDependenciesWhenNotThreading)
	{
		return;
	}

	SCOPE_CYCLE_COUNTER(MovieSceneEval_SystemDependencyCost)

	if (ComponentTask)
	{
		if (!Subsequents)
		{
			Subsequents = MakeShared<FSystemTaskPrerequisites>();
			Graph->Nodes.Array[NodeID].SubsequentTasks = Subsequents;
		}

		Subsequents->AddComponentTask(ComponentType, ComponentTask);
		AllTasks->Add(ComponentTask);
	}
}


} // namespace MovieScene
} // namespace UE
