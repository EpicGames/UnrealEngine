// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneTaskScheduler.h"
#include "Tasks/Task.h"
#include "Algo/RandomShuffle.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Stats/Stats.h"
#include "Misc/AutomationTest.h"

namespace UE::MovieScene
{

DECLARE_CYCLE_STAT(TEXT("Anonymous MovieScene Task"), MovieSceneEval_AnonymousTask, STATGROUP_MovieSceneECS);

/** CVar that disables our task scheduler. When disabled, all systems that are normally in the Scheduling phase will be executed in the Evaluation phase with their OnRun function */
bool GSequencerCustomTaskScheduling = true;
static FAutoConsoleVariableRef CVarSequencerCustomTaskScheduling(
	TEXT("Sequencer.CustomTaskScheduling"),
	GSequencerCustomTaskScheduling,
	TEXT("(Default: true. Enables more efficient custom task scheduling of asynchronous Sequencer evaluation.")
	);

/** Flag structure to pass when executing a task. */
struct FTaskExecutionFlags
{
	explicit FTaskExecutionFlags()
	{
		bCanInlineSubsequents = true;
	}

	/**
	 * When false, prevents any other tasks from being executed inline on completion of this task.
	 * This should be used when a task is being forced inline to prevent a cascade of inlined tasks from blocking the scheduling of other async work
	 */
	uint8 bCanInlineSubsequents : 1;
};

FScheduledTask::FScheduledTask(FEntityAllocationWriteContext InWriteContextOffset)
	: StatId(GET_STATID(MovieSceneEval_AnonymousTask))
	, WriteContextOffset(InWriteContextOffset)
	, TaskFunctionType(FScheduledTaskFuncionPtr::EType::None)
{
	bForceGameThread = false;
	bForceInline = false;
}

FScheduledTask::FScheduledTask(FScheduledTask&& InTask)
	: ComputedSubsequents(MoveTemp(InTask.ComputedSubsequents))
	, TaskFunction(InTask.TaskFunction)
	, TaskContext(InTask.TaskContext)
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	, DebugName(InTask.DebugName)
#endif
	, StatId(InTask.StatId)
	, WriteContextOffset(InTask.WriteContextOffset)
	, LockedComponentData(MoveTemp(InTask.LockedComponentData))
	, NumPrerequisites(InTask.NumPrerequisites)
	, WaitCount(InTask.WaitCount.Load(EEntityThreadingModel::NoThreading))
	, ChildCompleteCount(InTask.ChildCompleteCount.Load(EEntityThreadingModel::NoThreading))
	, Parent(InTask.Parent)
	, NumChildren(InTask.NumChildren)
	, TaskFunctionType(InTask.TaskFunctionType)
{
	bForceGameThread = InTask.bForceGameThread;
	bForceInline = InTask.bForceInline;
}

FScheduledTask::~FScheduledTask()
{}

void FScheduledTask::SetFunction(TaskFunctionPtr InFunction)
{
	switch (InFunction.GetIndex())
	{
	case 0: TaskFunctionType = TaskFunction.Assign(InFunction.Get<UnboundTaskFunctionPtr>()); break;
	case 1: TaskFunctionType = TaskFunction.Assign(InFunction.Get<AllocationFunctionPtr>()); break;
	case 2: TaskFunctionType = TaskFunction.Assign(InFunction.Get<AllocationItemFunctionPtr>()); break;
	case 3: TaskFunctionType = TaskFunction.Assign(InFunction.Get<PreLockedAllocationItemFunctionPtr>()); break;
	}
}

void FScheduledTask::Run(const FEntitySystemScheduler* Scheduler, FTaskExecutionFlags InFlags) const
{
	if (TaskFunctionType != FScheduledTaskFuncionPtr::EType::None)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

		UE_LOGF(LogMovieSceneECS, VeryVerbose, "Running task \"%ls\"", *DebugName);

		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*DebugName);
#endif

#if STATS || ENABLE_STATNAMEDEVENTS
		FScopeCycleCounter Scope(StatId);
#endif
		FEntityAllocationWriteContext ThisWriteContext = Scheduler->GetWriteContextOffset().Add(WriteContextOffset);

		switch(TaskFunctionType)
		{
		case FScheduledTaskFuncionPtr::EType::Unbound:
			(TaskFunction.UnboundTask)(TaskContext.Get(), ThisWriteContext);
			break;
		case FScheduledTaskFuncionPtr::EType::AllocationPtr:
			{
				check(LockedComponentData.AllocationIndex != MAX_uint16);

				FEntityAllocationProxy Allocation = FEntityAllocationProxy::MakeInstance(Scheduler->GetEntityManager(), LockedComponentData.AllocationIndex);
				(TaskFunction.Allocation)(Allocation, TaskContext.Get(), ThisWriteContext);
			}
			break;
		case FScheduledTaskFuncionPtr::EType::AllocationItem:
			{
				check(LockedComponentData.AllocationIndex != MAX_uint16);
				FEntityAllocationProxy Allocation = FEntityAllocationProxy::MakeInstance(Scheduler->GetEntityManager(), LockedComponentData.AllocationIndex);
				(TaskFunction.AllocationItem)(Allocation, TaskContext.Get(), ThisWriteContext);
			}
			break;
		case FScheduledTaskFuncionPtr::EType::PreLockedAllocationItem:
			{
				check(LockedComponentData.AllocationIndex != MAX_uint16);
				FEntityAllocationProxy Allocation = FEntityAllocationProxy::MakeInstance(Scheduler->GetEntityManager(), LockedComponentData.AllocationIndex);

				TArrayView<const FPreLockedDataPtr> PreLockedData(LockedComponentData.PreLockedComponentData);
				(TaskFunction.PreLockedAllocationItem)(Allocation, PreLockedData, TaskContext.Get(), ThisWriteContext);
			}
			break;
		}
	}

	// Now the task is finished, schedule any children to run, or any subsequents
	// If we are a parent we do not call CompleteTask until _all_ our children have finished:
	//     this happens in FEntitySystemScheduler::CompleteTask if Parent is valid
	if (NumChildren > 0)
	{
		// Increment the child completion count to protect CompleteTask being called for _this_ parent task
		// while the loop over ChildTasks is running. This prevents a race condition where the final child can end up being
		// the last task altogether, which triggers OnAllTasksFinished, potentially allowing the waiting thread to continue
		// and destroy or  otherwise mutate the contents of FEntitySystemScheduler resulting in a crash.
		//
		// Once our loop has finished we check the child complete count to see if this was the last one
		ChildCompleteCount.Add(Scheduler->GetEntityManager()->GetThreadingModel(), 1);

		const FScheduledTask* Children = this + 1;
		for (uint16 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
		{
			Scheduler->PrerequisiteCompleted(&Children[ChildIndex], nullptr);
		}

		// Subtract our count added on ln 205. If this is the last count, complete this task (all children have completed)
		const int32 PreviousCompleteCount = ChildCompleteCount.Sub(Scheduler->GetEntityManager()->GetThreadingModel(), 1);
		if (PreviousCompleteCount == 1)
		{
			Scheduler->CompleteTask(this, InFlags);
		}
	}
	else
	{
		checkSlow(ChildCompleteCount.Load(Scheduler->GetEntityManager()->GetThreadingModel()) == 0);
		Scheduler->CompleteTask(this, InFlags);
	}
}

FEntitySystemScheduler::FEntitySystemScheduler(FEntityManager* InEntityManager)
	: EntityManager(InEntityManager)
{
}

FEntitySystemScheduler::~FEntitySystemScheduler()
{
	check(!GameThreadSignal);
}

bool FEntitySystemScheduler::IsCustomSchedulingEnabled()
{
	return GSequencerCustomTaskScheduling;
}

FTaskID FEntitySystemScheduler::CreateForkedAllocationTask(const FTaskParams& InParams, TSharedPtr<ITaskContext> InTaskContext, TaskFunctionPtr InTaskFunction, TFunctionRef<void(FEntityAllocationIteratorItem,TArray<FPreLockedDataPtr>&)> InPreLockFunc, const FEntityComponentFilter& Filter, const FComponentMask& ReadDeps, const FComponentMask& WriteDeps)
{
	FEntityAllocationWriteContext WriteContext(*EntityManager);
	WriteContext.Subtract(WriteContextBase);

	const int32 StartingNumTasks = Tasks.Num();

	// We should never encounter both read and write dependencies for the same component
	ensure(FComponentMask::BitwiseAND(ReadDeps, WriteDeps, EBitwiseOperatorFlags::MinSize).NumComponents() == 0);

	FTaskID LastTaskID, ParentTaskID;

	for (FEntityAllocationIteratorItem Allocation : EntityManager->Iterate(&Filter))
	{
		// If we haven't created a parent yet, create that now
		if (!ParentTaskID)
		{
			ParentTaskID = FTaskID(Tasks.Emplace(WriteContext));

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			TStringBuilder<128> ParentTaskName;
			ParentTaskName += TEXT("Parent task for ");
			ParentTaskName += InParams.DebugName;
			if (InParams.DebugName)
			{
				ParentTaskName += InParams.DebugName;
			}
	#if STATS
			else
			{
				InParams.StatId.GetName().ToString(ParentTaskName);
			}
	#endif
#endif
			Tasks[ParentTaskID.Index].bForceInline = true;
			Tasks[ParentTaskID.Index].bForceGameThread = false;

			// We're dependent upon the output of any explicit upstream dependency (not bound to a specific allocation)
			if (CurrentPrerequisites)
			{
				if (InParams.bForceConsumeUpstream)
				{
					for (int32 SystemPrereq : CurrentPrerequisites->SystemWidePrerequisites)
					{
						AddPrerequisite(FTaskID(SystemPrereq), ParentTaskID);
					}
				}

				for (int32 SystemPrereq : CurrentPrerequisites->ForcedSystemWidePrerequisites)
				{
					AddPrerequisite(FTaskID(SystemPrereq), ParentTaskID);
				}
			}

			CurrentSubsequents.SystemWidePrerequisites.SetBit(ParentTaskID.Index);
			if (InParams.bForcePropagateDownstream)
			{
				CurrentSubsequents.ForcedSystemWidePrerequisites.SetBit(ParentTaskID.Index);
			}
		}

		const int32 AllocationIndex = Allocation.GetAllocationIndex();

		// Set up a new task for this allocation
		const int32 AllocationTask = Tasks.Num();
		const FTaskID ThisTask(AllocationTask);

		FScheduledTask::FLockedComponentData LockedComponentData;
		LockedComponentData.AllocationIndex = static_cast<uint16>(AllocationIndex);

		InPreLockFunc(Allocation, LockedComponentData.PreLockedComponentData);

		// Create the task

		const int32 ChildTaskIndex = Tasks.Num();
		FScheduledTask& NewTask = Tasks.Emplace_GetRef(WriteContext);
		NewTask.SetFunction(InTaskFunction);
		NewTask.StatId = InParams.StatId;
		NewTask.Parent = FTaskID(ParentTaskID.Index);
		NewTask.TaskContext = InTaskContext;
		NewTask.bForceGameThread = InParams.bForceGameThread;
		NewTask.NumPrerequisites = 1; // +1 Because the parent triggers us as well when it starts
		NewTask.LockedComponentData = MoveTemp(LockedComponentData);
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (InParams.DebugName)
		{
			NewTask.DebugName = InParams.DebugName;
		}
	#if STATS
		else
		{
			NewTask.DebugName = InParams.StatId.GetName().ToString();
		}
	#endif
#endif

		++Tasks[ParentTaskID.Index].NumChildren;

		// If we're forking these tasks, the parent schedules this allocation task as soon as possible
		// In this case that means as soon as everything upstream that writes to the same components on the same allocation is finished
		// This guarantees we can never read/write from/to component data that is being written to on another thread

		// Component reads only depend up upstream writes
		for (FComponentMaskIterator It = ReadDeps.Iterate(); It; ++It)
		{
			FComponentTypeID Type = FComponentTypeID::FromBitIndex(It.GetIndex());

			// Otherwise we can be scheduled as soon as the last write task to this allocation is done
			FComponentDependencies& Dependencies = ComponentDepedenciesByAllocation.FindOrAdd(MakeTuple(AllocationIndex, Type));
			if (Dependencies.WriteTask)
			{
				AddPrerequisite(Dependencies.WriteTask, ThisTask);
			}

			Dependencies.ReadTasks.SetBit(ThisTask.Index);
		}

		// Component writes depend up upstream reads and writes, and become the new write dependency for anything downstream
		for (FComponentMaskIterator It = WriteDeps.Iterate(); It; ++It)
		{
			FComponentTypeID Type = FComponentTypeID::FromBitIndex(It.GetIndex());

			FComponentDependencies& Dependencies = ComponentDepedenciesByAllocation.FindOrAdd(MakeTuple(AllocationIndex, Type));
			for (int32 Dep : Dependencies.ReadTasks)
			{
				AddPrerequisite(FTaskID(Dep), ThisTask);
			}
			if (Dependencies.WriteTask)
			{
				AddPrerequisite(Dependencies.WriteTask, ThisTask);
			}

			Dependencies.ReadTasks = FTaskBitSet();
			Dependencies.WriteTask = ThisTask;
		}

		// If the tasks are serial, we depend on the last child we made
		if (InParams.bSerialTasks && LastTaskID)
		{
			AddPrerequisite(LastTaskID, ThisTask);
		}

		LastTaskID = ThisTask;
	}

	check(!ParentTaskID || ParentTaskID.Index + Tasks[ParentTaskID.Index].NumChildren == Tasks.Num()-1);

	return ParentTaskID;
}

void FEntitySystemScheduler::AddPrerequisite(FTaskID Prerequisite, FTaskID Subsequent)
{
	// Take care not to add the same dependency multiple times.
	// This will happen if the same upstream task writes to many components
	// that this task needs.
	if (Prerequisite && Subsequent && Prerequisite.Index != Subsequent.Index && !Tasks[Prerequisite.Index].ComputedSubsequents.IsBitSet(Subsequent.Index))
	{
		Tasks[Prerequisite.Index].ComputedSubsequents.SetBit(Subsequent.Index);
		Tasks[Subsequent.Index].NumPrerequisites += 1;
	}
}

void FEntitySystemScheduler::AddChildFront(FTaskID Parent, FTaskID Child)
{
	if (Parent && Child && ensure(Child.Index > Parent.Index && Child.Index == Parent.Index + Tasks[Parent.Index].NumChildren + 1))
	{
		// Children can't be prerequisites of their parent or visa-versa
		check(!Tasks[Parent.Index].ComputedSubsequents.IsBitSet(Child.Index) && !Tasks[Child.Index].ComputedSubsequents.IsBitSet(Parent.Index));

		// All existing children must come after this
		const int32 FirstChild = Parent.Index+1;
		const int32 ChildrenEnd = FirstChild + Tasks[Parent.Index].NumChildren;
		for (int32 ChildIndex = FirstChild; ChildIndex < ChildrenEnd; ++ChildIndex)
		{
			AddPrerequisite(Child, FTaskID(ChildIndex));
		}

		++Tasks[Parent.Index].NumChildren;

		// The parent will signal this task when it is run
		Tasks[Child.Index].NumPrerequisites += 1;
		Tasks[Child.Index].Parent = Parent;
	}
}

void FEntitySystemScheduler::AddChildBack(FTaskID Parent, FTaskID Child)
{
	// Child tasks must be immediately reported
	if (Parent && Child && ensure(Child.Index > Parent.Index && Child.Index == Parent.Index + Tasks[Parent.Index].NumChildren + 1))
	{
		// Children can't be prerequisites of their parent or visa-versa
		check(!Tasks[Parent.Index].ComputedSubsequents.IsBitSet(Child.Index) && !Tasks[Child.Index].ComputedSubsequents.IsBitSet(Parent.Index));

		// All existing children must come before this
		const int32 FirstChild = Parent.Index+1;
		const int32 ChildrenEnd = FirstChild + Tasks[Parent.Index].NumChildren;
		for (int32 ChildIndex = FirstChild; ChildIndex < ChildrenEnd; ++ChildIndex)
		{
			AddPrerequisite(FTaskID(ChildIndex), Child);
		}

		++Tasks[Parent.Index].NumChildren;

		// The parent will signal this task when it is run
		Tasks[Child.Index].NumPrerequisites += 1;
		Tasks[Child.Index].Parent = Parent;
	}
}

FTaskID FEntitySystemScheduler::AddTask(const FTaskParams& InParams, TSharedPtr<ITaskContext> InTaskContext, TaskFunctionPtr InTaskFunction)
{
	FEntityAllocationWriteContext WriteContext(*EntityManager);
	WriteContext.Subtract(WriteContextBase);

	const int32 TaskIndex = Tasks.Num();
	FScheduledTask& NewTask = Tasks.Emplace_GetRef(WriteContext);

	NewTask.SetFunction(InTaskFunction);
	NewTask.TaskContext   = InTaskContext;
	NewTask.StatId        = InParams.StatId;
	NewTask.bForceGameThread = InParams.bForceGameThread;
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	NewTask.DebugName     = InParams.DebugName;
#endif

	FTaskID ThisTask(TaskIndex);

	// We're dependent upon the output of any explicit upstream dependency (not bound to a specific allocation)
	if (CurrentPrerequisites)
	{
		if (InParams.bForceConsumeUpstream)
		{
			for (int32 SystemPrereq : CurrentPrerequisites->SystemWidePrerequisites)
			{
				AddPrerequisite(FTaskID(SystemPrereq), ThisTask);
			}
		}

		for (int32 SystemPrereq : CurrentPrerequisites->ForcedSystemWidePrerequisites)
		{
			AddPrerequisite(FTaskID(SystemPrereq), ThisTask);
		}
	}

	CurrentSubsequents.SystemWidePrerequisites.SetBit(TaskIndex);
	if (InParams.bForcePropagateDownstream)
	{
		CurrentSubsequents.ForcedSystemWidePrerequisites.SetBit(TaskIndex);
	}

	return ThisTask;
}

FTaskID FEntitySystemScheduler::AddNullTask()
{
	FEntityAllocationWriteContext WriteContext(*EntityManager);
	WriteContext.Subtract(WriteContextBase);

	const int32 TaskIndex = Tasks.Num();
	FScheduledTask& NewTask = Tasks.Emplace_GetRef(WriteContext);
	return FTaskID(TaskIndex);
}

void FEntitySystemScheduler::ShuffleTasks()
{
	TArray<int32> RemainingIndices;

	// We have to keep children contiguous in memory with their parents so only shuffle parents
	for (int32 Index = 0; Index < Tasks.Num(); ++Index)
	{
		RemainingIndices.Emplace(Index);
		Index += Tasks[Index].NumChildren;
	}

	Algo::RandomShuffle(RemainingIndices);

	TArray<int32> ReverseShuffledIndices;
	ReverseShuffledIndices.SetNum(Tasks.Num());
	for (int32 Index = 0, ShuffledIndex = 0; Index < RemainingIndices.Num(); ++Index)
	{
		int32 TaskIndex = RemainingIndices[Index];

		ReverseShuffledIndices[ShuffledIndex] = TaskIndex;

		const int32 ShuffeldChildStart   = ShuffledIndex + 1;
		const int32 UnshuffeldChildStart = TaskIndex + 1;

		const int32 NumChildren = Tasks[TaskIndex].NumChildren;

		ShuffledIndex += 1 + NumChildren;
		for (int32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
		{
			ReverseShuffledIndices[ShuffeldChildStart + ChildIndex] = UnshuffeldChildStart + ChildIndex;
		}
	}

	TArray<int32> ShuffledIndices;
	ShuffledIndices.SetNum(Tasks.Num());
	for (int32 Index = 0; Index < ReverseShuffledIndices.Num(); ++Index)
	{
		const int32 OriginalTaskIndex = ReverseShuffledIndices[Index];
		ShuffledIndices[OriginalTaskIndex] = Index;
	}

	auto RedirectMask = [&ShuffledIndices](FTaskBitSet& InOutBitSet)
	{
		FTaskBitSet NewBits;
		for (int32 Index : InOutBitSet)
		{
			NewBits.SetBit(ShuffledIndices[Index]);
		}
		Swap(InOutBitSet, NewBits);
	};

	for (int32 Index = 0; Index < Tasks.Num(); ++Index)
	{
		FScheduledTask& Task = Tasks[Index];

		if (Task.Parent)
		{
			Task.Parent = FTaskID(ShuffledIndices[Task.Parent.Index]);
		}

		RedirectMask(Task.ComputedSubsequents);
	}

	RedirectMask(InitialTasks);

	TArray<FScheduledTask> OldTasks;
	Swap(Tasks, OldTasks);

	for (int32 Index = 0; Index < OldTasks.Num(); ++Index)
	{
		Tasks.Emplace(MoveTemp(OldTasks[ReverseShuffledIndices[Index]]));
	}
}

void FEntitySystemScheduler::ExecuteTasks()
{
	if (!Tasks.Num())
	{
		return;
	}

	const EEntityThreadingModel ThreadingModelToUse = EntityManager->GetThreadingModel();

	const int32 PreviousNumRemaining = NumTasksRemaining.Exchange(ThreadingModelToUse, Tasks.Num());
	check(PreviousNumRemaining == 0);

	ThreadingModel = ThreadingModelToUse;
	// In a transaction we can only support the no-threading mode.
	check(!AutoRTFM::IsTransactional() || (EEntityThreadingModel::NoThreading == ThreadingModel));

	WriteContextBase = FEntityAllocationWriteContext(*EntityManager);

	// Condition 1: No threading
	//              Initiate all tasks immediately. Their subsequents will be triggered inline
	if (ThreadingModel == EEntityThreadingModel::NoThreading)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("Run Scheduled Tasks");
		for (int32 Index : InitialTasks)
		{
			Tasks[Index].Run(this, FTaskExecutionFlags());
		}

		check(NumTasksRemaining.Load(ThreadingModel) == 0);
		EntityManager->IncrementSystemSerial(SystemSerialIncrement);
		return;
	}

	// Condition 2: Task graph threading
	//              Schedule initial tasks tasks immediately. Gamethread tasks will be added to the GT queue to ensure that threaded work can be scheduled asap.

	// We need to get a game thread signal from the event pool for the execution
	// which we'll return to the pool after we've completed execution.
	check(!GameThreadSignal);
	GameThreadSignal = FPlatformProcess::GetSynchEventFromPool();

	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("Dispatch Scheduled Tasks");

		int32 NumInitialTasks = 0;
		for (int32 Index : InitialTasks)
		{
			// If it has to run on the game thread, add it to the task list.
			// This allows us to schedule threaded tasks first, then run the game
			// thread ones while they are in flight
			if (Tasks[Index].bForceGameThread)
			{
				GameThreadTaskList.Push(const_cast<FScheduledTask*>(&Tasks[Index]));
			}
			else if (GameThreadTaskList.IsEmpty())
			{
				GameThreadTaskList.Push(const_cast<FScheduledTask*>(&Tasks[Index]));
			}
			else
			{
				UE::Tasks::Launch(TEXT("MovieSceneTask"), [this, Index](){
					this->Tasks[Index].Run(this, FTaskExecutionFlags());
				}, UE::Tasks::ETaskPriority::High);
			}

			++NumInitialTasks;
		}

		ensure(NumInitialTasks != 0);
	}

	TRACE_CPUPROFILER_EVENT_SCOPE_STR("Wait For Scheduled Tasks");
	for(;;)
	{
		while (const FScheduledTask* Task = GameThreadTaskList.Pop())
		{
			Task->Run(this, FTaskExecutionFlags());
		}

		// Process exit conditions for this loop, where we have one of two situations:
		//
		//    1. The final task was just processed on this thread.
		//       In this case OnAllTasksFinished will have decremented
		//       NumTasksRemaining to -1 and we can break immediately.
		//
		//    2. The final task was completed on a thread.
		//       In this case, this thread may see NumTasksRemaining as either 0 or -1
		//       depending on how far through OnAllTasksFinished the bg thread is.
		//       In this case, -1 means OnAllTasksFinished has finished, 0 means it is
		//       part way through, so we just spin on the atomic until it hits -1
		const int32 RemainingCount = NumTasksRemaining.Load(ThreadingModel);
		if (RemainingCount == -1)
		{
			break;
		}
		else if (RemainingCount == 0)
		{
			// Handle the race condition where this code runs in-between
			//    NumTasksRemaining reaching 0 and the GameThreadSignal being triggered.
			// If the Trigger is called before the Wait below, some platforms may deadlock
			while (NumTasksRemaining.Load(ThreadingModel) != -1);
			break;
		}

		GameThreadSignal->Wait();
	}

	check(NumTasksRemaining.Load(ThreadingModel) == -1);
	EntityManager->IncrementSystemSerial(SystemSerialIncrement);

	NumTasksRemaining.Exchange(ThreadingModel, 0);

	// Lastly return the game thread signal event to the pool as we are
	// done executing.
	check(GameThreadSignal);
	FPlatformProcess::ReturnSynchEventToPool(GameThreadSignal);
	GameThreadSignal = nullptr;
}

void FEntitySystemScheduler::CompleteTask(const FScheduledTask* Task, FTaskExecutionFlags InFlags) const
{
	// Reset the WaitCount ready for the next run
	const int32 PreviousWaitCount = Task->WaitCount.Exchange(ThreadingModel, Task->NumPrerequisites);
	const int32 PreviousChildCount = Task->ChildCompleteCount.Exchange(ThreadingModel, Task->NumChildren);

	checkSlow(PreviousWaitCount == 0 && PreviousChildCount == 0);

	int32 FirstInlineIndex = INDEX_NONE;
	for (int32 Index : Task->ComputedSubsequents)
	{
		PrerequisiteCompleted(FTaskID(Index), InFlags.bCanInlineSubsequents ? &FirstInlineIndex : nullptr);
	}

	// Complete our parent if this is the last child
	if (Task->Parent)
	{
		const FScheduledTask* Parent = &Tasks[Task->Parent.Index];
		const int32 PreviousCompleteCount = Parent->ChildCompleteCount.Sub(ThreadingModel, 1);
		if (PreviousCompleteCount == 1)
		{
			CompleteTask(Parent, InFlags);
		}
	}

	if (FirstInlineIndex != INDEX_NONE)
	{
		Tasks[FirstInlineIndex].Run(this, FTaskExecutionFlags());
	}

	const int32 PreviousNumRemaining = NumTasksRemaining.Sub(ThreadingModel, 1);
	if (PreviousNumRemaining == 1)
	{
		OnAllTasksFinished();
	}
}

void FEntitySystemScheduler::PrerequisiteCompleted(FTaskID TaskID, int32* OptRunInlineIndex) const
{
	PrerequisiteCompleted(&Tasks[TaskID.Index], OptRunInlineIndex);
}

void FEntitySystemScheduler::PrerequisiteCompleted(const FScheduledTask* Task, int32* OptRunInlineIndex) const
{
	// We either need to not be using threading, or have a valid game thread signal event!
	check((ThreadingModel == EEntityThreadingModel::NoThreading) || (GameThreadSignal != nullptr));

	int32 PreviousWaitCount = Task->WaitCount.Sub(ThreadingModel, 1);
	if (PreviousWaitCount <= 1)
	{
		if (PreviousWaitCount <= 0)
		{
			// This is an error
			ensureMsgf(false, TEXT("Sequencer Task Prerequisite Count underflow!"));
			if (GameThreadSignal)
			{
				// Trigger the game thread to wake up if necessary
				GameThreadSignal->Trigger();
			}
		}
		else if (ThreadingModel == EEntityThreadingModel::NoThreading)
		{
			Task->Run(this, FTaskExecutionFlags());
		}
		else if (Task->bForceInline)
		{
			FTaskExecutionFlags Flags;
			// Don't let the completion of this task inline any others
			// to prevent cascades of inlined tasks suffocating the dispatch of others
			Flags.bCanInlineSubsequents = false;

			Task->Run(this, Flags);
		}
		else if (Task->bForceGameThread)
		{
			// Push this onto the GT list even if we are already on the game thread.
			// This ensures other subsequent tasks being processed in the same loop
			// have a chance to be scheduled before we do any potentially time-consuming task work.
			GameThreadTaskList.Push(const_cast<FScheduledTask*>(Task));
			GameThreadSignal->Trigger();
		}
		else if (OptRunInlineIndex && *OptRunInlineIndex == INDEX_NONE)
		{
			const int32 TaskIndex = (Task - Tasks.GetData());
			*OptRunInlineIndex = TaskIndex;
		}
		else if (GameThreadTaskList.IsEmpty())
		{
			GameThreadTaskList.Push(const_cast<FScheduledTask*>(Task));
			GameThreadSignal->Trigger();
		}
		else
		{
			UE::Tasks::Launch(TEXT("MovieSceneTask"), [this, Task](){
				Task->Run(this, FTaskExecutionFlags());
			}, UE::Tasks::ETaskPriority::High);
		}
	}
}

void FEntitySystemScheduler::OnAllTasksFinished() const
{
	if (ThreadingModel != EEntityThreadingModel::NoThreading)
	{
		GameThreadSignal->Trigger();
		NumTasksRemaining.Sub(ThreadingModel, 1);
	}
}

bool FEntitySystemScheduler::HasAnyTasksToPropagateDownstream() const
{
	return CurrentSubsequents.SystemWidePrerequisites.CountSetBits() != 0
		|| CurrentSubsequents.ForcedSystemWidePrerequisites.CountSetBits() != 0;
}

void FEntitySystemScheduler::BeginConstruction()
{
	WriteContextBase = FEntityAllocationWriteContext(*EntityManager);
	SystemSerialIncrement = EntityManager->GetSystemSerial();

	Tasks.Reset();
	InitialTasks = FTaskBitSet();
}

void FEntitySystemScheduler::BeginSystem(uint16 NodeID)
{
	CurrentSubsequents.Reset();
	CurrentPrerequisites = AllPrerequisites.Find(NodeID);
}

void FEntitySystemScheduler::PropagatePrerequisite(uint16 ToNodeID)
{
	FTaskPrerequisiteCache& DownstreamPrerequisites = AllPrerequisites.FindOrAdd(ToNodeID);
	DownstreamPrerequisites.SystemWidePrerequisites |= CurrentSubsequents.SystemWidePrerequisites;
	DownstreamPrerequisites.ForcedSystemWidePrerequisites |= CurrentSubsequents.ForcedSystemWidePrerequisites;
}

void FEntitySystemScheduler::EndSystem(uint16 NodeID)
{

}

void FEntitySystemScheduler::EndConstruction()
{
	// SystemSerialIncrement is currently the system serial number from BeginConstruction
	// Make this a diff by subtracting the current system serial so we can increment by it each time we run our tasks
	const uint64 CurrentSerial = EntityManager->GetSystemSerial();
	if (SystemSerialIncrement <= CurrentSerial)
	{
		SystemSerialIncrement = CurrentSerial - SystemSerialIncrement;
	}
	else
	{
		SystemSerialIncrement = 0;
	}

	Tasks.Shrink();

	AllPrerequisites.Empty();
	ComponentDepedenciesByAllocation.Empty();

	CurrentSubsequents.Reset();
	CurrentPrerequisites = nullptr;

	for (int32 Index = 0; Index < Tasks.Num(); ++Index)
	{
		FScheduledTask& Task = Tasks[Index];
		Task.WaitCount.Exchange(ThreadingModel, Task.NumPrerequisites);
		Task.ChildCompleteCount.Exchange(ThreadingModel, Task.NumChildren);

		if (Task.NumPrerequisites == 0)
		{
			InitialTasks.SetBit(Index);
		}
	}

#if WITH_AUTOMATION_TESTS && (!UE_BUILD_SHIPPING && !UE_BUILD_TEST)
	if (GIsAutomationTesting)
	{
		ShuffleTasks();
	}
#endif

	UE_LOGF(LogMovieSceneECS, VeryVerbose, "Finalized building task graph:\n %ls", *ToString());
}

FString FEntitySystemScheduler::ToString() const
{
	TStringBuilder<1024> GraphString;
	GraphString += TEXT("\ndigraph FEntitySystemScheduler {\n");
	GraphString += TEXT("\trankdir=\"LR\"\n");
	GraphString += TEXT("\tcompound=true\n");
	GraphString += TEXT("\tnode [shape=record,height=.1];\n");

	TBitArray<> ParentTasks(false, Tasks.Num());
	TBitArray<> ChildTasks(false, Tasks.Num());

	for (int32 Index = 0; Index < Tasks.Num(); ++Index)
	{
		const FScheduledTask& Task = Tasks[Index];
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		const TCHAR* DebugName = *Task.DebugName;
#else
		const TCHAR* DebugName = TEXT("");
#endif
		FString SanitizedName(DebugName);
		SanitizedName.ReplaceCharInline('<', ' ');
		SanitizedName.ReplaceCharInline('>', ' ');

		if (Task.NumChildren != 0)
		{
			ParentTasks[Index] = true;
			GraphString.Appendf(TEXT("\tsubgraph cluster_%d{ \n\t label=\"Parent Task %d: %s\";\n"), Index, Index, *SanitizedName);
			GraphString.Appendf(TEXT("\t\ttask_%d[label=\"[All Children]\" style=invis];\n"), Index);

			const int32 FirstChild = Index+1;
			for (int32 ChildIndex = FirstChild; ChildIndex < FirstChild + Task.NumChildren; ++ChildIndex)
			{
				ChildTasks[ChildIndex] = true;
				GraphString.Appendf(TEXT("\t\ttask_%d;\n"), ChildIndex);
			}
			GraphString.Append(TEXT("\t}\n"));
		}
		else
		{
			GraphString.Appendf(TEXT("\ttask_%d[label=\"Task %d: %s (%d prerequisites)\"];\n"),
				Index, Index, *SanitizedName, Task.NumPrerequisites);
		}
	}

	GraphString += TEXT("\n\n");

	// Draw edges
	for (int32 Index = 0; Index < Tasks.Num(); ++Index)
	{
		const FScheduledTask& Task = Tasks[Index];

		FString Color = FLinearColor::MakeRandomColor().ToFColorSRGB().ToHex();

		if (ParentTasks[Index])
		{
			for (int32 SubsequentIndex : Task.ComputedSubsequents)
			{
				if (ParentTasks[SubsequentIndex])
				{
					GraphString.Appendf(TEXT("\ttask_%d -> task_%d [ltail=cluster_%d, lhead=cluster_%d, color=\"#%s\"];\n"), Index, SubsequentIndex, Index, SubsequentIndex, *Color);
				}
				else
				{
					GraphString.Appendf(TEXT("\ttask_%d -> task_%d [ltail=cluster_%d, color=\"#%s\"];\n"), Index, SubsequentIndex, Index, *Color);
				}
				
			}
		}
		else
		{
			for (int32 SubsequentIndex : Task.ComputedSubsequents)
			{
				if (ParentTasks[SubsequentIndex])
				{
					GraphString.Appendf(TEXT("\ttask_%d -> task_%d [lhead=cluster_%d, color=\"#%s\"];\n"), Index, SubsequentIndex, SubsequentIndex, *Color);
				}
				else
				{
					GraphString.Appendf(TEXT("\ttask_%d -> task_%d [color=\"#%s\"];\n"), Index, SubsequentIndex, *Color);
				}
			}
		}
	}
	GraphString += TEXT("}\n");
	return GraphString.ToString();
}

} // namespace UE::MovieScene