// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/SparseBitSet.h"
#include "Containers/Map.h"
#include "Containers/Array.h"
#include "Tasks/Task.h"
#include "Containers/LockFreeList.h"
#include "Containers/StringView.h"
#include "EntitySystem/RelativePtr.h"
#include "EntitySystem/IMovieSceneTaskScheduler.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/MovieSceneMaybeAtomic.h"
#include "Misc/TransactionallySafeCriticalSection.h"

namespace UE::MovieScene
{

struct FTaskExecutionFlags;
class FEntitySystemScheduler;

using FTaskBitSet = TDynamicSparseBitSet<uint32, TDynamicSparseBitSetBucketStorage<uint16, 0>>; // Buckets of 512 task bits

/**
 * Structure used for tracking task dependencies that must be propagated from system to system
 * @note: This structure is not used or required for tracking component read/write dependencies
 *        unless such tasks are explicitly passed down or consumed by systems.
 */
struct FTaskPrerequisiteCache
{
	/**
	 * Bitset that contains all tasks produced by systems that the current system depend on.
	 * Only consumed for tasks that specify bForceConsumeUpstream on construction
	 */
	FTaskBitSet SystemWidePrerequisites;

	/**
	 * Bitset that contains all tasks that the current system must depend on as mandated by any upstream system
	 */
	FTaskBitSet ForcedSystemWidePrerequisites;

	/** Reset this cache */
	void Reset()
	{
		SystemWidePrerequisites = FTaskBitSet();
		ForcedSystemWidePrerequisites = FTaskBitSet();
	}
};

struct FScheduledTaskFuncionPtr
{
	enum class EType : uint8
	{
		None,
		Unbound,
		AllocationPtr,
		AllocationItem,
		PreLockedAllocationItem
	};

	EType Assign(UnboundTaskFunctionPtr InUnboundTask)
	{
		UnboundTask = InUnboundTask;
		return EType::Unbound;
	}
	EType Assign(AllocationFunctionPtr InAllocation)
	{
		Allocation = InAllocation;
		return EType::AllocationPtr;
	}
	EType Assign(AllocationItemFunctionPtr InAllocationItem)
	{
		AllocationItem = InAllocationItem;
		return EType::AllocationItem;
	}
	EType Assign(PreLockedAllocationItemFunctionPtr InPreLockedAllocationItem)
	{
		PreLockedAllocationItem = InPreLockedAllocationItem;
		return EType::PreLockedAllocationItem;
	}

	union
	{
		UnboundTaskFunctionPtr             UnboundTask;
		AllocationFunctionPtr              Allocation;
		AllocationItemFunctionPtr          AllocationItem;
		PreLockedAllocationItemFunctionPtr PreLockedAllocationItem;
	};
};

/**
 * Task structure that contains all the information required for dispatching and running an async task that reads/writes to component data
 */
struct FScheduledTask
{
	/**
	 * Construct a new task from a write context. The write context is used as an additive offset from the base write context at when tasks are first dispatched.
	 */
	explicit FScheduledTask(FEntityAllocationWriteContext InWriteContextOffset);
	~FScheduledTask();

	/** Move constructor used when shuffling tasks */
	FScheduledTask(FScheduledTask&& InTask);

	struct FLockedComponentData
	{
		/** Relative pointer to the start of our prelocked data inside FEntitySystemScheduler::PreLockedComponentData */
		TArray<FPreLockedDataPtr> PreLockedComponentData;
		/** Allocation index within FEntityManager::EntityAllocations */
		uint16 AllocationIndex = MAX_uint16;
	};

	/**
	 * Run this task immediately and signal any subsequent tasks to run if necessary
	 */
	void Run(const FEntitySystemScheduler* Scheduler, FTaskExecutionFlags InFlags) const;

	/**
	 * Assign this task's function
	 */
	void SetFunction(TaskFunctionPtr InFunction);

	/** 32 Bytes - Bit set of all tasks that are waiting for this */
	FTaskBitSet ComputedSubsequents;
	/** 32 Bytes - Bitset of children */
	FTaskBitSet ChildTasks;

	/** 8 Bytes - Function ptr of type defined by FScheduledTask::TaskFunctionType */
	FScheduledTaskFuncionPtr TaskFunction;

	/** 16 Bytes - Context pointer potentially shared between all forked tasks of the same operation */
	TSharedPtr<ITaskContext> TaskContext = nullptr;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	/** 16 Bytes - This task's parent (or None() if it is not a child task) */
	FString DebugName;
#endif

	/** 8 Bytes - stat ID for this task */
	TStatId StatId;
	/** 8 Bytes - Write context offset for this task. Added to the current Entity Manager write context on execution. */
	FEntityAllocationWriteContext WriteContextOffset;

	/** 6 bytes - Pre-locked component data specifying the direct pointers to the data required by this task */
	FLockedComponentData LockedComponentData;

	/** 4 Bytes - the total number of tasks that must complete before this one can begin */
	int32 NumPrerequisites = 0;
	/** 4 Bytes - the number of outstanding prerequisite tasks this task is waiting on. Reset to NumPrerequisites on completion */
	mutable FEntitySystemMaybeAtomicInt32 WaitCount = 0;
	/** 4 Bytes - the number of child tasks that must be completed before this task is considered complete */
	mutable FEntitySystemMaybeAtomicInt32 ChildCompleteCount = 0;

	/** 4 Bytes - This task's parent (or None() if it is not a child task) */
	FTaskID Parent;

	uint16 NumChildren = 0;

	/** 1 Byte - The type of the TaskFunction function pointer */
	FScheduledTaskFuncionPtr::EType TaskFunctionType;

	/** 1 Byte - execution flags */
	uint8 bForceGameThread : 1;

	/** When true, this task will be forcibly run inline as soon as it is able. Generally used for parent tasks that don't do any meaningful work but schedule their children. */
	uint8 bForceInline : 1;
};

class FEntitySystemScheduler : public IEntitySystemScheduler
{
public:

	/**
	 * Construction from an entity manager pointer that must outlive the instance of this class
	 */
	explicit FEntitySystemScheduler(FEntityManager* InEntityManager);

	~FEntitySystemScheduler();

	/**
	 * Check whether custom task scheduling is enabled based on the state of the Sequencer.CustomTaskScheduling console variable
	 */
	static bool IsCustomSchedulingEnabled();

public:

	/**
	 * Reset this task scheduler to its default state ready to start rebuilding its task graph
	 */
	void BeginConstruction();

	/**
	 * Begin construction of tasks for a system with the specified unique Node ID. Only one system at a time is supported. Must call EndSystem with the same ID before the next system can begin.
	 */
	void BeginSystem(uint16 NodeID);

	/**
	 * Check whether the currently open node has produced any tasks that need to be propagated to downstream system dependencies
	 */
	bool HasAnyTasksToPropagateDownstream() const;

	/**
	 * Propagate the outputs of this system to the specified downstream node as prerequisites for (optional) consumption when it is built.
	 * Normally systems do not depend on the outputs of other systems other than strict read/write ordering on components, unless there is a strict system dependency
	 */
	void PropagatePrerequisite(uint16 ToNodeID);

	/**
	 * Begin construction of tasks for a system with the specified unique Node ID
	 */
	void EndSystem(uint16 NodeID);

	/**
	 * Complete construction of the task graph and clean up transient data
	 */
	void EndConstruction();

public:
	/*~ Builder functionality */

	/**
	 * Add a new task of the specified type for the currently open node ID
	 *
	 * Example usage:
	 * TaskScheduler->AddTask<FMyTaskType>(FTaskParams(GET_STAT_ID(StatId)));
	 * TaskScheduler->AddTask<FMyTaskType2>(FTaskParams(GET_STAT_ID(StatId)), ConstructorArg1, ConstructorArg2);
	 */
	template<typename TaskType, typename ...TaskArgTypes>
	FTaskID AddTask(const FTaskParams& InParams, TaskArgTypes&&... Args)
	{
		struct FExecute
		{
			static void Execute(const ITaskContext* Context, FEntityAllocationWriteContext WriteContext)
			{
				static_cast<const TaskType*>(Context)->Run(WriteContext);
			}
		};

		TaskFunctionPtr Function(TInPlaceType<UnboundTaskFunctionPtr>(), FExecute::Execute);
		return AddTask(InParams, MakeShared<TaskType>(Forward<TaskArgTypes>(Args)...), Function);
	}

	/**
	 * Add a 'null' task that can be used to join many tasks into a single dependency
	 */
	FTaskID AddNullTask();

	/**
	 * Add an anonymous unbound task for doing non-ecs work
	 */
	FTaskID AddTask(const FTaskParams& InParams, TSharedPtr<ITaskContext> InTaskContext, TaskFunctionPtr InTaskFunction);

	/**
	 * Create one task for each of the entity allocations that match the specified filter
	 */
	FTaskID CreateForkedAllocationTask(const FTaskParams& InParams, TSharedPtr<ITaskContext> InTaskContext, TaskFunctionPtr InTaskFunction, TFunctionRef<void(FEntityAllocationIteratorItem,TArray<FPreLockedDataPtr>&)> InPreLockFunc, const FEntityComponentFilter& Filter, const FComponentMask& ReadDeps, const FComponentMask& WriteDeps);

	/**
	 * Define a prerequisite for the given task 
	 */
	void AddPrerequisite(FTaskID Prerequisite, FTaskID Subsequent);

	/**
	 * Add a child to the front of a previously created 'forked' task. Used for defining 'PreTask' work
	 */
	void AddChildBack(FTaskID Parent, FTaskID Child);

	/**
	 * Add a child to the back of a previously created 'forked' task. Used for defining 'PostTask' work
	 */
	void AddChildFront(FTaskID Parent, FTaskID Child);

	/**
	 * (Debug only) Shuffle the task buffer to test determinism regardless of construction order
	 */
	void ShuffleTasks();

	/*~ End Builder functionality */

public:

	/*~ Execution functionality */

	const FEntityManager* GetEntityManager() const
	{
		return EntityManager;
	}

	FEntityAllocationWriteContext GetWriteContextOffset() const
	{
		return WriteContextBase;
	}

	/**
	 * Execute all tasks as soon as possible, waiting for the result
	 */
	void ExecuteTasks();

	/**
	 * Called when a task has been completed
	 */
	void CompleteTask(const FScheduledTask* Task, FTaskExecutionFlags InFlags) const;

	/**
	 * Called when a task that was defined as a prerequisite for TaskID has been completed.
	 * If this was the last remaining prerequisite, the task will be scheduled
	 *
	 * @param TaskID               The subsequent task that may be ready to be scheduled
	 * @param OptRunInlineIndex    (Optional) When non-null, can be assigned a new task to run inline after this function call to avoid scheduling overhead
	 */
	void PrerequisiteCompleted(FTaskID TaskID, int32* OptRunInlineIndex) const;
	void PrerequisiteCompleted(const FScheduledTask* Task, int32* OptRunInlineIndex) const;

	/**
	 * Called when all tasks have been completed
	 */
	void OnAllTasksFinished() const;

	/*~ End execution functionality */

public:

	FString ToString() const;

private:
	/** Array of task data. Constant once EndConstruction has been called */
	TArray<FScheduledTask> Tasks;

	/** Pointer to the current node's system prerequisites if any. Only valid during construction. */
	FTaskPrerequisiteCache* CurrentPrerequisites = nullptr;

	/** Cache of the current node's task outputs. Only valid during construction. */
	FTaskPrerequisiteCache CurrentSubsequents;

	/** Sparse bit set of all the tasks that have no prerequisites. Only valid after EndConstruction has been called. */
	FTaskBitSet InitialTasks;

	struct FComponentDependencies
	{
		FTaskBitSet ReadTasks;
		FTaskID WriteTask;
	};
	/** Map that defines tasks that read from/write to specific components on specific allocations. */
	TMap<TPair<int32, FComponentTypeID>, FComponentDependencies> ComponentDepedenciesByAllocation;

	TMap<uint16, FTaskPrerequisiteCache> AllPrerequisites;

	FEntityManager* EntityManager;

	mutable FEntitySystemMaybeAtomicInt32 NumTasksRemaining = 0;

	FEvent* GameThreadSignal = nullptr;
	mutable TLockFreePointerListFIFO<FScheduledTask, PLATFORM_CACHE_LINE_SIZE> GameThreadTaskList;
	FEntityAllocationWriteContext WriteContextBase = FEntityAllocationWriteContext::NewAllocation();
	uint32 SystemSerialIncrement = 0;
	EEntityThreadingModel ThreadingModel = EEntityThreadingModel::NoThreading;
};


} // namespace UE::MovieScene