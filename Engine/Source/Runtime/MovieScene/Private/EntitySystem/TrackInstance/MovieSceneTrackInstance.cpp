// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/TrackInstance/MovieSceneTrackInstance.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedCaptureSource.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedCaptureSources.inl"
#include "Algo/Sort.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneTrackInstance)

void UMovieSceneTrackInstance::Initialize(UObject* InAnimatedObject, UMovieSceneEntitySystemLinker* InLinker)
{
	// We make the difference between a root track instance, and a bound track instance that lost its binding,
	// by remembering if this track instance was initialized with a valid bound object.
	WeakAnimatedObject = InAnimatedObject;
	bIsRootTrackInstance = (InAnimatedObject == nullptr);

	PrivateLinker = InLinker;

	OnInitialize();
}

void UMovieSceneTrackInstance::Animate()
{
	OnAnimate();
}

void UMovieSceneTrackInstance::Destroy()
{
	using namespace UE::MovieScene;

	if (bIsRootTrackInstance || !UE::MovieScene::FBuiltInComponentTypes::IsBoundObjectGarbage(WeakAnimatedObject.Get()))
	{
		OnDestroyed();
	}

	FPreAnimatedTrackInstanceInputCaptureSources* InputMetaData = PrivateLinker->PreAnimatedState.GetTrackInstanceInputMetaData();
	if (InputMetaData)
	{
		for (const FMovieSceneTrackInstanceInput& Input : Inputs)
		{
			InputMetaData->StopTrackingCaptureSource(Input);
		}
	}

	FPreAnimatedTrackInstanceCaptureSources* TrackInstanceMetaData = PrivateLinker->PreAnimatedState.GetTrackInstanceMetaData();
	if (TrackInstanceMetaData)
	{
		TrackInstanceMetaData->StopTrackingCaptureSource(this);
	}
}

void UMovieSceneTrackInstance::UpdateInputs(TArray<FMovieSceneTrackInstanceInput>&& InNewInputs)
{
	using namespace UE::MovieScene;

	Algo::Sort(InNewInputs);

	// Fast path if they are the same - this includes checking whether bInputHasBeenProcessed is the
	// same on all pre/post inputs, which technically should never happen for us to have gotten this far
	if (Inputs == InNewInputs)
	{
		// We still call OnBegin/EndUpdateInputs since some of them must have been invalidated for us to get this far
		OnBeginUpdateInputs();
		OnEndUpdateInputs();
		return;
	}

	FPreAnimatedTrackInstanceInputCaptureSources* InputMetaData = PrivateLinker->PreAnimatedState.GetTrackInstanceInputMetaData();

	// We know they are different in some way - now 
	OnBeginUpdateInputs();

	int32 OldIndex = 0;
	int32 NewIndex = 0;

	const int32 OldNum = Inputs.Num();
	const int32 NewNum = InNewInputs.Num();

	TArray<int32> StopTrackingIndices;

	for ( ; OldIndex < OldNum || NewIndex < NewNum; )
	{
		while (OldIndex < OldNum && NewIndex < NewNum && Inputs[OldIndex].IsSameInput(InNewInputs[NewIndex]))
		{
			if (!InNewInputs[NewIndex].bInputHasBeenProcessed)
			{
				// This input is the same as one already existing in the track instance
				// But it is being reimported - remove it and add it again
				OnInputRemoved(Inputs[OldIndex]);
				// We don't delay ending tracking for this input, because it will most likely get tracked
				// in the call to OnInputAdded, which won't do anything if it's already tracked. And then
				// later in the delayed end-tracking we would remove it. We don't want that.
				if (InputMetaData)
				{
					InputMetaData->StopTrackingCaptureSource(Inputs[OldIndex]);
				}

				FScopedPreAnimatedCaptureSource CaptureSource(PrivateLinker, InNewInputs[NewIndex]);

				InNewInputs[NewIndex].bInputHasBeenProcessed = true;
				OnInputAdded(InNewInputs[NewIndex]);
			}

			++OldIndex;
			++NewIndex;
		}

		if (OldIndex >= OldNum && NewIndex >= NewNum)
		{
			break;
		}
		else if (OldIndex < OldNum && NewIndex < NewNum)
		{
			if (Inputs[OldIndex].Section < InNewInputs[NewIndex].Section)
			{
				// Out with the old
				OnInputRemoved(Inputs[OldIndex]);
				StopTrackingIndices.Add(OldIndex);
				++OldIndex;
			}
			else
			{
				// and in with the new
				FScopedPreAnimatedCaptureSource CaptureSource(PrivateLinker, InNewInputs[NewIndex]);

				InNewInputs[NewIndex].bInputHasBeenProcessed = true;
				OnInputAdded(InNewInputs[NewIndex]);
				++NewIndex;
			}
		}
		else if (OldIndex < OldNum)
		{
			// Out with the old
			OnInputRemoved(Inputs[OldIndex]);
			StopTrackingIndices.Add(OldIndex);
			++OldIndex;
		}
		else if (ensure(NewIndex < NewNum))
		{
			// and in with the new
			FScopedPreAnimatedCaptureSource CaptureSource(PrivateLinker, InNewInputs[NewIndex]);

			InNewInputs[NewIndex].bInputHasBeenProcessed = true;
			OnInputAdded(InNewInputs[NewIndex]);
			++NewIndex;
		}
	}

	if (InputMetaData)
	{
		for (int32 Index : StopTrackingIndices)
		{
			InputMetaData->StopTrackingCaptureSource(Inputs[Index]);
		}
	}

	Swap(Inputs, InNewInputs);

	OnEndUpdateInputs();
}

UWorld* UMovieSceneTrackInstance::GetWorld() const
{
	UObject* AnimatedObject = WeakAnimatedObject.Get();
	return AnimatedObject ? AnimatedObject->GetWorld() : Super::GetWorld();
}

