// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/TrackInstance/MovieSceneTrackInstanceSystem.h"
#include "EntitySystem/TrackInstance/MovieSceneTrackInstance.h"
#include "EntitySystem/MovieSceneBoundObjectInstantiator.h"
#include "EntitySystem/MovieSceneBoundSceneComponentInstantiator.h"
#include "EntitySystem/MovieSceneRootInstantiatorSystem.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneInstanceRegistry.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntityFactoryTemplates.h"
#include "MovieSceneSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneTrackInstanceSystem)

DECLARE_CYCLE_STAT(TEXT("Generic Track Instances"), MovieSceneEval_GenericTrackInstances, STATGROUP_MovieSceneECS);
DECLARE_CYCLE_STAT(TEXT("Generic Track Instances Task"), MovieSceneEval_GenericTrackInstanceTask, STATGROUP_MovieSceneECS);

namespace UE
{
namespace MovieScene
{

struct FTrackInstanceInputComponentInitializer : TChildEntityInitializer<FMovieSceneTrackInstanceComponent, FTrackInstanceInputComponent>
{
	UMovieSceneTrackInstanceInstantiator* Instantiator;

	explicit FTrackInstanceInputComponentInitializer(UMovieSceneTrackInstanceInstantiator* InInstantiator)
		: TChildEntityInitializer<FMovieSceneTrackInstanceComponent, FTrackInstanceInputComponent>(FBuiltInComponentTypes::Get()->TrackInstance, FBuiltInComponentTypes::Get()->TrackInstanceInput)
		, Instantiator(InInstantiator)
	{}

	virtual void Run(const FEntityRange& ChildRange, const FEntityAllocation* ParentAllocation, TArrayView<const int32> ParentAllocationOffsets) override
	{
		check(ParentAllocationOffsets.Num() == ChildRange.Num);

		FEntityAllocationWriteContext FreshWriteContext = FEntityAllocationWriteContext::NewAllocation();

		TComponentReader<FMovieSceneTrackInstanceComponent> TrackInstances  = ParentAllocation->ReadComponents(GetParentComponent());
		TComponentWriter<FTrackInstanceInputComponent>      Inputs          = ChildRange.Allocation->WriteComponents(GetChildComponent(), FreshWriteContext);
		TOptionalComponentReader<UObject*>                  BoundObjects    = ChildRange.Allocation->TryReadComponents(FBuiltInComponentTypes::Get()->BoundObject);

		if (BoundObjects)
		{
			for (int32 Index = 0; Index < ChildRange.Num; ++Index)
			{
				const int32 ParentIndex = ParentAllocationOffsets[Index];
				const int32 ChildIndex  = ChildRange.ComponentStartOffset + Index;

				// Initialize the output index
				Inputs[ChildIndex].OutputIndex = Instantiator->MakeOutput(BoundObjects[ChildIndex], TrackInstances[ParentIndex].TrackInstanceClass);
			}
		}
		else for (int32 Index = 0; Index < ChildRange.Num; ++Index)
		{
			const int32 ParentIndex = ParentAllocationOffsets[Index];
			const int32 ChildIndex  = ChildRange.ComponentStartOffset + Index;

			// Initialize the output index
			Inputs[ChildIndex].OutputIndex = Instantiator->MakeOutput(nullptr, TrackInstances[ParentIndex].TrackInstanceClass);
		}
	}
};


} // MovieScene
} // UE


UMovieSceneTrackInstanceInstantiator::UMovieSceneTrackInstanceInstantiator(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UMovieSceneRootInstantiatorSystem::StaticClass(), GetClass());
		DefineComponentConsumer(GetClass(), FBuiltInComponentTypes::Get()->BoundObject);
	}
}

void UMovieSceneTrackInstanceInstantiator::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (!Ar.IsLoading())
	{
		for (FMovieSceneTrackInstanceEntry& Entry : TrackInstances)
		{
			FMovieSceneTrackInstanceEntry::StaticStruct()->SerializeItem(Ar, &Entry, nullptr);
		}

		Ar << BoundObjectToInstances;
	}
}

void UMovieSceneTrackInstanceInstantiator::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UMovieSceneTrackInstanceInstantiator* This = CastChecked<UMovieSceneTrackInstanceInstantiator>(InThis);

	for (FMovieSceneTrackInstanceEntry& Entry : This->TrackInstances)
	{
		Collector.AddReferencedObject(Entry.BoundObject, This);
		Collector.AddReferencedObject(Entry.TrackInstance, This);
	}

	for (auto& Pair : This->BoundObjectToInstances)
	{
		Collector.AddReferencedObject(Pair.Key, This);
	}
}

int32 UMovieSceneTrackInstanceInstantiator::MakeOutput(UObject* BoundObject, UClass* TrackInstanceClass)
{
	for (auto It = BoundObjectToInstances.CreateConstKeyIterator(BoundObject); It; ++It)
	{
		const int32 AnimatorIndex = It.Value();
		if (TrackInstances[AnimatorIndex].TrackInstance->GetClass() == TrackInstanceClass)
		{
			InvalidatedOutputs.PadToNum(AnimatorIndex + 1, false);
			InvalidatedOutputs[AnimatorIndex] = true;
			return AnimatorIndex;
		}
	}

	FMovieSceneTrackInstanceEntry NewEntry;
	NewEntry.BoundObject   = BoundObject;
	NewEntry.TrackInstance = NewObject<UMovieSceneTrackInstance>(this, TrackInstanceClass);
	NewEntry.TrackInstance->Initialize(BoundObject, Linker);

	const int32 NewAnimatorIndex = TrackInstances.Add(NewEntry);
	BoundObjectToInstances.Add(BoundObject, NewAnimatorIndex);
	InvalidatedOutputs.PadToNum(NewAnimatorIndex + 1, false);
	InvalidatedOutputs[NewAnimatorIndex] = true;
	return NewAnimatorIndex;
}

void UMovieSceneTrackInstanceInstantiator::OnLink()
{
	using namespace UE::MovieScene;

	ChildInitializerIndex = Linker->EntityManager.DefineInstancedChildInitializer(FTrackInstanceInputComponentInitializer(this));
}

void UMovieSceneTrackInstanceInstantiator::OnUnlink()
{
	Linker->EntityManager.DestroyInstancedChildInitializer(ChildInitializerIndex);
}

void UMovieSceneTrackInstanceInstantiator::OnTagGarbage()
{
	using namespace UE::MovieScene;

	TArray<FMovieSceneEntityID> Garbage;

	auto FindGarbage = [this, &Garbage](FMovieSceneEntityID EntityID, FTrackInstanceInputComponent InputComponent)
	{
		if (FBuiltInComponentTypes::IsBoundObjectGarbage(InputComponent.Section))
		{
			Garbage.Add(EntityID);
			this->InvalidatedOutputs.PadToNum(InputComponent.OutputIndex + 1, false);
			this->InvalidatedOutputs[InputComponent.OutputIndex] = true;
		}
	};

	FEntityTaskBuilder()
	.ReadEntityIDs()
	.Read(FBuiltInComponentTypes::Get()->TrackInstanceInput)
	.Iterate_PerEntity(&Linker->EntityManager, FindGarbage);

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	for (FMovieSceneEntityID ID : Garbage)
	{
		Linker->EntityManager.AddComponent(ID, BuiltInComponents->Tags.NeedsUnlink, EEntityRecursion::Full);
	}
}

void UMovieSceneTrackInstanceInstantiator::OnCleanTaggedGarbage()
{
	using namespace UE::MovieScene;

	// Garbage has been tagged with NeedsUnlink, so visit those and remove any track instances that are
	// associated with them. Note that this is different garbage compared to what we tagged in OnTagGarbage.
	// Here, we remove things that were tagged as garbage by *other* systems, such as GC'ed bound objects.
	auto DoCleanTaggedGarbage = [this](FMovieSceneEntityID EntityID, FTrackInstanceInputComponent InputComponent)
	{
		this->InvalidatedOutputs.PadToNum(InputComponent.OutputIndex + 1, false);
		this->InvalidatedOutputs[InputComponent.OutputIndex] = true;
	};

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	FEntityTaskBuilder()
	.ReadEntityIDs()
	.Read(FBuiltInComponentTypes::Get()->TrackInstanceInput)
	.FilterAny({ BuiltInComponents->Tags.NeedsUnlink })
	.Iterate_PerEntity(&Linker->EntityManager, DoCleanTaggedGarbage);
}

void UMovieSceneTrackInstanceInstantiator::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	auto InvalidateOutputs = [this](FTrackInstanceInputComponent InputComponent)
	{
		this->InvalidatedOutputs.PadToNum(InputComponent.OutputIndex + 1, false);
		this->InvalidatedOutputs[InputComponent.OutputIndex] = true;
	};
	FEntityTaskBuilder().Read(BuiltInComponents->TrackInstanceInput).FilterAny({ BuiltInComponents->Tags.NeedsUnlink, BuiltInComponents->Tags.NeedsLink }).Iterate_PerEntity(&Linker->EntityManager, InvalidateOutputs);

	// If we've nothing else to do, don't do anything else...
	if (InvalidatedOutputs.Find(true) == INDEX_NONE)
	{
		return;
	}

	// Gather all the inputs for any invalidated output indices
	TSortedMap<int32, TArray<FMovieSceneTrackInstanceInput> > NewInputs;
	{
		auto ReLinkInputs = [this, &NewInputs, BuiltInComponents](FEntityAllocationIteratorItem Item, const FInstanceHandle* SourceInstances, const FTrackInstanceInputComponent* InputComponents)
		{
			const int32 Num = Item.GetAllocation()->Num();

			// If the input does not have the NeedsLink tag, it has already been processed, so doesn't need removing and re-adding
			const bool bInputHasBeenProcessed = !Item.GetAllocationType().Contains(BuiltInComponents->Tags.NeedsLink);
			
			for (int32 Index = 0; Index < Num; ++Index)
			{
				const int32 OutputIndex = InputComponents[Index].OutputIndex;

				FMovieSceneTrackInstanceInput NewInput{ InputComponents[Index].Section, SourceInstances[Index], bInputHasBeenProcessed };
				if (this->InvalidatedOutputs.IsValidIndex(OutputIndex) && this->InvalidatedOutputs[OutputIndex] == true)
				{
					NewInputs.FindOrAdd(OutputIndex).Add(NewInput);
				}
			}
			
		};
		FEntityTaskBuilder()
		.Read(BuiltInComponents->InstanceHandle)
		.Read(BuiltInComponents->TrackInstanceInput)
		.FilterNone({ BuiltInComponents->Tags.NeedsUnlink })
		.Iterate_PerAllocation(&Linker->EntityManager, ReLinkInputs);
	}

	// Update the inputs for each of the invalidated indices
	for (TTuple< int32, TArray<FMovieSceneTrackInstanceInput> >& NewInput : NewInputs)
	{
		const int32 ThisIndex = NewInput.Key;

		// Clear the bit so it doesn't get destroyed in the next loop
		InvalidatedOutputs[ThisIndex] = false;

		// This check should never fire - the inputs should always have been populated above if the output index was added
		check(NewInput.Value.Num() != 0);

		TrackInstances[ThisIndex].TrackInstance->UpdateInputs(MoveTemp(NewInput.Value));
	}

	for (TConstSetBitIterator<> SetBits(InvalidatedOutputs); SetBits; ++SetBits)
	{
		// Destroy any outputs before this output that were invalidated and now have no inputs
		const int32 DestroyIndex = SetBits.GetIndex();

		FMovieSceneTrackInstanceEntry& Entry = TrackInstances[DestroyIndex];
		Entry.TrackInstance->Destroy();

		// Remove the entry from our LUTs
		BoundObjectToInstances.Remove(Entry.BoundObject, DestroyIndex);
		TrackInstances.RemoveAt(DestroyIndex);
	}

	InvalidatedOutputs.Reset();
}

UMovieSceneTrackInstanceSystem::UMovieSceneTrackInstanceSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	Phase = UE::MovieScene::ESystemPhase::Scheduling;
	RelevantComponent = UE::MovieScene::FBuiltInComponentTypes::Get()->TrackInstance;
}

void UMovieSceneTrackInstanceSystem::OnLink()
{
	Instantiator = Linker->LinkSystem<UMovieSceneTrackInstanceInstantiator>();
	Linker->SystemGraph.AddReference(this, Instantiator);
}

void UMovieSceneTrackInstanceSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;
	if (this->Instantiator->GetTrackInstances().Num() != 0)
	{
		TaskScheduler->AddMemberFunctionTask(FTaskParams(TEXT("Evaluate Track Instances")).ForceGameThread(), this, &UMovieSceneTrackInstanceSystem::EvaluateAllInstances);
	}
}

void UMovieSceneTrackInstanceSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	SCOPE_CYCLE_COUNTER(MovieSceneEval_GenericTrackInstances)

	if (this->Instantiator->GetTrackInstances().Num() != 0)
	{
		if (Linker->EntityManager.GetThreadingModel() == EEntityThreadingModel::NoThreading)
		{
			this->EvaluateAllInstances();
		}
		else
		{
			FGraphEventRef Task = FFunctionGraphTask::CreateAndDispatchWhenReady([this]{ this->EvaluateAllInstances(); }, GET_STATID(MovieSceneEval_GenericTrackInstanceTask), InPrerequisites.All(), Linker->EntityManager.GetGatherThread());
			Subsequents.AddRootTask(Task);
		}
	}
}

void UMovieSceneTrackInstanceSystem::EvaluateAllInstances()
{
	for (const FMovieSceneTrackInstanceEntry& Entry : this->Instantiator->GetTrackInstances())
	{
		if (ensure(Entry.TrackInstance))
		{
			Entry.TrackInstance->Animate();
		}
	}
}
