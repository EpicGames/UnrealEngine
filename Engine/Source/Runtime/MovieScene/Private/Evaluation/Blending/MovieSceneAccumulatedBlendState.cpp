// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/Blending/MovieSceneAccumulatedBlendState.h"
#include "IMovieScenePlayer.h"

void FMovieSceneAccumulatedBlendState::Consolidate(TMap<FMovieSceneBlendingKey, FActuatorTokenStackPtr>& InOutBlendState, FMovieSceneEvaluationOperand InOperand, IMovieScenePlayer& Player)
{
	if (TokensToBlend.Num() == 0)
	{
		return;
	}

	if (InOperand.ObjectBindingID.IsValid())
	{
		for (TWeakObjectPtr<> WeakObj : Player.FindBoundObjects(InOperand))
		{
			if (UObject* Obj = WeakObj.Get())
			{
				for (TInlineValue<FTokenEntry, 128>& Token : TokensToBlend)
				{
					FMovieSceneBlendingKey Key = { Obj, Token->GetActuatorID() };

					Token->Consolidate(InOutBlendState.FindOrAdd(Key));
				}
			}
		}
	}
	else
	{
		// Explicit nullptr means root tracks
		Consolidate(InOutBlendState);
	}
}

void FMovieSceneAccumulatedBlendState::Consolidate(TMap<FMovieSceneBlendingKey, FActuatorTokenStackPtr>& InOutBlendState)
{
	for (TInlineValue<FTokenEntry, 128>& Token : TokensToBlend)
	{
		FMovieSceneBlendingKey Key = { nullptr, Token->GetActuatorID() };

		Token->Consolidate(InOutBlendState.FindOrAdd(Key));
	}
}
