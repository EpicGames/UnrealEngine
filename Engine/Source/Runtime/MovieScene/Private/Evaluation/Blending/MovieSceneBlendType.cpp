// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/Blending/MovieSceneBlendType.h"
#include "Misc/EnumClassFlags.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneBlendType)

namespace UE
{
namespace MovieScene
{
	ENUM_CLASS_FLAGS(EMovieSceneBlendType)
}
}

FMovieSceneBlendTypeField::FMovieSceneBlendTypeField()
	: BlendTypeField((EMovieSceneBlendType)0)
{
}

FMovieSceneBlendTypeField FMovieSceneBlendTypeField::All()
{
	FMovieSceneBlendTypeField New;
	New.Add(EMovieSceneBlendType::Absolute, EMovieSceneBlendType::Additive, EMovieSceneBlendType::Relative,
			EMovieSceneBlendType::AdditiveFromBase, EMovieSceneBlendType::Override);
	return New;
}

FMovieSceneBlendTypeField FMovieSceneBlendTypeField::None()
{
	FMovieSceneBlendTypeField New;
	return New;
}

void FMovieSceneBlendTypeField::Add(EMovieSceneBlendType Type)
{
	using namespace UE::MovieScene;
	BlendTypeField |= Type;
}

void FMovieSceneBlendTypeField::Add(FMovieSceneBlendTypeField Field)
{
	using namespace UE::MovieScene;
	BlendTypeField |= Field.BlendTypeField;
}

void FMovieSceneBlendTypeField::Remove(EMovieSceneBlendType Type)
{
	using namespace UE::MovieScene;
	BlendTypeField |= Type;
}

void FMovieSceneBlendTypeField::Remove(FMovieSceneBlendTypeField Field)
{
	using namespace UE::MovieScene;
	BlendTypeField &= ~Field.BlendTypeField;
}

FMovieSceneBlendTypeField FMovieSceneBlendTypeField::Invert() const
{
	using namespace UE::MovieScene;
	return FMovieSceneBlendTypeField(~BlendTypeField);
}

bool FMovieSceneBlendTypeField::Contains(EMovieSceneBlendType InBlendType) const
{
	return EnumHasAnyFlags(BlendTypeField, InBlendType);
}

int32 FMovieSceneBlendTypeField::Num() const
{
	return
		(Contains(EMovieSceneBlendType::Absolute) ? 1 : 0) +
		(Contains(EMovieSceneBlendType::Relative) ? 1 : 0) +
		(Contains(EMovieSceneBlendType::Additive) ? 1 : 0) +
		(Contains(EMovieSceneBlendType::AdditiveFromBase) ? 1 : 0) +
		(Contains(EMovieSceneBlendType::Override) ? 1 : 0);
}


void FMovieSceneBlendTypeFieldIterator::IterateToNext()
{
	do
	{
		++Offset;
	} while (*this && !Field.Contains(EMovieSceneBlendType(1 << Offset)));
}

EMovieSceneBlendType FMovieSceneBlendTypeFieldIterator::operator*()
{
	return EMovieSceneBlendType(1 << Offset);
}

FMovieSceneBlendTypeFieldIterator begin(const FMovieSceneBlendTypeField& InField)
{
	FMovieSceneBlendTypeFieldIterator It;
	It.Field = InField;
	It.Offset = -1;
	It.IterateToNext();
	return It;
}

FMovieSceneBlendTypeFieldIterator end(const FMovieSceneBlendTypeField& InField)
{
	FMovieSceneBlendTypeFieldIterator It;
	It.Field = InField;
	It.Offset = FMovieSceneBlendTypeFieldIterator::MaxValidOffset() + 1;
	return It;
}

