// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/Blending/MovieSceneBlendingAccumulator.h"
#include "Compilation/MovieSceneTemplateInterrogation.h"

FMovieSceneAnimTypeID GetInitialValueTypeID()
{
	static FMovieSceneAnimTypeID ID = FMovieSceneAnimTypeID::Unique();
	return ID;
}


void FMovieSceneBlendingAccumulator::Apply(const FMovieSceneContext& Context, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player)
{
	TMap<FMovieSceneBlendingKey, FActuatorTokenStackPtr> BlendState;
	BlendState.Reserve(128);

	UnboundBlendState.Consolidate(BlendState, FMovieSceneEvaluationOperand(), Player);
	for (TTuple<FMovieSceneEvaluationOperand, FMovieSceneAccumulatedBlendState>& Pair : OperandToBlendState)
	{
		Pair.Value.Consolidate(BlendState, Pair.Key, Player);
	}

	for (TTuple<FMovieSceneBlendingKey, FActuatorTokenStackPtr>& Pair : BlendState)
	{
		IBlendableTokenStack* TokenStack = Pair.Value.GetPtr();

		UObject* Object = Pair.Key.ObjectPtr;
		FMovieSceneBlendingActuatorID ActuatorType = Pair.Key.ActuatorType;

		TokenStack->ComputeAndActuate(Object, *this, ActuatorType, Context, PersistentData, Player);
	}

	UnboundBlendState.Reset();
	for (TTuple<FMovieSceneEvaluationOperand, FMovieSceneAccumulatedBlendState>& Pair : OperandToBlendState)
	{
		Pair.Value.Reset();
	}
}

void FMovieSceneBlendingAccumulator::Interrogate(const FMovieSceneContext& Context, FMovieSceneInterrogationData& InterrogationData, UObject* AnimatedObject)
{
	//@todo: interrogation currently does not work on entire sequences, so we just accumulate into a single map.
	TMap<FMovieSceneBlendingKey, FActuatorTokenStackPtr> BlendState;

	UnboundBlendState.Consolidate(BlendState);
	for (TTuple<FMovieSceneEvaluationOperand, FMovieSceneAccumulatedBlendState>& Pair : OperandToBlendState)
	{
		Pair.Value.Consolidate(BlendState);
	}

	// Evaluate the token stacks
	for (TTuple<FMovieSceneBlendingKey, FActuatorTokenStackPtr>& Pair : BlendState)
	{
		IBlendableTokenStack* TokenStack = Pair.Value.GetPtr();
		FMovieSceneBlendingActuatorID ActuatorType = Pair.Key.ActuatorType;

		TokenStack->Interrogate(AnimatedObject, InterrogationData, *this, ActuatorType, Context);
	}

	UnboundBlendState.Reset();
	for (auto& Pair : OperandToBlendState)
	{
		Pair.Value.Reset();
	}
}
