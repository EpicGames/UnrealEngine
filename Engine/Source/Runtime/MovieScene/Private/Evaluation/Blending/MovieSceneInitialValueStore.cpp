// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/Blending/MovieSceneInitialValueStore.h"
#include "Evaluation/Blending/MovieSceneBlendingActuator.h"

struct FMovieSceneRemoveInitialValueToken : IMovieScenePreAnimatedToken
{
	FMovieSceneRemoveInitialValueToken(TWeakPtr<IMovieSceneBlendingActuator> InWeakActuator)
		: WeakActuator(InWeakActuator)
	{}

	virtual void RestoreState(UObject& Object, const UE::MovieScene::FRestoreStateParams& Params) override
	{
		TSharedPtr<IMovieSceneBlendingActuator> Store = WeakActuator.Pin();
		if (Store.IsValid())
		{
			Store->RemoveInitialValueForObject(FObjectKey(&Object));
		}
	}

private:
	/** The store to remove the initial value from */
	TWeakPtr<IMovieSceneBlendingActuator> WeakActuator;
};

struct FMovieSceneRemoveInitialGlobalValueToken : IMovieScenePreAnimatedGlobalToken
{
	FMovieSceneRemoveInitialGlobalValueToken(TWeakPtr<IMovieSceneBlendingActuator> InWeakActuator)
		: WeakActuator(InWeakActuator)
	{}

	virtual void RestoreState(const UE::MovieScene::FRestoreStateParams& Params) override
	{
		TSharedPtr<IMovieSceneBlendingActuator> Store = WeakActuator.Pin();
		if (Store.IsValid())
		{
			Store->RemoveInitialValueForObject(FObjectKey());
		}
	}

private:
	/** The store to remove the initial value from */
	TWeakPtr<IMovieSceneBlendingActuator> WeakActuator;
};

FMovieSceneRemoveInitialValueTokenProducer::FMovieSceneRemoveInitialValueTokenProducer(TWeakPtr<IMovieSceneBlendingActuator> InWeakActuator)
	: WeakActuator(InWeakActuator)
{
}

FMovieSceneRemoveInitialValueTokenProducer::~FMovieSceneRemoveInitialValueTokenProducer() = default;

IMovieScenePreAnimatedTokenPtr FMovieSceneRemoveInitialValueTokenProducer::CacheExistingState(UObject& Object) const
{
	return FMovieSceneRemoveInitialValueToken(WeakActuator);
}

FMovieSceneRemoveInitialGlobalValueTokenProducer::FMovieSceneRemoveInitialGlobalValueTokenProducer(TWeakPtr<IMovieSceneBlendingActuator> InWeakActuator)
	: WeakActuator(InWeakActuator)
{
}

FMovieSceneRemoveInitialGlobalValueTokenProducer::~FMovieSceneRemoveInitialGlobalValueTokenProducer() = default;

IMovieScenePreAnimatedGlobalTokenPtr FMovieSceneRemoveInitialGlobalValueTokenProducer::CacheExistingState() const
{
	return FMovieSceneRemoveInitialGlobalValueToken(WeakActuator);
}
