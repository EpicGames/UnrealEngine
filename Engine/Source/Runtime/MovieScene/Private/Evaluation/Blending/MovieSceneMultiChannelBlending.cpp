// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/Blending/MovieSceneMultiChannelBlending.h"

template<> FMovieSceneAnimTypeID GetBlendingDataType<int32>()		{ static FMovieSceneAnimTypeID TypeID = FMovieSceneAnimTypeID::Unique(); return TypeID; }
template<> FMovieSceneAnimTypeID GetBlendingDataType<float>()		{ static FMovieSceneAnimTypeID TypeID = FMovieSceneAnimTypeID::Unique(); return TypeID; }
template<> FMovieSceneAnimTypeID GetBlendingDataType<FVector2D>()	{ static FMovieSceneAnimTypeID TypeID = FMovieSceneAnimTypeID::Unique(); return TypeID; }
template<> FMovieSceneAnimTypeID GetBlendingDataType<FVector>()		{ static FMovieSceneAnimTypeID TypeID = FMovieSceneAnimTypeID::Unique(); return TypeID; }
template<> FMovieSceneAnimTypeID GetBlendingDataType<FVector4>()	{ static FMovieSceneAnimTypeID TypeID = FMovieSceneAnimTypeID::Unique(); return TypeID; }
template<> FMovieSceneAnimTypeID GetBlendingDataType<FTransform>()	{ static FMovieSceneAnimTypeID TypeID = FMovieSceneAnimTypeID::Unique(); return TypeID; }
template<> FMovieSceneAnimTypeID GetBlendingDataType<FEulerTransform>()	{ static FMovieSceneAnimTypeID TypeID = FMovieSceneAnimTypeID::Unique(); return TypeID; }
template<> FMovieSceneAnimTypeID GetBlendingDataType<FLinearColor>()   { static FMovieSceneAnimTypeID TypeId = FMovieSceneAnimTypeID::Unique(); return TypeId; }