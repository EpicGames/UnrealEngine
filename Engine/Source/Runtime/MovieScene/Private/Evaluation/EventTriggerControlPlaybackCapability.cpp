// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/EventTriggerControlPlaybackCapability.h"

namespace UE::MovieScene
{

UE_DEFINE_MOVIESCENE_PLAYBACK_CAPABILITY(FEventTriggerControlPlaybackCapability)

bool FEventTriggerControlPlaybackCapability::IsDisablingEventTriggers(FFrameTime& OutDisabledUntilTime) const
{
	if (DisableEventTriggersUntilTime.IsSet())
	{
		OutDisabledUntilTime = DisableEventTriggersUntilTime.GetValue();
		return true;
	}
	return false;
}

}  // namespace UE::MovieScene

