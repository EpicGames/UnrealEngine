// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/IMovieSceneCustomClockSource.h"
#include "UObject/Package.h"
#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IMovieSceneCustomClockSource)

FMovieSceneTimeController_Custom::FMovieSceneTimeController_Custom(const FSoftObjectPath& InObjectPath, TWeakObjectPtr<> InWeakPlaybackContext)
	: WeakPlaybackContext(InWeakPlaybackContext)
	, InterfacePtr(nullptr)
	, ObjectPath(InObjectPath)
{
#if WITH_EDITOR
	UObject* PlaybackContext = WeakPlaybackContext.Get();

	UPackage* Package = PlaybackContext ? PlaybackContext->GetOutermost() : nullptr;
	if (Package && Package->GetPIEInstanceID() != INDEX_NONE)
	{
		if (!ObjectPath.FixupForPIE(Package->GetPIEInstanceID()))
		{
			// log error?
		}
	}
#endif

	ResolveInterfacePtr();
}

void FMovieSceneTimeController_Custom::OnTick(float DeltaSeconds, float InPlayRate)
{
	if (WeakObject.IsStale())
	{
		ResolveInterfacePtr();
	}

	if (WeakObject.IsValid())
	{
		InterfacePtr->OnTick(DeltaSeconds, InPlayRate);
	}
}

void FMovieSceneTimeController_Custom::OnStartPlaying(const FQualifiedFrameTime& InStartTime)
{
	if (WeakObject.IsStale())
	{
		ResolveInterfacePtr();
	}

	if (WeakObject.IsValid())
	{
		InterfacePtr->OnStartPlaying(InStartTime);
	}
}

void FMovieSceneTimeController_Custom::OnStopPlaying(const FQualifiedFrameTime& InStopTime)
{
	if (WeakObject.IsStale())
	{
		ResolveInterfacePtr();
	}

	if (WeakObject.IsValid())
	{
		InterfacePtr->OnStopPlaying(InStopTime);
	}
}

FFrameTime FMovieSceneTimeController_Custom::OnRequestCurrentTime(const FQualifiedFrameTime& InCurrentTime, float InPlayRate)
{
	if (WeakObject.IsStale())
	{
		ResolveInterfacePtr();
	}

	if (WeakObject.IsValid())
	{
		return InterfacePtr->OnRequestCurrentTime(InCurrentTime, InPlayRate);
	}

	return InCurrentTime.Time;
}

void FMovieSceneTimeController_Custom::ResolveInterfacePtr()
{
	WeakObject = nullptr;
	InterfacePtr = nullptr;

	if (UObject* ResolvedClockSource = ObjectPath.ResolveObject())
	{
		const bool bObjectHasCompatibleInterface = ResolvedClockSource->GetClass()->ImplementsInterface(UMovieSceneCustomClockSource::StaticClass());
		if (bObjectHasCompatibleInterface)
		{
			WeakObject = ResolvedClockSource;
			InterfacePtr = static_cast<IMovieSceneCustomClockSource*>(ResolvedClockSource->GetInterfaceAddress(UMovieSceneCustomClockSource::StaticClass()));
			return;
		}

		if (AActor* AsActor = Cast<AActor>(ResolvedClockSource))
		{
			TArray<UActorComponent*> ClockSources = AsActor->GetComponentsByInterface(UMovieSceneCustomClockSource::StaticClass());
			if (ClockSources.Num() == 0)
			{
				return;
			}
			// hmmmm... warn if actor has more than one possible clock source? 
			WeakObject = ClockSources[0];
			InterfacePtr = static_cast<IMovieSceneCustomClockSource*>(ClockSources[0]->GetInterfaceAddress(UMovieSceneCustomClockSource::StaticClass()));
		}
	}
}
