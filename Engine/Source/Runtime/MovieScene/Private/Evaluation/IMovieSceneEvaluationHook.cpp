// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/IMovieSceneEvaluationHook.h"

#include "IMovieScenePlayer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IMovieSceneEvaluationHook)

void IMovieSceneEvaluationHook::Begin(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Begin(UE::MovieScene::FPlayerIndexPlaybackCapability::GetPlayer(SharedPlaybackState), Params);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void IMovieSceneEvaluationHook::Update(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Update(UE::MovieScene::FPlayerIndexPlaybackCapability::GetPlayer(SharedPlaybackState), Params);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void IMovieSceneEvaluationHook::End(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	End(UE::MovieScene::FPlayerIndexPlaybackCapability::GetPlayer(SharedPlaybackState), Params);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void IMovieSceneEvaluationHook::Trigger(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Trigger(UE::MovieScene::FPlayerIndexPlaybackCapability::GetPlayer(SharedPlaybackState), Params);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

