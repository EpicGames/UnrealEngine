// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/IMovieScenePlaybackCapability.h"
#include <limits.h>

namespace UE::MovieScene
{

FPlaybackCapabilityIDRegistry* GPlaybackCapabilityIDRegistryForDebuggingVisualizers = nullptr;

FPlaybackCapabilityID FPlaybackCapabilityID::Register(const TCHAR* InDebugName)
{
	return FPlaybackCapabilityIDRegistry::Get()->RegisterNewID(InDebugName);
}

FPlaybackCapabilityIDRegistry* FPlaybackCapabilityIDRegistry::Get()
{
	static FPlaybackCapabilityIDRegistry Instance;
	return &Instance;
}

FPlaybackCapabilityID FPlaybackCapabilityIDRegistry::RegisterNewID(const TCHAR* InDebugName)
{
	FPlaybackCapabilityIDInfo NewInfo;
#if UE_MOVIESCENE_ENTITY_DEBUG
	NewInfo.DebugName = InDebugName;
#endif
	const int32 NewID = Infos.Add(MoveTemp(NewInfo));
	// The ID is a bit index inside FPlaybackCapabilitiesImpl's AllCapabilities uint32 bitmask,
	// so we don't want to overflow this.
	checkf((size_t)NewID < sizeof(uint32) * CHAR_BIT, TEXT("Exceeded the maximum possible amount of playback capabilities!"));

	return FPlaybackCapabilityID{ NewID };
}

} // namespace UE::MovieScene

