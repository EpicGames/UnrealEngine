// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/Instances/MovieSceneTrackEvaluator.h"
#include "Containers/SortedMap.h"
#include "IMovieScenePlayer.h"
#include "MovieSceneSequence.h"
#include "Sections/MovieSceneSubSection.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedCaptureSources.h"
#include "Stats/Stats.h"

#include "IMovieSceneModule.h"
#include "Algo/Sort.h"
#include "Algo/Find.h"

DECLARE_CYCLE_STAT(TEXT("Template Evaluation Cost"), MovieSceneEval_TrackEvaluator, STATGROUP_MovieSceneEval);
DECLARE_CYCLE_STAT(TEXT("Gather Entries For Frame"), MovieSceneEval_GatherEntries, STATGROUP_MovieSceneEval);
DECLARE_CYCLE_STAT(TEXT("Call Setup() and TearDown()"), MovieSceneEval_CallSetupTearDown, STATGROUP_MovieSceneEval);
DECLARE_CYCLE_STAT(TEXT("Evaluate Group"), MovieSceneEval_EvaluateGroup, STATGROUP_MovieSceneEval);


/** Scoped helper class that facilitates the delayed restoration of preanimated state for specific evaluation keys */
struct FDelayedPreAnimatedStateRestore
{
	FDelayedPreAnimatedStateRestore(IMovieScenePlayer& InPlayer)
		: Player(InPlayer)
	{}

	~FDelayedPreAnimatedStateRestore()
	{
		RestoreNow();
	}

	void Add(FMovieSceneEvaluationKey Key)
	{
		KeysToRestore.Add(Key);
	}

	void RestoreNow()
	{
		using namespace UE::MovieScene;

		UMovieSceneEntitySystemLinker* Linker = Player.GetSharedPlaybackState()->GetLinker();
		FPreAnimatedTemplateCaptureSources* TemplateMetaData = Linker->PreAnimatedState.GetTemplateMetaData();
		if (TemplateMetaData)
		{
			const FRootInstanceHandle RootInstanceHandle = Player.GetEvaluationTemplate().GetRootInstanceHandle();
			for (FMovieSceneEvaluationKey Key : KeysToRestore)
			{
				TemplateMetaData->StopTrackingCaptureSource(Key, RootInstanceHandle);
			}
		}
		KeysToRestore.Reset();
	}

private:
	/** The movie scene player to restore with */
	IMovieScenePlayer& Player;
	/** The array of keys to restore */
	TArray<FMovieSceneEvaluationKey> KeysToRestore;
};


FMovieSceneTrackEvaluator::FMovieSceneTrackEvaluator(UMovieSceneSequence* InRootSequence, FMovieSceneCompiledDataID InRootCompiledDataID, UMovieSceneCompiledDataManager* InCompiledDataManager)
	: RootSequence(InRootSequence)
	, RootCompiledDataID(InRootCompiledDataID)
	, RootID(MovieSceneSequenceID::Root)
	, CompiledDataManager(InCompiledDataManager)
{
	CachedReallocationVersion = 0;
}

FMovieSceneTrackEvaluator::~FMovieSceneTrackEvaluator()
{
	const bool bHasFinished = (GExitPurge || ThisFrameMetaData.ActiveEntities.Num() == 0);
	if (!bHasFinished)
	{
		UE_LOGF(LogMovieSceneECS, Verbose, "Evaluator instance being torn down without calling Finish (ThisFrameMetaData has data)");
	}
}

void FMovieSceneTrackEvaluator::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(CompiledDataManager);
}

void FMovieSceneTrackEvaluator::Finish(IMovieScenePlayer& Player)
{
	Swap(ThisFrameMetaData, LastFrameMetaData);
	ThisFrameMetaData.Reset();

	ConstructEvaluationPtrCache();
	CallSetupTearDown(Player);
}

void FMovieSceneTrackEvaluator::Evaluate(FMovieSceneContext Context, IMovieScenePlayer& Player, FMovieSceneSequenceID InOverrideRootID)
{
	SCOPE_CYCLE_COUNTER(MovieSceneEval_TrackEvaluator);

	Swap(ThisFrameMetaData, LastFrameMetaData);
	ThisFrameMetaData.Reset();

	ConstructEvaluationPtrCache();

	if (RootID != InOverrideRootID)
	{
		// Tear everything down if we're evaluating a different root sequence
		CallSetupTearDown(Player);
		LastFrameMetaData.Reset();
	}

	UMovieSceneSequence* OverrideRootSequence = GetSequence(InOverrideRootID);
	if (!OverrideRootSequence)
	{
		CallSetupTearDown(Player);
		return;
	}

	SCOPE_CYCLE_UOBJECT(ContextScope, OverrideRootSequence);

	const FMovieSceneEvaluationGroup* GroupToEvaluate = SetupFrame(OverrideRootSequence, InOverrideRootID, Context);
	if (!GroupToEvaluate)
	{
		CallSetupTearDown(Player);
		return;
	}

	// Cause stale tracks to not restore until after evaluation. This fixes issues when tracks that are set to 'Restore State' are regenerated, causing the state to be restored then re-animated by the new track
	FDelayedPreAnimatedStateRestore DelayedRestore(Player);

	// Run the post root evaluate steps which invoke tear downs for anything no longer evaluated.
	// Do this now to ensure they don't undo any of the current frame's execution tokens 
	CallSetupTearDown(Player, &DelayedRestore);

	// Ensure any null objects are not cached
	if (FMovieSceneEvaluationState* State = Player.GetEvaluationState())
	{
		State->InvalidateExpiredObjects();
	}

	// Accumulate execution tokens into this structure
	EvaluateGroup(*GroupToEvaluate, Context, Player);

	// Process execution tokens
	ExecutionTokens.Apply(Context, Player);
}

void FMovieSceneTrackEvaluator::ConstructEvaluationPtrCache()
{
	// Cache all the pointers needed for this frame
	const uint32 CurrentReallocationVersion = CompiledDataManager->GetReallocationVersion();
	if (CachedPtrs.Num() == 0 || CurrentReallocationVersion > CachedReallocationVersion)
	{
		check(CompiledDataManager);

		const FMovieSceneSequenceHierarchy* RootHierarchy = CompiledDataManager->FindHierarchy(RootCompiledDataID);
		if (RootHierarchy)
		{
			// Cache all sub-sequence ptrs
			for (const TTuple<FMovieSceneSequenceID, FMovieSceneSubSequenceData>& Pair : RootHierarchy->AllSubSequenceData())
			{
				UMovieSceneSequence*                 SubSequence = Pair.Value.GetSequence();
				FMovieSceneCompiledDataID            SubDataID   = CompiledDataManager->FindDataID(SubSequence);
				const FMovieSceneEvaluationTemplate* SubTemplate = CompiledDataManager->FindTrackTemplate(SubDataID);
				if (SubTemplate)
				{
					CachedPtrs.Add(Pair.Key, FCachedPtrs{ SubSequence, SubTemplate, &Pair.Value });
				}
			}
		}

		// Find the root template from the template store
		const FMovieSceneEvaluationTemplate* RootTemplate = CompiledDataManager->FindTrackTemplate(RootCompiledDataID);
		if (RootTemplate)
		{
			CachedPtrs.Add(MovieSceneSequenceID::Root, FCachedPtrs{ RootSequence.Get(), RootTemplate, nullptr });
		}
	}
}

const FMovieSceneEvaluationGroup* FMovieSceneTrackEvaluator::SetupFrame(UMovieSceneSequence* OverrideRootSequence, FMovieSceneSequenceID InOverrideRootID, FMovieSceneContext Context)
{
	const FMovieSceneSequenceHierarchy* RootHierarchy = CompiledDataManager->FindHierarchy(RootCompiledDataID);
	check(OverrideRootSequence);

	RootID = InOverrideRootID;
	RootOverridePath.Reset(InOverrideRootID, RootHierarchy);

	const FMovieSceneEvaluationField* OverrideRootField = nullptr;
	FFrameTime RootTime = Context.GetEvaluationFieldTime();

	if (InOverrideRootID == MovieSceneSequenceID::Root)
	{
		OverrideRootField = CompiledDataManager->FindTrackTemplateField(RootCompiledDataID);
	}
	else
	{
		check(RootHierarchy);

		// Evaluate Sub Sequences in Isolation is turned on
		FMovieSceneCompiledDataID OverrideRootDataID = CompiledDataManager->FindDataID(OverrideRootSequence);
		OverrideRootField = CompiledDataManager->FindTrackTemplateField(OverrideRootDataID);
		if (const FMovieSceneSubSequenceData* OverrideSubData = RootHierarchy->FindSubData(InOverrideRootID))
		{
			RootTime *= OverrideSubData->RootToSequenceTransform;
		}
	}

	if (OverrideRootField == nullptr)
	{
		return nullptr;
	}

	// The one that we want to evaluate is either the first or last index in the range.
	// FieldRange is always of the form [First, Last+1)
	const int32 TemplateFieldIndex = OverrideRootField->GetSegmentFromTime(RootTime.FloorToFrame());
	if (TemplateFieldIndex != INDEX_NONE)
	{
		// Set meta-data
		ThisFrameMetaData = OverrideRootField->GetMetaData(TemplateFieldIndex);
		return &OverrideRootField->GetGroup(TemplateFieldIndex);
	}

	return nullptr;
}

void FMovieSceneTrackEvaluator::EvaluateGroup(const FMovieSceneEvaluationGroup& Group, const FMovieSceneContext& RootContext, IMovieScenePlayer& Player)
{
	MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_EvaluateGroup);

	FPersistentEvaluationData PersistentDataProxy(Player);

	FMovieSceneEvaluationOperand Operand;

	FMovieSceneContext Context = RootContext;
	FMovieSceneContext SubContext = Context;

	int32 TrackIndex = 0;
	const FMovieSceneFieldEntry_ChildTemplate* Templates = Group.SectionLUT.GetData();

	for (const FMovieSceneEvaluationGroupLUTIndex& Index : Group.LUTIndices)
	{
		const int32 LastInitTrack = TrackIndex + Index.NumInitPtrs;

		// Initialize anything that wants to be initialized first
		for ( ; TrackIndex < LastInitTrack; ++TrackIndex)
		{
			FMovieSceneFieldEntry_EvaluationTrack TrackEntry = Group.TrackLUT[TrackIndex];
			FMovieSceneEvaluationFieldTrackPtr    TrackPtr   = TrackEntry.TrackPtr;

			// Ensure we're able to find the sequence instance in our root if we've overridden
			TrackPtr.SequenceID = RootOverridePath.ResolveChildSequenceID(TrackPtr.SequenceID);

			const FCachedPtrs&                EvalPtrs = CachedPtrs.FindChecked(TrackPtr.SequenceID);
			const FMovieSceneEvaluationTrack* Track    = EvalPtrs.Template->FindTrack(TrackPtr.TrackIdentifier);

			if (Track)
			{
				Operand.ObjectBindingID = Track->GetObjectBindingID();
				Operand.SequenceID = TrackPtr.SequenceID;

				FMovieSceneEvaluationKey TrackKey(TrackPtr.SequenceID, TrackPtr.TrackIdentifier);

				PersistentDataProxy.SetTrackKey(TrackKey);
				FScopedPreAnimatedCaptureSource CaptureSource(&Player.PreAnimatedState, TrackKey, false);

				SubContext = Context;
				if (EvalPtrs.SubData)
				{
					SubContext = Context.Transform(EvalPtrs.SubData->RootToSequenceTransform, EvalPtrs.SubData->TickResolution);

					// Hittest against the sequence's pre and postroll ranges
					SubContext.ReportOuterSectionRanges(EvalPtrs.SubData->PreRollRange.Value, EvalPtrs.SubData->PostRollRange.Value);
					SubContext.SetHierarchicalBias(EvalPtrs.SubData->HierarchicalBias);
				}

				TArrayView<const FMovieSceneFieldEntry_ChildTemplate> ChildTemplates(Templates, TrackEntry.NumChildren);
				Track->Initialize(ChildTemplates, Operand, SubContext, PersistentDataProxy, Player);
			}

			Templates += TrackEntry.NumChildren;
		}

		// Then evaluate
		const int32 LastEvalTrack = TrackIndex + Index.NumEvalPtrs;
		for (; TrackIndex < LastEvalTrack; ++TrackIndex)
		{
			FMovieSceneFieldEntry_EvaluationTrack TrackEntry = Group.TrackLUT[TrackIndex];
			FMovieSceneEvaluationFieldTrackPtr    TrackPtr   = TrackEntry.TrackPtr;

			// Ensure we're able to find the sequence instance in our root if we've overridden
			TrackPtr.SequenceID = RootOverridePath.ResolveChildSequenceID(TrackPtr.SequenceID);

			const FCachedPtrs&                EvalPtrs = CachedPtrs.FindChecked(TrackPtr.SequenceID);
			const FMovieSceneEvaluationTrack* Track    = EvalPtrs.Template->FindTrack(TrackPtr.TrackIdentifier);

			if (Track)
			{
				Operand.ObjectBindingID = Track->GetObjectBindingID();
				Operand.SequenceID = TrackPtr.SequenceID;

				FMovieSceneEvaluationKey TrackKey(TrackPtr.SequenceID, TrackPtr.TrackIdentifier);

				PersistentDataProxy.SetTrackKey(TrackKey);

				ExecutionTokens.SetOperand(Operand);
				ExecutionTokens.SetCurrentScope(FMovieSceneEvaluationScope(TrackKey, EMovieSceneCompletionMode::KeepState));

				SubContext = Context;
				if (EvalPtrs.SubData)
				{
					SubContext = Context.Transform(EvalPtrs.SubData->RootToSequenceTransform, EvalPtrs.SubData->TickResolution);

					// Hittest against the sequence's pre and postroll ranges
					SubContext.ReportOuterSectionRanges(EvalPtrs.SubData->PreRollRange.Value, EvalPtrs.SubData->PostRollRange.Value);
					SubContext.SetHierarchicalBias(EvalPtrs.SubData->HierarchicalBias);
				}

				TArrayView<const FMovieSceneFieldEntry_ChildTemplate> ChildTemplates(Templates, TrackEntry.NumChildren);
				Track->Evaluate(
					ChildTemplates,
					Operand,
					SubContext,
					PersistentDataProxy,
					ExecutionTokens);
			}

			Templates += TrackEntry.NumChildren;
		}

		ExecutionTokens.Apply(Context, Player);
	}
}

void FMovieSceneTrackEvaluator::CallSetupTearDown(IMovieScenePlayer& Player, FDelayedPreAnimatedStateRestore* DelayedRestore)
{
	using namespace UE::MovieScene;

	MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_CallSetupTearDown);

	UMovieSceneEntitySystemLinker* Linker = Player.GetEvaluationTemplate().GetEntitySystemLinker();
	FPreAnimatedTemplateCaptureSources* TemplateMetaData = Linker->PreAnimatedState.GetTemplateMetaData();
	const FRootInstanceHandle RootInstanceHandle = Player.GetEvaluationTemplate().GetRootInstanceHandle();

	FPersistentEvaluationData PersistentDataProxy(Player);

	TArray<FMovieSceneOrderedEvaluationKey> ExpiredEntities;
	TArray<FMovieSceneOrderedEvaluationKey> NewEntities;
	ThisFrameMetaData.DiffEntities(LastFrameMetaData, &NewEntities, &ExpiredEntities);

	for (const FMovieSceneOrderedEvaluationKey& OrderedKey : ExpiredEntities)
	{
		FMovieSceneEvaluationKey Key = OrderedKey.Key;

		// Ensure we're able to find the sequence instance in our root if we've overridden
		Key.SequenceID = RootOverridePath.ResolveChildSequenceID(Key.SequenceID);

		const FCachedPtrs* EvalPtrs = CachedPtrs.Find(Key.SequenceID);
		if (EvalPtrs)
		{
			const FMovieSceneEvaluationTrack* Track = EvalPtrs->Template->FindTrack(Key.TrackIdentifier);
			const bool bStaleTrack = EvalPtrs->Template->IsTrackStale(Key.TrackIdentifier);

			// Track data key may be required by both tracks and sections
			PersistentDataProxy.SetTrackKey(Key.AsTrack());

			if (Key.SectionIndex == uint32(-1))
			{
				if (Track)
				{
					Track->OnEndEvaluation(PersistentDataProxy, Player);
				}
				
				PersistentDataProxy.ResetTrackData();
			}
			else if (Track && Track->HasChildTemplate(Key.SectionIndex))
			{
				PersistentDataProxy.SetSectionKey(Key);
				Track->GetChildTemplate(Key.SectionIndex).OnEndEvaluation(PersistentDataProxy, Player);

				PersistentDataProxy.ResetSectionData();
			}

			if (bStaleTrack && DelayedRestore)
			{
				DelayedRestore->Add(Key);
			}
			else if (TemplateMetaData)
			{
				TemplateMetaData->StopTrackingCaptureSource(Key, RootInstanceHandle);
			}
		}
		else if (TemplateMetaData)
		{
			// If the track has been destroyed since it was last evaluated, we can still restore state for anything it made
			// In particular this is needed for movie renders, where it will enable/disable shots between cuts in order
			// to render handle frames
			TemplateMetaData->StopTrackingCaptureSource(Key, RootInstanceHandle);
		}
	}

	for (const FMovieSceneOrderedEvaluationKey& OrderedKey : NewEntities)
	{
		FMovieSceneEvaluationKey Key = OrderedKey.Key;

		// Ensure we're able to find the sequence instance in our root if we've overridden
		Key.SequenceID = RootOverridePath.ResolveChildSequenceID(Key.SequenceID);

		const FCachedPtrs&                EvalPtrs = CachedPtrs.FindChecked(Key.SequenceID);
		const FMovieSceneEvaluationTrack* Track    = EvalPtrs.Template->FindTrack(Key.TrackIdentifier);

		if (Track)
		{
			PersistentDataProxy.SetTrackKey(Key.AsTrack());

			if (Key.SectionIndex == uint32(-1))
			{
				Track->OnBeginEvaluation(PersistentDataProxy, Player);
			}
			else
			{
				PersistentDataProxy.SetSectionKey(Key);
				Track->GetChildTemplate(Key.SectionIndex).OnBeginEvaluation(PersistentDataProxy, Player);
			}
		}
	}
}

void FMovieSceneTrackEvaluator::InvalidateCachedData()
{
	CachedPtrs.Empty();
	CachedReallocationVersion = 0;
}

void FMovieSceneTrackEvaluator::CopyActuators(FMovieSceneBlendingAccumulator& Accumulator) const
{
	Accumulator.Actuators = ExecutionTokens.GetBlendingAccumulator().Actuators;
}

UMovieSceneSequence* FMovieSceneTrackEvaluator::GetSequence(FMovieSceneSequenceIDRef SequenceID) const
{
	UMovieSceneSequence* RootSequencePtr = RootSequence.Get();
	if (!RootSequencePtr)
	{
		return nullptr;
	}
	else if (SequenceID == MovieSceneSequenceID::Root)
	{
		return RootSequencePtr;
	}
	
	const FMovieSceneSequenceHierarchy* RootHierarchy = CompiledDataManager->FindHierarchy(RootCompiledDataID);
	const FMovieSceneSubSequenceData*   SubData       = RootHierarchy ? RootHierarchy->FindSubData(SequenceID) : nullptr;
	return SubData ? SubData->GetSequence() : nullptr;
}
