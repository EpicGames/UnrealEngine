// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Map.h"
#include "Containers/SortedMap.h"
#include "UObject/GCObject.h"
#include "MovieSceneSequenceID.h"
#include "Compilation/MovieSceneCompiledDataID.h"
#include "Evaluation/MovieSceneExecutionTokens.h"
#include "Evaluation/MovieSceneEvaluationField.h"
#include "Evaluation/MovieSceneRootOverridePath.h"

struct FMovieSceneContext;
struct FMovieSceneSubSequenceData;
struct FMovieSceneEvaluationGroup;
struct FMovieSceneEvaluationTemplate;
struct FMovieSceneBlendingAccumulator;
struct FDelayedPreAnimatedStateRestore;

class IMovieScenePlayer;
class UMovieSceneSequence;
class UMovieSceneCompiledDataManager;

/**
 * Root evaluation template instance used to play back any sequence
 */
struct FMovieSceneTrackEvaluator : FGCObject
{
	FMovieSceneTrackEvaluator(UMovieSceneSequence* InRootSequence, FMovieSceneCompiledDataID InRootCompiledDataID, UMovieSceneCompiledDataManager* InCompiledDataManager);
	~FMovieSceneTrackEvaluator();

	/**
	 * Evaluate this sequence
	 *
	 * @param Context				Evaluation context containing the time (or range) to evaluate
	 * @param Player				The player responsible for playback
	 * @param OverrideRootID		The ID of the sequence from which to evaluate.
	 */
	void Evaluate(FMovieSceneContext Context, IMovieScenePlayer& Player, FMovieSceneSequenceID OverrideRootID = MovieSceneSequenceID::Root);

	/**
	 * Indicate that we're not going to evaluate this instance again, and that we should tear down any current state
	 *
	 * @param Player				The player responsible for playback
	 */
	void Finish(IMovieScenePlayer& Player);

	void InvalidateCachedData();

public:

	/**
	 * Attempt to locate the underlying sequence given a sequence ID
	 *
	 * @param SequenceID 			ID of the sequence to locate
	 * @return The sequence, or nullptr if the ID was not found
	 */
	MOVIESCENE_API UMovieSceneSequence* GetSequence(FMovieSceneSequenceIDRef SequenceID) const;

	/**
 	 * Cache of everything that is evaluated this frame 
	 */
	const FMovieSceneEvaluationMetaData& GetThisFrameMetaData() const
	{
		return ThisFrameMetaData;
	}

	/**
	 * Copy any actuators from this template instance into the specified accumulator
	 *
	 * @param Accumulator 			The accumulator to copy actuators into
	 */
	MOVIESCENE_API void CopyActuators(FMovieSceneBlendingAccumulator& Accumulator) const;

private:

	/**
	 * Setup the current frame by finding or generating the necessary evaluation group and meta-data
	 *
	 * @param OverrideRootSequence	Pointer to the sequence that is considered the root for this evaluation (that maps to InOverrideRootID)
	 * @param OverrideRootID		The sequence ID of the currently considered root (normally MovieSceneSequenceID::Root unless Evaluate Sub-Sequences in Isolation is active)
	 * @param Context				The evaluation context for this frame
	 * @return The evaluation group within the root template's evaluation field to evaluate, or nullptr if none could be found or compiled
	 */
	const FMovieSceneEvaluationGroup* SetupFrame(UMovieSceneSequence* OverrideRootSequence, FMovieSceneSequenceID InOverrideRootID, FMovieSceneContext Context);

	/**
	 * Process entities that are newly evaluated, and those that are no longer being evaluated
	 */
	void CallSetupTearDown(IMovieScenePlayer& Player, FDelayedPreAnimatedStateRestore* DelayedRestore = nullptr);

	/**
	 * Evaluate a particular group of a segment
	 */
	void EvaluateGroup(const FMovieSceneEvaluationGroup& Group, const FMovieSceneContext& Context, IMovieScenePlayer& Player);

	/**
	 * Construct all the template and sub-data ptrs required for this frame by combining all those needed last frame, with those needed this frame
	 */
	void ConstructEvaluationPtrCache();

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FMovieSceneTrackEvaluator");
	}

private:

	struct FCachedPtrs
	{
		/** The cached sequence ptr - always non-nullptr */
		UMovieSceneSequence*                 Sequence;
		/** The cached template ptr - always non-nullptr */
		const FMovieSceneEvaluationTemplate* Template;
		/** The cached sub data ptr from the hierarchy. Only valid for sub sequences. */
		const FMovieSceneSubSequenceData*    SubData;
	};

	/** Map from sequence ID to pointers cached from UMovieSceneCompiledDataManager so we don't have to look them up for every track. These pointers are protected by CachedReallocationVersion in the event that the cached pointers are reallocated within the manager */
	TSortedMap<FMovieSceneSequenceID, FCachedPtrs, TInlineAllocator<8>> CachedPtrs;

	TWeakObjectPtr<UMovieSceneSequence> RootSequence;

	FMovieSceneCompiledDataID RootCompiledDataID;

	/** Sequence ID that was last used to evaluate from */
	FMovieSceneSequenceID RootID;

	/** Cache of everything that was evaluated last frame */
	FMovieSceneEvaluationMetaData LastFrameMetaData;
	/** Cache of everything that is evaluated this frame */
	FMovieSceneEvaluationMetaData ThisFrameMetaData;

	/** Instance responsible for supplying and generating compiled data for a given sequence */
	TObjectPtr<UMovieSceneCompiledDataManager> CompiledDataManager;

	/** Override path that is used to remap inner sequence IDs to the root space when evaluating with a root override */
	UE::MovieScene::FSubSequencePath RootOverridePath;

	/** Execution tokens that are used to apply animated state */
	FMovieSceneExecutionTokens ExecutionTokens;

	/** The value of UMovieSceneCompiledDataManager::GetReallocationVersion when we last cached pointers from it */
	uint32 CachedReallocationVersion;
};