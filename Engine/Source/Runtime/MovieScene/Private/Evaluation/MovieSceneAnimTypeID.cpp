// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/MovieSceneAnimTypeID.h"

FString LexToString(const FMovieSceneAnimTypeID& AnimTypeID)
{
	return FString::Printf(TEXT("%#010" UINT64_x_FMT), AnimTypeID.ID);
}

uint64 FMovieSceneAnimTypeID::Initialize(uint64* StaticPtr, uint32 Seed)
{
	const uint64 NewHash = GenerateHash(StaticPtr, Seed);
	// Store the cached ID in the static itself
	*StaticPtr = NewHash;
	return NewHash;
}

uint64 FMovieSceneAnimTypeID::GenerateHash(void* StaticPtr, uint32 Seed)
{
	// The bitvalue of the address with the upper 32 bits hashed with the seed
	uint64 Address = (uint64)StaticPtr;
	return (uint64(HashCombine(Address >> 32, Seed)) << 32) | (uint64)HashCombine(Address & 0xFFFFFFFF, Seed);
}

namespace
{
	static uint64 RunningCount = 0;
	struct FUniqueTypeID : FMovieSceneAnimTypeID
	{
		UE_AUTORTFM_ALWAYS_OPEN FUniqueTypeID()
		{
			ID = GenerateHash(&RunningCount, ++RunningCount);
		}
	};
}

FMovieSceneAnimTypeID FMovieSceneAnimTypeID::Unique()
{
	return FUniqueTypeID();
}

FMovieSceneAnimTypeID FMovieSceneAnimTypeID::Combine(FMovieSceneAnimTypeID A, FMovieSceneAnimTypeID B)
{
	FMovieSceneAnimTypeID Combined;
	// Combine the two IDs by hashing both the lower and upper 32 bits together separately
	Combined.ID = (uint64(HashCombine(A.ID >> 32, B.ID >> 32)) << 32) | HashCombine((uint32)A.ID, (uint32)B.ID);
	return Combined;
}