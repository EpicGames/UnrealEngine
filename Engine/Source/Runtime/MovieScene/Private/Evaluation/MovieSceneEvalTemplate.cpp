// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/MovieSceneEvalTemplate.h"
#include "Evaluation/MovieSceneEvalTemplateSerializer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneEvalTemplate)

float FMovieSceneEvalTemplate::EvaluateEasing(FFrameTime CurrentTime) const
{
	return SourceSectionPtr.IsValid() ? SourceSectionPtr->EvaluateEasing(CurrentTime) : 1.f;
}

bool FMovieSceneEvalTemplatePtr::Serialize(FArchive& Ar)
{
	bool bShouldWarn = !WITH_EDITORONLY_DATA;
	return SerializeInlineValue(*this, Ar, bShouldWarn);
}

