// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/InlineValue.h"
#include "MovieSceneFwd.h"
#include "Evaluation/MovieSceneEvaluationCustomVersion.h"
#include "Evaluation/MovieSceneEvalTemplateBase.h"

struct FMovieSceneEmptyStruct;

/** Serialize an inline value that has a GetScriptStruct method */
template<typename T, uint16 N>
bool SerializeInlineValue(TInlineValue<T, N>& Impl, FArchive& Ar, bool bWarnOnError)
{
	Ar.UsingCustomVersion(FMovieSceneEvaluationCustomVersion::GUID);
	
	if (Ar.IsLoading())
	{
		FString TypeName;
		Ar << TypeName;

		// If it's empty, then we've got nothing
		if (TypeName.IsEmpty())
		{
			return true;
		}

		// Find the script struct of the thing that was serialized
		UScriptStruct* Struct = FindObject<UScriptStruct>(nullptr, *TypeName);
		if (!Struct)
		{
			if (bWarnOnError)
			{
				// We only throw this warning in cooked builds as that is the only place where this deserialized data matters
				UE_LOGF(LogMovieScene, Warning, "Unknown or invalid type (%ls) found in serialized data. This will no longer work.", *TypeName);
			}

			// If it wasn't found, just deserialize an empty struct instead, and set ourselves to default
			FMovieSceneEmptyStruct Empty;
			FMovieSceneEmptyStruct::StaticStruct()->SerializeItem(Ar, &Empty, nullptr);
			return true;
		}

		int32 StructSize = Struct->GetCppStructOps()->GetSize();
		int32 StructAlignment = Struct->GetCppStructOps()->GetAlignment();
		void* Allocation = Impl.Reserve(StructSize, StructAlignment);

		Struct->GetCppStructOps()->Construct(Allocation);
		Struct->SerializeItem(Ar, Allocation, nullptr);

		return true;
	}
	else if (Ar.IsSaving())
	{
		if (Impl.IsValid())
		{
			auto* ImplPtr = &Impl.GetValue();
			UScriptStruct& Struct = ImplPtr->GetScriptStruct();
			FString TypeName = Struct.GetPathName();
			Ar << TypeName;

			Struct.SerializeItem(Ar, ImplPtr, nullptr);
		}
		else
		{
			// Just serialize an empty name
			FString Name;
			Ar << Name;
		}

		return true;
	}

	return false;
}
