// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/MovieSceneEvaluationCustomVersion.h"
#include "Serialization/CustomVersion.h"

// Register the custom version with core
FCustomVersionRegistration GRegisterMovieSceneEvaluationCustomVersion(FMovieSceneEvaluationCustomVersion::GUID, FMovieSceneEvaluationCustomVersion::LatestVersion, TEXT("MovieSceneEvaluation"));
