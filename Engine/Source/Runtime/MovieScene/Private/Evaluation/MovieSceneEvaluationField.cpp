// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/MovieSceneEvaluationField.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Evaluation/MovieSceneEvaluationTree.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneTimeHelpers.h"
#include "Algo/Sort.h"
#include "Algo/BinarySearch.h"
#include "Algo/IndexOf.h"

#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntityManager.h"

#include "MovieSceneSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneEvaluationField)


FMovieSceneEntityComponentFieldBuilder::FMovieSceneEntityComponentFieldBuilder(FMovieSceneEntityComponentField* InField)
	: Field(InField)
	, SharedMetaDataIndex(Field->SharedMetaData.Emplace())
{}

FMovieSceneEntityComponentFieldBuilder::~FMovieSceneEntityComponentFieldBuilder()
{
	const bool bContainsValidEntities = Algo::AnyOf(KeyToFieldIndex, [](FKeyToIndex In) { return In.FieldIndex != INDEX_NONE; });
	if (!bContainsValidEntities)
	{
		if (ensureMsgf(Field->SharedMetaData.Num() == SharedMetaDataIndex+1, TEXT("Additional shared meta-data has been added since this builder was constructed, recursive builders are not supported")))
		{
			Field->SharedMetaData.RemoveAt(SharedMetaDataIndex, EAllowShrinking::No);
		}
	}
}

FMovieSceneEvaluationFieldSharedEntityMetaData& FMovieSceneEntityComponentFieldBuilder::GetSharedMetaData()
{
	return Field->SharedMetaData[SharedMetaDataIndex];
}

int32 FMovieSceneEntityComponentFieldBuilder::GetSharedMetaDataIndex() const
{
	return SharedMetaDataIndex;
}

int32 FMovieSceneEntityComponentFieldBuilder::FindOrAddEntity(UObject* EntityOwner, uint32 EntityID)
{
	FMovieSceneEvaluationFieldEntityKey Key = { EntityOwner, EntityID };

	int32 LocalIndex = Algo::IndexOfBy(KeyToFieldIndex, Key, &FKeyToIndex::Key);
	if (LocalIndex != INDEX_NONE)
	{
		return LocalIndex;
	}

	return KeyToFieldIndex.Add(FKeyToIndex{ Key, INDEX_NONE });
}

int32 FMovieSceneEntityComponentFieldBuilder::AddMetaData(const FMovieSceneEvaluationFieldEntityMetaData& InMetaData)
{
	if (InMetaData.IsRedundant())
	{
		return INDEX_NONE;
	}

	int32 LocalIndex = Algo::IndexOfBy(MetaDataToFieldIndex, InMetaData, &FMetaDataToIndex::MetaData);
	if (LocalIndex == INDEX_NONE)
	{
		LocalIndex = MetaDataToFieldIndex.Add(FMetaDataToIndex{ InMetaData, INDEX_NONE });
	}

	return LocalIndex;
}

void FMovieSceneEntityComponentFieldBuilder::AddPersistentEntity(const TRange<FFrameNumber>& Range, UObject* EntityOwner, uint32 EntityID, int32 LocalMetaDataIndex)
{
	AddPersistentEntity(Range, FindOrAddEntity(EntityOwner, EntityID), LocalMetaDataIndex);
}

void FMovieSceneEntityComponentFieldBuilder::AddPersistentEntity(const TRange<FFrameNumber>& Range, int32 LocalIndex, int32 LocalMetaDataIndex)
{
	FMovieSceneEvaluationFieldEntityTree::FEntityAndMetaDataIndex Data = {
		LocalEntityIndexToFieldIndex(LocalIndex),
		LocalMetaDataIndexToFieldIndex(LocalMetaDataIndex)
	};
	Field->PersistentEntityTree.SerializedData.AddUnique(Range, Data);
}

void FMovieSceneEntityComponentFieldBuilder::AddOneShotEntity(const TRange<FFrameNumber>& OneShotRange, UObject* EntityOwner, uint32 EntityID, int32 LocalMetaDataIndex)
{
	AddOneShotEntity(OneShotRange, FindOrAddEntity(EntityOwner, EntityID), LocalMetaDataIndex);
}

void FMovieSceneEntityComponentFieldBuilder::AddOneShotEntity(const TRange<FFrameNumber>& OneShotRange, int32 LocalIndex, int32 LocalMetaDataIndex)
{
	FMovieSceneEvaluationFieldEntityTree::FEntityAndMetaDataIndex Data = {
		LocalEntityIndexToFieldIndex(LocalIndex),
		LocalMetaDataIndexToFieldIndex(LocalMetaDataIndex)
	};

	Field->OneShotEntityTree.SerializedData.AddUnique(OneShotRange, Data);
}

int32 FMovieSceneEntityComponentFieldBuilder::LocalEntityIndexToFieldIndex(int32 LocalIndex)
{
	checkf(KeyToFieldIndex.IsValidIndex(LocalIndex), TEXT("Invalid local entity index specified"));

	FKeyToIndex& KeyToIndex = KeyToFieldIndex[LocalIndex];
	if (KeyToIndex.FieldIndex == INDEX_NONE)
	{
		KeyToIndex.FieldIndex = Field->Entities.Emplace(FMovieSceneEvaluationFieldEntity{ KeyToIndex.Key, SharedMetaDataIndex });
	}

	return KeyToIndex.FieldIndex;
}

int32 FMovieSceneEntityComponentFieldBuilder::LocalMetaDataIndexToFieldIndex(int32 LocalIndex)
{
	if (LocalIndex == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	checkf(MetaDataToFieldIndex.IsValidIndex(LocalIndex), TEXT("Invalid local meta-data index specified"));

	FMetaDataToIndex& MetaDataToIndex = MetaDataToFieldIndex[LocalIndex];
	if (MetaDataToIndex.FieldIndex == INDEX_NONE)
	{
		MetaDataToIndex.FieldIndex = Field->EntityMetaData.Emplace(MetaDataToIndex.MetaData);
	}

	return MetaDataToIndex.FieldIndex;
}

void FMovieSceneEntityComponentField::QueryPersistentEntities(FFrameNumber QueryTime, TRange<FFrameNumber>& OutRange, FMovieSceneEvaluationFieldEntitySet& OutEntities) const
{
	FMovieSceneEvaluationTreeRangeIterator Iterator = PersistentEntityTree.SerializedData.IterateFromTime(QueryTime);
	check(Iterator);

	OutRange = Iterator.Range();
	for (FMovieSceneEvaluationFieldEntityTree::FEntityAndMetaDataIndex Pair : PersistentEntityTree.SerializedData.GetAllData(Iterator.Node()))
	{
		OutEntities.Add(FMovieSceneEvaluationFieldEntityQuery{
			GetEntity(Pair.EntityIndex),
			Pair.MetaDataIndex
		});
	}
}

void FMovieSceneEntityComponentField::QueryPersistentEntities(FFrameNumber QueryTime, TFunctionRef<bool(const FMovieSceneEvaluationFieldEntityQuery&)> QueryCallback, TRange<FFrameNumber>& OutRange) const
{
	FMovieSceneEvaluationTreeRangeIterator Iterator = PersistentEntityTree.SerializedData.IterateFromTime(QueryTime);
	check(Iterator);

	OutRange = Iterator.Range();
	for (FMovieSceneEvaluationFieldEntityTree::FEntityAndMetaDataIndex Pair : PersistentEntityTree.SerializedData.GetAllData(Iterator.Node()))
	{
		FMovieSceneEvaluationFieldEntityQuery Query{
			GetEntity(Pair.EntityIndex),
			Pair.MetaDataIndex
		};

		if (!QueryCallback(Query))
		{
			return;
		}
	}
}

bool FMovieSceneEntityComponentField::HasAnyOneShotEntities() const
{
	return !OneShotEntityTree.SerializedData.IsEmpty();
}

void FMovieSceneEntityComponentField::QueryOneShotEntities(const TRange<FFrameNumber>& QueryRange, FMovieSceneEvaluationFieldEntitySet& OutEntities) const
{
	FMovieSceneEvaluationTreeRangeIterator Iterator = OneShotEntityTree.SerializedData.IterateFromLowerBound(QueryRange.GetLowerBound());
	check(Iterator);

	for ( ; Iterator && QueryRange.Overlaps(Iterator.Range()); ++Iterator )
	{
		for (FMovieSceneEvaluationFieldEntityTree::FEntityAndMetaDataIndex Pair : OneShotEntityTree.SerializedData.GetAllData(Iterator.Node()))
		{
			OutEntities.Add(FMovieSceneEvaluationFieldEntityQuery{
				GetEntity(Pair.EntityIndex),
				Pair.MetaDataIndex,
			});
		}
	}
}

int32 FMovieSceneEvaluationField::GetSegmentFromTime(FFrameNumber Time) const
{
	// Linear search
	// @todo: accelerated search based on the last evaluated index?
	for (int32 Index = 0; Index < Ranges.Num(); ++Index)
	{
		if (Ranges[Index].Value.Contains(Time))
		{
			return Index;
		}
	}

	return INDEX_NONE;
}

TRange<int32> FMovieSceneEvaluationField::OverlapRange(const TRange<FFrameNumber>& Range) const
{
	if (Ranges.Num() == 0)
	{
		return TRange<int32>::Empty();
	}

	TArrayView<const FMovieSceneFrameRange> RangesToSearch(Ranges);

	// Binary search the first lower bound that's greater than the input range's lower bound
	int32 StartIndex = Algo::UpperBoundBy(RangesToSearch, Range.GetLowerBound(), &FMovieSceneFrameRange::GetLowerBound, MovieSceneHelpers::SortLowerBounds);

	// StartIndex is always <= RangesToSearch.Num(). If the previous range overlaps the input range, include that
	if (StartIndex > 0 && RangesToSearch[StartIndex-1].Value.Overlaps(Range))
	{
		StartIndex = StartIndex - 1;
	}

	if (StartIndex == RangesToSearch.Num())
	{
		return TRange<int32>::Empty();
	}


	// Search the remaining ranges for the last upper bound greater than the input range
	RangesToSearch = RangesToSearch.Slice(StartIndex, RangesToSearch.Num() - StartIndex);

	// Binary search the first lower bound that is greater than or equal to the input range's upper bound
	int32 Length = Range.GetUpperBound().IsOpen() ? RangesToSearch.Num() : Algo::UpperBoundBy(RangesToSearch, Range.GetUpperBound(), &FMovieSceneFrameRange::GetUpperBound, MovieSceneHelpers::SortUpperBounds);

	// Length is always <= RangesToSearch.Num(). If the previous range overlaps the input range, include that
	if (Length < RangesToSearch.Num() && RangesToSearch[Length].Value.Overlaps(Range))
	{
		Length = Length + 1;
	}

	return Length > 0 ? TRange<int32>(StartIndex, StartIndex + Length) : TRange<int32>::Empty();
}

void FMovieSceneEvaluationField::Invalidate(const TRange<FFrameNumber>& Range)
{
	TRange<int32> OverlappingRange = OverlapRange(Range);
	if (!OverlappingRange.IsEmpty())
	{
		Ranges.RemoveAt(OverlappingRange.GetLowerBoundValue(), OverlappingRange.Size<int32>(), EAllowShrinking::No);
		Groups.RemoveAt(OverlappingRange.GetLowerBoundValue(), OverlappingRange.Size<int32>(), EAllowShrinking::No);
		MetaData.RemoveAt(OverlappingRange.GetLowerBoundValue(), OverlappingRange.Size<int32>(), EAllowShrinking::No);

#if WITH_EDITORONLY_DATA
		Signature = FGuid::NewGuid();
#endif
	}
}

int32 FMovieSceneEvaluationField::Insert(const TRange<FFrameNumber>& InRange, FMovieSceneEvaluationGroup&& InGroup, FMovieSceneEvaluationMetaData&& InMetaData)
{
	const int32 InsertIndex = Algo::UpperBoundBy(Ranges, InRange.GetLowerBound(), &FMovieSceneFrameRange::GetLowerBound, MovieSceneHelpers::SortLowerBounds);

	const bool bOverlapping = 
		(Ranges.IsValidIndex(InsertIndex  ) && Ranges[InsertIndex  ].Value.Overlaps(InRange)) ||
		(Ranges.IsValidIndex(InsertIndex-1) && Ranges[InsertIndex-1].Value.Overlaps(InRange));

	if (!ensureAlwaysMsgf(!bOverlapping, TEXT("Attempting to insert an overlapping range into the evaluation field.")))
	{
		return INDEX_NONE;
	}

	Ranges.Insert(InRange, InsertIndex);
	MetaData.Insert(MoveTemp(InMetaData), InsertIndex);
	Groups.Insert(MoveTemp(InGroup), InsertIndex);

#if WITH_EDITORONLY_DATA
	Signature = FGuid::NewGuid();
#endif

	return InsertIndex;
}

void FMovieSceneEvaluationField::Add(const TRange<FFrameNumber>& InRange, FMovieSceneEvaluationGroup&& InGroup, FMovieSceneEvaluationMetaData&& InMetaData)
{
	if (ensureAlwaysMsgf(!Ranges.Num() || !Ranges.Last().Value.Overlaps(InRange), TEXT("Attempting to add overlapping ranges to sequence evaluation field.")))
	{
		Ranges.Add(InRange);
		MetaData.Add(MoveTemp(InMetaData));
		Groups.Add(MoveTemp(InGroup));

#if WITH_EDITORONLY_DATA
		Signature = FGuid::NewGuid();
#endif
	}
}

void FMovieSceneEvaluationMetaData::DiffSequences(const FMovieSceneEvaluationMetaData& LastFrame, TArray<FMovieSceneSequenceID>* NewSequences, TArray<FMovieSceneSequenceID>* ExpiredSequences) const
{
	// This algorithm works on the premise that each array is sorted, and each ID can only appear once
	auto ThisFrameIDs = ActiveSequences.CreateConstIterator();
	auto LastFrameIDs = LastFrame.ActiveSequences.CreateConstIterator();

	// Iterate both arrays together
	while (ThisFrameIDs && LastFrameIDs)
	{
		FMovieSceneSequenceID ThisID = *ThisFrameIDs;
		FMovieSceneSequenceID LastID = *LastFrameIDs;

		// If they're the same, skip
		if (ThisID == LastID)
		{
			++ThisFrameIDs;
			++LastFrameIDs;
			continue;
		}

		if (LastID < ThisID)
		{
			// Last frame iterator is less than this frame's, which means the last ID is no longer evaluated
			if (ExpiredSequences)
			{
				ExpiredSequences->Add(LastID);
			}
			++LastFrameIDs;
		}
		else
		{
			// Last frame iterator is greater than this frame's, which means this ID is new
			if (NewSequences)
			{
				NewSequences->Add(ThisID);
			}

			++ThisFrameIDs;
		}
	}

	// Add any remaining expired sequences
	if (ExpiredSequences)
	{
		while (LastFrameIDs)
		{
			ExpiredSequences->Add(*LastFrameIDs);
			++LastFrameIDs;
		}
	}

	// Add any remaining new sequences
	if (NewSequences)
	{
		while (ThisFrameIDs)
		{
			NewSequences->Add(*ThisFrameIDs);
			++ThisFrameIDs;
		}
	}
}

void FMovieSceneEvaluationMetaData::DiffEntities(const FMovieSceneEvaluationMetaData& LastFrame, TArray<FMovieSceneOrderedEvaluationKey>* NewKeys, TArray<FMovieSceneOrderedEvaluationKey>* ExpiredKeys) const
{
	// This algorithm works on the premise that each array is sorted, and each ID can only appear once
	auto ThisFrameKeys = ActiveEntities.CreateConstIterator();
	auto LastFrameKeys = LastFrame.ActiveEntities.CreateConstIterator();

	// Iterate both arrays together
	while (ThisFrameKeys && LastFrameKeys)
	{
		FMovieSceneOrderedEvaluationKey ThisKey = *ThisFrameKeys;
		FMovieSceneOrderedEvaluationKey LastKey = *LastFrameKeys;

		// If they're the same, skip
		if (ThisKey.Key == LastKey.Key)
		{
			++ThisFrameKeys;
			++LastFrameKeys;
			continue;
		}

		if (LastKey.Key < ThisKey.Key)
		{
			// Last frame iterator is less than this frame's, which means the last entity is no longer evaluated
			if (ExpiredKeys)
			{
				ExpiredKeys->Add(LastKey);
			}
			++LastFrameKeys;
		}
		else
		{
			// Last frame iterator is greater than this frame's, which means this entity is new
			if (NewKeys)
			{
				NewKeys->Add(ThisKey);
			}

			++ThisFrameKeys;
		}
	}

	// Add any remaining expired entities
	if (ExpiredKeys)
	{
		while (LastFrameKeys)
		{
			ExpiredKeys->Add(*LastFrameKeys);
			++LastFrameKeys;
		}

		Algo::SortBy(*ExpiredKeys, &FMovieSceneOrderedEvaluationKey::TearDownIndex);
	}

	// Add any remaining new entities
	if (NewKeys)
	{
		while (ThisFrameKeys)
		{
			NewKeys->Add(*ThisFrameKeys);
			++ThisFrameKeys;
		}

		Algo::SortBy(*NewKeys, &FMovieSceneOrderedEvaluationKey::SetupIndex);
	}
}

