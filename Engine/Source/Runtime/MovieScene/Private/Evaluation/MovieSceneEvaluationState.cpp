// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/MovieSceneEvaluationState.h"

#include "Algo/Sort.h"
#include "Algo/Unique.h"
#include "Engine/World.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneInstanceRegistry.h"
#include "EntitySystem/MovieSceneSequenceInstance.h"
#include "IMovieScenePlaybackClient.h"
#include "IMovieScenePlayer.h"
#include "MovieScene.h"
#include "MovieSceneObjectBindingID.h"
#include "MovieSceneSequence.h"
#include "UniversalObjectLocatorResolveParams.h"

DECLARE_CYCLE_STAT(TEXT("Find Bound Objects"), MovieSceneEval_FindBoundObjects, STATGROUP_MovieSceneEval);
DECLARE_CYCLE_STAT(TEXT("Iterate Bound Objects"), MovieSceneEval_IterateBoundObjects, STATGROUP_MovieSceneEval);

namespace UE::MovieScene
{

UE_DEFINE_MOVIESCENE_PLAYBACK_CAPABILITY(IObjectBindingNotifyPlaybackCapability)
UE_DEFINE_MOVIESCENE_PLAYBACK_CAPABILITY(IStaticBindingOverridesPlaybackCapability)

FMovieSceneEvaluationOperand* FStaticBindingOverrides::GetBindingOverride(const FMovieSceneEvaluationOperand& InOperand)
{
	return BindingOverrides.Find(InOperand);
}

void FStaticBindingOverrides::AddBindingOverride(const FMovieSceneEvaluationOperand& InOperand, const FMovieSceneEvaluationOperand& InOverrideOperand)
{
	BindingOverrides.Add(InOperand, InOverrideOperand);
}

void FStaticBindingOverrides::RemoveBindingOverride(const FMovieSceneEvaluationOperand& InOperand)
{
	BindingOverrides.Remove(InOperand);
}

}  // namespace UE::MovieScene

FMovieSceneSharedDataId FMovieSceneSharedDataId::Allocate()
{
	static uint32 Counter = 0;

	FMovieSceneSharedDataId Value;
	Value.UniqueId = ++Counter;
	check(Counter != -1);
	return Value;
}

TArrayView<TWeakObjectPtr<>> FMovieSceneObjectCache::FindBoundObjects(const FGuid& InBindingID, TSharedRef<const FSharedPlaybackState> InSharedPlaybackState)
{
	MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_FindBoundObjects)
	
	// Fast route - where everything's cached
	FBoundObjects* Bindings = BoundObjects.Find(InBindingID);
	if (Bindings && Bindings->bUpToDate)
	{
		return TArrayView<TWeakObjectPtr<>>(
			Bindings->Objects.GetData(),
			Bindings->Objects.Num()
			);
	}

	// Attempt to update the bindings
	UpdateBindings(InBindingID, InSharedPlaybackState);

	Bindings = BoundObjects.Find(InBindingID);
	if (Bindings)
	{
		return TArrayView<TWeakObjectPtr<>>(Bindings->Objects.GetData(), Bindings->Objects.Num());
	}

	// Just return nothing
	return TArrayView<TWeakObjectPtr<>>();
}

TArrayView<const TWeakObjectPtr<>> FMovieSceneObjectCache::IterateBoundObjects(const FGuid& InBindingID) const
{
	MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_IterateBoundObjects)

	const FBoundObjects* Bindings = BoundObjects.Find(InBindingID);
	if (Bindings && Bindings->bUpToDate)
	{
		return TArrayView<const TWeakObjectPtr<>>(
			Bindings->Objects.GetData(),
			Bindings->Objects.Num()
			);
	}

	// Just return nothing
	return TArrayView<TWeakObjectPtr<>>();
}

FGuid FMovieSceneObjectCache::FindObjectId(UObject& InObject, TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	UMovieSceneSequence* Sequence = WeakSequence.Get();
	UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;
	if (!MovieScene)
	{
		return FGuid();
	}

	if (!bReentrantUpdate)
	{
		// @todo: Currently we delete the entire object cache when attempting to find an object's ID to ensure that we do a 
		// complete lookup from scratch. This is required for UMG as it interchanges content slots without notifying sequencer.
		Clear(SharedPlaybackState);
	}

	return FindCachedObjectId(InObject, SharedPlaybackState);
}

FGuid FMovieSceneObjectCache::FindCachedObjectId(UObject& InObject, TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	UMovieSceneSequence* Sequence = WeakSequence.Get();
	UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;
	if (!MovieScene)
	{
		return FGuid();
	}

	TWeakObjectPtr<> ObjectToFind(&InObject);

	// Search all possessables
	for (int32 Index = 0; Index < MovieScene->GetPossessableCount(); ++Index)
	{
		FGuid ThisGuid = MovieScene->GetPossessable(Index).GetGuid();
		if (FindBoundObjects(ThisGuid, SharedPlaybackState).Contains(ObjectToFind))
		{
			return ThisGuid;
		}
	}

	// Search all spawnables
	for (int32 Index = 0; Index < MovieScene->GetSpawnableCount(); ++Index)
	{
		FGuid ThisGuid = MovieScene->GetSpawnable(Index).GetGuid();
		if (FindBoundObjects(ThisGuid, SharedPlaybackState).Contains(ObjectToFind))
		{
			return ThisGuid;
		}
	}

	return FGuid();
}

void FMovieSceneObjectCache::FilterObjectBindings(UObject* PredicateObject, TSharedRef<const FSharedPlaybackState> SharedPlaybackState, TArray<FMovieSceneObjectBindingID>* OutBindings)
{
	check(OutBindings);

	TArray<FGuid, TInlineAllocator<8>> OutOfDateBindings;
	for (const TTuple<FGuid, FBoundObjects>& Pair : BoundObjects)
	{
		if (Pair.Value.bUpToDate)
		{
			for (TWeakObjectPtr<> WeakObject : Pair.Value.Objects)
			{
				UObject* Object = WeakObject.Get();
				if (Object && Object == PredicateObject)
				{
					OutBindings->Add(UE::MovieScene::FFixedObjectBindingID(Pair.Key, SequenceID));
					break;
				}
			}
		}
		else
		{
			OutOfDateBindings.Add(Pair.Key);
		}
	}

	for (const FGuid& DirtyBinding : OutOfDateBindings)
	{
		UpdateBindings(DirtyBinding, SharedPlaybackState);

		const FBoundObjects& Bindings = BoundObjects.FindChecked(DirtyBinding);
		for (TWeakObjectPtr<> WeakObject : Bindings.Objects)
		{
			UObject* Object = WeakObject.Get();
			if (Object && Object == PredicateObject)
			{
				OutBindings->Add(UE::MovieScene::FFixedObjectBindingID(DirtyBinding, SequenceID));
				break;
			}
		}
	}
}

void FMovieSceneObjectCache::InvalidateExpiredObjects()
{
	for (auto& Pair : BoundObjects)
	{
		if (!Pair.Value.bUpToDate)
		{
			continue;
		}

		for (TWeakObjectPtr<>& Ptr : Pair.Value.Objects)
		{
			if (!Ptr.Get())
			{
				InvalidateInternal(Pair.Key);
				break;
			}
		}
	}

	if (UMovieSceneSequence* Sequence = WeakSequence.Get())
	{
		TArray<FGuid> InvalidObjectIDs;
		Sequence->GatherExpiredObjects(*this, InvalidObjectIDs);

		for (const FGuid& ObjectID : InvalidObjectIDs)
		{
			InvalidateInternal(ObjectID);
		}
	}

	UpdateSerialNumber();
}

void FMovieSceneObjectCache::InvalidateIfValid(const FGuid& InGuid)
{
	const bool bInvalidated = InvalidateIfValidInternal(InGuid);
	if (bInvalidated)
	{
		UpdateSerialNumber();
	}
}

bool FMovieSceneObjectCache::GetBindingActivation(const FGuid& InGuid) const
{
	return !InactiveBindingIds.Contains(InGuid);
}

void FMovieSceneObjectCache::SetBindingActivation(const FGuid& InGuid, bool bActive)
{
	if (bActive)
	{
		InactiveBindingIds.Remove(InGuid);
	}
	else
	{
		InactiveBindingIds.Add(InGuid);
	}
	InvalidateInternal(InGuid);
}

bool FMovieSceneObjectCache::InvalidateIfValidInternal(const FGuid& InGuid)
{
	// Don't manipulate the actual map structure, since this can be called from inside an iterator
	FBoundObjects* Cache = BoundObjects.Find(InGuid);

	if (Cache && Cache->bUpToDate == true)
	{
		Cache->bUpToDate = false;

		auto* Children = ChildBindings.Find(InGuid);
		if (Children)
		{
			for (const FGuid& Child : *Children)
			{
				InvalidateIfValidInternal(Child);
			}
		}

		OnBindingInvalidated.Broadcast(InGuid);

		return true;
	}

	return false;
}

void FMovieSceneObjectCache::Invalidate(const FGuid& InGuid)
{
	InvalidateInternal(InGuid);
	UpdateSerialNumber();
}

void FMovieSceneObjectCache::Invalidate(const FGuid& InGuid, FMovieSceneSequenceIDRef InSequenceID)
{
	if (InSequenceID == SequenceID)
	{
		Invalidate(InGuid);
	}
	else
	{
		FMovieSceneObjectBindingID BindingID(UE::MovieScene::FFixedObjectBindingID(InGuid, InSequenceID));
		if (FGuidArray* ReferencedGuids = ReverseMappedBindings.Find(BindingID))
		{
			for (FGuid ReferencedGuid : *ReferencedGuids)
			{
				Invalidate(ReferencedGuid);
			}
			ReverseMappedBindings.Remove(BindingID);
		}
	}
}

bool FMovieSceneObjectCache::InvalidateInternal(const FGuid& InGuid)
{
	// Don't manipulate the actual map structure, since this can be called from inside an iterator
	FBoundObjects* Cache = BoundObjects.Find(InGuid);
	if (Cache)
	{
		Cache->bUpToDate = false;

		auto* Children = ChildBindings.Find(InGuid);
		if (Children)
		{
			for (const FGuid& Child : *Children)
			{
				InvalidateInternal(Child);
			}
		}
	}



	OnBindingInvalidated.Broadcast(InGuid);

	return true;
}

void FMovieSceneObjectCache::Clear(TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	using namespace UE::MovieScene;

	BoundObjects.Reset();
	ChildBindings.Reset();
	ReverseMappedBindings.Reset();

	UpdateSerialNumber();

	if (IObjectBindingNotifyPlaybackCapability* Notify = SharedPlaybackState->FindCapability<IObjectBindingNotifyPlaybackCapability>())
	{
		Notify->NotifyBindingsChanged();
	}
	OnBindingInvalidated.Broadcast(FGuid());
}


void FMovieSceneObjectCache::SetSequence(UMovieSceneSequence& InSequence, FMovieSceneSequenceIDRef InSequenceID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	if (WeakSequence != &InSequence)
	{
		Clear(SharedPlaybackState);
	}

	WeakSequence = &InSequence;
	SequenceID = InSequenceID;
}

void FMovieSceneObjectCache::UpdateBindings(const FGuid& InGuid, TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	using namespace UE::MovieScene;

	TGuardValue<bool> ReentrancyGuard(bReentrantUpdate, true);

	// Invalidate existing bindings, we're going to rebuild them.
	FBoundObjects* Bindings = &BoundObjects.FindOrAdd(InGuid);
	Bindings->Objects.Reset();

	// Update our serial now so it's done before any early returns.
	UpdateSerialNumber();

	if (auto* Children = ChildBindings.Find(InGuid))
	{
		for (const FGuid& Child : *Children)
		{
			InvalidateIfValidInternal(Child);
		}
	}

	// Find the sequence for this cache.
	UMovieSceneSequence* Sequence = WeakSequence.Get();
	if (!Sequence)
	{
		return;
	}

	// Binding is inactive, do not resolve.
	if (InactiveBindingIds.Contains(InGuid))
	{
		return;
	}

	// If we have overrides for this binding, ask the player to find it for us (most probably in a different cache
	// for a different sequence).
	// TODO-lchabant: we could technically end up in a circular override that creates an infinite loop...
	const FMovieSceneEvaluationOperand Operand(SequenceID, InGuid);
	FMovieSceneEvaluationState* State = SharedPlaybackState->FindCapability<FMovieSceneEvaluationState>();
	IStaticBindingOverridesPlaybackCapability* StaticOverrides = SharedPlaybackState->FindCapability<IStaticBindingOverridesPlaybackCapability>();

	if (const FMovieSceneEvaluationOperand* OverrideOperand = StaticOverrides ? StaticOverrides->GetBindingOverride(Operand) : nullptr)
	{
		const TArrayView<TWeakObjectPtr<>> OverrideBoundObjects = State->FindBoundObjects(*OverrideOperand, SharedPlaybackState);
		Bindings->Objects.Append(OverrideBoundObjects.GetData(), OverrideBoundObjects.Num());
	}
	else
	{
		const bool bUseParentsAsContext = Sequence->AreParentContextsSignificant();

		UObject* Context = SharedPlaybackState->GetPlaybackContext();
		IMovieScenePlayer* Player = FPlayerIndexPlaybackCapability::GetPlayer(SharedPlaybackState);

		const FMovieScenePossessable* Possessable = Sequence->GetMovieScene()->FindPossessable(InGuid);
		if (Possessable)
		{
			UObject* ResolutionContext = Context;

			// Because these are ordered parent-first, the parent must have already been bound, if it exists
			if (Possessable->GetParent().IsValid())
			{
				TArrayView<TWeakObjectPtr<>> ParentBoundObjects = FindBoundObjects(Possessable->GetParent(), SharedPlaybackState);

				ChildBindings.FindOrAdd(Possessable->GetParent()).AddUnique(InGuid);

				// Refresh bindings in case of map changes
				Bindings = BoundObjects.Find(InGuid);
				for (TWeakObjectPtr<> Parent : ParentBoundObjects)
				{
					if (bUseParentsAsContext)
					{
						ResolutionContext = Parent.Get();
						if (!ResolutionContext)
						{
							continue;
						}
					}

					TArray<UObject*, TInlineAllocator<1>> FoundObjects;
					
					if (Possessable->GetSpawnableObjectBindingID().IsValid())
					{
						for (TWeakObjectPtr<> BoundObject : Possessable->GetSpawnableObjectBindingID().ResolveBoundObjects(SequenceID, SharedPlaybackState))
						{
							if (BoundObject.IsValid())
							{
								FoundObjects.Add(BoundObject.Get());
							}
						}
					}
					else
					{
						UE::UniversalObjectLocator::FResolveParams ResolveParams(ResolutionContext);
						if (Player)
						{
							Player->ResolveBoundObjects(ResolveParams, InGuid, SequenceID, *Sequence, FoundObjects);
						}
						else
						{
							Sequence->LocateBoundObjects(InGuid, ResolveParams, SharedPlaybackState, FoundObjects);
						}
					}
					
					Bindings = BoundObjects.Find(InGuid);
					for (UObject* Object : FoundObjects)
					{
						Bindings->Objects.Add(Object);
					}
				}
			}
			else
			{
				TArray<UObject*, TInlineAllocator<1>> FoundObjects;

				if (Possessable->GetSpawnableObjectBindingID().IsValid())
				{
					// We resolve this binding to fixed here, as we conveniently have a Player pointer already, and when being invalidated,
					// the binding ID passed down will be relative to the root.
					FMovieSceneObjectBindingID SpawnableFixedBindingID = Possessable->GetSpawnableObjectBindingID().ResolveToFixed(SequenceID, SharedPlaybackState);
					ReverseMappedBindings.FindOrAdd(SpawnableFixedBindingID).AddUnique(InGuid);
					for (TWeakObjectPtr<> BoundObject : Possessable->GetSpawnableObjectBindingID().ResolveBoundObjects(SequenceID, SharedPlaybackState))
					{
						if (BoundObject.IsValid())
						{
							FoundObjects.Add(BoundObject.Get());
						}
					}
				}
				else
				{
					UE::UniversalObjectLocator::FResolveParams ResolveParams(ResolutionContext);
					if (Player)
					{
						Player->ResolveBoundObjects(ResolveParams, InGuid, SequenceID, *Sequence, FoundObjects);
					}
					else
					{
						Sequence->LocateBoundObjects(InGuid, ResolveParams, SharedPlaybackState, FoundObjects);
					}
				}
				
				Bindings = BoundObjects.Find(InGuid);
				for (UObject* Object : FoundObjects)
				{
					Bindings->Objects.Add(Object);
				}
			}
		}
		else
		{
			// Probably a FMovieSceneSpawnable then (or an phantom)
			bool bUseDefault = true;

			// Allow external overrides for spawnables
			const IMovieScenePlaybackClient* DynamicOverrides = SharedPlaybackState->FindCapability<IMovieScenePlaybackClient>();
			if (DynamicOverrides)
			{
				TArray<UObject*, TInlineAllocator<1>> FoundObjects;
				bUseDefault = DynamicOverrides->RetrieveBindingOverrides(InGuid, SequenceID, FoundObjects);
				for (UObject* Object : FoundObjects)
				{
					Bindings->Objects.Add(Object);
				}
			}

			// If we have no overrides, or they want to allow the default spawnable, do that now
			if (bUseDefault)
			{
				const FMovieSceneSpawnRegister* SpawnRegister = SharedPlaybackState->FindCapability<FMovieSceneSpawnRegister>();
				UObject* SpawnedObject = SpawnRegister ? SpawnRegister->FindSpawnedObject(InGuid, SequenceID, 0).Get() : nullptr;
				if (SpawnedObject)
				{
					Bindings->Objects.Add(SpawnedObject);
				}
			}
		}
	}

	const int32 NumBoundObjects = Bindings->Objects.Num();

	// Remove duplicates from bound objects
	if (NumBoundObjects > 1)
	{
		Algo::Sort(Bindings->Objects, [](const TWeakObjectPtr<>& A, const TWeakObjectPtr<>& B) { return A.Get() < B.Get(); });
		const int32 EndIndex = Algo::Unique(Bindings->Objects);
		if (EndIndex < NumBoundObjects)
		{
			FText ObjectDisplayName = Sequence->GetMovieScene()->GetObjectDisplayName(InGuid);
			UE_LOGF(LogMovieScene, Warning, "Found %d duplicate object(s) while resolving binding %ls (%ls) in %ls",
					NumBoundObjects - EndIndex, 
					*ObjectDisplayName.ToString(), *LexToString(InGuid), 
					*Sequence->GetPathName());
			Bindings->Objects.SetNum(EndIndex);
		}
	}

	if (NumBoundObjects > 0)
	{
		Bindings->bUpToDate = true;

		if (IObjectBindingNotifyPlaybackCapability* Notify = SharedPlaybackState->FindCapability<IObjectBindingNotifyPlaybackCapability>())
		{
			Notify->NotifyBindingUpdate(InGuid, SequenceID, Bindings->Objects);
		}

		if (auto* Children = ChildBindings.Find(InGuid))
		{
			for (const FGuid& Child : *Children)
			{
				InvalidateIfValidInternal(Child);
			}
		}
		ChildBindings.Remove(InGuid);
	}
}

void FMovieSceneObjectCache::UpdateSerialNumber()
{
	// Ok to overflow.
	++SerialNumber;
}

TArrayView<TWeakObjectPtr<>> FMovieSceneObjectCache::FindBoundObjects(const FGuid& InBindingID, IMovieScenePlayer& Player)
{
	return FindBoundObjects(InBindingID, Player.GetSharedPlaybackState());
}

void FMovieSceneObjectCache::SetSequence(UMovieSceneSequence& InSequence, FMovieSceneSequenceIDRef InSequenceID, IMovieScenePlayer& Player)
{
	SetSequence(InSequence, InSequenceID, Player.GetSharedPlaybackState());
}

FGuid FMovieSceneObjectCache::FindObjectId(UObject& InObject, IMovieScenePlayer& Player)
{
	return FindObjectId(InObject, Player.GetSharedPlaybackState());
}

FGuid FMovieSceneObjectCache::FindCachedObjectId(UObject& InObject, IMovieScenePlayer& Player)
{
	return FindCachedObjectId(InObject, Player.GetSharedPlaybackState());
}

void FMovieSceneObjectCache::Clear(IMovieScenePlayer& Player)
{
	Clear(Player.GetSharedPlaybackState());
}

void FMovieSceneObjectCache::FilterObjectBindings(UObject* PredicateObject, IMovieScenePlayer& Player, TArray<FMovieSceneObjectBindingID>* OutBindings)
{
	FilterObjectBindings(PredicateObject, Player.GetSharedPlaybackState(), OutBindings);
}

UE_DEFINE_MOVIESCENE_PLAYBACK_CAPABILITY(FMovieSceneEvaluationState)

void FMovieSceneEvaluationState::InvalidateExpiredObjects()
{
	for (auto& Pair : ObjectCaches)
	{
		Pair.Value.ObjectCache.InvalidateExpiredObjects();
	}
}

void FMovieSceneEvaluationState::Invalidate(const FGuid& InGuid, FMovieSceneSequenceIDRef SequenceID)
{
	// We need to send the invalidation method to all of the caches, as there may be other bindings in other sequences referencing this one that is being invalidated
	for (auto& Pair : ObjectCaches)
	{
		Pair.Value.ObjectCache.Invalidate(InGuid, SequenceID);
	}
}


bool FMovieSceneEvaluationState::GetBindingActivation(const FGuid& InGuid, FMovieSceneSequenceIDRef InSequenceID) const
{
	if (const FMovieSceneObjectCache* Cache = FindObjectCache(InSequenceID))
	{
		return Cache->GetBindingActivation(InGuid);
	}
	return true;
}

void FMovieSceneEvaluationState::SetBindingActivation(const FGuid& InGuid, FMovieSceneSequenceIDRef InSequenceID, bool bActive)
{
	GetObjectCache(InSequenceID).SetBindingActivation(InGuid, bActive);
}

void FMovieSceneEvaluationState::ClearObjectCaches(TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	for (auto& Pair : ObjectCaches)
	{
		Pair.Value.ObjectCache.Clear(SharedPlaybackState);
	}
}

void FMovieSceneEvaluationState::AssignSequence(FMovieSceneSequenceIDRef InSequenceID, UMovieSceneSequence& InSequence, TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	GetObjectCache(InSequenceID).SetSequence(InSequence, InSequenceID, SharedPlaybackState);
}

UMovieSceneSequence* FMovieSceneEvaluationState::FindSequence(FMovieSceneSequenceIDRef InSequenceID) const
{
	const FVersionedObjectCache* Cache = ObjectCaches.Find(InSequenceID);
	return Cache ? Cache->ObjectCache.GetSequence() : nullptr;
}

FMovieSceneSequenceID FMovieSceneEvaluationState::FindSequenceId(const UMovieSceneSequence* InSequence) const
{
	return FindSequenceId(const_cast<UMovieSceneSequence*>(InSequence));
}

FMovieSceneSequenceID FMovieSceneEvaluationState::FindSequenceId(UMovieSceneSequence* InSequence) const
{
	for (auto& Pair : ObjectCaches)
	{
		if (Pair.Value.ObjectCache.GetSequence() == InSequence)
		{
			return Pair.Key;
		}
	}

	return FMovieSceneSequenceID();
}

FGuid FMovieSceneEvaluationState::FindObjectId(UObject& Object, FMovieSceneSequenceIDRef InSequenceID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	FVersionedObjectCache* Cache = ObjectCaches.Find(InSequenceID);
	return Cache ? Cache->ObjectCache.FindObjectId(Object, SharedPlaybackState) : FGuid();
}

FGuid FMovieSceneEvaluationState::FindCachedObjectId(UObject& Object, FMovieSceneSequenceIDRef InSequenceID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	FVersionedObjectCache* Cache = ObjectCaches.Find(InSequenceID);
	return Cache ? Cache->ObjectCache.FindCachedObjectId(Object, SharedPlaybackState) : FGuid();
}

void FMovieSceneEvaluationState::FilterObjectBindings(UObject* PredicateObject, TSharedRef<const FSharedPlaybackState> SharedPlaybackState, TArray<FMovieSceneObjectBindingID>* OutBindings)
{
	check(OutBindings);

	for (TTuple<FMovieSceneSequenceID, FVersionedObjectCache>& Cache : ObjectCaches)
	{
		Cache.Value.ObjectCache.FilterObjectBindings(PredicateObject, SharedPlaybackState, OutBindings);
	}
}

uint32 FMovieSceneEvaluationState::GetSerialNumber()
{
	bool bUpdateSerial = false;
	for (TTuple<FMovieSceneSequenceID, FVersionedObjectCache>& Cache : ObjectCaches)
	{
		const uint32 CurrentCacheSerial = Cache.Value.ObjectCache.GetSerialNumber();
		bUpdateSerial = bUpdateSerial || (CurrentCacheSerial != Cache.Value.LastKnownSerial);
		Cache.Value.LastKnownSerial = CurrentCacheSerial;
	}
	if (bUpdateSerial)
	{
		// Ok to overflow.
		++SerialNumber;
	}
	return SerialNumber;
}

void FMovieSceneEvaluationState::Initialize(TSharedRef<const FSharedPlaybackState> Owner)
{
	UMovieSceneEntitySystemLinker* Linker = Owner->GetLinker();
	RegisterObjectCacheEvents(Linker, Owner->GetRootInstanceHandle(), MovieSceneSequenceID::Root);
}

void FMovieSceneEvaluationState::OnSubInstanceCreated(TSharedRef<const FSharedPlaybackState> Owner, const UE::MovieScene::FInstanceHandle InstanceHandle)
{
	UMovieSceneEntitySystemLinker* Linker = Owner->GetLinker();
	const UE::MovieScene::FSequenceInstance& SubInstance = Linker->GetInstanceRegistry()->GetInstance(InstanceHandle);
	RegisterObjectCacheEvents(Linker, InstanceHandle, SubInstance.GetSequenceID());
}

void FMovieSceneEvaluationState::RegisterObjectCacheEvents(UMovieSceneEntitySystemLinker* Linker, const UE::MovieScene::FInstanceHandle& InstanceHandle, const FMovieSceneSequenceID SequenceID)
{
	FMovieSceneObjectCache& ObjectCache = GetObjectCache(SequenceID);  // Make sure the cache is created...
	FVersionedObjectCache& VersionedObjectCache = ObjectCaches.FindChecked(SequenceID);
	VersionedObjectCache.OnInvalidateObjectBindingHandle = ObjectCache.OnBindingInvalidated.AddUObject(
			Linker, &UMovieSceneEntitySystemLinker::InvalidateObjectBinding, InstanceHandle);
}

void FMovieSceneEvaluationState::AssignSequence(FMovieSceneSequenceIDRef InSequenceID, UMovieSceneSequence& InSequence, IMovieScenePlayer& Player)
{
	AssignSequence(InSequenceID, InSequence, Player.GetSharedPlaybackState());
}

FGuid FMovieSceneEvaluationState::FindObjectId(UObject& Object, FMovieSceneSequenceIDRef InSequenceID, IMovieScenePlayer& Player)
{
	return FindObjectId(Object, InSequenceID, Player.GetSharedPlaybackState());
}

FGuid FMovieSceneEvaluationState::FindCachedObjectId(UObject& Object, FMovieSceneSequenceIDRef InSequenceID, IMovieScenePlayer& Player)
{
	return FindCachedObjectId(Object, InSequenceID, Player.GetSharedPlaybackState());
}

void FMovieSceneEvaluationState::FilterObjectBindings(UObject* PredicateObject, IMovieScenePlayer& Player, TArray<FMovieSceneObjectBindingID>* OutBindings)
{
	FilterObjectBindings(PredicateObject, Player.GetSharedPlaybackState(), OutBindings);
}

void FMovieSceneEvaluationState::ClearObjectCaches(IMovieScenePlayer& Player)
{
	ClearObjectCaches(Player.GetSharedPlaybackState());
}

