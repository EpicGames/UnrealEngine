// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/MovieSceneEvaluationTemplate.h"
#include "MovieSceneSequence.h"
#include "Sections/MovieSceneSubSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneEvaluationTemplate)

FMovieSceneSubSectionData::FMovieSceneSubSectionData(UMovieSceneSubSection& InSubSection, const FGuid& InObjectBindingId, ESectionEvaluationFlags InFlags)
	: Section(&InSubSection), ObjectBindingId(InObjectBindingId), Flags(InFlags)
{}

FMovieSceneTrackIdentifier FMovieSceneTemplateGenerationLedger::FindTrackIdentifier(const FGuid& InSignature) const
{
	return TrackSignatureToTrackIdentifier.FindRef(InSignature);
}

void FMovieSceneTemplateGenerationLedger::AddTrack(const FGuid& InSignature, FMovieSceneTrackIdentifier Identifier)
{
	ensure(!TrackSignatureToTrackIdentifier.Contains(InSignature));
	TrackSignatureToTrackIdentifier.Add(InSignature, Identifier);
}

#if WITH_EDITORONLY_DATA
void FMovieSceneEvaluationTemplate::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		for (auto& Pair : Tracks)
		{
			if (TemplateLedger.LastTrackIdentifier == FMovieSceneTrackIdentifier::Invalid() || TemplateLedger.LastTrackIdentifier.Value < Pair.Key.Value)
			{
				// Reset previously serialized, invalid data
				*this = FMovieSceneEvaluationTemplate();
				break;
			}
		}
	}
}
#endif

FMovieSceneTrackIdentifier FMovieSceneEvaluationTemplate::AddTrack(const FGuid& InSignature, FMovieSceneEvaluationTrack&& InTrack)
{
	FMovieSceneTrackIdentifier NewIdentifier = ++TemplateLedger.LastTrackIdentifier;

	InTrack.SetupOverrides();
	Tracks.Add(NewIdentifier, MoveTemp(InTrack));
	TemplateLedger.AddTrack(InSignature, NewIdentifier);

	return NewIdentifier;
}

void FMovieSceneEvaluationTemplate::RemoveTrack(const FGuid& InSignature)
{
	FMovieSceneTrackIdentifier TrackIdentifier = TemplateLedger.FindTrackIdentifier(InSignature);
	if (TrackIdentifier == FMovieSceneTrackIdentifier::Invalid())
	{
		return;
	}

	FMovieSceneEvaluationTrack* Track = Tracks.Find(TrackIdentifier);
	if (Track)
	{
		StaleTracks.Add(TrackIdentifier, MoveTemp(*Track));
	}

	Tracks.Remove(TrackIdentifier);
	TemplateLedger.TrackSignatureToTrackIdentifier.Remove(InSignature);
}

void FMovieSceneEvaluationTemplate::RemoveStaleData(const TSet<FGuid>& ActiveSignatures)
{
	{
		TArray<FGuid> SignaturesToRemove;

		// Go through the template ledger, and remove anything that is no longer referenced
		for (auto& Pair : TemplateLedger.TrackSignatureToTrackIdentifier)
		{
			if (!ActiveSignatures.Contains(Pair.Key))
			{
				SignaturesToRemove.Add(Pair.Key);
			}
		}

		// Remove the signatures, updating entries in the evaluation field as we go
		for (const FGuid& Signature : SignaturesToRemove)
		{
			RemoveTrack(Signature);
		}
	}

	// Remove stale sub sections
	{
		TArray<TTuple<FGuid, FMovieSceneFrameRange>> SubSectionsToRemove;

		for (const TTuple<FGuid, FMovieSceneFrameRange>& Pair : TemplateLedger.SubSectionRanges)
		{
			if (!ActiveSignatures.Contains(Pair.Key))
			{
				SubSectionsToRemove.Add(Pair);
			}
		}

		for (const TTuple<FGuid, FMovieSceneFrameRange>& Pair : SubSectionsToRemove)
		{
			TemplateLedger.SubSectionRanges.Remove(Pair.Key);
		}
	}
}

const TMap<FMovieSceneTrackIdentifier, FMovieSceneEvaluationTrack>& FMovieSceneEvaluationTemplate::GetTracks() const
{
	return Tracks;
}

TMap<FMovieSceneTrackIdentifier, FMovieSceneEvaluationTrack>& FMovieSceneEvaluationTemplate::GetTracks()
{
	return Tracks;
}

const TMap<FMovieSceneTrackIdentifier, FMovieSceneEvaluationTrack>& FMovieSceneEvaluationTemplate::GetStaleTracks() const
{
	return StaleTracks;
}

