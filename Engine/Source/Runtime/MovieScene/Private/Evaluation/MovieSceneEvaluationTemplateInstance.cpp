// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Containers/SortedMap.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "IMovieScenePlayer.h"
#include "MovieSceneSequence.h"
#include "MovieSceneSequence.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "Evaluation/Instances/MovieSceneTrackEvaluator.h"
#include "IMovieScenePlaybackClient.h"

#include "Sections/MovieSceneSubSection.h"

#include "UObject/Package.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneEvaluationTemplateInstance)

DECLARE_CYCLE_STAT(TEXT("Entire Evaluation Cost"), MovieSceneEval_EntireEvaluationCost, STATGROUP_MovieSceneEval);


FMovieSceneRootEvaluationTemplateInstance::FMovieSceneRootEvaluationTemplateInstance()
	: EntitySystemLinker(nullptr)
	, bOwnsSharedPlaybackState(false)
{
#if WITH_EDITOR
	EmulatedNetworkMask = EMovieSceneServerClientMask::All;
#endif
}

void FMovieSceneRootEvaluationTemplateInstance::TearDown()
{
	using namespace UE::MovieScene;

	// Let's clear our shared pointer before destroying the sequence instance. This is because the sequence
	// instance checks that no one is holding onto it, making sure it's destroyed along with it and that nobody
	// is leaking it.
	FRootInstanceHandle RootInstanceHandle = SharedPlaybackState ?
		SharedPlaybackState->GetRootInstanceHandle() : FRootInstanceHandle();
	SharedPlaybackState.Reset();

	// Avoid redundant work if the linker is being destroyed anyway
	if (bOwnsSharedPlaybackState && 
			RootInstanceHandle.IsValid() &&
			EntitySystemLinker && 
			IsValidChecked(EntitySystemLinker) && 
			!EntitySystemLinker->IsUnreachable() && 
			!EntitySystemLinker->HasAnyFlags(RF_BeginDestroyed))
	{
		EntitySystemLinker->DestroyInstanceImmediately(RootInstanceHandle);
	}

	EntitySystemLinker = nullptr;
}

FMovieSceneRootEvaluationTemplateInstance::~FMovieSceneRootEvaluationTemplateInstance()
{
	TearDown();
}

UMovieSceneEntitySystemLinker* FMovieSceneRootEvaluationTemplateInstance::ConstructEntityLinker(IMovieScenePlayer& Player)
{
	UMovieSceneEntitySystemLinker* Linker = Player.ConstructEntitySystemLinker();
	if (Linker)
	{
		return Linker;
	}

	UObject* PlaybackContext = Player.GetPlaybackContext();
	return UMovieSceneEntitySystemLinker::FindOrCreateLinker(PlaybackContext, UE::MovieScene::EEntitySystemLinkerRole::Standalone, TEXT("DefaultEntitySystemLinker"));
}

void FMovieSceneRootEvaluationTemplateInstance::Initialize(UMovieSceneSequence& InRootSequence, IMovieScenePlayer& Player, UMovieSceneCompiledDataManager* InCompiledDataManager)
{
	using namespace UE::MovieScene;

	bool bReinitialize = (
			// Initialize if we weren't initialized before and this is our first sequence.
			!SharedPlaybackState.IsValid() ||
			SharedPlaybackState->GetRootSequence() == nullptr ||
			// Initialize if we lost our linker.
			EntitySystemLinker == nullptr);

	// Reinitialize if the compiled data manager has changed.
	const UMovieSceneCompiledDataManager* PreviousCompiledDataManager = SharedPlaybackState ? 
		SharedPlaybackState->GetCompiledDataManager().Get() : nullptr;
	if (!InCompiledDataManager)
	{
#if WITH_EDITOR
		EMovieSceneServerClientMask Mask = EmulatedNetworkMask;
		if (Mask == EMovieSceneServerClientMask::All)
		{
			UObject* PlaybackContext = Player.GetPlaybackContext();
			UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;

			if (World)
			{
				ENetMode NetMode = World->GetNetMode();
				if (NetMode == ENetMode::NM_DedicatedServer)
				{
					Mask = EMovieSceneServerClientMask::Server;
				}
				else if (NetMode == ENetMode::NM_Client)
				{
					Mask = EMovieSceneServerClientMask::Client;
				}
			}
		}
		InCompiledDataManager = UMovieSceneCompiledDataManager::GetPrecompiledData(Mask);
#else
		InCompiledDataManager = UMovieSceneCompiledDataManager::GetPrecompiledData();
#endif
	}
	bReinitialize |= (PreviousCompiledDataManager != InCompiledDataManager);

	// Reinitialize if the runner has changed.
	TSharedPtr<FMovieSceneEntitySystemRunner> PreviousRunner = SharedPlaybackState ? SharedPlaybackState->GetRunner() : nullptr;

	// Reinitialize if the root sequence has changed.
	UMovieSceneSequence* PreviousRootSequence = SharedPlaybackState ? 
		SharedPlaybackState->GetRootSequence() : nullptr;
	bReinitialize |= (PreviousRootSequence != &InRootSequence);

	if (bReinitialize)
	{
		FRootInstanceHandle OldRootInstanceHandle = SharedPlaybackState ?
			SharedPlaybackState->GetRootInstanceHandle() : FRootInstanceHandle();

		if (OldRootInstanceHandle.IsValid() && bOwnsSharedPlaybackState)
		{
			if (PreviousRunner)
			{
				// Reset out SharedPlaybackState in a callback executed soon after the last update.
				// This is because:
				//
				// 1) We want it to still be valid during the last update, as some legacy things might go through
				//    the IMovieScenePlayer::GetEvaluationTemplate() method, which is us, and we need a valid
				//    SharedPlaybackState to still be functional during that time.
				//
				// 2) We also want it to be null when the runner finally destroys the FSequenceInstance, because
				//    its destructor has a leak-detection assert that checks that nobody has lingering references
				//    to the SharedPlaybackState.
				//
				auto OnFlushBeforeDestroy = [this]()
				{
					SharedPlaybackState.Reset();
				};

				PreviousRunner->AbandonAndDestroyInstance(
						OldRootInstanceHandle,
						FSimpleDelegate::CreateLambda(OnFlushBeforeDestroy));
			}
			else if (EntitySystemLinker)
			{
				EntitySystemLinker->GetInstanceRegistry()->DestroyInstance(OldRootInstanceHandle);
			}
			else
			{
				ensureMsgf(false, TEXT("Unable to destroy previously allocated sequence instance - this could indicate a memory leak."));
			}
		}

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Player.State.PersistentEntityData.Reset();
		Player.State.PersistentSharedData.Reset();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		
		// Build a new linker and immediately attach the runner to it, so that we can find both
		// when initializing the new root instance and shared playback state.
		EntitySystemLinker = ConstructEntityLinker(Player);

		// Create the new root instance and save its new shared playback state.
		FRootInstanceHandle NewRootInstanceHandle;
		if (EntitySystemLinker != nullptr && EntitySystemLinker->GetInstanceRegistry())
		{
			UObject* PlaybackContext = Player.GetPlaybackContext();
			FInstanceRegistry* InstanceRegistry = EntitySystemLinker->GetInstanceRegistry();
			NewRootInstanceHandle = InstanceRegistry->AllocateRootInstance(InRootSequence, PlaybackContext, InCompiledDataManager);
			SharedPlaybackState = InstanceRegistry->GetInstance(NewRootInstanceHandle).GetSharedPlaybackState();
			Player.InitializeRootInstance(SharedPlaybackState.ToSharedRef());
		}

		if (NewRootInstanceHandle.IsValid())
		{
			Player.PreAnimatedState.Initialize(EntitySystemLinker, NewRootInstanceHandle);
		}

		bOwnsSharedPlaybackState = true;
	}
}

void FMovieSceneRootEvaluationTemplateInstance::Initialize(TSharedRef<UE::MovieScene::FSharedPlaybackState> InSharedPlaybackState)
{
	EntitySystemLinker = InSharedPlaybackState->GetLinker();
	SharedPlaybackState = InSharedPlaybackState;
	bOwnsSharedPlaybackState = false;
}

void FMovieSceneRootEvaluationTemplateInstance::EvaluateSynchronousBlocking(FMovieSceneContext Context, IMovieScenePlayer& Player)
{
	EvaluateSynchronousBlocking(Context);
}

void FMovieSceneRootEvaluationTemplateInstance::EvaluateSynchronousBlocking(FMovieSceneContext Context)
{
	if (TSharedPtr<FMovieSceneEntitySystemRunner> Runner = GetRunner())
	{
		Runner->QueueUpdate(Context, GetRootInstanceHandle());
		Runner->Flush();
	}
}

void FMovieSceneRootEvaluationTemplateInstance::ResetDirectorInstances()
{
	using namespace UE::MovieScene;

	if (SharedPlaybackState)
	{
		if (FSequenceDirectorPlaybackCapability* Cap = SharedPlaybackState->FindCapability<FSequenceDirectorPlaybackCapability>())
		{
			Cap->ResetDirectorInstances();
		}
	}
}

bool FMovieSceneRootEvaluationTemplateInstance::IsValid() const
{
	return EntitySystemLinker && SharedPlaybackState;
}

TSharedPtr<UE::MovieScene::FSharedPlaybackState> FMovieSceneRootEvaluationTemplateInstance::GetSharedPlaybackState()
{
	return SharedPlaybackState;
}

TSharedPtr<const UE::MovieScene::FSharedPlaybackState> FMovieSceneRootEvaluationTemplateInstance::GetSharedPlaybackState() const
{
	return SharedPlaybackState;
}

UE::MovieScene::FRootInstanceHandle FMovieSceneRootEvaluationTemplateInstance::GetRootInstanceHandle() const
{
	return SharedPlaybackState ? SharedPlaybackState->GetRootInstanceHandle() : UE::MovieScene::FRootInstanceHandle();
}

UMovieSceneSequence* FMovieSceneRootEvaluationTemplateInstance::GetRootSequence() const
{
	using namespace UE::MovieScene;

	if (SharedPlaybackState)
	{
		return SharedPlaybackState->GetRootSequence();
	}
	return nullptr;
}

FMovieSceneCompiledDataID FMovieSceneRootEvaluationTemplateInstance::GetCompiledDataID() const
{
	return SharedPlaybackState ? SharedPlaybackState->GetRootCompiledDataID() : FMovieSceneCompiledDataID();
}

UMovieSceneCompiledDataManager* FMovieSceneRootEvaluationTemplateInstance::GetCompiledDataManager() const
{
	return SharedPlaybackState ? SharedPlaybackState->GetCompiledDataManager() : nullptr;
}

TSharedPtr<FMovieSceneEntitySystemRunner> FMovieSceneRootEvaluationTemplateInstance::GetRunner() const
{
	return SharedPlaybackState ? SharedPlaybackState->GetRunner() : TSharedPtr<FMovieSceneEntitySystemRunner>();
}

void FMovieSceneRootEvaluationTemplateInstance::EnableGlobalPreAnimatedStateCapture()
{
	using namespace UE::MovieScene;

	if (ensure(SharedPlaybackState))
	{
		SharedPlaybackState->GetPreAnimatedState().EnableGlobalPreAnimatedStateCapture();
	}
}

UMovieSceneSequence* FMovieSceneRootEvaluationTemplateInstance::GetSequence(FMovieSceneSequenceIDRef SequenceID) const
{
	return SharedPlaybackState ? SharedPlaybackState->GetSequence(SequenceID) : nullptr;
}

UMovieSceneEntitySystemLinker* FMovieSceneRootEvaluationTemplateInstance::GetEntitySystemLinker() const
{
	return EntitySystemLinker;
}

bool FMovieSceneRootEvaluationTemplateInstance::HasEverUpdated() const
{
	const UE::MovieScene::FRootInstanceHandle RootInstanceHandle = GetRootInstanceHandle();
	if (EntitySystemLinker && RootInstanceHandle.IsValid())
	{
		const UE::MovieScene::FSequenceInstance& SequenceInstance = EntitySystemLinker->GetInstanceRegistry()->GetInstance(RootInstanceHandle);
		return SequenceInstance.HasEverUpdated();
	}

	return false;
}

const FMovieSceneSequenceHierarchy* FMovieSceneRootEvaluationTemplateInstance::GetHierarchy() const
{
	return SharedPlaybackState ? SharedPlaybackState->GetHierarchy() : nullptr;
}

void FMovieSceneRootEvaluationTemplateInstance::GetSequenceParentage(const UE::MovieScene::FInstanceHandle InstanceHandle, TArray<UE::MovieScene::FInstanceHandle>& OutParentHandles) const
{
	using namespace UE::MovieScene;

	if (!ensure(EntitySystemLinker))
	{
		return;
	}

	// Get the root instance so we can find all necessary sub-instances from it.
	const FInstanceRegistry* InstanceRegistry = EntitySystemLinker->GetInstanceRegistry();

	check(InstanceHandle.IsValid());
	const FSequenceInstance& Instance = InstanceRegistry->GetInstance(InstanceHandle);

	const UE::MovieScene::FRootInstanceHandle RootInstanceHandle = GetRootInstanceHandle();
	checkf(Instance.GetRootInstanceHandle() == RootInstanceHandle, TEXT("The provided instance handle relates to a different root sequence."));
	const FSequenceInstance& RootInstance = InstanceRegistry->GetInstance(RootInstanceHandle);

	// Find the hierarchy node for the provided instance, and walk up from there to populate the output array.
	const FMovieSceneSequenceHierarchy* Hierarchy = GetHierarchy();
	if (!ensure(Hierarchy))
	{
		return;
	}

	const FMovieSceneSequenceHierarchyNode* HierarchyNode = Hierarchy->FindNode(Instance.GetSequenceID());
	while (HierarchyNode && HierarchyNode->ParentID.IsValid())
	{
		if (HierarchyNode->ParentID != MovieSceneSequenceID::Root)
		{
			const FInstanceHandle ParentHandle = RootInstance.FindSubInstance(HierarchyNode->ParentID);
			OutParentHandles.Add(ParentHandle);
		}
		else
		{
			OutParentHandles.Add(RootInstanceHandle);
		}
		HierarchyNode = Hierarchy->FindNode(HierarchyNode->ParentID);
	}
}

const UE::MovieScene::FSequenceInstance* FMovieSceneRootEvaluationTemplateInstance::GetRootInstance() const
{
	const UE::MovieScene::FRootInstanceHandle RootInstanceHandle = GetRootInstanceHandle();
	if (RootInstanceHandle.IsValid())
	{
		const UE::MovieScene::FInstanceRegistry* InstanceRegistry = EntitySystemLinker->GetInstanceRegistry();
		return &InstanceRegistry->GetInstance(RootInstanceHandle);
	}
	return nullptr;
}

UE::MovieScene::FSequenceInstance* FMovieSceneRootEvaluationTemplateInstance::FindInstance(FMovieSceneSequenceID SequenceID)
{
	using namespace UE::MovieScene;

	const UE::MovieScene::FRootInstanceHandle RootInstanceHandle = GetRootInstanceHandle();
	if (ensure(EntitySystemLinker && RootInstanceHandle.IsValid()))
	{
		FSequenceInstance* SequenceInstance = &EntitySystemLinker->GetInstanceRegistry()->MutateInstance(RootInstanceHandle);

		if (SequenceID == MovieSceneSequenceID::Root)
		{
			return SequenceInstance;
		}

		FInstanceHandle SubHandle = SequenceInstance->FindSubInstance(SequenceID);
		if (SubHandle.IsValid())
		{
			return &EntitySystemLinker->GetInstanceRegistry()->MutateInstance(SubHandle);
		}
	}

	return nullptr;
}

const UE::MovieScene::FSequenceInstance* FMovieSceneRootEvaluationTemplateInstance::FindInstance(FMovieSceneSequenceID SequenceID) const
{
	using namespace UE::MovieScene;

	const UE::MovieScene::FRootInstanceHandle RootInstanceHandle = GetRootInstanceHandle();
	if (ensure(EntitySystemLinker && RootInstanceHandle.IsValid()))
	{
		const FSequenceInstance* SequenceInstance = &EntitySystemLinker->GetInstanceRegistry()->GetInstance(RootInstanceHandle);

		if (SequenceID == MovieSceneSequenceID::Root)
		{
			return SequenceInstance;
		}

		FInstanceHandle SubHandle = SequenceInstance->FindSubInstance(SequenceID);
		if (SubHandle.IsValid())
		{
			return &EntitySystemLinker->GetInstanceRegistry()->GetInstance(SubHandle);
		}
	}

	return nullptr;
}

UE::MovieScene::FMovieSceneEntityID FMovieSceneRootEvaluationTemplateInstance::FindEntityFromOwner(UObject* Owner, uint32 EntityID, FMovieSceneSequenceID SequenceID) const
{
	using namespace UE::MovieScene;

	if (const FSequenceInstance* SequenceInstance = FindInstance(SequenceID))
	{
		return SequenceInstance->FindEntity(Owner, EntityID);
	}

	return FMovieSceneEntityID::Invalid();
}

void FMovieSceneRootEvaluationTemplateInstance::FindEntitiesFromOwner(UObject* Owner, FMovieSceneSequenceID SequenceID, TArray<UE::MovieScene::FMovieSceneEntityID>& OutEntityIDs) const
{
	using namespace UE::MovieScene;

	if (const FSequenceInstance* SequenceInstance = FindInstance(SequenceID))
	{
		SequenceInstance->FindEntities(Owner, OutEntityIDs);
	}
}

UObject* FMovieSceneRootEvaluationTemplateInstance::GetOrCreateDirectorInstance(FMovieSceneSequenceIDRef SequenceID, IMovieScenePlayer& Player)
{
	using namespace UE::MovieScene;

	if (SharedPlaybackState)
	{
		if (FSequenceDirectorPlaybackCapability* Cap = SharedPlaybackState->FindCapability<FSequenceDirectorPlaybackCapability>())
		{
			return Cap->GetOrCreateDirectorInstance(SharedPlaybackState.ToSharedRef(), SequenceID);
		}
	}
	return nullptr;
}

void FMovieSceneRootEvaluationTemplateInstance::PlaybackContextChanged(IMovieScenePlayer& Player)
{
	using namespace UE::MovieScene;

	if (!ensure(bOwnsSharedPlaybackState))
	{
		return;
	}

	// Only the playback context changed, so we keep the same sequence, runner, and compiled data manager.
	UMovieSceneSequence* RootSequence = SharedPlaybackState->GetRootSequence();
	UMovieSceneCompiledDataManager* CompiledDataManager = SharedPlaybackState->GetCompiledDataManager();

	const bool bGlobalCapture = SharedPlaybackState->GetPreAnimatedState().IsCapturingGlobalPreAnimatedState();
	const FRootInstanceHandle PreviousRootInstanceHandle = SharedPlaybackState->GetRootInstanceHandle();

	if (PreviousRootInstanceHandle.IsValid() &&
			EntitySystemLinker && 
			IsValidChecked(EntitySystemLinker) &&
			!EntitySystemLinker->IsUnreachable() && 
			!EntitySystemLinker->HasAnyFlags(RF_BeginDestroyed))
	{
		EntitySystemLinker->CleanupInvalidBoundObjects();

		// Update the playback client from the player - in some cases this points to an actor that could be destroyed when the playback context changes
		//   so we overwrite it with the latest value (which could be nullptr, but that's fine)
		if (SharedPlaybackState->HasCapability<IMovieScenePlaybackClient>())
		{
			SharedPlaybackState->GetCapabilities().OverwriteCapabilityRaw(Player.GetPlaybackClient());
		}

		TSharedPtr<FMovieSceneEntitySystemRunner> Runner = SharedPlaybackState->GetRunner();
		if (Runner)
		{
			if (Runner->QueueFinalUpdate(PreviousRootInstanceHandle))
			{
				Runner->Flush();
			}
		}

		if (bGlobalCapture)
		{
			Player.RestorePreAnimatedState();
		}

		// Forget our previous shared playback state before we destroy our root instance, because the instance
		// wants to make sure the state will be destroyed with it.
		SharedPlaybackState.Reset();

		EntitySystemLinker->GetInstanceRegistry()->DestroyInstance(PreviousRootInstanceHandle);
	}

	EntitySystemLinker = ConstructEntityLinker(Player);

	UObject* PlaybackContext = Player.GetPlaybackContext();
	FInstanceRegistry* InstanceRegistry = EntitySystemLinker->GetInstanceRegistry();
	const FRootInstanceHandle RootInstanceHandle = InstanceRegistry->AllocateRootInstance(
			*RootSequence, PlaybackContext, CompiledDataManager);
	FSequenceInstance& RootInstance = InstanceRegistry->MutateInstance(RootInstanceHandle);
	SharedPlaybackState = RootInstance.GetSharedPlaybackState();
	Player.InitializeRootInstance(SharedPlaybackState.ToSharedRef());

	Player.PreAnimatedState.Initialize(EntitySystemLinker, RootInstanceHandle);
	if (bGlobalCapture)
	{
		SharedPlaybackState->GetPreAnimatedState().EnableGlobalPreAnimatedStateCapture();
	}
}

const FMovieSceneSubSequenceData* FMovieSceneRootEvaluationTemplateInstance::FindSubData(FMovieSceneSequenceIDRef SequenceID) const
{
	if (const FMovieSceneSequenceHierarchy* Hierarchy = SharedPlaybackState->GetHierarchy())
	{
		return Hierarchy->FindSubData(SequenceID);
	}
	return nullptr;
}

void FMovieSceneRootEvaluationTemplateInstance::CopyActuators(FMovieSceneBlendingAccumulator& Accumulator) const
{
	using namespace UE::MovieScene;

	FRootInstanceHandle RootInstanceHandle = SharedPlaybackState->GetRootInstanceHandle();
	const FSequenceInstance& SequenceInstance = EntitySystemLinker->GetInstanceRegistry()->GetInstance(RootInstanceHandle);
	const FMovieSceneTrackEvaluator* LegacyEvaluator = SequenceInstance.GetLegacyEvaluator();

	if (LegacyEvaluator)
	{
		LegacyEvaluator->CopyActuators(Accumulator);
	}
}

#if WITH_EDITOR
void FMovieSceneRootEvaluationTemplateInstance::SetEmulatedNetworkMask(EMovieSceneServerClientMask InNewMask, IMovieScenePlayer& Player)
{
	SetEmulatedNetworkMask(InNewMask);
}
void FMovieSceneRootEvaluationTemplateInstance::SetEmulatedNetworkMask(EMovieSceneServerClientMask InNewMask)
{
	check(InNewMask != EMovieSceneServerClientMask::None);
	EmulatedNetworkMask = InNewMask;
}
EMovieSceneServerClientMask FMovieSceneRootEvaluationTemplateInstance::GetEmulatedNetworkMask() const
{
	return EmulatedNetworkMask;
}
#endif

