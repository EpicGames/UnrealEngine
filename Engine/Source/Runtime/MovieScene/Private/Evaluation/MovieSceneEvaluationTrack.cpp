// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/MovieSceneEvaluationTrack.h"
#include "Evaluation/MovieSceneExecutionTokens.h"

#include "Compilation/MovieSceneCompilerRules.h"
#include "MovieSceneTrack.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneTimeHelpers.h"

#include "Algo/BinarySearch.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneEvaluationTrack)


#if WITH_DEV_AUTOMATION_TESTS

	struct FGetTrackSegmentBlender
	{
		FMovieSceneTrackSegmentBlender* operator()(const UMovieSceneTrack* Track, FMovieSceneTrackSegmentBlenderPtr& OutOwnerPtr) const
		{
			if (Override.IsValid())
			{
				return &Override.GetValue();
			}
			else if (Track)
			{
				OutOwnerPtr = Track->GetTrackSegmentBlender();
			}

			return OutOwnerPtr.GetPtr();
		}

		static FMovieSceneTrackSegmentBlenderPtr Override;

	} GetTrackSegmentBlender;

	FMovieSceneTrackSegmentBlenderPtr FGetTrackSegmentBlender::Override;

	FScopedOverrideTrackSegmentBlender::FScopedOverrideTrackSegmentBlender(FMovieSceneTrackSegmentBlenderPtr&& InTrackSegmentBlender)
	{
		check(!GetTrackSegmentBlender.Override.IsValid());
		GetTrackSegmentBlender.Override = MoveTemp(InTrackSegmentBlender);
	}

	FScopedOverrideTrackSegmentBlender::~FScopedOverrideTrackSegmentBlender()
	{
		GetTrackSegmentBlender.Override.Reset();
	}

#else	// WITH_DEV_AUTOMATION_TESTS

	FMovieSceneTrackSegmentBlender* GetTrackSegmentBlender(const UMovieSceneTrack* Track, FMovieSceneTrackSegmentBlenderPtr& OutOwnerPtr)
	{
		if (Track)
		{
			OutOwnerPtr = Track->GetTrackSegmentBlender();
		}
		return OutOwnerPtr.GetPtr();
	}

#endif	// WITH_DEV_AUTOMATION_TESTS


FMovieSceneEvaluationTrack::FMovieSceneEvaluationTrack()
	: EvaluationPriority(1000)
	, EvaluationMethod(EEvaluationMethod::Static)
	, SourceTrack(nullptr)
	, bEvaluateInPreroll(true)
	, bEvaluateInPostroll(true)
	, bTearDownPriority(false)
{
}

FMovieSceneEvaluationTrack::FMovieSceneEvaluationTrack(const FGuid& InObjectBindingID)
	: ObjectBindingID(InObjectBindingID)
	, EvaluationPriority(1000)
	, EvaluationMethod(EEvaluationMethod::Static)
	, SourceTrack(nullptr)
	, bEvaluateInPreroll(true)
	, bEvaluateInPostroll(true)
	, bTearDownPriority(false)
{
}

void FMovieSceneEvaluationTrack::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading() && !Ar.IsObjectReferenceCollector())
	{
		SetupOverrides();
	}
}

void FMovieSceneEvaluationTrack::DefineAsSingleTemplate(FMovieSceneEvalTemplatePtr&& InTemplate)
{
	ChildTemplates.Reset(1);
	ChildTemplates.Add(MoveTemp(InTemplate));
}

int32 FMovieSceneEvaluationTrack::AddChildTemplate(FMovieSceneEvalTemplatePtr&& InTemplate)
{
	return ChildTemplates.Add(MoveTemp(InTemplate));
}

void FMovieSceneEvaluationTrack::SetupOverrides()
{
	for (FMovieSceneEvalTemplatePtr& ChildTemplate : ChildTemplates)
	{
		if (ChildTemplate.IsValid())
		{
			ChildTemplate->SetupOverrides();
		}
	}

	if (TrackTemplate.IsValid())
	{
		TrackTemplate->SetupOverrides();
	}
}

void FMovieSceneEvaluationTrack::Initialize(TArrayView<const FMovieSceneFieldEntry_ChildTemplate> Children, const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) const
{
	if (!TrackTemplate.IsValid() || !TrackTemplate->HasCustomInitialize())
	{
		DefaultInitialize(Children, Operand, Context, PersistentData, Player);
	}
	else
	{
		TrackTemplate->Initialize(*this, Children, Operand, Context, PersistentData, Player);
	}
}

void FMovieSceneEvaluationTrack::DefaultInitialize(TArrayView<const FMovieSceneFieldEntry_ChildTemplate> Children, const FMovieSceneEvaluationOperand& Operand, FMovieSceneContext Context, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) const
{
	for (FMovieSceneFieldEntry_ChildTemplate FieldEntry : Children)
	{
		const FMovieSceneEvalTemplate* Template = HasChildTemplate(FieldEntry.ChildIndex) ? &GetChildTemplate(FieldEntry.ChildIndex) : nullptr;
		if (!Template)
		{
			continue;
		}

		if (Template->RequiresInitialization())
		{
			PersistentData.DeriveSectionKey(FieldEntry.ChildIndex);
			Context.OverrideTime(FieldEntry.ForcedTime);

			const bool bIsPreRoll  = EnumHasAnyFlags(FieldEntry.Flags, ESectionEvaluationFlags::PreRoll);
			const bool bIsPostRoll = EnumHasAnyFlags(FieldEntry.Flags, ESectionEvaluationFlags::PostRoll);

			Context.ApplySectionPrePostRoll(bIsPreRoll, bIsPostRoll);

			Template->Initialize(Operand, Context, PersistentData, Player);
		}
	}
}

void FMovieSceneEvaluationTrack::Evaluate(TArrayView<const FMovieSceneFieldEntry_ChildTemplate> Children, const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const
{
	if (!TrackTemplate.IsValid() || !TrackTemplate->HasCustomEvaluate())
	{
		DefaultEvaluate(Children, Operand, Context, PersistentData, ExecutionTokens);
	}
	else
	{
		TrackTemplate->Evaluate(*this, Children, Operand, Context, PersistentData, ExecutionTokens);
	}
}

void FMovieSceneEvaluationTrack::DefaultEvaluate(TArrayView<const FMovieSceneFieldEntry_ChildTemplate> Children, const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const
{
	switch (EvaluationMethod)
	{
	case EEvaluationMethod::Static:
		EvaluateStatic(Children, Operand, Context, PersistentData, ExecutionTokens);
		break;
	case EEvaluationMethod::Swept:
		EvaluateSwept(Children, Operand, Context, PersistentData, ExecutionTokens);
		break;
	}
}

void FMovieSceneEvaluationTrack::EvaluateStatic(TArrayView<const FMovieSceneFieldEntry_ChildTemplate> Children, const FMovieSceneEvaluationOperand& Operand, FMovieSceneContext Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const
{
	// If we only have a single child template and a single segment, short-circuit the lookup into the impl array to avoid the unnecessary cache misses
	for (FMovieSceneFieldEntry_ChildTemplate FieldEntry : Children)
	{
		const FMovieSceneEvalTemplate* Template = HasChildTemplate(FieldEntry.ChildIndex) ? &GetChildTemplate(FieldEntry.ChildIndex) : nullptr;
		if (!Template)
		{
			continue;
		}

		Context.OverrideTime(FieldEntry.ForcedTime);

		const bool bIsPreRoll  = EnumHasAnyFlags(FieldEntry.Flags, ESectionEvaluationFlags::PreRoll);
		const bool bIsPostRoll = EnumHasAnyFlags(FieldEntry.Flags, ESectionEvaluationFlags::PostRoll);

		Context.ApplySectionPrePostRoll(bIsPreRoll, bIsPostRoll);

		EMovieSceneCompletionMode CompletionMode =
			EnumHasAnyFlags(FieldEntry.Flags, ESectionEvaluationFlags::ForceRestoreState) ? EMovieSceneCompletionMode::RestoreState :
			EnumHasAnyFlags(FieldEntry.Flags, ESectionEvaluationFlags::ForceKeepState)    ? EMovieSceneCompletionMode::KeepState :
			Template->GetCompletionMode();

		PersistentData.DeriveSectionKey(FieldEntry.ChildIndex);
		ExecutionTokens.SetCurrentScope(FMovieSceneEvaluationScope(PersistentData.GetSectionKey(), CompletionMode));
		ExecutionTokens.SetContext(Context);

		Template->Evaluate(Operand, Context, PersistentData, ExecutionTokens);
	}
}

void FMovieSceneEvaluationTrack::EvaluateSwept(TArrayView<const FMovieSceneFieldEntry_ChildTemplate> Children, const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const
{
	FFrameNumber Time = Context.GetTime().FrameNumber;
	for (const FMovieSceneTrackEvaluationFieldEntry& Entry : SourceTrack->GetEvaluationField().Entries)
	{
		if (!Entry.Range.Contains(Time) || Entry.Flags != ESectionEvaluationFlags::None)
		{
			continue;
		}

		auto MatchTemplateToSection = [Section=Entry.Section](const FMovieSceneEvalTemplatePtr& ChildTemplate)
		{
			return ChildTemplate.IsValid() && ChildTemplate->GetSourceSection() == Section;
		};

		const int32 ChildIndex = ChildTemplates.IndexOfByPredicate(MatchTemplateToSection);
		if (ChildIndex != INDEX_NONE)
		{
			const FMovieSceneEvalTemplate& Template = *ChildTemplates[ChildIndex];

			EMovieSceneCompletionMode CompletionMode =
				EnumHasAnyFlags(Entry.Flags, ESectionEvaluationFlags::ForceRestoreState) ? EMovieSceneCompletionMode::RestoreState :
				EnumHasAnyFlags(Entry.Flags, ESectionEvaluationFlags::ForceKeepState)    ? EMovieSceneCompletionMode::KeepState :
				Template.GetCompletionMode();

			PersistentData.DeriveSectionKey(ChildIndex);
			ExecutionTokens.SetCurrentScope(FMovieSceneEvaluationScope(PersistentData.GetSectionKey(), CompletionMode));
			ExecutionTokens.SetContext(Context);

			Template.EvaluateSwept(
				Operand,
				Context,
				TRange<FFrameNumber>::Intersection(Entry.Range, Context.GetFrameNumberRange()),
				PersistentData,
				ExecutionTokens);
		}
	}
}

void FMovieSceneEvaluationTrack::Interrogate(const FMovieSceneContext& Context, FMovieSceneInterrogationData& Container, UObject* BindingOverride) const
{
	if (TrackTemplate.IsValid() && TrackTemplate->Interrogate(Context, Container, BindingOverride))
	{
		return;
	}

	FFrameNumber Time = Context.GetTime().FrameNumber;
	for (const FMovieSceneTrackEvaluationFieldEntry& Entry : SourceTrack->GetEvaluationField().Entries)
	{
		if (!Entry.Range.Contains(Time) || Entry.Flags != ESectionEvaluationFlags::None)
		{
			continue;
		}

		auto MatchTemplateToSection = [Section=Entry.Section](const FMovieSceneEvalTemplatePtr& ChildTemplate)
		{
			return ChildTemplate.IsValid() && ChildTemplate->GetSourceSection() == Section;
		};

		const int32 ChildIndex = ChildTemplates.IndexOfByPredicate(MatchTemplateToSection);
		if (ChildIndex != INDEX_NONE)
		{
			ChildTemplates[ChildIndex]->Interrogate(Context, Container, BindingOverride);
		}
	}

	// @todo: this should live higher up the callstack when whole template interrogation is supported
	Container.Finalize(Context, BindingOverride);
}

