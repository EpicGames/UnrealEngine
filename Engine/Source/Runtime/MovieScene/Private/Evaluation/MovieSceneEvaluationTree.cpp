// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/MovieSceneEvaluationTree.h"
#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "Compilation/MovieSceneSegmentCompiler.h"
#include "Compilation/MovieSceneCompilerRules.h"
#include "Evaluation/MovieSceneEvaluationField.h"
#include "MovieSceneCommonHelpers.h"
#include "UObject/Package.h"
#include "Templates/Tuple.h"
#include "MovieSceneTimeHelpers.h"


FMovieSceneEvaluationTreeRangeIterator::FMovieSceneEvaluationTreeRangeIterator(const FMovieSceneEvaluationTree& InTree)
	: CurrentRange(TRange<FFrameNumber>::All()), CurrentNodeHandle(FMovieSceneEvaluationTreeNodeHandle::Root()), Tree(&InTree)
{
	// Compute the starting range by inspecting the front of the tree
	const FMovieSceneEvaluationTreeNode* CurrentNode = &Tree->GetRootNode();
	TArrayView<const FMovieSceneEvaluationTreeNode> Children;

	// Find the first child-most node that doesn't match the opening range bound
	for(;;)
	{
		Children = Tree->GetChildren(*CurrentNode);
		if (Children.Num() && Children[0].Range.GetLowerBound().IsOpen())
		{
			// Recurse into the child
			CurrentNodeHandle = FMovieSceneEvaluationTreeNodeHandle(CurrentNode->ChildrenID, 0);
			CurrentNode = &Tree->GetNode(CurrentNodeHandle);
		}
		else
		{
			break;
		}
	}

	// The upper bound is either the current node's upper bound (if it has no children), or the inverse of its first child's lower bound
	TRangeBound<FFrameNumber> TrailingBound = Children.Num() ? TRangeBound<FFrameNumber>::FlipInclusion(Children[0].Range.GetLowerBound()) : CurrentNode->Range.GetUpperBound();
	CurrentRange = TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Open(), TrailingBound);
}

FMovieSceneEvaluationTreeRangeIterator::FMovieSceneEvaluationTreeRangeIterator(const FMovieSceneEvaluationTree& InTree, TRangeBound<FFrameNumber> StartingBound)
	: CurrentNodeHandle(FMovieSceneEvaluationTreeNodeHandle::Root()), Tree(&InTree)
{
	auto GetLowerBound = [](const FMovieSceneEvaluationTreeNode& In) {
		return In.Range.GetLowerBound();
	};

	TRange<FFrameNumber> CompareRange(StartingBound, TRangeBound<FFrameNumber>::Open());

	for (;;)
	{
		const FMovieSceneEvaluationTreeNode& ThisNode = Tree->GetNode(CurrentNodeHandle);

		// Keep binary searching child nodes until we hit a leaf
		CurrentRange = ThisNode.Range;
		if (!ThisNode.ChildrenID.IsValid())
		{
			break;
		}

		TArrayView<const FMovieSceneEvaluationTreeNode> Children = Tree->GetChildren(ThisNode);

		// Binary search children's lower bounds for the first that's >= the starting bound. That results in either ChildIndex or ChildIndex-1 being the overlapping range (if any).
		int32 ChildIndex = Algo::LowerBoundBy(Children, StartingBound, GetLowerBound, MovieSceneHelpers::SortLowerBounds);
		if (Children.IsValidIndex(ChildIndex) && Children[ChildIndex].Range.GetLowerBound() == StartingBound)
		{
			CurrentNodeHandle = FMovieSceneEvaluationTreeNodeHandle(ThisNode.ChildrenID, ChildIndex);
		}
		else if (Children.IsValidIndex(ChildIndex-1) && Children[ChildIndex-1].Range.Overlaps(CompareRange))
		{
			CurrentNodeHandle = FMovieSceneEvaluationTreeNodeHandle(ThisNode.ChildrenID, ChildIndex - 1);
		}
		else
		{
			// We're at some empty space between children
			CurrentRange = TRange<FFrameNumber>(
				Children.IsValidIndex(ChildIndex-1) ? TRangeBound<FFrameNumber>::FlipInclusion(Children[ChildIndex-1].Range.GetUpperBound()) : CurrentRange.GetLowerBound(),
				Children.IsValidIndex(ChildIndex)   ? TRangeBound<FFrameNumber>::FlipInclusion(Children[ChildIndex].Range.GetLowerBound())   : CurrentRange.GetUpperBound()
			);

			break;
		}
	}

	check(CurrentRange.Overlaps(CompareRange));
}

void FMovieSceneEvaluationTreeRangeIterator::Iter(bool bForwards)
{
	if (!CurrentNodeHandle.IsValid())
	{
		return;
	}

	const FMovieSceneEvaluationTreeNode& CurrentNode = Tree->GetNode(CurrentNodeHandle);
	if (GetTrailingBound(bForwards, CurrentNode.Range) == GetTrailingBound(bForwards, CurrentRange))
	{
		// We're done with this node, iterate forwards from the same position in the parent
		CurrentNodeHandle = CurrentNode.Parent;
		Iter(bForwards);
		return;
	}

	TRangeBound<FFrameNumber> NewLeadingBound = CurrentRange.IsEmpty() ? TRangeBound<FFrameNumber>::Open() : TRangeBound<FFrameNumber>::FlipInclusion(GetTrailingBound(bForwards, CurrentRange));

	// Iterate into children when possible (where the leading bound matches the current new leading bound)
	FMovieSceneEvaluationTreeNodeHandle NextChildHandle = FindNextChild(CurrentNodeHandle, NewLeadingBound, bForwards);
	while (NextChildHandle.IsValid() && CompareBound(bForwards, Tree->GetNode(NextChildHandle).Range, NewLeadingBound))
	{
		CurrentNodeHandle = NextChildHandle;
		NextChildHandle = FindNextChild(CurrentNodeHandle, NewLeadingBound, bForwards);
	}

	// The new traililng bound is either the trailing bound of the whole node, or the inverse of the leading bound of the next child (which will only be valid if it's not the same as the leading bound).
	// The latter means this is space in-between child nodes
	TRangeBound<FFrameNumber> NewTrailingBound = GetTrailingBound(bForwards, Tree->GetNode(CurrentNodeHandle).Range);
	if (NextChildHandle.IsValid())
	{
		const FMovieSceneEvaluationTreeNode& NextChildNode = Tree->GetNode(NextChildHandle);
		if (!CompareBound(bForwards, NextChildNode.Range, NewLeadingBound))
		{
			NewTrailingBound = TRangeBound<FFrameNumber>::FlipInclusion(GetLeadingBound(bForwards, NextChildNode.Range));
		}
	}

	CurrentRange = TRange<FFrameNumber>(bForwards ? NewLeadingBound : NewTrailingBound, bForwards ? NewTrailingBound : NewLeadingBound);
}

FMovieSceneEvaluationTreeNodeHandle FMovieSceneEvaluationTreeRangeIterator::FindNextChild(FMovieSceneEvaluationTreeNodeHandle ParentNodeHandle, TRangeBound<FFrameNumber> PredicateBound, bool bForwards)
{
	auto GetLowerBound = [](const FMovieSceneEvaluationTreeNode& In) {
		return In.Range.GetLowerBound();
	};
	auto GetUpperBound = [](const FMovieSceneEvaluationTreeNode& In) {
		return In.Range.GetUpperBound();
	};

	const FMovieSceneEvaluationTreeNode& ParentNode = Tree->GetNode(ParentNodeHandle);
	TArrayView<const FMovieSceneEvaluationTreeNode> Children = Tree->GetChildren(ParentNode);
	if (Children.Num())
	{
		// Find the index of the next >= bound
		int32 SearchIndex = bForwards
			? Algo::LowerBoundBy(Children, PredicateBound, GetLowerBound, MovieSceneHelpers::SortLowerBounds)
			: Algo::UpperBoundBy(Children, PredicateBound, GetUpperBound, MovieSceneHelpers::SortUpperBounds)-1;

		if (Children.IsValidIndex(SearchIndex))
		{
			return FMovieSceneEvaluationTreeNodeHandle(ParentNode.ChildrenID, SearchIndex);
		}
	}

	return FMovieSceneEvaluationTreeNodeHandle::Invalid();
}


TArrayView<const FMovieSceneEvaluationTreeNode> FMovieSceneEvaluationTree::GetChildren(const FMovieSceneEvaluationTreeNode& InNode) const
{
	return InNode.ChildrenID.IsValid() ? ChildNodes.Get(InNode.ChildrenID) : TArrayView<const FMovieSceneEvaluationTreeNode>();
}

TArrayView<FMovieSceneEvaluationTreeNode> FMovieSceneEvaluationTree::GetChildren(const FMovieSceneEvaluationTreeNode& InNode)
{
	return InNode.ChildrenID.IsValid() ? ChildNodes.Get(InNode.ChildrenID) : TArrayView<FMovieSceneEvaluationTreeNode>();
}

FMovieSceneEvaluationTreeRangeIterator FMovieSceneEvaluationTree::IterateFromTime(FFrameNumber InTime) const
{
	return IterateFromLowerBound(TRangeBound<FFrameNumber>::Inclusive(InTime));
}

FMovieSceneEvaluationTreeRangeIterator FMovieSceneEvaluationTree::IterateFromLowerBound(TRangeBound<FFrameNumber> InStartingLowerBound) const
{
	return FMovieSceneEvaluationTreeRangeIterator(*this, InStartingLowerBound);
}

void FMovieSceneEvaluationTree::InsertNewChild(TRange<FFrameNumber> InEffectiveRange, const IMovieSceneEvaluationTreeNodeOperator& InOperator, int32 InsertIndex, FMovieSceneEvaluationTreeNodeHandle InParent)
{
	// Check if this parent node has any 
	FEvaluationTreeEntryHandle ChildID = GetNode(InParent).ChildrenID;
	if (!ChildID.IsValid())
	{
		// AllocateEntry may reallocate the whole node array
		ChildID = ChildNodes.AllocateEntry(1);
		GetNode(InParent).ChildrenID = ChildID;
	}
	else
	{
		// Fixup parent handles for any grandchild nodes that reside as subsequent children
		TArrayView<FMovieSceneEvaluationTreeNode> ExistingChildren = ChildNodes.Get(ChildID);
		for (int32 Index = InsertIndex; Index < ExistingChildren.Num(); ++Index)
		{
			FMovieSceneEvaluationTreeNodeHandle FixedUpParentNode(ChildID, Index+1);
			for (FMovieSceneEvaluationTreeNode& GrandChild : GetChildren(ExistingChildren[Index]))
			{
				GrandChild.Parent = FixedUpParentNode;
			}
		}
	}

	FMovieSceneEvaluationTreeNode NewChildNode(InEffectiveRange, InParent);
	InOperator(NewChildNode);
	ChildNodes.Insert(ChildID, InsertIndex, MoveTemp(NewChildNode));
}

void FMovieSceneEvaluationTree::AddTimeRange(TRange<FFrameNumber> InTimeRange)
{
	struct FNullOp : IMovieSceneEvaluationTreeNodeOperator
	{
		virtual void operator()(FMovieSceneEvaluationTreeNode& Node) const {}
	};

	AddTimeRange(InTimeRange, FNullOp(), FMovieSceneEvaluationTreeNodeHandle::Root(), nullptr);
}

void FMovieSceneEvaluationTree::AddTimeRange(TRange<FFrameNumber> InTimeRange, const IMovieSceneEvaluationTreeNodeOperator& InOperator, FMovieSceneEvaluationTreeNodeHandle InParent, const TFunctionRef<bool(FMovieSceneEvaluationTreeNodeHandle)>* Predicate)
{
	// Take a temporary copy of the node as the container may be reallocated in this function
	FMovieSceneEvaluationTreeNode ThisNode = GetNode(InParent);
	if (!ensure(ThisNode.Range.Overlaps(InTimeRange)))
	{
		return;
	}
	else if (Predicate && !(*Predicate)(InParent))
	{
		return;
	}

	if (InTimeRange.Contains(ThisNode.Range))
	{
		// If the section range entirely contains this node's range or it's exactly the same, just add the data to this entry
		// We must be sure to pass a reference to the actual here, not the temporary copy
		InOperator(GetNode(InParent));
	}
	else if (!ThisNode.ChildrenID.IsValid())
	{
		// We have no child nodes yet, so just add the intersection as a child at the start
		TRange<FFrameNumber> Intersection = TRange<FFrameNumber>::Intersection(ThisNode.Range, InTimeRange);
		InsertNewChild(Intersection, InOperator, 0, InParent);
	}
	else
	{
		// It must be a sub-division of this range, find the intersection and add new children as necessary
		TRangeBound<FFrameNumber> LastBound = ThisNode.Range.GetLowerBound();

		// Iterate all existing children, adding new children for gaps that the new section intersects
		int32 ChildIndex = 0;
		while (ChildIndex < ChildNodes.Get(ThisNode.ChildrenID).Num())
		{
			FMovieSceneEvaluationTreeNode ChildNode = GetNode(ThisNode.ChildrenID, ChildIndex);

			if (!ChildNode.Range.GetLowerBound().IsOpen())
			{
				TRange<FFrameNumber> PrecedingSpace(LastBound, TRangeBound<FFrameNumber>::FlipInclusion(ChildNode.Range.GetLowerBound()));
				TRange<FFrameNumber> PrecedingIntersection = TRange<FFrameNumber>::Intersection(PrecedingSpace, InTimeRange);

				if (!PrecedingIntersection.IsEmpty())
				{
					// Insert this child at the current child index
					InsertNewChild(PrecedingIntersection, InOperator, ChildIndex, InParent);

					// Skip over the child we just added
					ChildIndex++;
				}
			}

			if (ChildNode.Range.Overlaps(InTimeRange))
			{
				// Find the node again since InsertNewChild may have re-allocated the nodes
				AddTimeRange(InTimeRange, InOperator, FMovieSceneEvaluationTreeNodeHandle(ThisNode.ChildrenID, ChildIndex), Predicate);
			}
			LastBound = TRangeBound<FFrameNumber>::FlipInclusion(ChildNode.Range.GetUpperBound());
			ChildIndex++;
		}

		// If the last child didn't fully fill this parent node range, add a new one for this new section
		if (!LastBound.IsOpen())
		{
			TRange<FFrameNumber> ProceedingSpace(LastBound, ThisNode.Range.GetUpperBound());
			TRange<FFrameNumber> ProceedingIntersection = TRange<FFrameNumber>::Intersection(ProceedingSpace, InTimeRange);

			if (!ProceedingIntersection.IsEmpty())
			{
				// Insert this child at the end
				InsertNewChild(ProceedingIntersection, InOperator, ChildNodes.Get(ThisNode.ChildrenID).Num(), InParent);
			}
		}
	}
}