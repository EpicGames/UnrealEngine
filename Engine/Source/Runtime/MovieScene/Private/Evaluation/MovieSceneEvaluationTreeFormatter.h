// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "Evaluation/MovieSceneEvaluationTree.h"
#include "Misc/StringBuilder.h"
#include "MovieSceneFwd.h"
#include "MovieSceneTimeHelpers.h"
#include "Templates/Tuple.h"

#if !NO_LOGGING

template<typename DataType>
struct TMovieSceneEvaluationTreeFormatter
{
	DECLARE_DELEGATE_TwoParams(FOnFormatData, const DataType&, TStringBuilder<256>&);

	const TMovieSceneEvaluationTree<DataType>& Tree;

	struct FLogRowItem
	{
		int32 ParentRowItemIndex = INDEX_NONE;
		const FMovieSceneEvaluationTreeNode* Node = nullptr;
		int32 Width = 0;
		int32 ChildrenWidthTotal = 0;
		int32 LeftOffset = 0;
	};

	using FLogRow = TArray<FLogRowItem>;
	TArray<FLogRow> LogRows;

	using FDataIndex = TTuple<int32, int32>;
	using FDataInfo = TTuple<int32, TRange<FFrameNumber>>;
	TMap<FDataIndex, FDataInfo> DataIndices;

	int32 ParentItemMargin = 8;

	FOnFormatData DataFormatter;
	
	TMovieSceneEvaluationTreeFormatter(const TMovieSceneEvaluationTree<DataType>& InTree)
		: Tree(InTree)
	{}

	void LogTree()
	{
		LogRows.Reset();
		DataIndices.Reset();

		BuildLogRows(INDEX_NONE, &Tree.RootNode, 0);
		AccumulateWidths();
		AccumulateLeftOffsets();
		OutputLogRows();

		OutputData();
	}

private:
	void BuildLogRows(int32 InParentRowItemIndex, const FMovieSceneEvaluationTreeNode* InNode, int32 InDepth)
	{
		if (LogRows.Num() <= InDepth)
		{
			LogRows.SetNum(InDepth + 1);
		}
		FLogRow& LogRow = LogRows[InDepth];

		TArrayView<const DataType> NodeData = Tree.GetDataForSingleNode(*InNode);
		const int32 NodeWidth = GetItemMinWidth(NodeData.Num());
		const int32 CurItemIndex = LogRow.Add(FLogRowItem{InParentRowItemIndex, InNode, NodeWidth});

		if (InNode->ChildrenID.IsValid())
		{
			TArrayView<const FMovieSceneEvaluationTreeNode> ChildNodes = Tree.ChildNodes.Get(InNode->ChildrenID);
			for (const FMovieSceneEvaluationTreeNode& ChildNode : ChildNodes)
			{
				BuildLogRows(CurItemIndex, &ChildNode, InDepth + 1);
			}
		}
	}

	void AccumulateWidths()
	{
		for (int32 RowIndex = LogRows.Num() - 1; RowIndex >= 0; --RowIndex)
		{
			FLogRow& LogRow = LogRows[RowIndex];

			if (RowIndex > 0)
			{
				FLogRow& ParentLogRow = LogRows[RowIndex - 1];

				for (FLogRowItem& RowItem : LogRow)
				{
					if (RowItem.ChildrenWidthTotal > 0)
					{
						RowItem.Width = FMath::Max(RowItem.Width, RowItem.ChildrenWidthTotal + ParentItemMargin);
					}

					if (ensure(ParentLogRow.IsValidIndex(RowItem.ParentRowItemIndex)))
					{
						FLogRowItem& ParentRowItem = ParentLogRow[RowItem.ParentRowItemIndex];
						ParentRowItem.ChildrenWidthTotal += RowItem.Width;
					}
				}
			}
			else
			{
				for (FLogRowItem& RowItem : LogRow)
				{
					if (RowItem.ChildrenWidthTotal > 0)
					{
						RowItem.Width = FMath::Max(RowItem.Width, RowItem.ChildrenWidthTotal + ParentItemMargin);
					}
				}
			}
		}
	}

	void AccumulateLeftOffsets()
	{
		for (int32 RowIndex = 1; RowIndex < LogRows.Num(); ++RowIndex)
		{
			FLogRow& LogRow = LogRows[RowIndex];
			FLogRow& ParentLogRow = LogRows[RowIndex - 1];

			int32 CurParentRowItemIndex = INDEX_NONE;
			int32 NextLeftOffset = 0;

			for (int32 ItemIndex = 0; ItemIndex < LogRow.Num(); ++ItemIndex)
			{
				FLogRowItem& RowItem = LogRow[ItemIndex];
				if (ensure(ParentLogRow.IsValidIndex(RowItem.ParentRowItemIndex)))
				{
					FLogRowItem& ParentRowItem = ParentLogRow[RowItem.ParentRowItemIndex];

					if (RowItem.ParentRowItemIndex != CurParentRowItemIndex)
					{
						NextLeftOffset = ParentRowItem.LeftOffset + ParentItemMargin / 2;
						CurParentRowItemIndex = RowItem.ParentRowItemIndex;
					}

					RowItem.LeftOffset = NextLeftOffset;
					NextLeftOffset += RowItem.Width;
				}
			}
		}
	}

	void OutputLogRows()
	{
		TStringBuilder<1024> Builder;

		for (const FLogRow& LogRow : LogRows)
		{
			Builder.Reset();

			for (const FLogRowItem& RowItem : LogRow)
			{
				// Indent
				const int32 CurLeftOffset = Builder.Len();
				ensure(RowItem.LeftOffset >= CurLeftOffset);
				int32 IndentWidth = FMath::Max(0, RowItem.LeftOffset - CurLeftOffset);
				for (int32 Index = 0; Index < IndentWidth; ++Index)
				{
					Builder.Append(TEXT(" "));
				}

				// Build display string: [====== 0,1,2 ======]
				const FString DataString = LexToStringAllDataIndexes(RowItem.Node);

				const int32 Padding = FMath::Max(0, RowItem.Width - DataString.Len() - 4);
				const int32 PaddingLeft = Padding / 2;
				const int32 PaddingRight = Padding - PaddingLeft;

				Builder.Append(TEXT("["));
				for (int32 Index = 0; Index < PaddingLeft; ++Index)
				{
					Builder.Append(TEXT("="));
				}
				Builder.Append(TEXT(" "));

				Builder.Append(DataString);
				
				Builder.Append(TEXT(" "));
				for (int32 Index = 0; Index < PaddingLeft; ++Index)
				{
					Builder.Append(TEXT("="));
				}
				Builder.Append(TEXT("]"));
			}

			FString RowString(Builder.ToString());
			UE_LOGF(LogMovieScene, Log, "%ls", *RowString);
		}
	}

	void OutputData()
	{
		if (!DataFormatter.IsBound())
		{
			UE_LOGF(LogMovieScene, Log, "No data formatter provided.");
			return;
		}

		FMovieSceneEvaluationTreeNode Dummy;
		TStringBuilder<256> DataStringBuilder;

		for (auto It : DataIndices)
		{
			DataStringBuilder.Reset();
			Dummy.DataID.EntryIndex = It.Key.template Get<0>();
			TArrayView<const DataType> DataView = Tree.GetDataForSingleNode(Dummy);
			const int32 DataIndex = It.Key.template Get<1>();
			if (ensure(DataView.IsValidIndex(DataIndex)))
			{
				const FDataInfo DataInfo = It.Value;

				DataStringBuilder.Appendf(TEXT("%d: "), DataInfo.template Get<0>());
				DataStringBuilder.Appendf(TEXT("%s "), *LexToString(DataInfo.template Get<1>()));

				const DataType& Data = DataView[DataIndex];
				DataFormatter.Execute(Data, DataStringBuilder);
				const FString DataString = DataStringBuilder.ToString();
				UE_LOGF(LogMovieScene, Log, "%ls", *DataString);
			}
		}
	}

	FString LexToStringAllDataIndexes(const FMovieSceneEvaluationTreeNode* Node)
	{
		TStringBuilder<64> DataBuilder;
		TArrayView<const DataType> DataView = Tree.GetDataForSingleNode(*Node);
		for (int32 Index = 0; Index < DataView.Num(); ++Index)
		{
			if (Index > 0)
			{
				DataBuilder.Append(TEXT(","));
			}

			const DataType& Data = DataView[Index];
			const FDataIndex Key(Node->DataID.EntryIndex, Index);
			const FDataInfo DefaultValue(DataIndices.Num(), Node->Range);
			const FDataInfo ActualValue = DataIndices.FindOrAdd(Key, DefaultValue);
			DataBuilder.Append(LexToString(ActualValue.template Get<0>()));
		}
		if (DataView.Num() == 0)
		{
			DataBuilder.Append(TEXT("nil"));
		}
		return DataBuilder.ToString();
	}

	static int32 GetItemMinWidth(int32 NumData)
	{
		// [= 0,1,2 =]
		if (NumData > 0)
		{
			return 
				NumData + // unit digit
				FMath::Max(0, NumData - 9) + // tens digit
				FMath::Max(0, NumData - 99) + // hundred digit
				(NumData - 1) + // commas
				4 + 2; // brackets with minimal padding
		}
		else
		{
			return
				3 + // nil
				4 + 2; // brackes with minimal padding
		}
	}
};

#endif

