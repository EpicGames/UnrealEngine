// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/MovieSceneExecutionTokens.h"
#include "IMovieScenePlayer.h"
#include "Algo/Sort.h"


DECLARE_CYCLE_STAT(TEXT("Apply Execution Tokens"), MovieSceneEval_ApplyExecutionTokens, STATGROUP_MovieSceneEval);
DECLARE_CYCLE_STAT(TEXT("Apply Execution Token"), MovieSceneEval_ApplyExecutionToken, STATGROUP_MovieSceneEval);

bool SortTokens(const TInlineValue<IMovieSceneSharedExecutionToken, 32>& A, const TInlineValue<IMovieSceneSharedExecutionToken, 32>& B)
{
	return A->Order < B->Order;
}

void FMovieSceneExecutionTokens::Apply(const FMovieSceneContext& RootContext, IMovieScenePlayer& Player)
{
	MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_ApplyExecutionTokens);

	FPersistentEvaluationData PersistentDataProxy(Player);

	TArray<TInlineValue<IMovieSceneSharedExecutionToken, 32>> SortedSharedTokens;
	SortedSharedTokens.Reserve(SharedTokens.Num());

	for (auto& Pair : SharedTokens)
	{
		SortedSharedTokens.Add(MoveTemp(Pair.Value));
	}

	Algo::Sort(SortedSharedTokens, SortTokens);

	// Reset track and section keys
	PersistentDataProxy.SetSectionKey(FMovieSceneEvaluationKey());
	PersistentDataProxy.SetTrackKey(FMovieSceneEvaluationKey());

	int32 SharedTokenIndex = 0;
	while (SharedTokenIndex < SortedSharedTokens.Num())
	{
		IMovieSceneSharedExecutionToken& Token = SortedSharedTokens[SharedTokenIndex].GetValue();
		if (Token.Order <= 0)
		{
			Token.Execute(PersistentDataProxy, Player);
			++SharedTokenIndex;
		}
		else
		{
			break;
		}
	}

	for (FMovieSceneExecutionTokens::FEntry& Entry : OrderedTokens)
	{
		PersistentDataProxy.SetTrackKey(Entry.Scope.Key.AsTrack());
		PersistentDataProxy.SetSectionKey(Entry.Scope.Key);

		FScopedPreAnimatedCaptureSource CaptureSource(&Player.PreAnimatedState, Entry.Scope.Key, Entry.Scope.CompletionMode == EMovieSceneCompletionMode::RestoreState);

		{
			MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_ApplyExecutionToken);
			Entry.Token->Execute(Entry.Context, Entry.Operand, PersistentDataProxy, Player);
		}
	}

	OrderedTokens.Reset();

	// Reset track and section keys
	PersistentDataProxy.SetSectionKey(FMovieSceneEvaluationKey());
	PersistentDataProxy.SetTrackKey(FMovieSceneEvaluationKey());

	while (SharedTokenIndex < SortedSharedTokens.Num())
	{
		SortedSharedTokens[SharedTokenIndex++]->Execute(PersistentDataProxy, Player);
	}

	SharedTokens.Reset();

	BlendingAccumulator.Apply(RootContext, PersistentDataProxy, Player);
}
