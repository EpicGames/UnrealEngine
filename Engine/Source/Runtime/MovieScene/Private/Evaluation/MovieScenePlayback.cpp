// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/MovieScenePlayback.h"
#include "MovieSceneTransformTypes.h"
#include "MovieScene.h"

namespace 
{
	TRange<FFrameTime> CalculateEvaluationRange(FFrameTime CurrentTime, FFrameTime PreviousTime, bool bInclusivePreviousTime)
	{
		if (CurrentTime == PreviousTime)
		{
			return TRange<FFrameTime>(CurrentTime);
		}
		else if (CurrentTime < PreviousTime)
		{
			return TRange<FFrameTime>(
				TRangeBound<FFrameTime>::Inclusive(CurrentTime),
				bInclusivePreviousTime
					? TRangeBound<FFrameTime>::Inclusive(PreviousTime)
					: TRangeBound<FFrameTime>::Exclusive(PreviousTime)
				);
		}

		return TRange<FFrameTime>(
			bInclusivePreviousTime
				? TRangeBound<FFrameTime>::Inclusive(PreviousTime)
				: TRangeBound<FFrameTime>::Exclusive(PreviousTime)
			, TRangeBound<FFrameTime>::Inclusive(CurrentTime)
			);
	}
}

FMovieSceneEvaluationRange::FMovieSceneEvaluationRange(FFrameTime InTime, FFrameRate InFrameRate)
	: EvaluationRange(InTime)
	, CurrentFrameRate(InFrameRate)
	, Direction(EPlayDirection::Forwards)
	, TimeOverride(FFrameNumber(TNumericLimits<int32>::Lowest()))
{
}

FMovieSceneEvaluationRange::FMovieSceneEvaluationRange(TRange<FFrameTime> InRange, FFrameRate InFrameRate, EPlayDirection InDirection)
	: EvaluationRange(InRange)
	, CurrentFrameRate(InFrameRate)
	, Direction(InDirection)
	, TimeOverride(FFrameNumber(TNumericLimits<int32>::Lowest()))
{
}

FMovieSceneEvaluationRange::FMovieSceneEvaluationRange(FFrameTime InCurrentTime, FFrameTime InPreviousTime, FFrameRate InFrameRate, bool bInclusivePreviousTime, EPlayDirection PreferredDirection)
	: EvaluationRange(CalculateEvaluationRange(InCurrentTime, InPreviousTime, bInclusivePreviousTime))
	, CurrentFrameRate(InFrameRate)
	, TimeOverride(TNumericLimits<int32>::Lowest())
{
	const FFrameTime ZeroTime;
	const FFrameTime RangeLength = (InCurrentTime - InPreviousTime);
	Direction = (RangeLength > ZeroTime) ? EPlayDirection::Forwards :
		((RangeLength < ZeroTime) ? EPlayDirection::Backwards :
		 PreferredDirection);
}

void FMovieSceneEvaluationRange::ResetRange(const TRange<FFrameTime>& NewRange)
{
	ensureMsgf(TimeOverride == TNumericLimits<int32>::Lowest(),
		TEXT("Should reset time ranges on a range with a fixed time override. This should never happen because such ranges are internal to the movie scene compiler."));
	EvaluationRange = NewRange;
}

TRange<FFrameNumber> FMovieSceneEvaluationRange::GetTraversedFrameNumberRange() const
{
	TRange<FFrameNumber> FrameNumberRange;

	if (!EvaluationRange.GetLowerBound().IsOpen())
	{
		FFrameNumber StartFrame = EvaluationRange.GetLowerBoundValue().FloorToFrame();
		FrameNumberRange.SetLowerBound(TRangeBound<FFrameNumber>::Inclusive(StartFrame));
	}

	if (!EvaluationRange.GetUpperBound().IsOpen())
	{
		FFrameNumber EndFrame = EvaluationRange.GetUpperBoundValue().FloorToFrame() + 1;
		FrameNumberRange.SetUpperBound(TRangeBound<FFrameNumber>::Exclusive(EndFrame));
	}

	return FrameNumberRange;
}

TRange<FFrameNumber> FMovieSceneEvaluationRange::TimeRangeToNumberRange(const TRange<FFrameTime>& InFrameTimeRange)
{
	TRange<FFrameNumber> FrameNumberRange;
	TOptional<FFrameTime> UpperTime;
	if (!InFrameTimeRange.GetUpperBound().IsOpen())
	{
		UpperTime = InFrameTimeRange.GetUpperBoundValue();
		// Similar to adjusting the lower bound, if there's a subframe on the upper bound, the frame number needs incrementing in order to evaluate keys in the subframe
		if (UpperTime.GetValue().GetSubFrame() != 0.f || InFrameTimeRange.GetUpperBound().IsInclusive())
		{
			UpperTime.GetValue().FrameNumber = UpperTime.GetValue().FrameNumber + 1;
		}
		FrameNumberRange.SetUpperBound(TRangeBound<FFrameNumber>::Exclusive(UpperTime.GetValue().FrameNumber));
	}

	if (!InFrameTimeRange.GetLowerBound().IsOpen())
	{
		FFrameTime LowerTime = InFrameTimeRange.GetLowerBoundValue();
		// If there is a sub frame on the start time, we're actually beyond that frame number, so it needs incrementing
		if (LowerTime.GetSubFrame() != 0.f || InFrameTimeRange.GetLowerBound().IsExclusive())
		{
			LowerTime.FrameNumber = (!UpperTime.IsSet() || LowerTime.FrameNumber < UpperTime.GetValue().FrameNumber) ?
				LowerTime.FrameNumber + 1 : LowerTime.FrameNumber;

		}
		FrameNumberRange.SetLowerBound(TRangeBound<FFrameNumber>::Inclusive(LowerTime.FrameNumber));
	}


	return FrameNumberRange;
}

TRange<FFrameTime> FMovieSceneEvaluationRange::NumberRangeToTimeRange(const TRange<FFrameNumber>& InFrameNumberRange)
{
	TRange<FFrameTime> FrameTimeRange;

	if (!InFrameNumberRange.GetLowerBound().IsOpen())
	{
		const FFrameNumber FrameNumber = InFrameNumberRange.GetLowerBoundValue();

		FrameTimeRange.SetLowerBound(
			InFrameNumberRange.GetLowerBound().IsExclusive()
			? TRangeBound<FFrameTime>::Exclusive(FrameNumber)
			: TRangeBound<FFrameTime>::Inclusive(FrameNumber)
		);
	}

	if (!InFrameNumberRange.GetUpperBound().IsOpen())
	{
		const FFrameNumber FrameNumber = InFrameNumberRange.GetUpperBoundValue();

		FrameTimeRange.SetUpperBound(
			InFrameNumberRange.GetUpperBound().IsExclusive()
			? TRangeBound<FFrameTime>::Exclusive(FrameNumber)
			: TRangeBound<FFrameTime>::Inclusive(FrameNumber)
		);
	}

	return FrameTimeRange;
}


FMovieSceneContext FMovieSceneContext::Transform(const FMovieSceneSequenceTransform& InTransform, FFrameRate NewFrameRate) const
{
	using namespace UE::MovieScene;

	FMovieSceneContext NewContext = *this;
	NewContext.RootToSequenceTransform = NewContext.RootToSequenceTransform * InTransform;
	NewContext.CurrentFrameRate = NewFrameRate;

	NewContext.EvaluationRange = InTransform.ComputeTraversedHull(EvaluationRange);

	if (NewContext.EvaluationRange.GetLowerBound().IsClosed() && NewContext.EvaluationRange.GetUpperBound().IsClosed() && NewContext.EvaluationRange.GetLowerBoundValue() > NewContext.EvaluationRange.GetUpperBoundValue())
	{
		TRangeBound<FFrameTime> OldLower = NewContext.EvaluationRange.GetLowerBound();
		TRangeBound<FFrameTime> OldUpper = NewContext.EvaluationRange.GetUpperBound();
		NewContext.EvaluationRange.SetLowerBound(OldUpper);
		NewContext.EvaluationRange.SetUpperBound(OldLower);
		NewContext.Direction = EPlayDirection::Backwards;
	}

	// Transform the current time so we get an idea in what loop(s) we are relative to the root sequence.
	InTransform.TransformTime(GetTime(), FTransformTimeParams().AppendBreadcrumbs(NewContext.RootToSequenceWarpCounter));

	return NewContext;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FMovieSceneTimeTransform FMovieSceneContext::GetSequenceToRootTransform() const
{
	return RootToSequenceTransform.Inverse().AsLegacyLinearTimeTransform();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FMovieSceneInverseSequenceTransform FMovieSceneContext::GetSequenceToRootSequenceTransform() const
{
	return RootToSequenceTransform.Inverse();
}

void FMovieScenePlaybackPosition::CheckInvariants() const
{
	checkf(InputRate.IsValid() && OutputRate.IsValid(), TEXT("Invalid input or output rate. SetTimeBase must be called before any use of this class."))
}

void FMovieScenePlaybackPosition::SetTimeBase(FFrameRate NewInputRate, FFrameRate NewOutputRate, EMovieSceneEvaluationType NewEvaluationType)
{
	// Move the current position if necessary
	if (InputRate.IsValid() && InputRate != NewInputRate)
	{
		FFrameTime NewPosition = ConvertFrameTime(CurrentPosition, InputRate, NewInputRate);
		if (NewEvaluationType == EMovieSceneEvaluationType::FrameLocked)
		{
			NewPosition = NewPosition.FloorToFrame();
		}

		Reset(NewPosition);
	}

	InputRate      = NewInputRate;
	OutputRate     = NewOutputRate;
	EvaluationType = NewEvaluationType;
}

void FMovieScenePlaybackPosition::Reset(FFrameTime StartPos)
{
	CurrentPosition = StartPos;
	PreviousPlayEvalPosition.Reset();
	LastRange.Reset();
}

FMovieSceneEvaluationRange FMovieScenePlaybackPosition::GetCurrentPositionAsRange() const
{
	CheckInvariants();

	FFrameTime OutputPosition = ConvertFrameTime(CurrentPosition, InputRate, OutputRate);
	return FMovieSceneEvaluationRange(OutputPosition, OutputRate);
}

FMovieSceneEvaluationRange FMovieScenePlaybackPosition::JumpTo(FFrameTime InputPosition, EPlayDirection PreferredDirection)
{
	CheckInvariants();

	PreviousPlayEvalPosition.Reset();

	// Floor to the current frame number if running frame-locked
	if (EvaluationType == EMovieSceneEvaluationType::FrameLocked)
	{
		InputPosition = InputPosition.FloorToFrame();
	}

	// Assign the cached input values
	CurrentPosition = InputPosition;

	// Convert to output time-base
	FFrameTime OutputPosition = ConvertFrameTime(InputPosition, InputRate, OutputRate);

	LastRange = FMovieSceneEvaluationRange(TRange<FFrameTime>(OutputPosition), OutputRate, PreferredDirection);
	return LastRange.GetValue();
}

FMovieSceneEvaluationRange FMovieScenePlaybackPosition::PlayTo(FFrameTime InputPosition, EPlayDirection PreferredDirection)
{
	CheckInvariants();

	// Floor to the current frame number if running frame-locked
	if (EvaluationType == EMovieSceneEvaluationType::FrameLocked)
	{
		InputPosition = InputPosition.FloorToFrame();
	}

	// Convert to output time-base
	FFrameTime InputEvalPositionFrom  = PreviousPlayEvalPosition.Get(CurrentPosition);
	FFrameTime OutputEvalPositionFrom = ConvertFrameTime(InputEvalPositionFrom, InputRate, OutputRate);
	FFrameTime OutputEvalPositionTo   = ConvertFrameTime(InputPosition, InputRate, OutputRate);

	LastRange = FMovieSceneEvaluationRange(OutputEvalPositionTo, OutputEvalPositionFrom, OutputRate, !PreviousPlayEvalPosition.IsSet(), PreferredDirection);

	// Assign the cached input values
	CurrentPosition          = InputPosition;
	PreviousPlayEvalPosition = InputPosition;

	return LastRange.GetValue();
}

TOptional<FMovieSceneEvaluationRange> FMovieScenePlaybackPosition::GetLastRange() const
{
	return LastRange;
}
