// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/MovieScenePlaybackCapabilities.h"

namespace UE::MovieScene
{

FPlaybackCapabilities::FPlaybackCapabilities(FPlaybackCapabilities&& RHS)
{
	Memory = RHS.Memory;
	Alignment = RHS.Alignment;
	Capacity = RHS.Capacity;
	Num = RHS.Num;
	AllCapabilities = RHS.AllCapabilities;

	new (&RHS) FPlaybackCapabilities();
}

FPlaybackCapabilities& FPlaybackCapabilities::operator=(FPlaybackCapabilities&& RHS)
{
	if (ensure(this != &RHS))
	{
		Destroy();

		Memory = RHS.Memory;
		Alignment = RHS.Alignment;
		Capacity = RHS.Capacity;
		Num = RHS.Num;
		AllCapabilities = RHS.AllCapabilities;

		new (&RHS) FPlaybackCapabilities();
	}
	return *this;
}

FPlaybackCapabilities::~FPlaybackCapabilities()
{
	Destroy();
}

void FPlaybackCapabilities::OnSubInstanceCreated(TSharedRef<const FSharedPlaybackState> Owner, const FInstanceHandle InstanceHandle)
{
	ForEachCapabilityInterface([Owner, &InstanceHandle](IPlaybackCapability& Cap) { Cap.OnSubInstanceCreated(Owner, InstanceHandle); });
}

void FPlaybackCapabilities::InvalidateCachedData(UMovieSceneEntitySystemLinker* Linker)
{
	ForEachCapabilityInterface([Linker](IPlaybackCapability& Cap) { Cap.InvalidateCachedData(Linker); });
}

void FPlaybackCapabilities::Destroy()
{
	check((Memory == nullptr) == (Num == 0));

	if (Memory)
	{
		TArrayView<const FPlaybackCapabilityHeader> Headers = GetHeaders();
		for (int32 Index = 0; Index < Headers.Num(); ++Index)
		{
			const FPlaybackCapabilityHeader& Header = Headers[Index];
			const FPlaybackCapabilityHelpers& ThisHelpers = Helpers[Index];
			check(ThisHelpers.Destructor != nullptr);
			{
				void* Ptr = Header.Capability.Resolve(Memory);
				(*ThisHelpers.Destructor)(Ptr);
			}
		}

		FMemory::Free(Memory);
	}
}

} // namespace UE::MovieScene
