// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/MovieScenePlaybackManager.h"

#include "Channels/MovieSceneTimeWarpChannel.h"
#include "Misc/CoreMiscDefines.h"
#include "MovieScene.h"
#include "MovieSceneFwd.h"
#include "MovieSceneSequence.h"

FMovieScenePlaybackManager::FMovieScenePlaybackManager()
{
}

FMovieScenePlaybackManager::FMovieScenePlaybackManager(UMovieSceneSequence* InSequence)
{
	Initialize(InSequence);
}

void FMovieScenePlaybackManager::Initialize(UMovieSceneSequence* InSequence)
{
	if (!ensure(InSequence))
	{
		return;
	}

	UMovieScene* MovieScene = InSequence->GetMovieScene();
	const EMovieSceneEvaluationType EvaluationType = MovieScene->GetEvaluationType();

	DisplayRate = MovieScene->GetDisplayRate();

	// Make our playback position work *only* in ticks. We will handle conversion to/from frames ourselves.
	const FFrameRate TickResolution = MovieScene->GetTickResolution();
	PlaybackPosition.SetTimeBase(TickResolution, TickResolution, EvaluationType);

	const TRange<FFrameNumber> PlaybackRange = MovieScene->GetPlaybackRange();
	SequenceStartTick = UE::MovieScene::DiscreteInclusiveLower(PlaybackRange);
	SequenceEndTick = UE::MovieScene::DiscreteExclusiveUpper(PlaybackRange);

	ResetPlaybackSettings();

	PlaybackPosition.Reset(SequenceStartTick);
}

void FMovieScenePlaybackManager::ResetPlaybackSettings()
{
	StartOffsetTicks = EndOffsetTicks = FFrameNumber(0);
	NumLoopsToPlay = 1;
	NumLoopsCompleted = 0;
	PlayRate = 1.0;
	PlayDirection = EPlayDirection::Forwards;
}

void FMovieScenePlaybackManager::Update(float InDeltaSeconds, FContexts& OutContexts)
{
	using namespace UE::MovieScene;

	if (PlaybackStatus != EMovieScenePlayerStatus::Playing)
	{
		return;
	}
	
	// Get the new time, advanced by InDeltaSeconds.
	const FFrameTime PreviousTick = PlaybackPosition.GetCurrentPosition();
	const FFrameTime DeltaTicks = PlaybackPosition.GetInputRate().AsFrameTime(
			(IsPlayingForward() ? InDeltaSeconds : (-InDeltaSeconds)) * PlayRate);
	const FFrameTime NextTick = PreviousTick + DeltaTicks;

	FFrameTime WarpedNextTick = NextTick;
	if (bTransformPlaybackTime)
	{
		if (TimeTransform.FindFirstWarpDomain() == ETimeWarpChannelDomain::PlayRate)
		{
			WarpedNextTick = TimeTransform.TransformTime(WarpedNextTick);
		}
	}

	InternalUpdateToTick(WarpedNextTick.RoundToFrame(), OutContexts);
}

void FMovieScenePlaybackManager::UpdateTo(const FFrameTime NextTime, FContexts& OutContexts)
{
	const FFrameTime NextTick = ConvertFrameTime(NextTime, DisplayRate, PlaybackPosition.GetInputRate());
	InternalUpdateToTick(NextTick.RoundToFrame(), OutContexts);
}

void FMovieScenePlaybackManager::InternalUpdateToTick(const FFrameNumber NextTick, FContexts& OutContexts)
{
	using namespace UE::MovieScene;

	// If we are stopped, just move the playhead to the given time, without generating evaluation contexts.
	if (PlaybackStatus == EMovieScenePlayerStatus::Stopped)
	{
		PlaybackPosition.Reset(NextTick);
		return;
	}

	// Check we have some loop counters that make sense.
	if (NumLoopsToPlay > 0 && !ensure(NumLoopsCompleted < NumLoopsToPlay))
	{
		PlaybackStatus = EMovieScenePlayerStatus::Stopped;
		PlaybackPosition.Reset(NextTick);
	}

	// Gather some information about this update.
	const bool bShouldJump = 
		(PlaybackStatus != EMovieScenePlayerStatus::Playing && PlaybackStatus != EMovieScenePlayerStatus::Scrubbing);

	const FFrameNumber EffectiveStartTick = SequenceStartTick + StartOffsetTicks;
	const FFrameNumber EffectiveEndTick = SequenceEndTick - EndOffsetTicks;
	const FFrameNumber EffectiveDurationTicks = FMath::Max(FFrameNumber(0), EffectiveEndTick - EffectiveStartTick);
	const FFrameNumber LastValidTick = GetLastValidTick();
	// IMPORTANT: we assume that LastValidTick is less than the duration (current implementation is 
	//			  duration minus one tick).

	bool bIsPlayingForward = IsPlayingForward();
	FFrameNumber LoopStartTick = bIsPlayingForward ? EffectiveStartTick : LastValidTick;
	FFrameNumber LoopLastTick = bIsPlayingForward ? LastValidTick : EffectiveStartTick;
	FFrameNumber LastLoopLastTick = PlaybackEndTick.Get(LoopLastTick);
	// If the start/end offsets make the duration 0, we treat each loop as one tick long.
	const FFrameNumber LoopDurationTicks = FMath::Max(FFrameNumber(1), EffectiveDurationTicks);

	// Figure out if we crossed the loop-end boundary, or the "stop at" time on the last loop.
	const bool bCrossedLoopEnd = (
			(bIsPlayingForward && NextTick > LoopLastTick) ||
			(!bIsPlayingForward && NextTick < LoopLastTick)
			);
	const bool bReachedLastLoopEnd = (
			(NumLoopsToPlay > 0 && NumLoopsCompleted >= NumLoopsToPlay - 1) &&
			(
			 (bIsPlayingForward && NextTick >= LastLoopLastTick) ||
			 (!bIsPlayingForward && NextTick <= LastLoopLastTick)
			));
	if (bReachedLastLoopEnd)
	{
		// Special case when we reach or pass the last tick of the last loop. Play the last
		// bit of the animation, and stop.
		OutContexts.Add(
				FMovieSceneContext(PlaybackPosition.PlayTo(LastLoopLastTick), PlaybackStatus)
				.SetHasJumped(bShouldJump)
				);

		++NumLoopsCompleted;
		ensure(NumLoopsToPlay > 0 && NumLoopsCompleted == NumLoopsToPlay);

		PlaybackStatus = EMovieScenePlayerStatus::Stopped;
	}
	else if (bCrossedLoopEnd)
	{
		// Compute how many times we crossed the loop-end boundary.
		// NOTE: we completed one or more loops. Any extra loop may be the last loop, and therefore
		//       may need to consider a different end time (see LastLoopLastTick).
		const FFrameNumber LoopRelativeTick = NextTick - LoopStartTick;
		const int32 NumLoopingsOver = FMath::Abs(LoopRelativeTick.Value) / LoopDurationTicks.Value;
		ensure(NumLoopingsOver > 0);

		// Get the actual number of loops we want to consider completed, and see if this will bring
		// us inside or past the last loop.
		const int32 NumLoopsNewlyCompleted = ((NumLoopsToPlay > 0) ?
				FMath::Min(NumLoopingsOver, NumLoopsToPlay - NumLoopsCompleted) :
				NumLoopingsOver);
		const bool bCompletesLastLoop = (
				NumLoopsToPlay > 0 && NumLoopsCompleted + NumLoopsNewlyCompleted >= NumLoopsToPlay);
		ensure(NumLoopsNewlyCompleted > 0);

		// Play the last bit of the loop if we are looping and doing any sort of dissections.
		// We know this isn't the last loop (otherwise we'd be in the bReachedLastLoopEnd block)
		// so we don't need to be worried about any custom end time.
		if (DissectLooping != EMovieSceneLoopDissection::None)
		{
			OutContexts.Add(
					FMovieSceneContext(PlaybackPosition.PlayTo(LoopLastTick), PlaybackStatus)
					.SetHasJumped(bShouldJump)
					);
		}

		// See if we need to generate more update ranges for the loops. This can happen if we had a
		// large delta-time, and the duration of a loop is pretty short (i.e. we could have looped 
		// several times in one update).
		if (NumLoopsNewlyCompleted > 1 && DissectLooping == EMovieSceneLoopDissection::DissectAll)
		{
			// Add an explicit update for each loop, from start to end. Do one less extra loop if
			// the last extra loop is the final loop. We will handle the last loop later (see below).
			const int32 ExtraLoops = NumLoopsNewlyCompleted - 1 - (bCompletesLastLoop ? 1 : 0);
			ensure(ExtraLoops >= 0);

			if (!bPingPongPlayback)
			{
				for (int32 Index = 0; Index < ExtraLoops; ++Index)
				{
					PlaybackPosition.Reset(LoopStartTick);
					OutContexts.Add(
							FMovieSceneContext(PlaybackPosition.PlayTo(LoopLastTick), PlaybackStatus)
							.SetHasJumped(true)
							.SetHasLooped(true)
							);
				}
			}
			else
			{
				ReversePlayDirection();

				for (int32 Index = 0; Index < ExtraLoops; ++Index)
				{
					if (IsPlayingForward())
					{
						PlaybackPosition.Reset(EffectiveStartTick);
						OutContexts.Add(
								FMovieSceneContext(PlaybackPosition.PlayTo(LastValidTick), PlaybackStatus)
								.SetHasJumped(true)
								.SetHasLooped(true)
								);
					}
					else
					{
						PlaybackPosition.Reset(LastValidTick);
						OutContexts.Add(
								FMovieSceneContext(PlaybackPosition.PlayTo(EffectiveStartTick), PlaybackStatus)
								.SetHasJumped(true)
								.SetHasLooped(true)
								);
					}

					ReversePlayDirection();
				}
			}
		}

		// If we are ping-pong'ing, keep track of the direction to go next. We don't need to do
		// this if we were dissecting each loop, because then we already updated the play direction
		// while dissecting (see above).
		if (bPingPongPlayback && DissectLooping != EMovieSceneLoopDissection::DissectAll)
		{
			// If we are reaching the end of the last loop, don't flip an extra time. Just finish
			// playing in that loop's direction.
			const int32 NumPingPongs = NumLoopsNewlyCompleted - (bCompletesLastLoop ? 1 : 0);
			if (NumPingPongs % 2 != 0)
			{
				ReversePlayDirection();
			}
		}

		// Complete the loops we said we completed.
		NumLoopsCompleted += NumLoopsNewlyCompleted;

		// Re-query the direction of play in case we were ping-pong'ing, and update loop times.
		bIsPlayingForward = IsPlayingForward();
		LoopStartTick = bIsPlayingForward ? EffectiveStartTick : LastValidTick;
		LoopLastTick = bIsPlayingForward ? LastValidTick : EffectiveStartTick;
		LastLoopLastTick = PlaybackEndTick.Get(LoopLastTick);

		if (bCompletesLastLoop)
		{
			// We completed one loop, plus one or more extra loops that brought us to the end.
			ensure(NumLoopsNewlyCompleted > 1);

			if (DissectLooping == EMovieSceneLoopDissection::None)
			{
				// If we didn't dissect anything, we play to the end time if that's possible from
				// where we were last, or reset and play the last loop from the start if not.
				const FFrameNumber CurrentTick = PlaybackPosition.GetCurrentPosition().FrameNumber;
				const bool bCanPlayToEndFromCurrentTick = 
						(bIsPlayingForward && CurrentTick < LastLoopLastTick) ||
						(!bIsPlayingForward && CurrentTick > LastLoopLastTick);
				if (bCanPlayToEndFromCurrentTick)
				{
					OutContexts.Add(
							FMovieSceneContext(PlaybackPosition.PlayTo(LastLoopLastTick), PlaybackStatus)
							.SetHasJumped(bShouldJump)
							);
				}
				else
				{
					PlaybackPosition.Reset(LoopStartTick);
					OutContexts.Add(
							FMovieSceneContext(PlaybackPosition.PlayTo(LastLoopLastTick), PlaybackStatus)
							.SetHasJumped(true)
							.SetHasLooped(true)
							);
				}
			}
			else
			{
				// If we dissect all loops, we left the last loop earlier to do now.
				// If we dissect only one loop, we play the last loop now.
				PlaybackPosition.Reset(LoopStartTick);
				OutContexts.Add(
						FMovieSceneContext(PlaybackPosition.PlayTo(LastLoopLastTick), PlaybackStatus)
						.SetHasJumped(true)
						.SetHasLooped(true)
						);
			}

			PlaybackStatus = EMovieScenePlayerStatus::Stopped;
		}
		else
		{
			// Start the next loop with any overplay from the update.
			//
			// When playing forward, we have, e.g.:
			// loop = [-30, -10], next time = -5, relative time = 25, mod(25, 20) = 5
			//
			// When playing backwards, we have, e.g.:
			// loop = [-30, -10], next time = -35, relative time = -25, mod(-25, 20) = -5
			const FFrameNumber OverplayTicks = LoopRelativeTick.Value % LoopDurationTicks.Value;

			// We reverse OverplayTicks when ping-pong'in since we're playing this overplay in the
			// reverse direction (otherwise, it already has the correct sign).
			FFrameTime EffectiveOverplayTick = LoopStartTick + ((!bPingPongPlayback) ? OverplayTicks : (-OverplayTicks));

			// If we have reached the last loop, clamp our overplay with any custom end time.
			const bool bIsLastLoop = (
					NumLoopsToPlay > 0 && NumLoopsCompleted >= NumLoopsToPlay - 1);
			if (bIsLastLoop)
			{
				EffectiveOverplayTick = bIsPlayingForward ?
					FMath::Min(EffectiveOverplayTick, FFrameTime(LastLoopLastTick)) :
					FMath::Max(EffectiveOverplayTick, FFrameTime(LastLoopLastTick));
			}

			// Evaluate the overplay.
			PlaybackPosition.Reset(bIsPlayingForward ? EffectiveStartTick : LastValidTick);
			OutContexts.Add(
					FMovieSceneContext(PlaybackPosition.PlayTo(EffectiveOverplayTick), PlaybackStatus)
					.SetHasJumped(true)
					.SetHasLooped(true)
					);

			// If the overplay leads us at/beyond the last tick of the last loop, let's count that
			// as a completed loop and finish playback. Otherwise, we'll wait for the next update 
			// to loop over in order to avoid counting that loop twice.
			if (bIsLastLoop && EffectiveOverplayTick == LastLoopLastTick)
			{
				++NumLoopsCompleted;

				PlaybackStatus = EMovieScenePlayerStatus::Stopped;
			}
		}
	}
	else
	{
		// We haven't crossed a loop-end boundary... just chug along.
		OutContexts.Add(
				FMovieSceneContext(PlaybackPosition.PlayTo(NextTick), PlaybackStatus)
				.SetHasJumped(bShouldJump)
				);
	}

	ensure(OutContexts.Num() > 0);
	ensure(OutContexts.Num() == 1 || (DissectLooping != EMovieSceneLoopDissection::None));
}

FFrameNumber FMovieScenePlaybackManager::GetLastValidTick() const
{
	// TODO: handle precision problems with float SubFrame, or change SubFrame to double.
	return SequenceEndTick - EndOffsetTicks - 1;  // minus one tick for exclusive end frame.
}

FMovieSceneContext FMovieScenePlaybackManager::UpdateToNextTick()
{
	const FFrameTime CurrentTick = PlaybackPosition.GetCurrentPosition();
	return FMovieSceneContext(PlaybackPosition.PlayTo(CurrentTick, PlayDirection), PlaybackStatus);
}

FMovieSceneContext FMovieScenePlaybackManager::UpdateAtCurrentTime() const
{
	return FMovieSceneContext(PlaybackPosition.GetCurrentPositionAsRange(), PlaybackStatus);
}

FFrameTime FMovieScenePlaybackManager::GetCurrentTime() const
{
	const FFrameRate TickResolution = PlaybackPosition.GetOutputRate();
	return ConvertFrameTime(PlaybackPosition.GetCurrentPosition(), TickResolution, DisplayRate);
}

void FMovieScenePlaybackManager::SetCurrentTime(const FFrameTime& InFrameTime)
{
	const FFrameRate TickResolution = PlaybackPosition.GetOutputRate();
	const FFrameNumber CurrentTick = ConvertFrameTime(InFrameTime, DisplayRate, TickResolution).RoundToFrame();
	const FFrameNumber EffectiveStartTick = SequenceStartTick + StartOffsetTicks;
	const FFrameNumber LastValidTick = GetLastValidTick();
	PlaybackPosition.Reset(FMath::Clamp(CurrentTick, EffectiveStartTick, LastValidTick));
}

void FMovieScenePlaybackManager::SetCurrentTimeOffset(const FFrameTime& InFrameTimeOffset)
{
	const FFrameRate TickResolution = PlaybackPosition.GetOutputRate();
	const FFrameNumber CurrentTickOffset = ConvertFrameTime(InFrameTimeOffset, DisplayRate, TickResolution).RoundToFrame();
	const FFrameNumber EffectiveStartTick = SequenceStartTick + StartOffsetTicks;
	const FFrameNumber LastValidTick = GetLastValidTick();
	PlaybackPosition.Reset(FMath::Clamp(EffectiveStartTick + CurrentTickOffset, EffectiveStartTick, LastValidTick));
}

TRange<FFrameTime> FMovieScenePlaybackManager::GetEffectivePlaybackRange() const
{
	const FFrameNumber StartTick = SequenceStartTick + StartOffsetTicks;
	const FFrameNumber EndTick = SequenceEndTick - EndOffsetTicks;

	const FFrameRate TickResolution = PlaybackPosition.GetOutputRate();

	const FFrameTime StartFrame = ConvertFrameTime(StartTick, TickResolution, DisplayRate);
	const FFrameTime EndFrame = ConvertFrameTime(EndTick, TickResolution, DisplayRate);

	return TRange<FFrameTime>(TRangeBound<FFrameTime>::Inclusive(StartFrame), TRangeBound<FFrameTime>::Exclusive(EndFrame));
}

FFrameTime FMovieScenePlaybackManager::GetEffectiveStartTime() const
{
	const FFrameRate TickResolution = PlaybackPosition.GetOutputRate();
	return ConvertFrameTime(SequenceStartTick + StartOffsetTicks, TickResolution, DisplayRate);
}

FFrameTime FMovieScenePlaybackManager::GetEffectiveEndTime() const
{
	const FFrameRate TickResolution = PlaybackPosition.GetOutputRate();
	return ConvertFrameTime(SequenceEndTick - EndOffsetTicks, TickResolution, DisplayRate);
}

void FMovieScenePlaybackManager::SetStartOffset(const FFrameTime& InStartOffset)
{
	const FFrameRate TickResolution = PlaybackPosition.GetOutputRate();
	const FFrameNumber InStartOffsetTicks = ConvertFrameTime(InStartOffset, DisplayRate, TickResolution).RoundToFrame();

	SetStartAndEndOffsetTicks(InStartOffsetTicks, EndOffsetTicks);
}

void FMovieScenePlaybackManager::SetEndOffset(const FFrameTime& InEndOffset)
{
	const FFrameRate TickResolution = PlaybackPosition.GetOutputRate();
	const FFrameNumber InEndOffsetTicks = ConvertFrameTime(InEndOffset, DisplayRate, TickResolution).RoundToFrame();

	SetStartAndEndOffsetTicks(StartOffsetTicks, InEndOffsetTicks);
}

void FMovieScenePlaybackManager::SetEndOffsetAsTime(const FFrameTime& InEndTime)
{
	const FFrameRate TickResolution = PlaybackPosition.GetOutputRate();
	const FFrameNumber InEndTick = ConvertFrameTime(InEndTime, DisplayRate, TickResolution).RoundToFrame();

	const FFrameNumber InEndOffsetTicks = SequenceEndTick - InEndTick;
	SetStartAndEndOffsetTicks(StartOffsetTicks, InEndOffsetTicks);
}

void FMovieScenePlaybackManager::SetStartAndEndOffsetTicks(FFrameNumber InStartOffsetTicks, FFrameNumber InEndOffsetTicks)
{
	const FFrameNumber SequenceDurationTicks = SequenceEndTick - SequenceStartTick;

	StartOffsetTicks = FMath::Min(
			FMath::Max(InStartOffsetTicks, FFrameNumber(0)),
			SequenceDurationTicks);

	EndOffsetTicks = FMath::Min(
			FMath::Max(InEndOffsetTicks, FFrameNumber(0)),
			SequenceDurationTicks - StartOffsetTicks);
}

FFrameTime FMovieScenePlaybackManager::GetStartOffset() const
{
	const FFrameRate TickResolution = PlaybackPosition.GetOutputRate();
	return ConvertFrameTime(StartOffsetTicks, TickResolution, DisplayRate);
}

FFrameTime FMovieScenePlaybackManager::GetEndOffset() const
{
	const FFrameRate TickResolution = PlaybackPosition.GetOutputRate();
	return ConvertFrameTime(EndOffsetTicks, TickResolution, DisplayRate);
}

TOptional<FFrameTime> FMovieScenePlaybackManager::GetPlaybackEndTime() const
{
	if (const FFrameNumber* EndTickValue = PlaybackEndTick.GetPtrOrNull())
	{
		const FFrameRate TickResolution = PlaybackPosition.GetOutputRate();
		return ConvertFrameTime(*EndTickValue, TickResolution, DisplayRate);
	}
	return TOptional<FFrameTime>();
}

void FMovieScenePlaybackManager::SetPlaybackEndTime(const FFrameTime& InEndTime)
{
	const FFrameRate TickResolution = PlaybackPosition.GetOutputRate();
	PlaybackEndTick = FMath::Clamp(
			ConvertFrameTime(InEndTime, DisplayRate, TickResolution).RoundToFrame(),
			SequenceStartTick + StartOffsetTicks,
			SequenceEndTick - EndOffsetTicks);
}

void FMovieScenePlaybackManager::ClearPlaybackEndTime()
{
	PlaybackEndTick.Reset();
}

void FMovieScenePlaybackManager::SetNumLoopsToPlay(int32 InNumLoopsToPlay)
{
	NumLoopsToPlay = InNumLoopsToPlay;
}

void FMovieScenePlaybackManager::ReversePlayDirection()
{
	if (PlayDirection == EPlayDirection::Forwards)
	{
		PlayDirection = EPlayDirection::Backwards;
	}
	else
	{
		PlayDirection = EPlayDirection::Forwards;
	}
}

void FMovieScenePlaybackManager::SetPingPongPlayback(bool bInPingPongPlayback)
{
	bPingPongPlayback = bInPingPongPlayback;
}

