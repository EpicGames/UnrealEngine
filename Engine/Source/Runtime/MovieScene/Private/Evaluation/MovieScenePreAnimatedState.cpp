// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/MovieScenePreAnimatedState.h"
#include "GameFramework/Actor.h"
#include "Components/ActorComponent.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneInstanceRegistry.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateExtension.h"
#include "Evaluation/PreAnimatedState/MovieSceneRestoreStateParams.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedCaptureSources.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedObjectTokenStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedRootTokenStorage.h"

DECLARE_CYCLE_STAT(TEXT("Save Pre Animated State"), MovieSceneEval_SavePreAnimatedState, STATGROUP_MovieSceneEval);

FMovieSceneInstancePreAnimatedState::FMovieSceneInstancePreAnimatedState(UMovieSceneEntitySystemLinker* InLinker, UE::MovieScene::FRootInstanceHandle InInstanceHandle)
	: WeakLinker(InLinker)
	, InstanceHandle(InInstanceHandle)
	, bCapturingGlobalPreAnimatedState(false)
{
}

FMovieSceneInstancePreAnimatedState::~FMovieSceneInstancePreAnimatedState()
{
	// Ensure that the global state capture request is removed
	UMovieSceneEntitySystemLinker* CurrentLinker = WeakLinker.Get();
	if (CurrentLinker && bCapturingGlobalPreAnimatedState)
	{
		checkf(CurrentLinker->PreAnimatedState.NumRequestsForGlobalState > 0, TEXT("Increment/Decrement mismatch on FPreAnimatedState::NumRequestsForGlobalState"));
		--CurrentLinker->PreAnimatedState.NumRequestsForGlobalState;
	}
}

UMovieSceneEntitySystemLinker* FMovieSceneInstancePreAnimatedState::GetLinker() const
{
	return WeakLinker.Get();
}

bool FMovieSceneInstancePreAnimatedState::IsCapturingGlobalPreAnimatedState() const
{
	return bCapturingGlobalPreAnimatedState;
}

void FMovieSceneInstancePreAnimatedState::EnableGlobalPreAnimatedStateCapture()
{
	if (bCapturingGlobalPreAnimatedState)
	{
		return;
	}

	bCapturingGlobalPreAnimatedState = true;

	UMovieSceneEntitySystemLinker* Linker = WeakLinker.Get();
	if (ensure(Linker))
	{
		++Linker->PreAnimatedState.NumRequestsForGlobalState;
	}
}

void FMovieSceneInstancePreAnimatedState::SavePreAnimatedState(UObject& InObject, FMovieSceneAnimTypeID InTokenType, const IMovieScenePreAnimatedTokenProducer& Producer)
{
	using namespace UE::MovieScene;

	UMovieSceneEntitySystemLinker* Linker = WeakLinker.Get();
	if (!Linker)
	{
		return;
	}

	if (bCapturingGlobalPreAnimatedState || Linker->PreAnimatedState.HasActiveCaptureSource())
	{
		Linker->PreAnimatedState.SavePreAnimatedStateDirectly(InObject, InTokenType, Producer);
	}
}

void FMovieSceneInstancePreAnimatedState::SavePreAnimatedState(FMovieSceneAnimTypeID InTokenType, const IMovieScenePreAnimatedGlobalTokenProducer& Producer)
{
	using namespace UE::MovieScene;

	UMovieSceneEntitySystemLinker* Linker = WeakLinker.Get();
	if (!Linker)
	{
		return;
	}

	if (bCapturingGlobalPreAnimatedState || Linker->PreAnimatedState.HasActiveCaptureSource())
	{
		Linker->PreAnimatedState.SavePreAnimatedStateDirectly(InTokenType, Producer);
	}
}

void FMovieSceneInstancePreAnimatedState::RestorePreAnimatedState()
{
	using namespace UE::MovieScene;

	UMovieSceneEntitySystemLinker* Linker = WeakLinker.Get();
	if (Linker)
	{
		Linker->PreAnimatedState.RestoreGlobalState(FRestoreStateParams{Linker, InstanceHandle});
	}
}

void FMovieSceneInstancePreAnimatedState::DiscardPreAnimatedState()
{
	using namespace UE::MovieScene;

	UMovieSceneEntitySystemLinker* Linker = WeakLinker.Get();
	if (Linker)
	{
		Linker->PreAnimatedState.DiscardGlobalState(FRestoreStateParams{ Linker, InstanceHandle });
	}
}

void FMovieSceneInstancePreAnimatedState::RestorePreAnimatedState(UObject& Object)
{
	using namespace UE::MovieScene;

	UMovieSceneEntitySystemLinker* Linker = WeakLinker.Get();
	if (!Linker)
	{
		return;
	}

	TSharedPtr<FPreAnimatedObjectGroupManager> ObjectGroupManager = Linker->PreAnimatedState.FindGroupManager<FPreAnimatedObjectGroupManager>();
	if (!ObjectGroupManager)
	{
		return;
	}

	FPreAnimatedStorageGroupHandle Group = ObjectGroupManager->FindGroupForKey(&Object);
	if (!Group)
	{
		return;
	}

	Linker->PreAnimatedState.RestoreStateForGroup(Group, FRestoreStateParams{Linker, InstanceHandle});
}

void FMovieSceneInstancePreAnimatedState::RestorePreAnimatedState(UClass* GeneratedClass)
{
	using namespace UE::MovieScene;

	UMovieSceneEntitySystemLinker* Linker = WeakLinker.Get();
	if (!Linker)
	{
		return;
	}

	TSharedPtr<FPreAnimatedObjectGroupManager> ObjectGroupManager = Linker->PreAnimatedState.FindGroupManager<FPreAnimatedObjectGroupManager>();
	if (ObjectGroupManager)
	{
		TArray<FPreAnimatedStorageGroupHandle> Handles;
		ObjectGroupManager->GetGroupsByClass(GeneratedClass, Handles);

		FRestoreStateParams Params{Linker, InstanceHandle};
		for (FPreAnimatedStorageGroupHandle GroupHandle : Handles)
		{
			Linker->PreAnimatedState.RestoreStateForGroup(GroupHandle, Params);
		}
	}
}


void FMovieSceneInstancePreAnimatedState::RestorePreAnimatedState(UObject& Object, TFunctionRef<bool(FMovieSceneAnimTypeID)> InFilter)
{
	using namespace UE::MovieScene;

	UMovieSceneEntitySystemLinker* Linker = WeakLinker.Get();
	if (!Linker)
	{
		return;
	}

	TSharedPtr<FAnimTypePreAnimatedStateObjectStorage> ObjectStorage = Linker->PreAnimatedState.FindStorage(FAnimTypePreAnimatedStateObjectStorage::StorageID);
	if (!ObjectStorage)
	{
		return;
	}

	struct FRestoreMask : FAnimTypePreAnimatedStateObjectStorage::IRestoreMask
	{
		TFunctionRef<bool(FMovieSceneAnimTypeID)>* Filter;

		virtual bool CanRestore(const FPreAnimatedObjectTokenTraits::KeyType& InKey) const override
		{
			return (*Filter)(InKey.Get<1>());
		}
	} RestoreMask;
	RestoreMask.Filter = &InFilter;

	ObjectStorage->SetRestoreMask(&RestoreMask);

	RestorePreAnimatedState(Object);

	ObjectStorage->SetRestoreMask(nullptr);
}

void FMovieSceneInstancePreAnimatedState::DiscardEntityTokens()
{
	using namespace UE::MovieScene;

	UMovieSceneEntitySystemLinker* Linker = WeakLinker.Get();
	if (Linker)
	{
		Linker->PreAnimatedState.DiscardTransientState();
	}
}

void FMovieSceneInstancePreAnimatedState::DiscardAndRemoveEntityTokensForObject(UObject& Object)
{
	using namespace UE::MovieScene;

	UMovieSceneEntitySystemLinker* Linker = WeakLinker.Get();
	if (!Linker)
	{
		return;
	}

	TSharedPtr<FPreAnimatedObjectGroupManager> ObjectGroupManager = Linker->PreAnimatedState.FindGroupManager<FPreAnimatedObjectGroupManager>();
	if (!ObjectGroupManager)
	{
		return;
	}

	FPreAnimatedStorageGroupHandle Group = ObjectGroupManager->FindGroupForKey(&Object);
	if (!Group)
	{
		return;
	}

	Linker->PreAnimatedState.DiscardStateForGroup(Group);
}

bool FMovieSceneInstancePreAnimatedState::ContainsAnyStateForSequence() const
{
	using namespace UE::MovieScene;

	UMovieSceneEntitySystemLinker* Linker = WeakLinker.Get();
	return Linker && InstanceHandle.IsValid() && Linker->PreAnimatedState.ContainsAnyStateForInstanceHandle(InstanceHandle);
}

void FMovieScenePreAnimatedState::Initialize(UMovieSceneEntitySystemLinker* Linker, UE::MovieScene::FRootInstanceHandle InInstanceHandle)
{
	WeakLinker = Linker;
	InstanceHandle = InInstanceHandle;
}

bool FMovieScenePreAnimatedState::IsCapturingGlobalPreAnimatedState() const
{
	if (FMovieSceneInstancePreAnimatedState* State = GetState())
	{
		return State->IsCapturingGlobalPreAnimatedState();
	}
	return false;
}

void FMovieScenePreAnimatedState::EnableGlobalPreAnimatedStateCapture()
{
	if (FMovieSceneInstancePreAnimatedState* State = GetState())
	{
		State->EnableGlobalPreAnimatedStateCapture();
	}
}

UMovieSceneEntitySystemLinker* FMovieScenePreAnimatedState::GetLinker() const
{
	return WeakLinker.Get();
}

void FMovieScenePreAnimatedState::SavePreAnimatedState(UObject& InObject, FMovieSceneAnimTypeID InTokenType, const IMovieScenePreAnimatedTokenProducer& Producer)
{
	if (FMovieSceneInstancePreAnimatedState* State = GetState())
	{
		State->SavePreAnimatedState(InObject, InTokenType, Producer);
	}
}

void FMovieScenePreAnimatedState::SavePreAnimatedState(FMovieSceneAnimTypeID InTokenType, const IMovieScenePreAnimatedGlobalTokenProducer& Producer)
{
	if (FMovieSceneInstancePreAnimatedState* State = GetState())
	{
		State->SavePreAnimatedState(InTokenType, Producer);
	}
}

void FMovieScenePreAnimatedState::RestorePreAnimatedState()
{
	if (FMovieSceneInstancePreAnimatedState* State = GetState())
	{
		State->RestorePreAnimatedState();
	}
}

void FMovieScenePreAnimatedState::RestorePreAnimatedState(UObject& Object)
{
	if (FMovieSceneInstancePreAnimatedState* State = GetState())
	{
		State->RestorePreAnimatedState(Object);
	}
}

void FMovieScenePreAnimatedState::RestorePreAnimatedState(UClass* GeneratedClass)
{
	if (FMovieSceneInstancePreAnimatedState* State = GetState())
	{
		State->RestorePreAnimatedState(GeneratedClass);
	}
}

void FMovieScenePreAnimatedState::RestorePreAnimatedState(UObject& Object, TFunctionRef<bool(FMovieSceneAnimTypeID)> InFilter)
{
	if (FMovieSceneInstancePreAnimatedState* State = GetState())
	{
		State->RestorePreAnimatedState(Object, InFilter);
	}
}

void FMovieScenePreAnimatedState::DiscardPreAnimatedState()
{
	if (FMovieSceneInstancePreAnimatedState* State = GetState())
	{
		State->DiscardPreAnimatedState();
	}
}

void FMovieScenePreAnimatedState::DiscardEntityTokens()
{
	if (FMovieSceneInstancePreAnimatedState* State = GetState())
	{
		State->DiscardEntityTokens();
	}
}

void FMovieScenePreAnimatedState::DiscardAndRemoveEntityTokensForObject(UObject& Object)
{
	if (FMovieSceneInstancePreAnimatedState* State = GetState())
	{
		State->DiscardAndRemoveEntityTokensForObject(Object);
	}
}

bool FMovieScenePreAnimatedState::ContainsAnyStateForSequence() const
{
	if (FMovieSceneInstancePreAnimatedState* State = GetState())
	{
		return State->ContainsAnyStateForSequence();
	}
	return false;
}

FMovieSceneInstancePreAnimatedState* FMovieScenePreAnimatedState::GetState() const
{
	using namespace UE::MovieScene;

	UMovieSceneEntitySystemLinker* Linker = WeakLinker.Get();
	if (Linker && InstanceHandle.IsValid() && Linker->GetInstanceRegistry()->IsHandleValid(InstanceHandle))
	{
		const FSequenceInstance& Instance = Linker->GetInstanceRegistry()->GetInstance(InstanceHandle);
		return &Instance.GetSharedPlaybackState()->GetPreAnimatedState();
	}
	return nullptr;
}

