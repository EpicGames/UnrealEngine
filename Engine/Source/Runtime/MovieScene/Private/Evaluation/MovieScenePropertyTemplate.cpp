// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/MovieScenePropertyTemplate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScenePropertyTemplate)

static TMovieSceneAnimTypeIDContainer<FString> PropertyTypeIDs;

// Default property ID to our own type - this implies an empty property
PropertyTemplate::FSectionData::FSectionData()
	: PropertyID(TMovieSceneAnimTypeID<FSectionData>())
{
}

void PropertyTemplate::FSectionData::Initialize(FName InPropertyName, FString InPropertyPath)
{
	PropertyID = PropertyTypeIDs.GetAnimTypeID(InPropertyPath);
	PropertyBindings = MakeShareable(new FTrackInstancePropertyBindings(InPropertyName, MoveTemp(InPropertyPath)));
}

FMovieScenePropertySectionTemplate::FMovieScenePropertySectionTemplate() = default;
FMovieScenePropertySectionTemplate::FMovieScenePropertySectionTemplate(const FMovieScenePropertySectionTemplate&) = default;
FMovieScenePropertySectionTemplate::~FMovieScenePropertySectionTemplate() = default;

FMovieScenePropertySectionTemplate::FMovieScenePropertySectionTemplate(FName PropertyName, const FString& InPropertyPath)
	: PropertyData(PropertyName, InPropertyPath)
{}

void FMovieScenePropertySectionTemplate::Setup(FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) const
{
	PropertyData.SetupTrack(PersistentData);
}

FMovieSceneAnimTypeID FMovieScenePropertySectionTemplate::GetPropertyTypeID() const
{
	return PropertyTypeIDs.GetAnimTypeID(PropertyData.PropertyPath);
}

const FMovieSceneInterrogationKey FMovieScenePropertySectionTemplate::GetFloatInterrogationKey()
{
	const static FMovieSceneAnimTypeID TypeID = FMovieSceneAnimTypeID::Unique();
	return TypeID;
}
const FMovieSceneInterrogationKey FMovieScenePropertySectionTemplate::GetInt32InterrogationKey()
{
	const static FMovieSceneAnimTypeID TypeID = FMovieSceneAnimTypeID::Unique();
	return TypeID;
}

const FMovieSceneInterrogationKey FMovieScenePropertySectionTemplate::GetVector4InterrogationKey()
{
	const static FMovieSceneAnimTypeID TypeID = FMovieSceneAnimTypeID::Unique();
	return TypeID;
}
const FMovieSceneInterrogationKey FMovieScenePropertySectionTemplate::GetVectorInterrogationKey()
{
	const static FMovieSceneAnimTypeID TypeID = FMovieSceneAnimTypeID::Unique();
	return TypeID;
}
const FMovieSceneInterrogationKey FMovieScenePropertySectionTemplate::GetVector2DInterrogationKey()
{
	const static FMovieSceneAnimTypeID TypeID = FMovieSceneAnimTypeID::Unique();
	return TypeID;
}

const FMovieSceneInterrogationKey FMovieScenePropertySectionTemplate::GetColorInterrogationKey()
{
	const static FMovieSceneAnimTypeID TypeID = FMovieSceneAnimTypeID::Unique();
	return TypeID;
}
