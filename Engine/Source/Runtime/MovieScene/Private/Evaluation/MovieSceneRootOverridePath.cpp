// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/MovieSceneRootOverridePath.h"
#include "Evaluation/MovieSceneSequenceHierarchy.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "IMovieScenePlayer.h"

#include "Algo/Find.h"

namespace UE
{
namespace MovieScene
{

FSubSequencePath::FSubSequencePath()
{}

FSubSequencePath::FSubSequencePath(FMovieSceneSequenceID LeafID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	Reset(LeafID, SharedPlaybackState->GetHierarchy());
}

FSubSequencePath::FSubSequencePath(FMovieSceneSequenceID LeafID, IMovieScenePlayer& Player)
{
	Reset(LeafID, Player.GetEvaluationTemplate().GetHierarchy());
}

FSubSequencePath::FSubSequencePath(FMovieSceneSequenceID LeafID, const FMovieSceneSequenceHierarchy* RootHierarchy)
{
	Reset(LeafID, RootHierarchy);
}

FMovieSceneSequenceID FSubSequencePath::FindCommonParent(const FSubSequencePath& A, const FSubSequencePath& B)
{
	if (A.PathToRoot.Num() == 0 || B.PathToRoot.Num() == 0)
	{
		return MovieSceneSequenceID::Root;
	}

	FMovieSceneSequenceID CommonParent = MovieSceneSequenceID::Root;

	int32 IndexA = A.PathToRoot.Num()-1;
	int32 IndexB = B.PathToRoot.Num()-1;

	// Keep walking into the path until we find a disprity in sequences
	for (; IndexA >= 0 && IndexB >= 0; --IndexA, --IndexB)
	{
		FMovieSceneSequenceID CurrentSequenceID = A.PathToRoot[IndexA].Accumulated;
		if (B.PathToRoot[IndexB].Accumulated == CurrentSequenceID)
		{
			CommonParent = CurrentSequenceID;
		}
		else
		{
			break;
		}
	}

	return CommonParent;
}

void FSubSequencePath::Reset()
{
	PathToRoot.Reset();
}

void FSubSequencePath::Reset(FMovieSceneSequenceID LeafID, const FMovieSceneSequenceHierarchy* RootHierarchy)
{
	check(LeafID != MovieSceneSequenceID::Invalid);

	PathToRoot.Reset();

	FMovieSceneSequenceID CurrentSequenceID = LeafID;

	check(LeafID == MovieSceneSequenceID::Root || RootHierarchy != nullptr);

	while (CurrentSequenceID != MovieSceneSequenceID::Root)
	{
		const FMovieSceneSequenceHierarchyNode* CurrentNode  = RootHierarchy->FindNode(CurrentSequenceID);
		const FMovieSceneSubSequenceData*       OuterSubData = RootHierarchy->FindSubData(CurrentSequenceID);
		if (!ensureAlwaysMsgf(CurrentNode && OuterSubData, TEXT("Malformed sequence hierarchy")))
		{
			return;
		}

		PathToRoot.Add(FSequenceIDPair{ OuterSubData->DeterministicSequenceID, CurrentSequenceID });
		CurrentSequenceID = CurrentNode->ParentID;
	}
}

bool FSubSequencePath::Contains(FMovieSceneSequenceID SequenceID) const
{
	return SequenceID == MovieSceneSequenceID::Root || Algo::FindBy(PathToRoot, SequenceID, &FSequenceIDPair::Accumulated) != nullptr;
}

int32 FSubSequencePath::NumGenerationsFromLeaf(FMovieSceneSequenceID SequenceID) const
{
	int32 Count = 0;

	// Count sequence IDs until we find the sequence ID
	// We must return 0 where SequenceID == Leaf, 1 for Immediate parents, 2 for grandparents etc
	for ( ; Count < PathToRoot.Num(); ++Count)
	{
		if (PathToRoot[Count].Accumulated == SequenceID)
		{
			return Count;
		}
	}

	ensureAlwaysMsgf(SequenceID == MovieSceneSequenceID::Root, TEXT("Specified SequenceID does not exist in this path"));
	return Count;
}

int32 FSubSequencePath::NumGenerationsFromRoot(FMovieSceneSequenceID SequenceID) const
{
	const int32 Num = PathToRoot.Num();
	int32 Count = 0;

	// Count sequence IDs from the root (ie, the tail of the path) until we find the sequence ID
	// We must return 0 where SequenceID == Root, 1 for children, 2 for grandchildren etc
	for ( ; Count < Num; ++Count)
	{
		const int32 Index = Num - Count - 1;
		if (PathToRoot[Index].Accumulated == SequenceID)
		{
			return Count;
		}
	}

	FMovieSceneSequenceID LeafID = PathToRoot.Num() != 0 ? PathToRoot.Last().Accumulated : MovieSceneSequenceID::Root;
	ensureAlwaysMsgf(LeafID == SequenceID, TEXT("Specified SequenceID does not exist in this path"));
	return Count;
}

FMovieSceneSequenceID FSubSequencePath::MakeLocalSequenceID(FMovieSceneSequenceID ParentSequenceID) const
{
	FMovieSceneSequenceID AccumulatedID = MovieSceneSequenceID::Root;
	for (FSequenceIDPair Pair : PathToRoot)
	{
		if (Pair.Accumulated == ParentSequenceID)
		{
			break;
		}

		AccumulatedID = AccumulatedID.AccumulateParentID(Pair.Unaccumulated);
	}

	return AccumulatedID;
}

FMovieSceneSequenceID FSubSequencePath::MakeLocalSequenceID(FMovieSceneSequenceID ParentSequenceID, FMovieSceneSequenceID TargetSequenceID) const
{
	FMovieSceneSequenceID AccumulatedID = MovieSceneSequenceID::Root;

	int32 Index = 0;

	// Skip over any that are not the target sequence ID
	for ( ; Index < PathToRoot.Num() && TargetSequenceID != PathToRoot[Index].Accumulated; ++Index)
	{}

	for ( ; Index < PathToRoot.Num(); ++Index)
	{
		FSequenceIDPair Pair = PathToRoot[Index];

		AccumulatedID = AccumulatedID.AccumulateParentID(Pair.Unaccumulated);

		if (Pair.Accumulated == ParentSequenceID)
		{
			break;
		}
	}

	return AccumulatedID;
}

void FSubSequencePath::PushGeneration(FMovieSceneSequenceID AccumulatedSequenceID, FMovieSceneSequenceID UnaccumulatedSequenceID)
{
	PathToRoot.Insert(FSequenceIDPair{ UnaccumulatedSequenceID, AccumulatedSequenceID }, 0);
}

void FSubSequencePath::PopTo(FMovieSceneSequenceID ParentSequenceID)
{
	const int32 NumToRemove = NumGenerationsFromLeaf(ParentSequenceID);
	PopGenerations(NumToRemove);
}

void FSubSequencePath::PopGenerations(int32 NumGenerations)
{
	if (NumGenerations != 0)
	{
		if(!ensureMsgf(NumGenerations <= PathToRoot.Num(), TEXT("FSubSequencePath::PopGenerations NumGenerations [%d] PathToRoot.Num [%d]. This can be caused by copy/pasting between sequences"), NumGenerations, PathToRoot.Num()))
		{
			NumGenerations = PathToRoot.Num();
		}

		// Remove children from the head of the array
		PathToRoot.RemoveAt(0, NumGenerations, EAllowShrinking::No);
	}
}


} // namespace MovieScene
} // namespace UE
