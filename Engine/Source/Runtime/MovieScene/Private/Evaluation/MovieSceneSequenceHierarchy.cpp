// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/MovieSceneSequenceHierarchy.h"
#include "Evaluation/MovieSceneEvaluationTreeFormatter.h"
#include "MovieSceneSequenceID.h"
#include "Sections/MovieSceneSubSection.h"
#include "MovieSceneSequence.h"
#include "MovieScene.h"
#include "MovieSceneTimeHelpers.h"
#include "MovieSceneTransformTypes.h"
#include "UObject/ReleaseObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "MovieSceneCommonHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSequenceHierarchy)

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FMovieSceneSubSequenceData::FMovieSceneSubSequenceData(const FMovieSceneSubSequenceData&) = default;
FMovieSceneSubSequenceData& FMovieSceneSubSequenceData::operator=(const FMovieSceneSubSequenceData&) = default;
FMovieSceneSubSequenceData::FMovieSceneSubSequenceData(FMovieSceneSubSequenceData&&) = default;
FMovieSceneSubSequenceData& FMovieSceneSubSequenceData::operator=(FMovieSceneSubSequenceData&&) = default;
FMovieSceneSubSequenceData::~FMovieSceneSubSequenceData() = default;

FMovieSceneSubSequenceTreeEntry::FMovieSceneSubSequenceTreeEntry() = default;
FMovieSceneSubSequenceTreeEntry::FMovieSceneSubSequenceTreeEntry(const FMovieSceneSubSequenceTreeEntry&) = default;
FMovieSceneSubSequenceTreeEntry& FMovieSceneSubSequenceTreeEntry::operator=(const FMovieSceneSubSequenceTreeEntry&) = default;
FMovieSceneSubSequenceTreeEntry::FMovieSceneSubSequenceTreeEntry(FMovieSceneSubSequenceTreeEntry&&) = default;
FMovieSceneSubSequenceTreeEntry& FMovieSceneSubSequenceTreeEntry::operator=(FMovieSceneSubSequenceTreeEntry&&) = default;
FMovieSceneSubSequenceTreeEntry::~FMovieSceneSubSequenceTreeEntry() = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FMovieSceneSubSequenceData::FMovieSceneSubSequenceData()
	: Sequence(nullptr)
	, HierarchicalBias(0)
	, AccumulatedFlags(EMovieSceneSubSectionFlags::None)
{}

FMovieSceneSubSequenceData::FMovieSceneSubSequenceData(const UMovieSceneSubSection& InSubSection)
	: Sequence(InSubSection.GetSequence())
	, DeterministicSequenceID(InSubSection.GetSequenceID())
	, ParentPlayRange(InSubSection.GetTrueRange())
	, HierarchicalBias(InSubSection.Parameters.HierarchicalBias)
	, AccumulatedFlags(InSubSection.Parameters.Flags)
	, bCanLoop(InSubSection.Parameters.bCanLoop)
#if WITH_EDITORONLY_DATA
	, SectionPath(*InSubSection.GetPathNameInMovieScene())
#endif
	, SubSectionSignature(InSubSection.GetSignature())
{
	using namespace UE::MovieScene;

	PreRollRange.Value  = TRange<FFrameNumber>::Empty();
	PostRollRange.Value = TRange<FFrameNumber>::Empty();


	UMovieSceneSequence* SequencePtr   = GetSequence();
	UMovieScene*         MovieScenePtr = SequencePtr ? SequencePtr->GetMovieScene() : nullptr;

	checkf(MovieScenePtr, TEXT("Attempting to construct sub sequence data with a null sequence."));

	TickResolution = MovieScenePtr->GetTickResolution();

	checkf(ParentPlayRange.GetLowerBound().IsClosed() && ParentPlayRange.GetUpperBound().IsClosed(), TEXT("Use of open (infinite) bounds with sub sections is not supported."));

	// Get the transform from the given section to its inner sequence.
	// Note that FMovieSceneCompiler will accumulate RootToSequenceTransform for us a bit later so that it ends up
	// being truly the full transform.
	OuterToInnerTransform = InSubSection.OuterToInnerTransform_NoInnerTimeWarp();

#if WITH_EDITORONLY_DATA
	RootToUnwarpedLocalTransform = OuterToInnerTransform;
	InSubSection.AppendInnerTimeWarpTransform(LocalToWarpedLocalTransform);
	OuterToInnerTransform.Append(LocalToWarpedLocalTransform);
#else
	InSubSection.AppendInnerTimeWarpTransform(OuterToInnerTransform);
#endif

	RootToSequenceTransform = OuterToInnerTransform;

	PlayRange.Value = UE::MovieScene::ConvertToDiscreteRange(OuterToInnerTransform.ComputeTraversedHull(ParentPlayRange.Value));

#if WITH_EDITORONLY_DATA
	// Set initial breadcrumbs
	OuterToInnerTransform.TransformTime(InSubSection.GetInclusiveStartFrame(), FTransformTimeParams().HarvestBreadcrumbs(StartTimeBreadcrumbs));
	OuterToInnerTransform.TransformTime(InSubSection.GetExclusiveEndFrame(), FTransformTimeParams().HarvestBreadcrumbs(EndTimeBreadcrumbs));
#endif // WITH_EDITORONLY_DATA

	// Make sure pre/postroll *ranges* are in the inner sequence's time space. Pre/PostRollFrames are in the outer sequence space.
	if (InSubSection.GetPreRollFrames() > 0)
	{
		FFrameNumber PreRollStart = OuterToInnerTransform.TransformTime(InSubSection.GetInclusiveStartFrame() - InSubSection.GetPreRollFrames()).FloorToFrame();
		FFrameNumber PreRollEnd   = OuterToInnerTransform.TransformTime(InSubSection.GetInclusiveStartFrame()).FloorToFrame();

		PreRollRange = TRange<FFrameNumber>(PreRollStart, PreRollEnd);
	}
	if (InSubSection.GetPostRollFrames() > 0)
	{
		FFrameNumber PostRollStart = OuterToInnerTransform.TransformTime(InSubSection.GetExclusiveEndFrame()).FloorToFrame();
		FFrameNumber PostRollEnd   = OuterToInnerTransform.TransformTime(InSubSection.GetExclusiveEndFrame() + InSubSection.GetPostRollFrames()).FloorToFrame();

		PostRollRange = TRange<FFrameNumber>(PostRollStart, PostRollEnd);
	}

	WeakCondition = MovieSceneHelpers::GetSequenceCondition(InSubSection.GetTypedOuter<UMovieSceneTrack>(), &InSubSection, true);
}

UMovieSceneSequence* FMovieSceneSubSequenceData::GetSequence() const
{
	UMovieSceneSequence* ResolvedSequence = GetLoadedSequence();
	if (!ResolvedSequence)
	{
		ResolvedSequence = Cast<UMovieSceneSequence>(Sequence.ResolveObject());

		if (!ResolvedSequence)
		{
			ResolvedSequence = Cast<UMovieSceneSequence>(Sequence.TryLoad());
		}

		CachedSequence = ResolvedSequence;
	}
	return ResolvedSequence;
}


UMovieSceneSequence* FMovieSceneSubSequenceData::GetLoadedSequence() const
{
	return CachedSequence.Get();
}

bool FMovieSceneSubSequenceData::IsDirty(const UMovieSceneSubSection& InSubSection) const
{
	return InSubSection.GetSignature() != SubSectionSignature || InSubSection.OuterToInnerTransform() != OuterToInnerTransform;
}

FMovieSceneSectionParameters FMovieSceneSubSequenceData::ToSubSectionParameters() const
{
	return FMovieSceneSectionParameters();
}

void FMovieSceneSequenceHierarchy::Add(const FMovieSceneSubSequenceData& Data, FMovieSceneSequenceIDRef ThisSequenceID, FMovieSceneSequenceIDRef ParentID)
{
	check(ParentID != MovieSceneSequenceID::Invalid);

	// Add (or update) the sub sequence data
	FMovieSceneSubSequenceData& AddedData = SubSequences.Add(ThisSequenceID, Data);

	// Set up the hierarchical information if we don't have any, or its wrong
	FMovieSceneSequenceHierarchyNode* ExistingHierarchyNode = FindNode(ThisSequenceID);
	if (!ExistingHierarchyNode || ExistingHierarchyNode->ParentID != ParentID)
	{
		if (!ExistingHierarchyNode)
		{
			// The node doesn't yet exist - create it
			FMovieSceneSequenceHierarchyNode Node(ParentID);
			Hierarchy.Add(ThisSequenceID, Node);
		}
		else
		{
			// The node exists already but under the wrong parent - we need to move it
			FMovieSceneSequenceHierarchyNode* Parent = FindNode(ExistingHierarchyNode->ParentID);
			check(Parent);
			// Remove it from its parent's children
			Parent->Children.Remove(ThisSequenceID);

			// Set the parent ID
			ExistingHierarchyNode->ParentID = ParentID;
		}

		// Add the node to its parent's children array
		FMovieSceneSequenceHierarchyNode* Parent = FindNode(ParentID);
		check(Parent);
		ensure(!Parent->Children.Contains(ThisSequenceID));
		Parent->Children.Add(ThisSequenceID);
	}
}

void FMovieSceneSequenceHierarchy::Remove(TArrayView<const FMovieSceneSequenceID> SequenceIDs)
{
	TArray<FMovieSceneSequenceID, TInlineAllocator<16>> IDsToRemove;
	IDsToRemove.Append(SequenceIDs.GetData(), SequenceIDs.Num());

	while (IDsToRemove.Num())
	{
		int32 NumRemaining = IDsToRemove.Num();
		for (int32 Index = 0; Index < NumRemaining; ++Index)
		{
			FMovieSceneSequenceID ID = IDsToRemove[Index];

			SubSequences.Remove(ID);

			// Remove all children too
			if (const FMovieSceneSequenceHierarchyNode* Node = FindNode(ID))
			{
				FMovieSceneSequenceHierarchyNode* Parent = FindNode(Node->ParentID);
				if (Parent)
				{
					Parent->Children.Remove(ID);
				}

				IDsToRemove.Append(Node->Children);
				Hierarchy.Remove(ID);
			}
		}

		IDsToRemove.RemoveAt(0, NumRemaining);
	}
}

void FMovieSceneSequenceHierarchy::AddRange(const TRange<FFrameNumber>& RootSpaceRange, FMovieSceneSequenceIDRef InSequenceID, ESectionEvaluationFlags InFlags, FMovieSceneWarpCounter RootToSequenceWarpCounter)
{
	AddRange(RootSpaceRange, InSequenceID, InFlags);
}

void FMovieSceneSequenceHierarchy::AddRange(const TRange<FFrameNumber>& RootSpaceRange, FMovieSceneSequenceIDRef InSequenceID, ESectionEvaluationFlags InFlags)
{
	Tree.Data.AddUnique(RootSpaceRange, FMovieSceneSubSequenceTreeEntry{ InSequenceID, InFlags });
}

FArchive& operator<<(FArchive& Ar, FMovieSceneSubSequenceTreeEntry& InOutEntry)
{
	Ar << InOutEntry.SequenceID << InOutEntry.Flags;
	return Ar;
}

bool operator==(const FMovieSceneSubSequenceTreeEntry& A, const FMovieSceneSubSequenceTreeEntry& B)
{
	return A.SequenceID == B.SequenceID 
		&& A.Flags == B.Flags;
}

#if !NO_LOGGING
void FMovieSceneSequenceHierarchy::LogHierarchy() const
{
	using FNodeInfo = TTuple<FMovieSceneSequenceID, const FMovieSceneSequenceHierarchyNode*, uint32>;

	TArray<FNodeInfo> NodeInfoStack;
	NodeInfoStack.Add(FNodeInfo{MovieSceneSequenceID::Root, &RootNode, 0});

	while (NodeInfoStack.Num() > 0)
	{
		const FNodeInfo CurNodeInfo = NodeInfoStack.Pop();

		const FMovieSceneSequenceID CurSequenceID = CurNodeInfo.Get<0>();
		const FMovieSceneSequenceHierarchyNode* CurNode = CurNodeInfo.Get<1>();
		const uint32 CurDepth = CurNodeInfo.Get<2>();

		if (CurSequenceID == MovieSceneSequenceID::Root)
		{
			UE_LOGF(LogMovieScene, Log, "ROOT SEQUENCE");
		}
		else
		{
			const FMovieSceneSubSequenceData* CurData = SubSequences.Find(CurSequenceID);

			FString Indent;
			Indent.Append(TEXT(" "), CurDepth * 2);
			UE_LOGF(LogMovieScene, Log, "%ls%ls Loop=%ls HBias=%d UnwarpedRange=%ls Transform=%ls", 
					*Indent, *CurData->GetSequence()->GetName(),
					*LexToString(CurData->bCanLoop),
					CurData->HierarchicalBias,
					*LexToString(CurData->ParentPlayRange.Value),
					*LexToString(CurData->RootToSequenceTransform));
		}

		const TArray<FMovieSceneSequenceID> CurNodeChildren = CurNodeInfo.Get<1>()->Children;
		for (int32 Index = CurNodeChildren.Num() - 1; Index >= 0; --Index)
		{
			const FMovieSceneSequenceID CurChildID = CurNodeChildren[Index];
			const FMovieSceneSequenceHierarchyNode* CurChild = Hierarchy.Find(CurChildID);
			if (ensure(CurChild && CurChild->ParentID == CurSequenceID))
			{
				NodeInfoStack.Add(FNodeInfo{CurChildID, CurChild, CurDepth + 1});
			}
		}
	}
}

void FMovieSceneSequenceHierarchy::LogSubSequenceTree() const
{
	using TreeFormatter = TMovieSceneEvaluationTreeFormatter<FMovieSceneSubSequenceTreeEntry>;
	TreeFormatter Formatter(Tree.Data);
	Formatter.DataFormatter = TreeFormatter::FOnFormatData::CreateLambda(
			[](const FMovieSceneSubSequenceTreeEntry& Entry, TStringBuilder<256>& Builder)
			{
				Builder.Appendf(TEXT("ID=%d"),
					Entry.SequenceID.GetInternalValue());
			});
	Formatter.LogTree();
}
#endif


