// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/MovieSceneSequenceTransform.h"
#include "MovieSceneTimeHelpers.h"
#include "Misc/FrameRate.h"
#include "Variants/MovieSceneTimeWarpGetter.h"
#include "Variants/MovieSceneTimeWarpVariantPayloads.h"
#include "MovieSceneTransformTypes.h"
#include "MovieSceneTimeHelpers.h"
#include "Algo/AllOf.h"
#include "Algo/AnyOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSequenceTransform)

namespace UE::MovieScene
{
	TRange<FFrameTime> TranslateRange(const TRange<FFrameTime>& InRange, FFrameTime Offset)
	{
		TRange<FFrameTime> Result = InRange;
		if (InRange.HasLowerBound())
		{
			Result.SetLowerBoundValue(InRange.GetLowerBoundValue() + Offset);
		}
		if (InRange.HasUpperBound())
		{
			Result.SetUpperBoundValue(InRange.GetUpperBoundValue() + Offset);
		}
		return Result;
	}

	void CorrectInsideOutRange(TRange<FFrameTime>& InOutRange)
	{
		if (InOutRange.HasUpperBound() && InOutRange.HasLowerBound())
		{
			FFrameTime LowerBoundValue = InOutRange.GetLowerBoundValue();
			if (LowerBoundValue > InOutRange.GetUpperBoundValue())
			{
				InOutRange.SetLowerBoundValue(InOutRange.GetUpperBoundValue());
				InOutRange.SetUpperBoundValue(LowerBoundValue);
			}
		}
	}

	FTransformTimeParams& FTransformTimeParams::HarvestBreadcrumbs(FMovieSceneTransformBreadcrumbs& OutBreadcrumbs)
	{
		OutBreadcrumbs.Reset();
		Breadcrumbs = &OutBreadcrumbs;
		return *this;
	}

	FTransformTimeParams& FTransformTimeParams::AppendBreadcrumbs(FMovieSceneTransformBreadcrumbs& OutBreadcrumbs)
	{
		Breadcrumbs = &OutBreadcrumbs;
		return *this;
	}

	FTransformTimeParams& FTransformTimeParams::TrackCycleCounts(TOptional<int32>* OutCycleCounter)
	{
		CycleCount = OutCycleCounter;
		return *this;
	}

	FTransformTimeParams& FTransformTimeParams::IgnoreClamps()
	{
		bIgnoreClamps = true;
		return *this;
	}

} // namespace UE::MovieScene;

FString LexToString(const FMovieSceneSequenceTransform& InTransform)
{
	TStringBuilder<256> Builder;

	Builder.Append(LexToString(InTransform.LinearTransform));

	int32 NestedIndex = 0;
	for (const FMovieSceneNestedSequenceTransform& Nested : InTransform.NestedTransforms)
	{
		if (Nested.IsIdentity())
		{
			Builder.Appendf(TEXT(" [ %d = "), NestedIndex);
			Nested.ToString(Builder);
			Builder.Append(TEXT(" ]"));
		}

		++NestedIndex;
	}

	return Builder.ToString();
}

FString LexToString(const FMovieSceneWarpCounter& InCounter)
{
	if (InCounter.Num() == 0)
	{
		return FString("[]");
	}

	TStringBuilder<256> Builder;

	Builder.Append(TEXT("["));
	int32 Index = 0;
	for (FFrameTime Breadcrumb : InCounter)
	{
		if (Index > 0)
		{
			Builder.Append(TEXT(","));
		}
		Builder.Appendf(TEXT("%.3f"), Breadcrumb.AsDecimal());
		++Index;
	}
	Builder.Append(TEXT("]"));

	FString OutString = Builder.ToString();
	return OutString;
}

FFrameTime operator*(FFrameTime InTime, const FMovieSceneSequenceTransform& RHS)
{
	const uint32 NestedTransformsSize = RHS.NestedTransforms.Num();
	if (NestedTransformsSize == 0)
	{
		return InTime * RHS.LinearTransform;
	}
	else
	{
		FFrameTime OutTime = InTime * RHS.LinearTransform;
		for (const FMovieSceneNestedSequenceTransform& NestedTransform : RHS.NestedTransforms)
		{
			OutTime = NestedTransform.TransformTime(OutTime);
		}
		return OutTime;
	}
}

FMovieSceneTransformBreadcrumbs::operator TArrayView<const FFrameTime>() const
{
	return Breadcrumbs;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FMovieSceneWarpCounter::FMovieSceneWarpCounter() = default;
FMovieSceneWarpCounter::FMovieSceneWarpCounter(const FMovieSceneWarpCounter&) = default;
FMovieSceneWarpCounter& FMovieSceneWarpCounter::operator=(const FMovieSceneWarpCounter&) = default;
FMovieSceneWarpCounter::FMovieSceneWarpCounter(FMovieSceneWarpCounter&&) = default;
FMovieSceneWarpCounter& FMovieSceneWarpCounter::operator=(FMovieSceneWarpCounter&&) = default;
FMovieSceneWarpCounter::~FMovieSceneWarpCounter() = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FMovieSceneNestedSequenceTransform::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		TimeScale.MakeWeakUnsafe();
	}
}

void FMovieSceneNestedSequenceTransform::ToString(TStringBuilderBase<TCHAR>& OutBuilder) const
{
	switch (TimeScale.GetType())
	{
	case EMovieSceneTimeWarpType::FixedPlayRate:
		OutBuilder.Append(LexToString(FMovieSceneTimeTransform(Offset, TimeScale.AsFixedPlayRateFloat())));
		return;  // Explicit return because there's nothing else top do

	case EMovieSceneTimeWarpType::FixedTime:
		{
			FMovieSceneTimeWarpFixedFrame Value = TimeScale.AsFixedTime();
			OutBuilder.Appendf(TEXT("Fixed Frame: %s"), *LexToString(FFrameTime(Value.FrameNumber) + Offset));
		}
		break;
	case EMovieSceneTimeWarpType::FrameRate:
		{
			FFrameRate FrameRate = TimeScale.AsFrameRate().GetFrameRate();
			OutBuilder.Appendf(TEXT("Frame Rate: [%i/%i]"), FrameRate.Numerator, FrameRate.Denominator);
		}
		break;
	case EMovieSceneTimeWarpType::Loop:
		{
			FMovieSceneTimeWarpLoop Value = TimeScale.AsLoop();
			OutBuilder.Appendf(TEXT("Loop [%s:%s)"), *LexToString(-Offset), *LexToString(-Offset+Value.Duration));
		}
		break;
	case EMovieSceneTimeWarpType::Clamp:
		{
			FMovieSceneTimeWarpClamp Value = TimeScale.AsClamp();
			OutBuilder.Appendf(TEXT("Clamp [%s:%s)"), *LexToString(-Offset), *LexToString(-Offset+Value.Max));
		}
		break;
	case EMovieSceneTimeWarpType::LoopFloat:
		{
			FMovieSceneTimeWarpLoopFloat Value = TimeScale.AsLoopFloat();
			OutBuilder.Appendf(TEXT("Loop [%s:%s)"), *LexToString(-Offset), *LexToString(-Offset+FFrameTime::FromDecimal(Value.Duration)));
		}
		break;
	case EMovieSceneTimeWarpType::ClampFloat:
		{
			FMovieSceneTimeWarpClampFloat Value = TimeScale.AsClampFloat();
			OutBuilder.Appendf(TEXT("Clamp [%s:%s)"), *LexToString(-Offset), *LexToString(-Offset+FFrameTime::FromDecimal(Value.Max)));
		}
		break;
	case EMovieSceneTimeWarpType::Custom:
		if (UMovieSceneTimeWarpGetter* Custom = TimeScale.AsCustom())
		{
			OutBuilder.Append(Custom->GetName());
		}
		break;
	}

	if (Offset != 0)
	{
		OutBuilder.Appendf(TEXT(" + %s"), *LexToString(Offset));
	}
}

FFrameTime FMovieSceneNestedSequenceTransform::TransformTime(FFrameTime InTime) const
{
	switch (TimeScale.GetType())
	{
	case EMovieSceneTimeWarpType::FixedPlayRate:
		return InTime * FMovieSceneTimeTransform(Offset, TimeScale.AsFixedPlayRateFloat());

	case EMovieSceneTimeWarpType::FixedTime:
		return TimeScale.AsFixedTime().FrameNumber + Offset;

	case EMovieSceneTimeWarpType::FrameRate:
		return ConvertFrameTime(InTime + Offset, TimeScale.AsFrameRate().GetFrameRate(), FFrameRate(1, 1));

	case EMovieSceneTimeWarpType::Loop:
		return TimeScale.AsLoop().LoopTime(InTime + Offset) - Offset;

	case EMovieSceneTimeWarpType::Clamp:
		return TimeScale.AsClamp().Clamp(InTime + Offset) - Offset;

	case EMovieSceneTimeWarpType::LoopFloat:
		return TimeScale.AsLoopFloat().LoopTime(InTime + Offset) - Offset;

	case EMovieSceneTimeWarpType::ClampFloat:
		return TimeScale.AsClampFloat().Clamp(InTime + Offset) - Offset;

	case EMovieSceneTimeWarpType::Custom:
		if (UMovieSceneTimeWarpGetter* Custom = TimeScale.AsCustom())
		{
			return Custom->RemapTime(InTime + Offset);
		}
		break;
	}

	return InTime;
}

FFrameTime FMovieSceneNestedSequenceTransform::TransformTime(FFrameTime InTime, const UE::MovieScene::FTransformTimeParams& Params) const
{
	switch (TimeScale.GetType())
	{
	case EMovieSceneTimeWarpType::FixedPlayRate:
		return InTime * FMovieSceneTimeTransform(Offset, TimeScale.AsFixedPlayRateFloat());

	case EMovieSceneTimeWarpType::FixedTime:
		return TimeScale.AsFixedTime().FrameNumber + Offset;

	case EMovieSceneTimeWarpType::FrameRate:
		return ConvertFrameTime(InTime + Offset, TimeScale.AsFrameRate().GetFrameRate(), FFrameRate(1, 1));

	case EMovieSceneTimeWarpType::Loop:
		if (Params.CycleCount)
		{
			*Params.CycleCount = 0;
			return TimeScale.AsLoop().LoopTime(InTime + Offset, Params.CycleCount->GetValue()) - Offset;
		}
		else
		{
			return TimeScale.AsLoop().LoopTime(InTime + Offset) - Offset;
		}

	case EMovieSceneTimeWarpType::Clamp:
		if (Params.bIgnoreClamps)
		{
			break;
		}
		return TimeScale.AsClamp().Clamp(InTime + Offset) - Offset;

	case EMovieSceneTimeWarpType::LoopFloat:
		return TimeScale.AsLoopFloat().LoopTime(InTime + Offset) - Offset;

	case EMovieSceneTimeWarpType::ClampFloat:
		if (Params.bIgnoreClamps)
		{
			break;
		}
		return TimeScale.AsClampFloat().Clamp(InTime + Offset) - Offset;

	case EMovieSceneTimeWarpType::Custom:
		if (UMovieSceneTimeWarpGetter* Custom = TimeScale.AsCustom())
		{
			return Custom->RemapTime(InTime + Offset);
		}
		break;
	}

	return InTime;
}

TRange<FFrameTime> FMovieSceneNestedSequenceTransform::ComputeTraversedHull(const TRange<FFrameTime>& InRange) const
{
	using namespace UE::MovieScene;

	EMovieSceneTimeWarpType Type = TimeScale.GetType();
	if (Type == EMovieSceneTimeWarpType::FixedPlayRate)
	{
		FMovieSceneTimeTransform LinearTransform(Offset, TimeScale.AsFixedPlayRate());

		TRange<FFrameTime> Result = InRange * LinearTransform;
		CorrectInsideOutRange(Result);
		return Result;
	}
	else if (Type == EMovieSceneTimeWarpType::FixedTime)
	{
		FFrameTime FixedFrame = Offset + TimeScale.AsFixedTime().FrameNumber;
		return TRange<FFrameTime>::Inclusive(FixedFrame, FixedFrame);
	}
	else if (Type == EMovieSceneTimeWarpType::FrameRate)
	{
		FFrameRate Rate = TimeScale.AsFrameRate().GetFrameRate();

		TRange<FFrameTime> Result = InRange;
		if (Result.HasLowerBound())
		{
			Result.SetLowerBoundValue(ConvertFrameTime(Result.GetLowerBoundValue() + Offset, Rate, FFrameRate(1, 1)));
		}
		if (Result.HasUpperBound())
		{
			Result.SetUpperBoundValue(ConvertFrameTime(Result.GetUpperBoundValue() + Offset, Rate, FFrameRate(1, 1)));
		}
		return Result;
	}

	// Handle other types
	TRange<FFrameTime> OffsetRange = TranslateRange(InRange, Offset);

	switch (Type)
	{

	case EMovieSceneTimeWarpType::Loop:
	{
		OffsetRange = TimeScale.AsLoop().ComputeTraversedHull(OffsetRange);
		break;
	}

	case EMovieSceneTimeWarpType::Clamp:
	{
		OffsetRange = TimeScale.AsClamp().ComputeTraversedHull(OffsetRange);
		break;
	}

	case EMovieSceneTimeWarpType::LoopFloat:
	{
		OffsetRange = TimeScale.AsLoopFloat().ComputeTraversedHull(OffsetRange);
		break;
	}

	case EMovieSceneTimeWarpType::ClampFloat:
	{
		OffsetRange = TimeScale.AsClampFloat().ComputeTraversedHull(OffsetRange);
		break;
	}

	case EMovieSceneTimeWarpType::Custom:
		if (UMovieSceneTimeWarpGetter* Custom = TimeScale.AsCustom())
		{
			OffsetRange = Custom->ComputeTraversedHull(OffsetRange);
		}
		return OffsetRange;
	}

	OffsetRange = TranslateRange(OffsetRange, -Offset);
	return OffsetRange;
}

bool FMovieSceneNestedSequenceTransform::ExtractBoundariesWithinRange(const TRange<FFrameTime>& Range, const TFunctionRef<bool(FFrameTime)>& InVisitor) const
{
	EMovieSceneTimeWarpType WarpType = TimeScale.GetType();

	auto VisitorOffsetWrapper = [this, Range, InVisitor](FFrameTime Time)
	{
		// Factor the loop offset
		Time -= this->Offset;
		if (Range.Contains(Time))
		{
			return InVisitor(Time);
		}
		return true;
	};

	switch (WarpType)
	{
	case EMovieSceneTimeWarpType::Loop:
		return TimeScale.AsLoop().ExtractBoundariesWithinRange(Range, VisitorOffsetWrapper);
	case EMovieSceneTimeWarpType::LoopFloat:
		return TimeScale.AsLoopFloat().ExtractBoundariesWithinRange(Range, VisitorOffsetWrapper);
	default:
		return false;
	}
}

bool FMovieSceneNestedSequenceTransform::SupportsBoundaries() const
{
	switch (TimeScale.GetType())
	{
	case EMovieSceneTimeWarpType::Loop:
	case EMovieSceneTimeWarpType::LoopFloat:
		return true;

	default:
		return false;
	}
}

TOptional<UE::MovieScene::ETimeWarpChannelDomain> FMovieSceneNestedSequenceTransform::GetWarpDomain() const
{
	UMovieSceneTimeWarpGetter* Getter = TimeScale.GetType() == EMovieSceneTimeWarpType::Custom
		? TimeScale.AsCustom()
		: nullptr;

	if (Getter)
	{
		return Getter->GetDomain();
	}

	return TOptional<UE::MovieScene::ETimeWarpChannelDomain>();
}

FMovieSceneInverseNestedSequenceTransform FMovieSceneNestedSequenceTransform::Inverse() const
{
	if (TimeScale.GetType() == EMovieSceneTimeWarpType::FixedPlayRate)
	{
		const double PlayRate = TimeScale.AsFixedPlayRate();
		checkf(!FMath::IsNearlyZero(PlayRate), TEXT("Play rate cannot be zero! This should be expressed as a nullptr TimeScale with FLAG_Zero."));

		FMovieSceneInverseNestedSequenceTransform InverseTransform;
		InverseTransform.Offset    = -Offset / PlayRate;
		InverseTransform.TimeScale = 1.0 / PlayRate;
		return InverseTransform;
	}

	FMovieSceneInverseNestedSequenceTransform InverseTransform;
	InverseTransform.Offset    = Offset;
	InverseTransform.TimeScale = TimeScale;
	return InverseTransform;
}

FMovieSceneTimeTransform FMovieSceneInverseNestedSequenceTransform::AsLinear() const
{
	const float PlayRate = TimeScale.AsFixedPlayRateFloat();
	checkf(!FMath::IsNearlyZero(PlayRate), TEXT("Play rate cannot be zero! This should be expressed as a nullptr TimeScale with FLAG_Zero."));
	return FMovieSceneTimeTransform(Offset, TimeScale.AsFixedPlayRateFloat());
}

bool FMovieSceneInverseNestedSequenceTransform::TransformTimeWithinRange(FFrameTime InTime, const TFunctionRef<bool(FFrameTime)>& InVisitor, FFrameTime RangeStart, FFrameTime RangeEnd) const
{
	auto OffsetVisitor = [this, &InVisitor](FFrameTime VisitTime)
	{
		return InVisitor(VisitTime - this->Offset);
	};

	FFrameTime TransformedTime;
	switch (TimeScale.GetType())
	{
	case EMovieSceneTimeWarpType::FixedPlayRate:
		TransformedTime = InTime * AsLinear();
		break;

	case EMovieSceneTimeWarpType::FixedTime:
		TransformedTime = TimeScale.AsFixedTime().FrameNumber + Offset;
		break;

	case EMovieSceneTimeWarpType::FrameRate:
		TransformedTime = ConvertFrameTime(InTime + Offset, FFrameRate(1, 1), TimeScale.AsFrameRate().GetFrameRate());
		break;

	case EMovieSceneTimeWarpType::Loop:
		if (RangeStart.FrameNumber.Value != MIN_int32)
		{
			RangeStart += Offset;
		}
		if (RangeEnd.FrameNumber.Value != MAX_int32)
		{
			RangeEnd += Offset;
		}

		return TimeScale.AsLoop().InverseRemapTimeWithinRange(InTime + Offset, RangeStart, RangeEnd, OffsetVisitor);

	case EMovieSceneTimeWarpType::Clamp:
		if (InTime < -Offset || InTime > TimeScale.AsClamp().Max - Offset)
		{
			return true;
		}

		TransformedTime = InTime;
		break;

	case EMovieSceneTimeWarpType::LoopFloat:
		if (RangeStart.FrameNumber.Value != MIN_int32)
		{
			RangeStart += Offset;
		}
		if (RangeEnd.FrameNumber.Value != MAX_int32)
		{
			RangeEnd += Offset;
		}

		return TimeScale.AsLoopFloat().InverseRemapTimeWithinRange(InTime + Offset, RangeStart, RangeEnd, OffsetVisitor);

	case EMovieSceneTimeWarpType::ClampFloat:
		if (InTime < -Offset || InTime > FFrameTime::FromDecimal(TimeScale.AsClampFloat().Max) - Offset)
		{
			return true;
		}

		TransformedTime = InTime;
		break;

	case EMovieSceneTimeWarpType::Custom:
		if (UMovieSceneTimeWarpGetter* Custom = TimeScale.AsCustom())
		{
			return Custom->InverseRemapTimeWithinRange(InTime, RangeStart, RangeEnd, OffsetVisitor);
		}
		return true;
	}

	if (TransformedTime >= RangeStart && TransformedTime <= RangeEnd)
	{
		return InVisitor(TransformedTime);
	}

	return true;
}

TOptional<FFrameTime> FMovieSceneInverseNestedSequenceTransform::TryTransformTime(FFrameTime InTime, FFrameTime Breadcrumb) const
{
	return TryTransformTime(InTime, Breadcrumb, UE::MovieScene::FInverseTransformTimeParams());
}

TOptional<FFrameTime> FMovieSceneInverseNestedSequenceTransform::TryTransformTime(FFrameTime InTime, FFrameTime Breadcrumb, const UE::MovieScene::FInverseTransformTimeParams& Params) const
{
	using namespace UE::MovieScene;

	switch (TimeScale.GetType())
	{
	case EMovieSceneTimeWarpType::FixedPlayRate:
		return InTime * FMovieSceneTimeTransform(Offset, TimeScale.AsFixedPlayRateFloat());

	case EMovieSceneTimeWarpType::FixedTime:
		if (InTime == TimeScale.AsFixedTime().FrameNumber + Offset)
		{
			return InTime;
		}
		break;

	case EMovieSceneTimeWarpType::FrameRate:
		return ConvertFrameTime(InTime, FFrameRate(1, 1), TimeScale.AsFrameRate().GetFrameRate()) - Offset;

	case EMovieSceneTimeWarpType::Loop:
		{
			TOptional<FFrameTime> Result = TimeScale.AsLoop().InverseRemapTimeCycled(InTime + Offset, Breadcrumb + Offset, Params);
			if (Result)
			{
				return Result.GetValue() - Offset;
			}
		}
		break;

	case EMovieSceneTimeWarpType::Clamp:
		if (EnumHasAnyFlags(Params.Flags, EInverseEvaluateFlags::IgnoreClamps) || ( InTime >= -Offset && InTime <= TimeScale.AsClamp().Max - Offset) )
		{
			return InTime;
		}
		break;

	case EMovieSceneTimeWarpType::LoopFloat:
		{
			TOptional<FFrameTime> Result = TimeScale.AsLoopFloat().InverseRemapTimeCycled(InTime + Offset, Breadcrumb + Offset, Params);
			if (Result)
			{
				return Result.GetValue() - Offset;
			}
		}
		break;

	case EMovieSceneTimeWarpType::ClampFloat:
		if (EnumHasAnyFlags(Params.Flags, EInverseEvaluateFlags::IgnoreClamps) || ( InTime >= -Offset && InTime <= FFrameTime::FromDecimal(TimeScale.AsClampFloat().Max) - Offset) )
		{
			return InTime;
		}
		break;

	case EMovieSceneTimeWarpType::Custom:
		if (UMovieSceneTimeWarpGetter* Custom = TimeScale.AsCustom())
		{
			TOptional<FFrameTime> Result = Custom->InverseRemapTimeCycled(InTime, Breadcrumb, Params);
			if (Result)
			{
				return Result.GetValue() - Offset;
			}
		}
		break;
	}

	return TOptional<FFrameTime>();
}

TOptional<FFrameTime> FMovieSceneInverseSequenceTransform::TryTransformTime(FFrameTime InTime) const
{
	return TryTransformTime(InTime, UE::MovieScene::FInverseTransformTimeParams());
}

TOptional<FFrameTime> FMovieSceneInverseSequenceTransform::TryTransformTime(FFrameTime InTime, const UE::MovieScene::FInverseTransformTimeParams& Params) const
{
	FFrameTime OutTime = InTime;

	for (const FMovieSceneInverseNestedSequenceTransform& NestedTransform : NestedTransforms)
	{
		FFrameTime Breadcrumb;

		TOptional<FFrameTime> NewTime = NestedTransform.TryTransformTime(OutTime, Breadcrumb, Params);
		if (!NewTime)
		{
			return TOptional<FFrameTime>();
		}
		OutTime = NewTime.GetValue();
	}

	return OutTime * LinearTransform;
}



TOptional<FFrameTime> FMovieSceneInverseSequenceTransform::TryTransformTime(FFrameTime InTime, const FMovieSceneTransformBreadcrumbs& InBreadcrumbs) const
{
	return TryTransformTime(InTime, InBreadcrumbs, UE::MovieScene::FInverseTransformTimeParams());
}

TOptional<FFrameTime> FMovieSceneInverseSequenceTransform::TryTransformTime(FFrameTime InTime, const FMovieSceneTransformBreadcrumbs& InBreadcrumbs, const UE::MovieScene::FInverseTransformTimeParams& Params) const
{
	FFrameTime OutTime = InTime;

	int32 BreadcrumbIndex = InBreadcrumbs.Num() - 1;
	for (const FMovieSceneInverseNestedSequenceTransform& NestedTransform : NestedTransforms)
	{
		FFrameTime Breadcrumb;

		const bool bShouldHaveBreadcrumb = (InBreadcrumbs.GetMode() == EMovieSceneBreadcrumbMode::Dense || NestedTransform.NeedsBreadcrumb());
		if (bShouldHaveBreadcrumb && InBreadcrumbs.IsValidIndex(BreadcrumbIndex))
		{
			Breadcrumb = InBreadcrumbs[BreadcrumbIndex--];
		}

		TOptional<FFrameTime> NewTime = NestedTransform.TryTransformTime(OutTime, Breadcrumb, Params);
		if (!NewTime)
		{
			return TOptional<FFrameTime>();
		}
		OutTime = NewTime.GetValue();
	}

	return OutTime * LinearTransform;
}

bool FMovieSceneInverseSequenceTransform::RecursiveTransformTimeWithinRange(int32 NestingIndex, FFrameTime InTime, const TFunctionRef<bool(FFrameTime)>& FinalVisitor, TArrayView<const FFrameTime> StartBreadcrumbs, TArrayView<const FFrameTime> EndBreadcrumbs) const
{
	for ( ; NestingIndex < NestedTransforms.Num(); ++NestingIndex)
	{
		const FMovieSceneInverseNestedSequenceTransform& NestedTransform = NestedTransforms[NestingIndex];

		// Linear transform is easy - keep looping them
		if (NestedTransform.IsLinear())
		{
			InTime = InTime * NestedTransform.AsLinear();
		}
		// Warped ranges may map to zero or more times in the outer sequence
		//     so perform a complete recursive expansion on all of them
		else if (ensureMsgf(StartBreadcrumbs.Num() > 0 && EndBreadcrumbs.Num() > 0,
			TEXT("Breadcrumb count mismatch in inverse transform computation")))
		{
			auto TransformNext = [this, FinalVisitor, NestingIndex, StartBreadcrumbs, EndBreadcrumbs](FFrameTime NextTime)
			{
				return this->RecursiveTransformTimeWithinRange(
					NestingIndex+1,
					NextTime,
					FinalVisitor,
					StartBreadcrumbs.LeftChop(1),
					EndBreadcrumbs.LeftChop(1)
				);
			};

			// RecursiveTransformTimeWithinRange will complete the recursion
			return NestedTransform.TransformTimeWithinRange(
				InTime,
				TransformNext,
				StartBreadcrumbs.Last(),
				EndBreadcrumbs.Last()
			);
		}
	}

	return FinalVisitor(InTime * LinearTransform);
}

bool FMovieSceneInverseSequenceTransform::TransformFiniteRangeWithinRange(const TRange<FFrameTime>& InRange, TFunctionRef<bool(TRange<FFrameTime>)> InVisitor, const FMovieSceneTransformBreadcrumbs& StartBreadcrumbs, const FMovieSceneTransformBreadcrumbs& EndBreadcrumbs) const
{
	check(!InRange.GetLowerBound().IsOpen() && !InRange.GetUpperBound().IsOpen());

	if (NestedTransforms.Num() == 0)
	{
		// Only one solution
		return InVisitor(InRange * LinearTransform);
	}

	TArray<FFrameTime> LowerBounds, UpperBounds;

	// Transform lower bounds
	auto VisitLower = [&LowerBounds](FFrameTime InFrameTime)
	{
		LowerBounds.Add(InFrameTime);
		return true;
	};
	TransformTimeWithinRange(InRange.GetLowerBoundValue(), VisitLower, StartBreadcrumbs, EndBreadcrumbs);

	// Transform upper bounds
	auto VisitUpper = [&UpperBounds](FFrameTime InFrameTime)
	{
		UpperBounds.Add(InFrameTime);
		return true;
	};
	TransformTimeWithinRange(InRange.GetUpperBoundValue(), VisitUpper, StartBreadcrumbs, EndBreadcrumbs);

	Algo::Sort(LowerBounds);
	Algo::Sort(UpperBounds);

	int32 LwrIndex = 0, UprIndex = 0;

	// Handle leading upper bounds - should only be one?
	while (UprIndex < UpperBounds.Num())
	{
		if (LwrIndex < LowerBounds.Num() && UpperBounds[UprIndex] >= LowerBounds[LwrIndex])
		{
			break;
		}

		// Maintain bound exclusivity
		TRange<FFrameTime> Result = InRange;
		if (UprIndex < UpperBounds.Num() - 1)
		{
			TRangeBound<FFrameTime> NewLower = Result.GetUpperBound();
			NewLower.SetValue(UpperBounds[UprIndex + 1]);
			Result.SetLowerBound(TRangeBound<FFrameTime>::FlipInclusion(NewLower));
		}
		else
		{
			Result.SetLowerBound(TRangeBound<FFrameTime>::Open());
		}

		Result.SetUpperBoundValue(UpperBounds[UprIndex]);
		if (!InVisitor(Result))
		{
			return false;
		}

		++UprIndex;
	}

	// Handle finite ranges
	while (LwrIndex < LowerBounds.Num() && UprIndex < UpperBounds.Num())
	{
		FFrameTime LowerBound = LowerBounds[LwrIndex];

		TRange<FFrameTime> Result = InRange;
		Result.SetLowerBoundValue(LowerBound);

		// Skip any upper bounds that are < this lower bound
		while (UprIndex < UpperBounds.Num() && UpperBounds[UprIndex] <= LowerBound)
		{
			++UprIndex;
		}
		
		if (UprIndex < UpperBounds.Num())
		{
			Result.SetUpperBoundValue(UpperBounds[UprIndex]);
			if (!Result.IsEmpty() && !InVisitor(Result))
			{
				return false;
			}
		}

		++LwrIndex;
		++UprIndex;
	}

	// Handle trailing lower bounds - there can be cases where there are multiple, especially in the case of nested
	//    if a looping subsequences that have their end cropped
	while (LwrIndex < LowerBounds.Num())
	{
		// Maintain bound exclusivity
		TRange<FFrameTime> Result = InRange;
		Result.SetLowerBoundValue(LowerBounds[LwrIndex]);
		if (LwrIndex < LowerBounds.Num()-1)
		{
			TRangeBound<FFrameTime> NewUpper = Result.GetLowerBound();
			NewUpper.SetValue(LowerBounds[LwrIndex + 1]);
			Result.SetUpperBound(TRangeBound<FFrameTime>::FlipInclusion(NewUpper));
		}
		else
		{
			Result.SetUpperBound(TRangeBound<FFrameTime>::Open());
		}

		if (!Result.IsEmpty() && !InVisitor(Result))
		{
			return false;
		}

		++LwrIndex;
	}

	return true;
}

bool FMovieSceneInverseSequenceTransform::TransformTimeWithinRange(FFrameTime InTime, const TFunctionRef<bool(FFrameTime)>& InVisitor, const FMovieSceneTransformBreadcrumbs& StartBreadcrumbs, const FMovieSceneTransformBreadcrumbs& EndBreadcrumbs) const
{
	return RecursiveTransformTimeWithinRange(0, InTime, InVisitor, StartBreadcrumbs, EndBreadcrumbs);
}

FMovieSceneTimeTransform FMovieSceneInverseSequenceTransform::AsLegacyLinearTimeTransform() const
{
	FMovieSceneTimeTransform Result;
	for (const FMovieSceneInverseNestedSequenceTransform& Nested : NestedTransforms)
	{
		if (Nested.IsLinear())
		{
			Result = Result * Nested.AsLinear();
		}
	}
	// Linear Transform should apply last, and transform multiplication applies RtL
	return LinearTransform * Result;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FMovieSceneNestedSequenceTransform::FMovieSceneNestedSequenceTransform() = default;
FMovieSceneNestedSequenceTransform::FMovieSceneNestedSequenceTransform(const FMovieSceneNestedSequenceTransform&) = default;
FMovieSceneNestedSequenceTransform& FMovieSceneNestedSequenceTransform::operator=(const FMovieSceneNestedSequenceTransform&) = default;
FMovieSceneNestedSequenceTransform::FMovieSceneNestedSequenceTransform(FMovieSceneNestedSequenceTransform&&) = default;
FMovieSceneNestedSequenceTransform& FMovieSceneNestedSequenceTransform::operator=(FMovieSceneNestedSequenceTransform&&) = default;
FMovieSceneNestedSequenceTransform::~FMovieSceneNestedSequenceTransform() = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FMovieSceneSequenceTransform::Add(FMovieSceneTimeTransform InTransform)
{
	if (InTransform.IsIdentity())
	{
		return;
	}

	if (IsLinear())
	{
		LinearTransform = InTransform * LinearTransform;
	}
	else
	{
		NestedTransforms.Emplace(InTransform);
	}
}

void FMovieSceneSequenceTransform::Add(FMovieSceneNestedSequenceTransform InTransform)
{
	if (InTransform.IsIdentity())
	{
		return;
	}

	if (IsLinear() && InTransform.IsLinear())
	{
		LinearTransform = InTransform.AsLinear() * LinearTransform;
	}
	else
	{
		NestedTransforms.Emplace(MoveTemp(InTransform));
	}
}

void FMovieSceneSequenceTransform::Add(FFrameTime InOffset, FMovieSceneTimeWarpVariant&& InTimeWarp)
{
	if (InTimeWarp.GetType() == EMovieSceneTimeWarpType::FixedPlayRate)
	{
		Add(FMovieSceneTimeTransform(InOffset, InTimeWarp.AsFixedPlayRateFloat()));
	}
	else
	{
		if (InTimeWarp.GetType() == EMovieSceneTimeWarpType::Custom)
		{
			UMovieSceneTimeWarpGetter* Getter = InTimeWarp.AsCustom();
			if (Getter && Getter->IsMuted())
			{
				return;
			}
		}

		NestedTransforms.Emplace(InOffset, MoveTemp(InTimeWarp));
	}
}

FFrameTime FMovieSceneSequenceTransform::TransformTime(FFrameTime InTime) const
{
	if (NestedTransforms.Num() == 0)
	{
		return InTime * LinearTransform;
	}

	FFrameTime OutTime = InTime * LinearTransform;
	for (const FMovieSceneNestedSequenceTransform& NestedTransform : NestedTransforms)
	{
		OutTime = NestedTransform.TransformTime(OutTime);
	}
	return OutTime;
}

FFrameTime FMovieSceneSequenceTransform::TransformTime(FFrameTime InTime, const UE::MovieScene::FTransformTimeParams& Params) const
{
	FFrameTime OutTime = InTime * LinearTransform;
	if (Params.Breadcrumbs && Params.Breadcrumbs->GetMode() == EMovieSceneBreadcrumbMode::Dense)
	{
		Params.Breadcrumbs->AddBreadcrumb(OutTime);
	}

	if (NestedTransforms.Num() == 0)
	{
		return OutTime;
	}

	for (const FMovieSceneNestedSequenceTransform& NestedTransform : NestedTransforms)
	{
		if (Params.Breadcrumbs && (Params.Breadcrumbs->GetMode() == EMovieSceneBreadcrumbMode::Dense || NestedTransform.NeedsBreadcrumb()) )
		{
			Params.Breadcrumbs->AddBreadcrumb(OutTime);
		}

		OutTime = NestedTransform.TransformTime(OutTime, Params);
	}
	return OutTime;
}

TRange<FFrameTime> FMovieSceneSequenceTransform::ComputeTraversedHull(const TRange<FFrameTime>& Range) const
{
	using namespace UE::MovieScene;

	TRange<FFrameTime> Result = Range * LinearTransform;
	CorrectInsideOutRange(Result);

	for (const FMovieSceneNestedSequenceTransform& NestedTransform : NestedTransforms)
	{
		Result = NestedTransform.ComputeTraversedHull(Result);
		if (Result.IsEmpty())
		{
			return Result;
		}
	}

	return Result;
}
TRange<FFrameTime> FMovieSceneSequenceTransform::ComputeTraversedHull(const TRange<FFrameNumber>& Range) const
{
	return ComputeTraversedHull(UE::MovieScene::ConvertToFrameTimeRange(Range));
}

bool FMovieSceneSequenceTransform::ExtractBoundariesWithinRange(FFrameTime Start, FFrameTime End, const TFunctionRef<bool(FFrameTime)>& InVisitor) const
{
	using namespace UE::MovieScene;

	FMovieSceneInverseSequenceTransform Inverse;

	FMovieSceneTransformBreadcrumbs StartBreadcrumbs, EndBreadcrumbs;

	TRange<FFrameTime> TraversedHull = TRange<FFrameTime>::All();

	if (Start != MIN_int32)
	{
		TraversedHull.SetLowerBound(Start * LinearTransform);
	}
	if (End != MAX_int32)
	{
		TraversedHull.SetUpperBound(End * LinearTransform);
	}
	CorrectInsideOutRange(TraversedHull);

	for (int32 NestedIndex = 0; NestedIndex < NestedTransforms.Num(); ++NestedIndex)
	{
		const FMovieSceneNestedSequenceTransform& NestedTransform = NestedTransforms[NestedIndex];

		// Find the first transform that has any boundaries
		if (!NestedTransform.SupportsBoundaries())
		{
			if (NestedTransform.NeedsBreadcrumb())
			{
				StartBreadcrumbs.AddBreadcrumb(TraversedHull.HasLowerBound() ? TraversedHull.GetLowerBoundValue() : FFrameTime(MIN_int32));
				EndBreadcrumbs.AddBreadcrumb(TraversedHull.HasUpperBound() ? TraversedHull.GetUpperBoundValue() : FFrameTime(MAX_int32));
			}
			TraversedHull = NestedTransform.ComputeTraversedHull(TraversedHull);
			continue;
		}


		FMovieSceneSequenceTransform RootToParentTransform = *this;
		RootToParentTransform.NestedTransforms.SetNum(NestedIndex);

		FMovieSceneInverseSequenceTransform ParentToRootTransform = RootToParentTransform.Inverse();

		auto VisitWrapper = [&InVisitor, &ParentToRootTransform, &StartBreadcrumbs, &EndBreadcrumbs](FFrameTime InBoundary)
		{
			return ParentToRootTransform.TransformTimeWithinRange(InBoundary, InVisitor, StartBreadcrumbs, EndBreadcrumbs);
		};

		return NestedTransform.ExtractBoundariesWithinRange(TraversedHull, VisitWrapper);
	}

	return false;
}

TOptional<UE::MovieScene::ETimeWarpChannelDomain> FMovieSceneSequenceTransform::FindFirstWarpDomain() const
{
	for (const FMovieSceneNestedSequenceTransform& NestedTransform : NestedTransforms)
	{
		TOptional<UE::MovieScene::ETimeWarpChannelDomain> Domain = NestedTransform.GetWarpDomain();
		if (Domain)
		{
			return Domain;
		}
	}

	return TOptional<UE::MovieScene::ETimeWarpChannelDomain>();
}

void FMovieSceneSequenceTransform::AddLoop(FFrameNumber InStart, FFrameNumber InEnd)
{
	check(InStart < InEnd);
	// Offset by -InStart because our looping variant can only loop from 0:Max
	NestedTransforms.Emplace(-InStart, FMovieSceneTimeWarpVariant(FMovieSceneTimeWarpLoop{ InEnd - InStart }));
}

bool FMovieSceneSequenceTransform::IsIdentity() const
{
	return LinearTransform.IsIdentity() && 
		Algo::AllOf(NestedTransforms, &FMovieSceneNestedSequenceTransform::IsIdentity);
}

FMovieSceneInverseSequenceTransform FMovieSceneSequenceTransform::Inverse() const
{
	FMovieSceneInverseSequenceTransform Result;

	if (NestedTransforms.Num() == 0)
	{
		Result.LinearTransform = LinearTransform.Inverse();
		return Result;
	}

	// Start accumulating the inverse transforms in reverse order.
	Result.NestedTransforms.Reserve(NestedTransforms.Num());

	for(int Index = NestedTransforms.Num() - 1; Index >= 0; --Index)
	{
		const FMovieSceneNestedSequenceTransform& Nested = NestedTransforms[Index];
		if (Nested.IsLinear())
		{
			Result.LinearTransform = Nested.AsLinear().Inverse() * Result.LinearTransform;
			continue;
		}

		if (!Result.LinearTransform.IsIdentity())
		{
			// If we have any linear transform that needs to get added to the stack before this one's inverse
			Result.NestedTransforms.Add(Result.LinearTransform);
			Result.LinearTransform = FMovieSceneTimeTransform();
		}

		Result.NestedTransforms.Add(NestedTransforms[Index].Inverse());
	}

	// Add the inverse of the main linear transform if not identity
	if (!LinearTransform.IsIdentity())
	{
		Result.LinearTransform = LinearTransform.Inverse() * Result.LinearTransform;
	}

	return Result;
}


void FMovieSceneSequenceTransform::Append(const FMovieSceneSequenceTransform& Tail)
{
	if (IsLinear())
	{
		if (!Tail.LinearTransform.IsIdentity())
		{
			LinearTransform = Tail.LinearTransform * LinearTransform;
		}
	}
	else if (!Tail.LinearTransform.IsIdentity())
	{
		NestedTransforms.Add(Tail.LinearTransform);
	}

	NestedTransforms.Append(Tail.NestedTransforms);
}

FMovieSceneSequenceTransform FMovieSceneSequenceTransform::operator*(const FMovieSceneSequenceTransform& RHS) const
{
	if (IsLinear() && RHS.IsLinear())
	{
		// None of the transforms are warping... we can combine them into another linear transform.
		return FMovieSceneSequenceTransform(LinearTransform * RHS.LinearTransform);
	}
	else if (IsLinear())
	{
		// LHS is linear, but RHS is warping. Since transforms are supposed to apply from right to left,
		// we need to append LHS at the "bottom" of RHS, i.e. add a new nested transform that's LHS. However
		// if LHS is identity, we have nothing to do, and if both LHS and RHS' deeper transform are linear,
		// we can combine both.
		FMovieSceneSequenceTransform Result(RHS);
		if (!LinearTransform.IsIdentity())
		{
			FMovieSceneNestedSequenceTransform& LastNested = Result.NestedTransforms.Last();
			if (LastNested.IsLinear())
			{
				FMovieSceneTimeTransform NewLinear = LinearTransform * LastNested.AsLinear();
				LastNested = FMovieSceneNestedSequenceTransform(NewLinear);
			}
			else
			{
				Result.NestedTransforms.Emplace(LinearTransform);
			}
		}
		return Result;
	}
	else if (RHS.IsLinear())
	{
		// RHS isn't warping, but LHS is, so we combine the linear transform parts and start looping
		// from there.
		FMovieSceneSequenceTransform Result;
		Result.LinearTransform = LinearTransform * RHS.LinearTransform;
		Result.NestedTransforms = NestedTransforms;
		return Result;
	}
	else
	{
		// Both are looping, we need to combine them. Usually, a warping transform doesn't use its linear part,
		// because whatever linear placement/scaling it has would be in the linear part of the nested transform
		// struct.
		FMovieSceneSequenceTransform Result(RHS);
		const bool bHasOnlyNested = LinearTransform.IsIdentity();
		if (!bHasOnlyNested)
		{
			Result.NestedTransforms.Add(FMovieSceneNestedSequenceTransform(LinearTransform));
		}
		Result.NestedTransforms.Append(NestedTransforms);
		return Result;
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool FMovieSceneSequenceTransform::IsLooping() const
{
	return Algo::AnyOf(NestedTransforms, &FMovieSceneNestedSequenceTransform::IsLooping);
}
void FMovieSceneSequenceTransform::TransformTime(FFrameTime InTime, FFrameTime& OutTime, FMovieSceneWarpCounter& OutWarpCounter) const
{
	OutTime = TransformTime(InTime, UE::MovieScene::FTransformTimeParams().HarvestBreadcrumbs(OutWarpCounter));
}
float FMovieSceneSequenceTransform::GetTimeScale() const
{
	float TimeScale = LinearTransform.TimeScale;
	for (const FMovieSceneNestedSequenceTransform& NestedTransform : NestedTransforms)
	{
		if (NestedTransform.IsLinear())
		{
			TimeScale *= NestedTransform.AsLinear().TimeScale;
		}
	}
	return TimeScale;
}
TRange<FFrameTime> FMovieSceneSequenceTransform::TransformRangeConstrained(const TRange<FFrameTime>& Range) const
{
	return ComputeTraversedHull(Range);
}
TRange<FFrameTime> FMovieSceneSequenceTransform::TransformRangePure(const TRange<FFrameTime>& Range) const
{
	return ComputeTraversedHull(Range);
}
TRange<FFrameTime> FMovieSceneSequenceTransform::TransformRangeUnwarped(const TRange<FFrameTime>& Range) const
{
	return ComputeTraversedHull(Range);
}
TRange<FFrameNumber> FMovieSceneSequenceTransform::TransformRangePure(const TRange<FFrameNumber>& Range) const
{
	using namespace UE::MovieScene;

	TRange<FFrameTime> TimeRange = TransformRangePure(ConvertRange<FFrameNumber, FFrameTime>(Range));
	return ConvertRange<FFrameTime, FFrameNumber>(TimeRange);
}
TRange<FFrameNumber> FMovieSceneSequenceTransform::TransformRangeUnwarped(const TRange<FFrameNumber>& Range) const
{
	using namespace UE::MovieScene;

	TRange<FFrameTime> TimeRange = ConvertRange<FFrameNumber, FFrameTime>(Range);
	TimeRange = TransformRangeUnwarped(TimeRange);
	return ConvertRange<FFrameTime, FFrameNumber>(TimeRange);
}
TRange<FFrameNumber> FMovieSceneSequenceTransform::TransformRangeConstrained(const TRange<FFrameNumber>& Range) const
{
	using namespace UE::MovieScene;

	TRange<FFrameTime> TimeRange = ConvertRange<FFrameNumber, FFrameTime>(Range);
	TimeRange = TransformRangeConstrained(TimeRange);
	return ConvertRange<FFrameTime, FFrameNumber>(TimeRange);
}
FMovieSceneTimeTransform FMovieSceneSequenceTransform::InverseLinearOnly() const
{
	ensureMsgf(!FMath::IsNearlyZero(LinearTransform.TimeScale), TEXT("Inverse of a zero timescale transform is undefined in a FMovieSceneTimeTransform. Please use InverseNoLooping for proper behavior."));
	return LinearTransform.Inverse();
}
FMovieSceneSequenceTransform FMovieSceneSequenceTransform::InverseNoLooping() const
{
	return FMovieSceneSequenceTransform();
}
FMovieSceneTimeTransform FMovieSceneSequenceTransform::InverseFromAllFirstWarps() const
{
	return FMovieSceneTimeTransform();
}
FMovieSceneSequenceTransform FMovieSceneSequenceTransform::InverseFromAllFirstLoops() const 
{
	return FMovieSceneSequenceTransform();
}
FMovieSceneTimeTransform FMovieSceneSequenceTransform::InverseFromWarp(const FMovieSceneWarpCounter& WarpCounter) const
{
	return FMovieSceneTimeTransform();
}
FMovieSceneTimeTransform FMovieSceneSequenceTransform::InverseFromWarp(const TArrayView<const uint32>& WarpCounts) const
{
	return FMovieSceneTimeTransform();
}
FMovieSceneSequenceTransform FMovieSceneSequenceTransform::InverseFromLoop(const FMovieSceneWarpCounter& LoopCounter) const
{
	return FMovieSceneSequenceTransform();
}

FMovieSceneSequenceTransform FMovieSceneSequenceTransform::InverseFromLoop(const TArrayView<const FFrameTime>& Breadcrumbs) const
{
	return FMovieSceneSequenceTransform();
}

FMovieSceneSequenceTransform FMovieSceneSequenceTransform::InverseFromLoop(const TArrayView<const uint32>& LoopCounts) const
{
	return FMovieSceneSequenceTransform();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS