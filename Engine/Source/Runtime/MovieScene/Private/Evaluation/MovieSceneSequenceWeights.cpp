// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/MovieSceneSequenceWeights.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/BuiltInComponentTypes.h"

namespace UE::MovieScene
{

FSequenceWeights::FSequenceWeights(UMovieSceneEntitySystemLinker* InLinker, FRootInstanceHandle InRootInstanceHandle)
	: WeakLinker(InLinker)
	, RootInstanceHandle(InRootInstanceHandle)
{
}

FSequenceWeights::~FSequenceWeights()
{
	UMovieSceneEntitySystemLinker* Linker = WeakLinker.Get();
	if (Linker)
	{
		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
		for (TPair<FMovieSceneSequenceID, FMovieSceneEntityID> Pair : WeightEntitiesBySequenceID)
		{
			Linker->EntityManager.AddComponent(Pair.Value, BuiltInComponents->Tags.NeedsUnlink);
		}
	}
}

void FSequenceWeights::SetWeight(FMovieSceneSequenceID InSequenceID, double Weight)
{
	UMovieSceneEntitySystemLinker* Linker = WeakLinker.Get();
	if (!ensureAlways(Linker))
	{
		return;
	}

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	FMovieSceneEntityID EntityID = WeightEntitiesBySequenceID.FindRef(InSequenceID);
	if (!EntityID)
	{
		EntityID = FEntityBuilder()
		.Add(BuiltInComponents->RootInstanceHandle, RootInstanceHandle)
		.Add(BuiltInComponents->HierarchicalEasingProvider, InSequenceID)
		.Add(BuiltInComponents->WeightResult, Weight)
		.Add(BuiltInComponents->WeightAndEasingResult, Weight)
		.AddTag(BuiltInComponents->Tags.NeedsLink)
		.CreateEntity(&Linker->EntityManager);

		WeightEntitiesBySequenceID.Add(InSequenceID, EntityID);
	}
	else
	{
		TOptionalComponentWriter<double> WeightComponent = Linker->EntityManager.WriteComponent(EntityID, BuiltInComponents->WeightResult);
		if (WeightComponent)
		{
			*WeightComponent = Weight;
		}
	}
}

void FSequenceWeights::SetWeights(const TSortedMap<FMovieSceneSequenceID, double>& Weights)
{
	UMovieSceneEntitySystemLinker* Linker = WeakLinker.Get();
	if (!ensureAlways(Linker))
	{
		return;
	}

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	for (TPair<FMovieSceneSequenceID, double> Pair : Weights)
	{
		FMovieSceneEntityID EntityID = WeightEntitiesBySequenceID.FindRef(Pair.Key);
		if (!EntityID)
		{
			EntityID = FEntityBuilder()
			.Add(BuiltInComponents->RootInstanceHandle, RootInstanceHandle)
			.Add(BuiltInComponents->HierarchicalEasingProvider, Pair.Key)
			.Add(BuiltInComponents->WeightResult, Pair.Value)
			.Add(BuiltInComponents->WeightAndEasingResult, Pair.Value)
			.AddTag(BuiltInComponents->Tags.NeedsLink)
			.CreateEntity(&Linker->EntityManager);

			WeightEntitiesBySequenceID.Add(Pair.Key, EntityID);
		}
		else
		{
			TOptionalComponentWriter<double> WeightComponent = Linker->EntityManager.WriteComponent(EntityID, BuiltInComponents->WeightResult);
			if (WeightComponent)
			{
				*WeightComponent = Pair.Value;
			}
		}
	}
}

void FSequenceWeights::RemoveWeight(FMovieSceneSequenceID InSequenceID)
{
	FMovieSceneEntityID            EntityID = WeightEntitiesBySequenceID.FindRef(InSequenceID);
	UMovieSceneEntitySystemLinker* Linker   = WeakLinker.Get();

	if (EntityID && ensureAlways(Linker))
	{
		Linker->EntityManager.AddComponent(EntityID, FBuiltInComponentTypes::Get()->Tags.NeedsUnlink);
		WeightEntitiesBySequenceID.Remove(InSequenceID);
	}
}


} // namespace UE::MovieScene