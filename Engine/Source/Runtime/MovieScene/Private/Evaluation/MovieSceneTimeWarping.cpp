// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/MovieSceneTimeWarping.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneTimeWarping)

const uint32 FMovieSceneTimeWarping::InvalidWarpCount = (uint32)(-1);

