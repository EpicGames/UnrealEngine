// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/MovieSceneTrackImplementation.h"
#include "Evaluation/MovieSceneEvalTemplate.h"
#include "Evaluation/MovieSceneEvalTemplateSerializer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneTrackImplementation)

bool FMovieSceneTrackImplementationPtr::Serialize(FArchive& Ar)
{
	bool bShouldWarn = !WITH_EDITORONLY_DATA;
	return SerializeInlineValue(*this, Ar, bShouldWarn);
}

