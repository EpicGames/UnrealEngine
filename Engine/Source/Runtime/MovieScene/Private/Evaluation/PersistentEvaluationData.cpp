// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/PersistentEvaluationData.h"
#include "IMovieScenePlayer.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Compilation/MovieSceneCompiledDataManager.h"

FPersistentEvaluationData::FPersistentEvaluationData(IMovieScenePlayer& InPlayer)
	: Player(InPlayer)
	, EntityData(nullptr)
	, SharedData(nullptr)
{

	FMovieSceneEvaluationState* EvaluationState = Player.GetEvaluationState();
	if (EvaluationState)
	{
		EntityData = &EvaluationState->PersistentEntityData;
		SharedData = &EvaluationState->PersistentSharedData;
	}
}

const FMovieSceneSequenceInstanceData* FPersistentEvaluationData::GetInstanceData() const
{
	FMovieSceneRootEvaluationTemplateInstance& Instance  = Player.GetEvaluationTemplate();
	const FMovieSceneSequenceHierarchy*        Hierarchy = Instance.GetCompiledDataManager()->FindHierarchy(Instance.GetCompiledDataID());
	if (!Hierarchy)
	{
		return nullptr;
	}

	const FMovieSceneSubSequenceData* SubData = Hierarchy->FindSubData(TrackKey.SequenceID);
	return SubData && SubData->InstanceData.IsValid() ? &SubData->InstanceData.GetValue() : nullptr;
}
