// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedCaptureSource.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedCaptureSources.inl"
#include "Evaluation/MovieScenePreAnimatedState.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "MovieSceneSection.h"
#include "MovieSceneSequence.h"

FScopedPreAnimatedCaptureSource::FScopedPreAnimatedCaptureSource(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, const FMovieSceneEvaluationKey& InEvalKey, bool bInWantsRestoreState)
	: Variant(TInPlaceType<FPreAnimatedEvaluationKeyType>(), FPreAnimatedEvaluationKeyType{ InEvalKey, SharedPlaybackState->GetRootInstanceHandle() })
	, RootInstanceHandle(SharedPlaybackState->GetRootInstanceHandle())
	, WeakLinker(SharedPlaybackState->GetLinker())
	, bWantsRestoreState(bInWantsRestoreState)
{
	FScopedPreAnimatedCaptureSource*& CaptureSourcePtr = GetCaptureSourcePtr();
	PrevCaptureSource = CaptureSourcePtr;
	CaptureSourcePtr = this;
}
FScopedPreAnimatedCaptureSource::FScopedPreAnimatedCaptureSource(FMovieScenePreAnimatedState* InPreAnimatedState, const FMovieSceneEvaluationKey& InEvalKey, bool bInWantsRestoreState)
	: Variant(TInPlaceType<FPreAnimatedEvaluationKeyType>(), FPreAnimatedEvaluationKeyType{ InEvalKey, InPreAnimatedState->InstanceHandle })
	, RootInstanceHandle(InPreAnimatedState->InstanceHandle)
	, WeakLinker(InPreAnimatedState->GetLinker())
	, bWantsRestoreState(bInWantsRestoreState)
{
	FScopedPreAnimatedCaptureSource*& CaptureSourcePtr = GetCaptureSourcePtr();
	PrevCaptureSource = CaptureSourcePtr;
	CaptureSourcePtr = this;
}
FScopedPreAnimatedCaptureSource::FScopedPreAnimatedCaptureSource(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, const UObject* InEvalHook, FMovieSceneSequenceID InSequenceID, bool bInWantsRestoreState)
	: Variant(TInPlaceType<FPreAnimatedEvalHookKeyType>(), FPreAnimatedEvalHookKeyType{ InEvalHook, SharedPlaybackState->GetRootInstanceHandle(), InSequenceID } )
	, RootInstanceHandle(SharedPlaybackState->GetRootInstanceHandle())
	, WeakLinker(SharedPlaybackState->GetLinker())
	, bWantsRestoreState(bInWantsRestoreState)
{
	FScopedPreAnimatedCaptureSource*& CaptureSourcePtr = GetCaptureSourcePtr();
	PrevCaptureSource = CaptureSourcePtr;
	CaptureSourcePtr = this;
}
FScopedPreAnimatedCaptureSource::FScopedPreAnimatedCaptureSource(FMovieScenePreAnimatedState* InPreAnimatedState, const UObject* InEvalHook, FMovieSceneSequenceID InSequenceID, bool bInWantsRestoreState)
	: Variant(TInPlaceType<FPreAnimatedEvalHookKeyType>(), FPreAnimatedEvalHookKeyType{InEvalHook, InPreAnimatedState->InstanceHandle, InSequenceID} )
	, RootInstanceHandle(InPreAnimatedState->InstanceHandle)
	, WeakLinker(InPreAnimatedState ? InPreAnimatedState->GetLinker() : nullptr)
	, bWantsRestoreState(bInWantsRestoreState)
{
	FScopedPreAnimatedCaptureSource*& CaptureSourcePtr = GetCaptureSourcePtr();
	PrevCaptureSource = CaptureSourcePtr;
	CaptureSourcePtr = this;
}
FScopedPreAnimatedCaptureSource::FScopedPreAnimatedCaptureSource(UMovieSceneEntitySystemLinker* InLinker, UMovieSceneTrackInstance* InTrackInstance, bool bInWantsRestoreState)
	: Variant(TInPlaceType<UMovieSceneTrackInstance*>(), InTrackInstance)
	, WeakLinker(InLinker)
	, bWantsRestoreState(bInWantsRestoreState)
{
	FScopedPreAnimatedCaptureSource*& CaptureSourcePtr = GetCaptureSourcePtr();
	PrevCaptureSource = CaptureSourcePtr;
	CaptureSourcePtr = this;
}
FScopedPreAnimatedCaptureSource::FScopedPreAnimatedCaptureSource(UMovieSceneEntitySystemLinker* InLinker, const FMovieSceneTrackInstanceInput& TrackInstanceInput)
	: Variant(TInPlaceType<FMovieSceneTrackInstanceInput>(), TrackInstanceInput)
	, WeakLinker(InLinker)
{
	EMovieSceneCompletionMode CompletionMode = TrackInstanceInput.Section->GetCompletionMode();
	if (CompletionMode == EMovieSceneCompletionMode::ProjectDefault)
	{
		CompletionMode = TrackInstanceInput.Section->GetTypedOuter<UMovieSceneSequence>()->DefaultCompletionMode;
	}
	bWantsRestoreState = (CompletionMode == EMovieSceneCompletionMode::RestoreState);

	FScopedPreAnimatedCaptureSource*& CaptureSourcePtr = GetCaptureSourcePtr();
	PrevCaptureSource = CaptureSourcePtr;
	CaptureSourcePtr = this;
}

FScopedPreAnimatedCaptureSource::~FScopedPreAnimatedCaptureSource()
{
	GetCaptureSourcePtr() = PrevCaptureSource;
}

FScopedPreAnimatedCaptureSource*& FScopedPreAnimatedCaptureSource::GetCaptureSourcePtr()
{
	// Implemented as a static thread-local for now since there are some tests that run without a linker,
	// so we can't put this on UMovieSceneEntitySystemLinker::PreAnimatedState where it should probably belong
	static thread_local FScopedPreAnimatedCaptureSource* GCaptureSource = nullptr;
	return GCaptureSource;
}

UE::MovieScene::FRootInstanceHandle FScopedPreAnimatedCaptureSource::GetRootInstanceHandle(UMovieSceneEntitySystemLinker* Linker) const
{
	if (RootInstanceHandle.IsValid())
	{
		return RootInstanceHandle;
	}
	if (const FMovieSceneTrackInstanceInput* TrackInstanceInput = Variant.TryGet<FMovieSceneTrackInstanceInput>())
	{
		return Linker->GetInstanceRegistry()->GetInstance(TrackInstanceInput->InstanceHandle).GetRootInstanceHandle();
	}
	return UE::MovieScene::FRootInstanceHandle();
}

void FScopedPreAnimatedCaptureSource::BeginTracking(const UE::MovieScene::FPreAnimatedStateMetaData& MetaData, UMovieSceneEntitySystemLinker* Linker)
{
	using namespace UE::MovieScene;

	ensureMsgf(!WeakLinker.IsValid() || WeakLinker.Get() == Linker, 
			TEXT("Attempting to track state on a capture source related to a different linker. Are you missing setting a scoped capture source?"));

	// All capture sources' meta-data is shared between all players

	if (FPreAnimatedEvaluationKeyType* EvalKey = Variant.TryGet<FPreAnimatedEvaluationKeyType>())
	{
		FPreAnimatedTemplateCaptureSources* TemplateMetaData = Linker->PreAnimatedState.GetOrCreateTemplateMetaData();
		ensureMsgf(EvalKey->RootInstanceHandle == MetaData.RootInstanceHandle, TEXT("Mismatched root handles between the scoped evaluation key and the given metadata"));
		TemplateMetaData->BeginTrackingCaptureSource(*EvalKey, MetaData);
	}
	else if (FPreAnimatedEvalHookKeyType* EvalHook = Variant.TryGet<FPreAnimatedEvalHookKeyType>())
	{
		FPreAnimatedEvaluationHookCaptureSources* EvaluationHookMetaData = Linker->PreAnimatedState.GetOrCreateEvaluationHookMetaData();
		ensureMsgf(EvalHook->RootInstanceHandle == MetaData.RootInstanceHandle, TEXT("Mismatched root handles between the scoped evaluation hook and the given metadata"));
		EvaluationHookMetaData->BeginTrackingCaptureSource(*EvalHook, MetaData);
	}
	else if (UMovieSceneTrackInstance* const * TrackInstance = Variant.TryGet<UMovieSceneTrackInstance*>())
	{
		FPreAnimatedTrackInstanceCaptureSources* TrackInstanceMetaData = Linker->PreAnimatedState.GetOrCreateTrackInstanceMetaData();
		TrackInstanceMetaData->BeginTrackingCaptureSource(*TrackInstance, MetaData);
	}
	else if (const FMovieSceneTrackInstanceInput* TrackInstanceInput = Variant.TryGet<FMovieSceneTrackInstanceInput>())
	{
		FPreAnimatedTrackInstanceInputCaptureSources* TrackInstanceInputMetaData = Linker->PreAnimatedState.GetOrCreateTrackInstanceInputMetaData();
		TrackInstanceInputMetaData->BeginTrackingCaptureSource(*TrackInstanceInput, MetaData);
	}
}
