// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedCaptureSources.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateExtension.h"
#include "Evaluation/PreAnimatedState/MovieSceneRestoreStateParams.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedCaptureSources.inl"
#include "EntitySystem/TrackInstance/MovieSceneTrackInstance.h"
#include "Algo/Find.h"

namespace UE
{
namespace MovieScene
{


// Explicit, exported template instantiation for these classes
template struct TPreAnimatedCaptureSources<FObjectKey>;
template struct TPreAnimatedCaptureSources<FPreAnimatedEvaluationKeyType>;
template struct TPreAnimatedCaptureSources<FPreAnimatedEvalHookKeyType>;
template struct TPreAnimatedCaptureSources<FMovieSceneEntityID>;

FPreAnimatedEntityCaptureSource::FPreAnimatedEntityCaptureSource(FPreAnimatedStateExtension* InOwner)
	: TPreAnimatedCaptureSources<FMovieSceneEntityID>(InOwner)
{}

void FPreAnimatedEntityCaptureSource::BeginTrackingEntity(const FPreAnimatedStateEntry& Entry, FMovieSceneEntityID EntityID, FRootInstanceHandle RootInstanceHandle, bool bWantsRestoreState)
{
	FPreAnimatedStateMetaData MetaData{ Entry, RootInstanceHandle, bWantsRestoreState };
	BeginTrackingCaptureSource(EntityID, MetaData);
}

FPreAnimatedTemplateCaptureSources::FPreAnimatedTemplateCaptureSources(FPreAnimatedStateExtension* InOwner)
	: TPreAnimatedCaptureSources<FPreAnimatedEvaluationKeyType>(InOwner)
{}

EPreAnimatedCaptureSourceState FPreAnimatedTemplateCaptureSources::BeginTrackingCaptureSource(const FMovieSceneEvaluationKey& EvaluationKey, const FPreAnimatedStateMetaData& MetaData)
{
	return TPreAnimatedCaptureSources<FPreAnimatedEvaluationKeyType>::BeginTrackingCaptureSource(FPreAnimatedEvaluationKeyType{ EvaluationKey, MetaData.RootInstanceHandle }, MetaData);
}

void FPreAnimatedTemplateCaptureSources::StopTrackingCaptureSource(const FMovieSceneEvaluationKey& EvaluationKey, FRootInstanceHandle RootInstanceHandle)
{
	TPreAnimatedCaptureSources<FPreAnimatedEvaluationKeyType>::StopTrackingCaptureSource(FPreAnimatedEvaluationKeyType{ EvaluationKey, RootInstanceHandle });
}

FPreAnimatedEvaluationHookCaptureSources::FPreAnimatedEvaluationHookCaptureSources(FPreAnimatedStateExtension* InOwner)
	: TPreAnimatedCaptureSources<FPreAnimatedEvalHookKeyType>(InOwner)
{}

EPreAnimatedCaptureSourceState FPreAnimatedEvaluationHookCaptureSources::BeginTrackingCaptureSource(const UObject* Hook, FMovieSceneSequenceID SequenceID, const FPreAnimatedStateMetaData& MetaData)
{
	return TPreAnimatedCaptureSources<FPreAnimatedEvalHookKeyType>::BeginTrackingCaptureSource(FPreAnimatedEvalHookKeyType{ Hook, MetaData.RootInstanceHandle, SequenceID }, MetaData);
}

void FPreAnimatedEvaluationHookCaptureSources::StopTrackingCaptureSource(const UObject* Hook, FRootInstanceHandle RootInstanceHandle, FMovieSceneSequenceID SequenceID)
{
	TPreAnimatedCaptureSources<FPreAnimatedEvalHookKeyType>::StopTrackingCaptureSource(FPreAnimatedEvalHookKeyType{ Hook, RootInstanceHandle, SequenceID });
}

FPreAnimatedTrackInstanceCaptureSources::FPreAnimatedTrackInstanceCaptureSources(FPreAnimatedStateExtension* InOwner)
	: TPreAnimatedCaptureSources<FObjectKey>(InOwner)
{}

EPreAnimatedCaptureSourceState FPreAnimatedTrackInstanceCaptureSources::BeginTrackingCaptureSource(UMovieSceneTrackInstance* TrackInstance, const FPreAnimatedStateMetaData& MetaData)
{
	FObjectKey Key(TrackInstance);
	return TPreAnimatedCaptureSources<FObjectKey>::BeginTrackingCaptureSource(Key, MetaData);
}

void FPreAnimatedTrackInstanceCaptureSources::StopTrackingCaptureSource(UMovieSceneTrackInstance* TrackInstance)
{
	FObjectKey Key(TrackInstance);
	TPreAnimatedCaptureSources<FObjectKey>::StopTrackingCaptureSource(Key);
}

FPreAnimatedTrackInstanceInputCaptureSources::FPreAnimatedTrackInstanceInputCaptureSources(FPreAnimatedStateExtension* InOwner)
	: TPreAnimatedCaptureSources<FMovieSceneTrackInstanceInput>(InOwner)
{}


} // namespace MovieScene
} // namespace UE
