// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedCaptureSources.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateExtension.h"
#include "Evaluation/PreAnimatedState/MovieSceneRestoreStateParams.h"
#include "Algo/Find.h"

namespace UE
{
namespace MovieScene
{

template<typename KeyType>
TPreAnimatedCaptureSources<KeyType>::TPreAnimatedCaptureSources(FPreAnimatedStateExtension* InOwner)
	: Owner(InOwner)
{}

template<typename KeyType>
EPreAnimatedCaptureSourceState TPreAnimatedCaptureSources<KeyType>::BeginTrackingCaptureSource(const KeyType& InKey, const FPreAnimatedStateMetaData& MetaData)
{
	FPreAnimatedStateMetaDataArray& Array = KeyToMetaData.FindOrAdd(InKey);
	if (FPreAnimatedStateMetaData* Existing = Algo::FindBy(Array, MetaData.Entry, &FPreAnimatedStateMetaData::Entry))
	{
		if (Existing->bWantsRestoreState != MetaData.bWantsRestoreState)
		{
			Existing->bWantsRestoreState = MetaData.bWantsRestoreState;
			Owner->UpdateMetaData(*Existing);

			return EPreAnimatedCaptureSourceState::Updated;
		}

		return EPreAnimatedCaptureSourceState::UpToDate;
	}

	Array.Add(MetaData);
	Owner->AddMetaData(MetaData);
	return EPreAnimatedCaptureSourceState::New;
}

template<typename KeyType>
void TPreAnimatedCaptureSources<KeyType>::StopTrackingCaptureSource(const KeyType& InKey, FPreAnimatedStorageID InStorageID)
{
	FPreAnimatedStateMetaDataArray* Array = KeyToMetaData.Find(InKey);
	if (Array)
	{
		for (int32 Index = Array->Num()-1; Index >= 0; --Index)
		{
			const FPreAnimatedStateMetaData& MetaData = (*Array)[Index];
			if (MetaData.Entry.ValueHandle.TypeID == InStorageID)
			{
				Owner->RemoveMetaData(MetaData);
				Array->RemoveAt(Index, EAllowShrinking::No);
			}
		}

		if (Array->Num() == 0)
		{
			KeyToMetaData.Remove(InKey);
		}
	}
}

template<typename KeyType>
void TPreAnimatedCaptureSources<KeyType>::StopTrackingCaptureSource(const KeyType& InKey)
{
	FPreAnimatedStateMetaDataArray* Array = KeyToMetaData.Find(InKey);
	if (Array)
	{
		for (int32 Index = Array->Num()-1; Index >= 0; --Index)
		{
			Owner->RemoveMetaData((*Array)[Index]);
		}

		KeyToMetaData.Remove(InKey);
	}
}

template<typename KeyType>
void TPreAnimatedCaptureSources<KeyType>::Reset()
{
	KeyToMetaData.Empty();
}

template<typename KeyType>
void TPreAnimatedCaptureSources<KeyType>::GatherAndRemoveExpiredMetaData(const FRestoreStateParams& Params, TArray<FPreAnimatedStateMetaData>& OutExpiredMetaData)
{
	FInstanceHandle InstanceHandle = Params.TerminalInstanceHandle;
	if (!InstanceHandle.IsValid())
	{
		for (auto It = KeyToMetaData.CreateIterator(); It; ++It)
		{
			OutExpiredMetaData.Append(It.Value());
		}
		KeyToMetaData.Empty();
	}
	else for (auto It = KeyToMetaData.CreateIterator(); It; ++It)
	{
		FPreAnimatedStateMetaDataArray& Array = It.Value();
		for (int32 Index = Array.Num()-1; Index >= 0; --Index)
		{
			FPreAnimatedStateMetaData& MetaData = Array[Index];
			if (MetaData.RootInstanceHandle == InstanceHandle)
			{
				OutExpiredMetaData.Add(MetaData);
				Array.RemoveAt(Index, 1, EAllowShrinking::No);
			}
		}

		if (Array.Num() == 0)
		{
			It.RemoveCurrent();
		}
	}
}

template<typename KeyType>
void TPreAnimatedCaptureSources<KeyType>::GatherAndRemoveMetaDataForGroup(FPreAnimatedStorageGroupHandle Group, TArray<FPreAnimatedStateMetaData>& OutExpiredMetaData)
{
	for (auto It = KeyToMetaData.CreateIterator(); It; ++It)
	{
		FPreAnimatedStateMetaDataArray& Array = It.Value();
		for (int32 Index = Array.Num()-1; Index >= 0; --Index)
		{
			const FPreAnimatedStateMetaData& MetaData = Array[Index];
			if (MetaData.Entry.GroupHandle == Group)
			{
				OutExpiredMetaData.Add(MetaData);
				Array.RemoveAt(Index, 1, EAllowShrinking::No);
			}
		}

		if (Array.Num() == 0)
		{
			It.RemoveCurrent();
		}
	}
}

template<typename KeyType>
void TPreAnimatedCaptureSources<KeyType>::GatherAndRemoveMetaDataForStorage(FPreAnimatedStorageID StorageID, FPreAnimatedStorageIndex StorageIndex, TArray<FPreAnimatedStateMetaData>& OutExpiredMetaData)
{
	for (auto It = KeyToMetaData.CreateIterator(); It; ++It)
	{
		FPreAnimatedStateMetaDataArray& Array = It.Value();
		for (int32 Index = Array.Num()-1; Index >= 0; --Index)
		{
			const FPreAnimatedStateMetaData& MetaData = Array[Index];
			if (MetaData.Entry.ValueHandle.TypeID == StorageID &&
					(!StorageIndex.IsValid() || MetaData.Entry.ValueHandle.StorageIndex == StorageIndex))
			{
				OutExpiredMetaData.Add(MetaData);
				Array.RemoveAt(Index, 1, EAllowShrinking::No);
			}
		}

		if (Array.Num() == 0)
		{
			It.RemoveCurrent();
		}
	}
}

template<typename KeyType>
void TPreAnimatedCaptureSources<KeyType>::GatherAndRemoveMetaDataForRootInstance(FRootInstanceHandle InstanceHandle, TArray<FPreAnimatedStateMetaData>& OutExpiredMetaData)
{
	for (auto It = KeyToMetaData.CreateIterator(); It; ++It)
	{
		FPreAnimatedStateMetaDataArray& Array = It.Value();
		for (int32 Index = Array.Num()-1; Index >= 0; --Index)
		{
			const FPreAnimatedStateMetaData& MetaData = Array[Index];
			if (MetaData.RootInstanceHandle == InstanceHandle)
			{
				OutExpiredMetaData.Add(MetaData);
				Array.RemoveAt(Index, 1, EAllowShrinking::No);
			}
		}

		if (Array.Num() == 0)
		{
			It.RemoveCurrent();
		}
	}
}

template<typename KeyType>
bool TPreAnimatedCaptureSources<KeyType>::ContainsInstanceHandle(FRootInstanceHandle RootInstanceHandle) const
{
	for (const TPair<KeyType, FPreAnimatedStateMetaDataArray>& Pair : KeyToMetaData)
	{
		for (const FPreAnimatedStateMetaData& MetaData : Pair.Value)
		{
			if (MetaData.RootInstanceHandle == RootInstanceHandle)
			{
				return true;
			}
		}
	}
	return false;
}

} // namespace MovieScene
} // namespace UE
