// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedObjectGroupManager.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.inl"
#include "EntitySystem/BuiltInComponentTypes.h"

namespace UE
{
namespace MovieScene
{

TAutoRegisterPreAnimatedStorageID<FPreAnimatedObjectGroupManager> FPreAnimatedObjectGroupManager::GroupManagerID;

void FPreAnimatedObjectGroupManager::OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap)
{
	TMap<FObjectKey, FPreAnimatedStorageGroupHandle> OldStorageGroupsByKey = MoveTemp(StorageGroupsByKey);
	StorageGroupsByKey.Reset();
	StorageGroupsByKey.Reserve(OldStorageGroupsByKey.Num());

	for (auto It = OldStorageGroupsByKey.CreateIterator(); It; ++It)
	{
		FPreAnimatedStorageGroupHandle GroupHandle = It.Value();

		UObject* Object = It.Key().ResolveObjectPtrEvenIfGarbage();
		if (UObject* ReplacedObject = ReplacementMap.FindRef(Object))
		{
			FObjectKey NewKey(ReplacedObject);

			StorageGroupsByKey.Add(NewKey, GroupHandle);
			// This will overwrite the existing entry
			StorageGroupsToKey.Add(GroupHandle, NewKey);

			Extension->ReplaceObjectForGroup(GroupHandle, It.Key(), NewKey);
		}
		else
		{
			StorageGroupsByKey.Add(It.Key(), It.Value());
		}
	}
}

void FPreAnimatedObjectGroupManager::GetGroupsByClass(UClass* GeneratedClass, TArray<FPreAnimatedStorageGroupHandle>& OutGroupHandles)
{
	for (auto It = StorageGroupsByKey.CreateConstIterator(); It; ++It)
	{
		UObject* Object = It.Key().ResolveObjectPtrEvenIfGarbage();
		if (Object && Object->IsA(GeneratedClass))
		{
			OutGroupHandles.Add(It.Value());
		}
	}
}

void FPreAnimatedObjectGroupManager::GatherStaleStorageGroups(TArray<FPreAnimatedStorageGroupHandle>& StaleGroupStorage) const
{
	for (auto It = StorageGroupsByKey.CreateConstIterator(); It; ++It)
	{
		if (FBuiltInComponentTypes::IsBoundObjectGarbage(It.Key().ResolveObjectPtr()))
		{
			StaleGroupStorage.Add(It.Value());
		}
	}
}

} // namespace MovieScene
} // namespace UE
