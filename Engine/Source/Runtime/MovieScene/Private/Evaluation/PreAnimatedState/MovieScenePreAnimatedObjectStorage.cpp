// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedObjectStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateExtension.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedObjectGroupManager.h"

namespace UE
{
namespace MovieScene
{

void FBoundObjectPreAnimatedStateTraits::Initialize(FPreAnimatedStorageID InStorageID, FPreAnimatedStateExtension* InParentExtension)
{
	ObjectGroupManager = InParentExtension->GetOrCreateGroupManager<FPreAnimatedObjectGroupManager>();
}

FPreAnimatedStorageGroupHandle FBoundObjectPreAnimatedStateTraits::FindGroupImpl(UObject* BoundObject)
{
	return ObjectGroupManager->FindGroupForKey(BoundObject);
}

FPreAnimatedStorageGroupHandle FBoundObjectPreAnimatedStateTraits::FindGroupImpl(const FObjectComponent& BoundObject)
{
	return ObjectGroupManager->FindGroupForKey(BoundObject.GetObject());
}

FPreAnimatedStorageGroupHandle FBoundObjectPreAnimatedStateTraits::MakeGroupImpl(UObject* BoundObject)
{
	return ObjectGroupManager->MakeGroupForKey(BoundObject);
}

FPreAnimatedStorageGroupHandle FBoundObjectPreAnimatedStateTraits::MakeGroupImpl(const FObjectComponent& BoundObject)
{
	return ObjectGroupManager->MakeGroupForKey(BoundObject.GetObject());
}

} // namespace MovieScene
} // namespace UE

