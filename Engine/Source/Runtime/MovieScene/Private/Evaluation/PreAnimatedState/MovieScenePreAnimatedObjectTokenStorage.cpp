// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedObjectTokenStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.inl"
#include "Evaluation/MovieSceneEvaluationKey.h"

namespace UE
{
namespace MovieScene
{

TAutoRegisterPreAnimatedStorageID<FAnimTypePreAnimatedStateObjectStorage> FAnimTypePreAnimatedStateObjectStorage::StorageID;

FPreAnimatedStateEntry FAnimTypePreAnimatedStateObjectStorage::FindEntry(UObject* Object, FMovieSceneAnimTypeID AnimTypeID)
{
	FPreAnimatedObjectTokenTraits::KeyType Key{ Object, AnimTypeID };

	// Begin by finding or creating a pre-animated state group for this bound object
	FPreAnimatedStorageGroupHandle Group = this->Traits.FindGroup(Object);

	// Find the storage index for the specific anim-type and object we're animating
	FPreAnimatedStorageIndex StorageIndex = FindStorageIndex(Key);

	return FPreAnimatedStateEntry{ Group, FPreAnimatedStateCachedValueHandle{ StorageID, StorageIndex } };
}

FPreAnimatedStateEntry FAnimTypePreAnimatedStateObjectStorage::MakeEntry(UObject* Object, FMovieSceneAnimTypeID AnimTypeID)
{
	FPreAnimatedObjectTokenTraits::KeyType Key{ Object, AnimTypeID };

	// Begin by finding or creating a pre-animated state group for this bound object
	FPreAnimatedStorageGroupHandle Group = this->Traits.MakeGroup(Object);

	// Find the storage index for the specific anim-type and object we're animating
	FPreAnimatedStorageIndex StorageIndex = GetOrCreateStorageIndex(Key);

	return FPreAnimatedStateEntry{ Group, FPreAnimatedStateCachedValueHandle{ StorageID, StorageIndex } };
}

} // namespace MovieScene
} // namespace UE






