// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedRootTokenStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.inl"

namespace UE
{
namespace MovieScene
{

TAutoRegisterPreAnimatedStorageID<FAnimTypePreAnimatedStateRootStorage> FAnimTypePreAnimatedStateRootStorage::StorageID;

void FAnimTypePreAnimatedStateRootStorage::Initialize(FPreAnimatedStorageID InStorageID, FPreAnimatedStateExtension* InParentExtension)
{
	TPreAnimatedStateStorage<FPreAnimatedRootTokenTraits>::Initialize(InStorageID, InParentExtension);
}

FPreAnimatedStateEntry FAnimTypePreAnimatedStateRootStorage::FindEntry(FMovieSceneAnimTypeID AnimTypeID)
{
	// Find the storage index for the specific anim-type and object we're animating
	FPreAnimatedStorageIndex StorageIndex = FindStorageIndex(AnimTypeID);
	return FPreAnimatedStateEntry{ FPreAnimatedStorageGroupHandle(), FPreAnimatedStateCachedValueHandle{ StorageID, StorageIndex } };
}

FPreAnimatedStateEntry FAnimTypePreAnimatedStateRootStorage::MakeEntry(FMovieSceneAnimTypeID AnimTypeID)
{
	// Find the storage index for the specific anim-type and object we're animating
	FPreAnimatedStorageIndex StorageIndex = GetOrCreateStorageIndex(AnimTypeID);
	return FPreAnimatedStateEntry{ FPreAnimatedStorageGroupHandle(), FPreAnimatedStateCachedValueHandle{ StorageID, StorageIndex } };
}



} // namespace MovieScene
} // namespace UE






