// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateExtension.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedCaptureSources.h"
#include "Evaluation/PreAnimatedState/MovieSceneRestoreStateParams.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedCaptureSource.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedObjectTokenStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedRootTokenStorage.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneComponentPtr.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "MovieSceneSequence.h"
#include "Algo/Find.h"
#include "Algo/IndexOf.h"
#include "Algo/BinarySearch.h"

namespace UE
{
namespace MovieScene
{

FPreAnimatedStateExtension::FPreAnimatedStateExtension()
	: NumRequestsForGlobalState(0)
	, Linker(nullptr)
	, bEntriesInvalidated(false)
{}

void FPreAnimatedStateExtension::Initialize(UMovieSceneEntitySystemLinker* InLinker)
{
	Linker = InLinker;
	InLinker->Events.AddReferencedObjects.AddRaw(this, &FPreAnimatedStateExtension::AddReferencedObjects);
}

FPreAnimatedStateExtension::~FPreAnimatedStateExtension()
{}

FPreAnimatedStorageID FPreAnimatedStateExtension::RegisterStorageInternal()
{
	static uint32 NextID = 0;
	return FPreAnimatedStorageID(++NextID);
}

FPreAnimatedStorageGroupHandle FPreAnimatedStateExtension::AllocateGroup(TSharedPtr<IPreAnimatedStateGroupManager> GroupManager)
{
	FPreAnimatedGroupMetaData NewEntry;
	NewEntry.GroupManagerPtr = GroupManager;

	const int32 NewIndex = GroupMetaData.Add(MoveTemp(NewEntry));
	return FPreAnimatedStorageGroupHandle{NewIndex};
}

void FPreAnimatedStateExtension::FreeGroup(FPreAnimatedStorageGroupHandle Handle)
{
	check(Handle);
	ensure(GroupMetaData[Handle.Value].AggregateMetaData.Num() == 0);

	FreeGroupInternal(Handle);
}

void FPreAnimatedStateExtension::FreeGroupInternal(FPreAnimatedStorageGroupHandle Handle)
{
	FPreAnimatedGroupMetaData& Group = GroupMetaData[Handle.Value];
	Group.GroupManagerPtr->OnGroupDestroyed(Handle);
	GroupMetaData.RemoveAt(Handle.Value);
}

void FPreAnimatedStateExtension::ReplaceObjectForGroup(FPreAnimatedStorageGroupHandle GroupHandle, const FObjectKey& OldObject, const FObjectKey& NewObject)
{
	FPreAnimatedGroupMetaData& Group = GroupMetaData[GroupHandle.Value];

	for (FAggregatePreAnimatedStateMetaData& MetaData : Group.AggregateMetaData)
	{
		TSharedPtr<IPreAnimatedStorage> Storage = GetStorageChecked(MetaData.ValueHandle.TypeID);
		Storage->OnObjectReplaced(MetaData.ValueHandle.StorageIndex, OldObject, NewObject);
	}
}

FPreAnimatedStateExtension::FAggregatePreAnimatedStateMetaData* FPreAnimatedStateExtension::FindMetaData(const FPreAnimatedStateEntry& Entry)
{
	if (Entry.GroupHandle.IsValid())
	{
		FPreAnimatedGroupMetaData&          Group     = GroupMetaData[Entry.GroupHandle.Value];
		FAggregatePreAnimatedStateMetaData* Aggregate = Algo::FindBy(Group.AggregateMetaData, Entry.ValueHandle, &FAggregatePreAnimatedStateMetaData::ValueHandle);

		return Aggregate;
	}
	else
	{
		return UngroupedMetaData.Find(Entry.ValueHandle);
	}
}

const FPreAnimatedStateExtension::FAggregatePreAnimatedStateMetaData* FPreAnimatedStateExtension::FindMetaData(const FPreAnimatedStateEntry& Entry) const
{
	if (Entry.GroupHandle.IsValid())
	{
		const FPreAnimatedGroupMetaData&          Group     = GroupMetaData[Entry.GroupHandle.Value];
		const FAggregatePreAnimatedStateMetaData* Aggregate = Algo::FindBy(Group.AggregateMetaData, Entry.ValueHandle, &FAggregatePreAnimatedStateMetaData::ValueHandle);

		return Aggregate;
	}
	else
	{
		return UngroupedMetaData.Find(Entry.ValueHandle);
	}
}

FPreAnimatedStateExtension::FAggregatePreAnimatedStateMetaData* FPreAnimatedStateExtension::GetOrAddMetaDataInternal(const FPreAnimatedStateEntry& Entry)
{
	FAggregatePreAnimatedStateMetaData* Aggregate = FindMetaData(Entry);
	if (!Aggregate)
	{
		if (Entry.GroupHandle.IsValid())
		{
			Aggregate = &GroupMetaData[Entry.GroupHandle.Value].AggregateMetaData.Emplace_GetRef(Entry.ValueHandle);
		}
		else
		{
			Aggregate = &UngroupedMetaData.Add(Entry.ValueHandle, FAggregatePreAnimatedStateMetaData{Entry.ValueHandle});
		}
	}

	return Aggregate;
}

EPreAnimatedStorageRequirement FPreAnimatedStateExtension::GetStorageRequirement(const FPreAnimatedStateEntry& Entry) const
{
	const FAggregatePreAnimatedStateMetaData* Aggregate = FindMetaData(Entry);

	if (ensure(Aggregate))
	{
		if (Aggregate->NumRestoreContributors != 0)
		{
			return EPreAnimatedStorageRequirement::Transient;
		}
		return EPreAnimatedStorageRequirement::Persistent;
	}
	return EPreAnimatedStorageRequirement::None;
}

void FPreAnimatedStateExtension::EnsureMetaData(const FPreAnimatedStateEntry& Entry)
{
	GetOrAddMetaDataInternal(Entry);
}

bool FPreAnimatedStateExtension::MetaDataExists(const FPreAnimatedStateEntry& Entry) const
{
	return FindMetaData(Entry) != nullptr;
}

void FPreAnimatedStateExtension::AddMetaData(const FPreAnimatedStateMetaData& MetaData)
{
	FAggregatePreAnimatedStateMetaData* Aggregate = GetOrAddMetaDataInternal(MetaData.Entry);

	++Aggregate->NumContributors;
	if (MetaData.bWantsRestoreState)
	{
		++Aggregate->NumRestoreContributors;
		Aggregate->bWantedRestore = true;
	}
}

void FPreAnimatedStateExtension::RemoveMetaData(const FPreAnimatedStateMetaData& MetaData)
{
	FAggregatePreAnimatedStateMetaData* Aggregate = FindMetaData(MetaData.Entry);

	if (!Aggregate)
	{
		return;
	}

	const int32 TotalNum = --Aggregate->NumContributors;
	if (MetaData.bWantsRestoreState)
	{
		if (--Aggregate->NumRestoreContributors == 0)
		{
			EPreAnimatedStorageRequirement NewRequirement = EPreAnimatedStorageRequirement::None;

			FRestoreStateParams Params = {Linker, MetaData.RootInstanceHandle};
			FPreAnimatedStorageIndex StorageIndex = MetaData.Entry.ValueHandle.StorageIndex;
			TSharedPtr<IPreAnimatedStorage> Storage = GetStorageChecked(MetaData.Entry.ValueHandle.TypeID);
			if (TotalNum == 0)
			{
				// We are completely done with the value (there are no more contributors). Restore the pre-animated 
				// value and discard it.
				EPreAnimatedStorageRequirement SourceRequirement = EPreAnimatedStorageRequirement::Transient;
				EPreAnimatedStorageRequirement TargetRequirement = EPreAnimatedStorageRequirement::None;
				NewRequirement = Storage->RestorePreAnimatedStateStorage(StorageIndex, SourceRequirement, TargetRequirement, Params);
			}
			else
			{
				// We still have contributors animating the value, but they're all set to KeepState.
				// When capturing global state, move the cached value into persistent storage. Otherwise leave the
				// storage entry untouched: it stays paired with the meta-data the remaining KeepState contributors
				// reference, and is released when that meta-data is. Freeing it here would leave the meta-data
				// pointing at a released entry for a later discard to free again.
				if (IsCapturingGlobalState())
				{
					NewRequirement = Storage->DiscardPreAnimatedStateStorage(StorageIndex, EPreAnimatedStorageRequirement::Transient);
				}
				else
				{
					NewRequirement = EPreAnimatedStorageRequirement::None;
				}
			}

			// Keep the aggregate and its group until the last contributor leaves; KeepState contributors still
			// reference both the aggregate and its storage entry.
			if (TotalNum == 0 && NewRequirement == EPreAnimatedStorageRequirement::None)
			{
				if (MetaData.Entry.GroupHandle.IsValid())
				{
					FPreAnimatedGroupMetaData& Group = GroupMetaData[MetaData.Entry.GroupHandle.Value];
					if (Group.AggregateMetaData.Num() == 1)
					{
						// If the group is going to be empty - just remove it all
						FreeGroupInternal(MetaData.Entry.GroupHandle.Value);
					}
					else
					{
						const int32 AggregateIndex = Aggregate - Group.AggregateMetaData.GetData();
						// Otherwise remove just this aggregate
						Group.AggregateMetaData.RemoveAt(AggregateIndex, EAllowShrinking::No);
					}
				}
				else
				{
					UngroupedMetaData.Remove(MetaData.Entry.ValueHandle);
				}

				return;
			}
		}
	}

	if (TotalNum == 0)
	{
		Aggregate->bWantedRestore = false;
		Aggregate->TerminalInstanceHandle = MetaData.RootInstanceHandle;
	}
}

void FPreAnimatedStateExtension::UpdateMetaData(const FPreAnimatedStateMetaData& MetaData)
{
	FAggregatePreAnimatedStateMetaData* Aggregate = FindMetaData(MetaData.Entry);
	if (ensure(Aggregate))
	{
		if (MetaData.bWantsRestoreState)
		{
			++Aggregate->NumRestoreContributors;
			Aggregate->bWantedRestore = true;
		}
		else
		{
			--Aggregate->NumRestoreContributors;
		}
	}
}

FPreAnimatedEntityCaptureSource* FPreAnimatedStateExtension::GetEntityMetaData() const
{
	return EntityCaptureSource.Get();
}

FPreAnimatedEntityCaptureSource* FPreAnimatedStateExtension::GetOrCreateEntityMetaData()
{
	if (!EntityCaptureSource)
	{
		EntityCaptureSource = MakeUnique<FPreAnimatedEntityCaptureSource>(this);
	}
	return EntityCaptureSource.Get();
}

FPreAnimatedTrackInstanceCaptureSources* FPreAnimatedStateExtension::GetTrackInstanceMetaData() const
{
	return TrackInstanceCaptureSource.Get();
}

FPreAnimatedTrackInstanceCaptureSources* FPreAnimatedStateExtension::GetOrCreateTrackInstanceMetaData()
{
	if (!TrackInstanceCaptureSource)
	{
		TrackInstanceCaptureSource = MakeUnique<FPreAnimatedTrackInstanceCaptureSources>(this);
	}
	return TrackInstanceCaptureSource.Get();
}

FPreAnimatedTrackInstanceInputCaptureSources* FPreAnimatedStateExtension::GetTrackInstanceInputMetaData() const
{
	return TrackInstanceInputCaptureSource.Get();
}

FPreAnimatedTrackInstanceInputCaptureSources* FPreAnimatedStateExtension::GetOrCreateTrackInstanceInputMetaData()
{
	if (!TrackInstanceInputCaptureSource)
	{
		TrackInstanceInputCaptureSource = MakeUnique<FPreAnimatedTrackInstanceInputCaptureSources>(this);
	}
	return TrackInstanceInputCaptureSource.Get();
}

FPreAnimatedTemplateCaptureSources* FPreAnimatedStateExtension::GetTemplateMetaData() const
{
	return TemplateCaptureSource.Get();
}

FPreAnimatedTemplateCaptureSources* FPreAnimatedStateExtension::GetOrCreateTemplateMetaData()
{
	if (!TemplateCaptureSource)
	{
		TemplateCaptureSource = MakeUnique<FPreAnimatedTemplateCaptureSources>(this);
	}
	return TemplateCaptureSource.Get();
}

FPreAnimatedEvaluationHookCaptureSources* FPreAnimatedStateExtension::GetEvaluationHookMetaData() const
{
	return EvaluationHookCaptureSource.Get();
}

FPreAnimatedEvaluationHookCaptureSources* FPreAnimatedStateExtension::GetOrCreateEvaluationHookMetaData()
{
	if (!EvaluationHookCaptureSource)
	{
		EvaluationHookCaptureSource = MakeUnique<FPreAnimatedEvaluationHookCaptureSources>(this);
	}
	return EvaluationHookCaptureSource.Get();
}

void FPreAnimatedStateExtension::AddWeakCaptureSource(TWeakPtr<IPreAnimatedCaptureSource> InWeakCaptureSource)
{
	WeakExternalCaptureSources.Add(InWeakCaptureSource);
}

void FPreAnimatedStateExtension::RemoveWeakCaptureSource(TWeakPtr<IPreAnimatedCaptureSource> InWeakCaptureSource)
{
	WeakExternalCaptureSources.Remove(InWeakCaptureSource);
}

void FPreAnimatedStateExtension::RestoreStateForGroup(FPreAnimatedStorageGroupHandle GroupHandle, const FRestoreStateParams& Params)
{
	FPreAnimatedGroupMetaData& Group = GroupMetaData[GroupHandle.Value];

	// Ensure that the entries are restored in strictly the reverse order they were cached in
	for (int32 AggregateIndex = Group.AggregateMetaData.Num()-1; AggregateIndex >= 0; --AggregateIndex)
	{
		FAggregatePreAnimatedStateMetaData& Aggregate = Group.AggregateMetaData[AggregateIndex];

		TSharedPtr<IPreAnimatedStorage> Storage = GetStorageChecked(Aggregate.ValueHandle.TypeID);
		Storage->RestorePreAnimatedStateStorage(Aggregate.ValueHandle.StorageIndex, EPreAnimatedStorageRequirement::Persistent, EPreAnimatedStorageRequirement::NoChange, Params);
	}
}

void FPreAnimatedStateExtension::RestoreGlobalState(const FRestoreStateParams& Params)
{
	TArray<FPreAnimatedStateMetaData> ExpiredMetaData;

	if (FPreAnimatedEntityCaptureSource* EntityMetaData = GetEntityMetaData())
	{
		EntityMetaData->GatherAndRemoveExpiredMetaData(Params, ExpiredMetaData);
	}

	if (FPreAnimatedTrackInstanceCaptureSources* TrackInstanceMetaData = GetTrackInstanceMetaData())
	{
		TrackInstanceMetaData->GatherAndRemoveExpiredMetaData(Params, ExpiredMetaData);
	}

	if (FPreAnimatedTrackInstanceInputCaptureSources* TrackInstanceInputMetaData = GetTrackInstanceInputMetaData())
	{
		TrackInstanceInputMetaData->GatherAndRemoveExpiredMetaData(Params, ExpiredMetaData);
	}

	if (FPreAnimatedTemplateCaptureSources* TemplateMetaData = GetTemplateMetaData())
	{
		TemplateMetaData->GatherAndRemoveExpiredMetaData(Params, ExpiredMetaData);
	}

	if (FPreAnimatedEvaluationHookCaptureSources* EvaluationHookMetaData = GetEvaluationHookMetaData())
	{
		EvaluationHookMetaData->GatherAndRemoveExpiredMetaData(Params, ExpiredMetaData);
	}

	for (int32 Index = WeakExternalCaptureSources.Num()-1; Index >= 0; --Index)
	{
		TSharedPtr<IPreAnimatedCaptureSource> MetaData = WeakExternalCaptureSources[Index].Pin();
		if (MetaData)
		{
			MetaData->GatherAndRemoveExpiredMetaData(Params, ExpiredMetaData);
		}
		else
		{
			// Order is not important in this array, so we can use the more efficient RemoveAtSwap algorithm
			WeakExternalCaptureSources.RemoveAtSwap(Index, 1);
		}
	}

	HandleMetaDataToRemove(Params, ExpiredMetaData,
			[Params](IPreAnimatedStorage& Storage, FPreAnimatedStorageIndex StorageIndex)
			{
				Storage.RestorePreAnimatedStateStorage(StorageIndex, EPreAnimatedStorageRequirement::Persistent, EPreAnimatedStorageRequirement::None, Params);
			});

	// Invalidate cached data for any sequence instance that belongs to the terminal instance.
	// This is because we have just restored pre-animated state, so trying to re-update the sequence will 
	// need to re-setup everything. It wouldn't do it unless we've invalidated it.
	if (Params.TerminalInstanceHandle.IsValid() && 
			ensureMsgf(
				Linker->GetInstanceRegistry()->IsHandleValid(Params.TerminalInstanceHandle),
				TEXT("Terminal instance handle is not valid anymore, was the sequence destroyed?")))
	{
		FSequenceInstance& TerminalInstance = Linker->GetInstanceRegistry()->MutateInstance(Params.TerminalInstanceHandle);
		TerminalInstance.InvalidateCachedData(ESequenceInstanceInvalidationType::All);
	}
}

void FPreAnimatedStateExtension::DiscardStaleObjectState()
{
	TArray<FPreAnimatedStorageGroupHandle> StaleStorageGroups;
	// Gather any groups from the group managers whose keys (e.g. FObjectKeys have become invalid)
	for (auto It = GroupManagers.CreateIterator(); It; ++It)
	{
		TWeakPtr<IPreAnimatedStateGroupManager> GroupManager = It.Value();
		if (auto GroupManagerPtr = GroupManager.Pin())
		{
			GroupManagerPtr->GatherStaleStorageGroups(StaleStorageGroups);
		}
	}
	// Discard the state for such groups
	for (FPreAnimatedStorageGroupHandle GroupHandle : StaleStorageGroups)
	{
		DiscardStateForGroup(GroupHandle);
	}

	bEntriesInvalidated = true;
}

void FPreAnimatedStateExtension::DiscardGlobalState(const FRestoreStateParams& Params)
{
	TArray<FPreAnimatedStateMetaData> ExpiredMetaData;

	if (FPreAnimatedEntityCaptureSource* EntityMetaData = GetEntityMetaData())
	{
		EntityMetaData->GatherAndRemoveExpiredMetaData(Params, ExpiredMetaData);
	}

	if (FPreAnimatedTrackInstanceCaptureSources* TrackInstanceMetaData = GetTrackInstanceMetaData())
	{
		TrackInstanceMetaData->GatherAndRemoveExpiredMetaData(Params, ExpiredMetaData);
	}

	if (FPreAnimatedTrackInstanceInputCaptureSources* TrackInstanceInputMetaData = GetTrackInstanceInputMetaData())
	{
		TrackInstanceInputMetaData->GatherAndRemoveExpiredMetaData(Params, ExpiredMetaData);
	}

	if (FPreAnimatedTemplateCaptureSources* TemplateMetaData = GetTemplateMetaData())
	{
		TemplateMetaData->GatherAndRemoveExpiredMetaData(Params, ExpiredMetaData);
	}

	if (FPreAnimatedEvaluationHookCaptureSources* EvaluationHookMetaData = GetEvaluationHookMetaData())
	{
		EvaluationHookMetaData->GatherAndRemoveExpiredMetaData(Params, ExpiredMetaData);
	}

	for (int32 Index = WeakExternalCaptureSources.Num() - 1; Index >= 0; --Index)
	{
		TSharedPtr<IPreAnimatedCaptureSource> MetaData = WeakExternalCaptureSources[Index].Pin();
		if (MetaData)
		{
			MetaData->GatherAndRemoveExpiredMetaData(Params, ExpiredMetaData);
		}
		else
		{
			// Order is not important in this array, so we can use the more efficient RemoveAtSwap algorithm
			WeakExternalCaptureSources.RemoveAtSwap(Index, 1);
		}
	}

	HandleMetaDataToRemove(Params, ExpiredMetaData,
			[](IPreAnimatedStorage& Storage, FPreAnimatedStorageIndex StorageIndex)
			{
				Storage.DiscardPreAnimatedStateStorage(StorageIndex, EPreAnimatedStorageRequirement::Persistent);
			});
}

void FPreAnimatedStateExtension::DiscardTransientState()
{
	if (FPreAnimatedEntityCaptureSource* EntityMetaData = GetEntityMetaData())
	{
		EntityMetaData->Reset();
	}

	if (FPreAnimatedTrackInstanceCaptureSources* TrackInstanceMetaData = GetTrackInstanceMetaData())
	{
		TrackInstanceMetaData->Reset();
	}

	if (FPreAnimatedTrackInstanceInputCaptureSources* TrackInstanceInputMetaData = GetTrackInstanceInputMetaData())
	{
		TrackInstanceInputMetaData->Reset();
	}

	if (FPreAnimatedTemplateCaptureSources* TemplateMetaData = GetTemplateMetaData())
	{
		TemplateMetaData->Reset();
	}

	if (FPreAnimatedEvaluationHookCaptureSources* EvaluationHookMetaData = GetEvaluationHookMetaData())
	{
		EvaluationHookMetaData->Reset();
	}

	for (int32 Index = WeakExternalCaptureSources.Num()-1; Index >= 0; --Index)
	{
		if (TSharedPtr<IPreAnimatedCaptureSource> MetaData = WeakExternalCaptureSources[Index].Pin())
		{
			MetaData->Reset();
		}
	}

	// Remove all contributions, whilst keeping the ledger of their entries within the storage
	for (FPreAnimatedGroupMetaData& Group : GroupMetaData)
	{
		for (FAggregatePreAnimatedStateMetaData& Aggregate : Group.AggregateMetaData)
		{
			Aggregate = FAggregatePreAnimatedStateMetaData(Aggregate.ValueHandle);

			TSharedPtr<IPreAnimatedStorage> Storage = GetStorageChecked(Aggregate.ValueHandle.TypeID);
			Storage->DiscardPreAnimatedStateStorage(Aggregate.ValueHandle.StorageIndex, EPreAnimatedStorageRequirement::Transient);
		}
	}
	for (TPair<FPreAnimatedStateCachedValueHandle, FAggregatePreAnimatedStateMetaData> Pair : UngroupedMetaData)
	{
		FAggregatePreAnimatedStateMetaData& Aggregate = Pair.Value;

		Aggregate = FAggregatePreAnimatedStateMetaData(Pair.Key);

		TSharedPtr<IPreAnimatedStorage> Storage = GetStorageChecked(Pair.Key.TypeID);
		Storage->DiscardPreAnimatedStateStorage(Pair.Key.StorageIndex, EPreAnimatedStorageRequirement::Transient);
	}
	bEntriesInvalidated = true;
}

void FPreAnimatedStateExtension::DiscardStateForGroup(FPreAnimatedStorageGroupHandle GroupHandle)
{
	TArray<FPreAnimatedStateMetaData> MetaDataToRemove;

	if (FPreAnimatedEntityCaptureSource* EntityMetaData = GetEntityMetaData())
	{
		EntityMetaData->GatherAndRemoveMetaDataForGroup(GroupHandle, MetaDataToRemove);
	}

	if (FPreAnimatedTrackInstanceCaptureSources* TrackInstanceMetaData = GetTrackInstanceMetaData())
	{
		TrackInstanceMetaData->GatherAndRemoveMetaDataForGroup(GroupHandle, MetaDataToRemove);
	}

	if (FPreAnimatedTrackInstanceInputCaptureSources* TrackInstanceInputMetaData = GetTrackInstanceInputMetaData())
	{
		TrackInstanceInputMetaData->GatherAndRemoveMetaDataForGroup(GroupHandle, MetaDataToRemove);
	}

	if (FPreAnimatedTemplateCaptureSources* TemplateMetaData = GetTemplateMetaData())
	{
		TemplateMetaData->GatherAndRemoveMetaDataForGroup(GroupHandle, MetaDataToRemove);
	}

	if (FPreAnimatedEvaluationHookCaptureSources* EvaluationHookMetaData = GetEvaluationHookMetaData())
	{
		EvaluationHookMetaData->GatherAndRemoveMetaDataForGroup(GroupHandle, MetaDataToRemove);
	}

	for (int32 Index = WeakExternalCaptureSources.Num()-1; Index >= 0; --Index)
	{
		if (TSharedPtr<IPreAnimatedCaptureSource> MetaData = WeakExternalCaptureSources[Index].Pin())
		{
			MetaData->GatherAndRemoveMetaDataForGroup(GroupHandle, MetaDataToRemove);
		}
	}

	// Throw away the meta data and reset any aggregates
	FPreAnimatedGroupMetaData& Group = GroupMetaData[GroupHandle.Value];

	for (FAggregatePreAnimatedStateMetaData& Aggregate : Group.AggregateMetaData)
	{
		TSharedPtr<IPreAnimatedStorage> Storage = GetStorageChecked(Aggregate.ValueHandle.TypeID);
		Storage->DiscardPreAnimatedStateStorage(Aggregate.ValueHandle.StorageIndex, EPreAnimatedStorageRequirement::Persistent);
	}

	Group.GroupManagerPtr->OnGroupDestroyed(GroupHandle.Value);
	GroupMetaData.RemoveAt(GroupHandle.Value, 1);

	bEntriesInvalidated = true;
}

void FPreAnimatedStateExtension::DiscardStateForStorage(FPreAnimatedStorageID StorageID, FPreAnimatedStorageIndex StorageIndex)
{
	TArray<FPreAnimatedStateMetaData> MetaDataToRemove;

	if (FPreAnimatedEntityCaptureSource* EntityMetaData = GetEntityMetaData())
	{
		EntityMetaData->GatherAndRemoveMetaDataForStorage(StorageID, StorageIndex, MetaDataToRemove);
	}

	if (FPreAnimatedTrackInstanceCaptureSources* TrackInstanceMetaData = GetTrackInstanceMetaData())
	{
		TrackInstanceMetaData->GatherAndRemoveMetaDataForStorage(StorageID, StorageIndex, MetaDataToRemove);
	}

	if (FPreAnimatedTrackInstanceInputCaptureSources* TrackInstanceInputMetaData = GetTrackInstanceInputMetaData())
	{
		TrackInstanceInputMetaData->GatherAndRemoveMetaDataForStorage(StorageID, StorageIndex, MetaDataToRemove);
	}

	if (FPreAnimatedTemplateCaptureSources* TemplateMetaData = GetTemplateMetaData())
	{
		TemplateMetaData->GatherAndRemoveMetaDataForStorage(StorageID, StorageIndex, MetaDataToRemove);
	}

	if (FPreAnimatedEvaluationHookCaptureSources* EvaluationHookMetaData = GetEvaluationHookMetaData())
	{
		EvaluationHookMetaData->GatherAndRemoveMetaDataForStorage(StorageID, StorageIndex, MetaDataToRemove);
	}

	for (int32 Index = WeakExternalCaptureSources.Num()-1; Index >= 0; --Index)
	{
		if (TSharedPtr<IPreAnimatedCaptureSource> MetaData = WeakExternalCaptureSources[Index].Pin())
		{
			MetaData->GatherAndRemoveMetaDataForStorage(StorageID, StorageIndex, MetaDataToRemove);
		}
	}

	// Remove all contributions
	for (const FPreAnimatedStateMetaData& MetaData : MetaDataToRemove)
	{
		FAggregatePreAnimatedStateMetaData* Aggregate = FindMetaData(MetaData.Entry);
		if (ensure(Aggregate))
		{
			const int32 TotalNum = --Aggregate->NumContributors;
			if (MetaData.bWantsRestoreState)
			{
				--Aggregate->NumRestoreContributors;
			}

			if (TotalNum == 0)
			{
				Aggregate->bWantedRestore = false;
				Aggregate->TerminalInstanceHandle = MetaData.RootInstanceHandle;
			}
		}
	}

	TSharedPtr<IPreAnimatedStorage> Storage = GetStorageChecked(StorageID);

	// Discard grouped entries
	for (int32 Index = 0; Index < GroupMetaData.GetMaxIndex(); ++Index)
	{
		if (!GroupMetaData.IsAllocated(Index))
		{
			continue;
		}

		FPreAnimatedGroupMetaData& Group = GroupMetaData[Index];

		for (int32 AggregateIndex = Group.AggregateMetaData.Num() - 1; AggregateIndex >= 0; --AggregateIndex)
		{
			FAggregatePreAnimatedStateMetaData& Aggregate = Group.AggregateMetaData[AggregateIndex];
			if (Aggregate.ValueHandle.TypeID == StorageID && 
					Aggregate.NumContributors == 0 &&
					(!StorageIndex.IsValid() || Aggregate.ValueHandle.StorageIndex == StorageIndex))
			{
				Storage->DiscardPreAnimatedStateStorage(Aggregate.ValueHandle.StorageIndex, EPreAnimatedStorageRequirement::Persistent);

				Group.AggregateMetaData.RemoveAt(AggregateIndex, EAllowShrinking::No);
			}

			if (Group.AggregateMetaData.Num() == 0)
			{
				// Remove at will not re-allocate the array or shuffle items within the sparse array, so this is safe
				Group.GroupManagerPtr->OnGroupDestroyed(Index);
				GroupMetaData.RemoveAt(Index);
			}
		}
	}

	// Discard ungrouped entries
	for (auto UngroupedIt = UngroupedMetaData.CreateIterator(); UngroupedIt; ++UngroupedIt)
	{
		FAggregatePreAnimatedStateMetaData& Aggregate = UngroupedIt.Value();
		if (Aggregate.ValueHandle.TypeID == StorageID && 
				Aggregate.NumContributors == 0 &&
				(!StorageIndex.IsValid() || Aggregate.ValueHandle.StorageIndex == StorageIndex))
		{
			Storage->DiscardPreAnimatedStateStorage(Aggregate.ValueHandle.StorageIndex, EPreAnimatedStorageRequirement::Persistent);

			UngroupedIt.RemoveCurrent();
		}
	}

	GroupMetaData.Shrink();

	bEntriesInvalidated = true;
}

bool FPreAnimatedStateExtension::ContainsAnyStateForInstanceHandle(FRootInstanceHandle RootInstanceHandle) const
{
	if (FPreAnimatedEntityCaptureSource* EntityMetaData = GetEntityMetaData())
	{
		if (EntityMetaData->ContainsInstanceHandle(RootInstanceHandle))
		{
			return true;
		}
	}

	if (FPreAnimatedTrackInstanceCaptureSources* TrackInstanceMetaData = GetTrackInstanceMetaData())
	{
		if (TrackInstanceMetaData->ContainsInstanceHandle(RootInstanceHandle))
		{
			return true;
		}
	}

	if (FPreAnimatedTrackInstanceInputCaptureSources* TrackInstanceInputMetaData = GetTrackInstanceInputMetaData())
	{
		if (TrackInstanceInputMetaData->ContainsInstanceHandle(RootInstanceHandle))
		{
			return true;
		}
	}

	if (FPreAnimatedTemplateCaptureSources* TemplateMetaData = GetTemplateMetaData())
	{
		if (TemplateMetaData->ContainsInstanceHandle(RootInstanceHandle))
		{
			return true;
		}
	}

	if (FPreAnimatedEvaluationHookCaptureSources* EvaluationHookMetaData = GetEvaluationHookMetaData())
	{
		if (EvaluationHookMetaData->ContainsInstanceHandle(RootInstanceHandle))
		{
			return true;
		}
	}

	for (int32 Index = WeakExternalCaptureSources.Num()-1; Index >= 0; --Index)
	{
		TSharedPtr<IPreAnimatedCaptureSource> MetaData = WeakExternalCaptureSources[Index].Pin();
		if (MetaData && MetaData->ContainsInstanceHandle(RootInstanceHandle))
		{
			return true;
		}
	}

	return false;
}

void FPreAnimatedStateExtension::HandleMetaDataToRemove(const FRestoreStateParams& Params, TArrayView<FPreAnimatedStateMetaData> MetaDataToRemove, FContributionRemover RemoveFunc)
{
	// Remove all contributions
	for (const FPreAnimatedStateMetaData& MetaData : MetaDataToRemove)
	{
		FAggregatePreAnimatedStateMetaData* Aggregate = FindMetaData(MetaData.Entry);
		if (ensure(Aggregate))
		{
			const int32 TotalNum = --Aggregate->NumContributors;
			if (MetaData.bWantsRestoreState)
			{
				--Aggregate->NumRestoreContributors;
			}

			if (TotalNum == 0)
			{
				Aggregate->bWantedRestore = false;
				Aggregate->TerminalInstanceHandle = MetaData.RootInstanceHandle;
			}
		}
	}

	// Ensure that the entries are removed in strictly the reverse order they were cached in
	for (int32 Index = 0; Index < GroupMetaData.GetMaxIndex(); ++Index)
	{
		if (!GroupMetaData.IsAllocated(Index))
		{
			continue;
		}

		FPreAnimatedGroupMetaData& Group = GroupMetaData[Index];

		for (int32 AggregateIndex = Group.AggregateMetaData.Num() - 1; AggregateIndex >= 0; --AggregateIndex)
		{
			FAggregatePreAnimatedStateMetaData& Aggregate = Group.AggregateMetaData[AggregateIndex];
			if (Aggregate.NumContributors == 0 && (!Aggregate.TerminalInstanceHandle.IsValid() || Aggregate.TerminalInstanceHandle == Params.TerminalInstanceHandle))
			{
				TSharedPtr<IPreAnimatedStorage> Storage = GetStorageChecked(Aggregate.ValueHandle.TypeID);
				RemoveFunc(*Storage.Get(), Aggregate.ValueHandle.StorageIndex);

				Group.AggregateMetaData.RemoveAt(AggregateIndex, EAllowShrinking::No);
			}

			if (Group.AggregateMetaData.Num() == 0)
			{
				// Remove at will not re-allocate the array or shuffle items within the sparse array, so this is safe
				Group.GroupManagerPtr->OnGroupDestroyed(Index);
				GroupMetaData.RemoveAt(Index);
			}
		}
	}

	for (auto UngroupedIt = UngroupedMetaData.CreateIterator(); UngroupedIt; ++UngroupedIt)
	{
		FAggregatePreAnimatedStateMetaData& Aggregate = UngroupedIt.Value();
		if (Aggregate.NumContributors == 0 && (!Aggregate.TerminalInstanceHandle.IsValid() || Aggregate.TerminalInstanceHandle == Params.TerminalInstanceHandle))
		{
			TSharedPtr<IPreAnimatedStorage> Storage = GetStorageChecked(Aggregate.ValueHandle.TypeID);
			RemoveFunc(*Storage.Get(), Aggregate.ValueHandle.StorageIndex);

			UngroupedIt.RemoveCurrent();
		}
	}

	GroupMetaData.Shrink();

	bEntriesInvalidated = true;
}

bool FPreAnimatedStateExtension::HasActiveCaptureSource() const
{
	FScopedPreAnimatedCaptureSource* CaptureSource = FScopedPreAnimatedCaptureSource::GetCaptureSourcePtr();
	ensureMsgf(!CaptureSource || CaptureSource->WeakLinker.Get() == Linker,
			TEXT("The current capture source is related to a different linker. Are you missing setting a scope capture source?"));
	return (CaptureSource && CaptureSource->bWantsRestoreState);
}

void FPreAnimatedStateExtension::AddSourceMetaData(const UE::MovieScene::FPreAnimatedStateEntry& Entry)
{
	using namespace UE::MovieScene;

	FScopedPreAnimatedCaptureSource* CaptureSource = FScopedPreAnimatedCaptureSource::GetCaptureSourcePtr();
	ensureMsgf(!CaptureSource || CaptureSource->WeakLinker.Get() == Linker,
			TEXT("The current capture source is related to a different linker. Are you missing setting a scope capture source?"));
	if (!CaptureSource)
	{
		EnsureMetaData(Entry);
	}
	else
	{
		FPreAnimatedStateMetaData MetaData;
		MetaData.Entry = Entry;
		MetaData.RootInstanceHandle = CaptureSource->GetRootInstanceHandle(Linker);
		MetaData.bWantsRestoreState = CaptureSource->bWantsRestoreState;

		CaptureSource->BeginTracking(MetaData, Linker);
	}
}

void FPreAnimatedStateExtension::SavePreAnimatedState(UObject& InObject, FMovieSceneAnimTypeID InTokenType, const IMovieScenePreAnimatedTokenProducer& Producer)
{
	if (HasActiveCaptureSource() || IsCapturingGlobalState())
	{
		SavePreAnimatedStateDirectly(InObject, InTokenType, Producer);
	}
}

void FPreAnimatedStateExtension::SavePreAnimatedStateDirectly(UObject& InObject, FMovieSceneAnimTypeID InTokenType, const IMovieScenePreAnimatedTokenProducer& Producer)
{
	TSharedPtr<FAnimTypePreAnimatedStateObjectStorage> ObjectStorage = WeakGenericObjectStorage.Pin();
	if (!ObjectStorage)
	{
		ObjectStorage = GetOrCreateStorage<FAnimTypePreAnimatedStateObjectStorage>();
		WeakGenericObjectStorage = ObjectStorage;
	}

	FPreAnimatedStateEntry   Entry        = ObjectStorage->MakeEntry(&InObject, InTokenType);
	FPreAnimatedStorageIndex StorageIndex = Entry.ValueHandle.StorageIndex;

	AddSourceMetaData(Entry);

	EPreAnimatedStorageRequirement StorageRequirement = GetStorageRequirement(Entry);
	if (!ObjectStorage->IsStorageRequirementSatisfied(StorageIndex, StorageRequirement))
	{
		IMovieScenePreAnimatedTokenPtr Token = Producer.CacheExistingState(InObject);
		if (Token.IsValid())
		{
			const bool bHasEverAninmated = ObjectStorage->HasEverAnimated(StorageIndex);
			if (!bHasEverAninmated)
			{
				Producer.InitializeObjectForAnimation(InObject);
			}

			ObjectStorage->AssignPreAnimatedValue(StorageIndex, StorageRequirement, MoveTemp(Token));
		}
	}
}

void FPreAnimatedStateExtension::SavePreAnimatedState(FMovieSceneAnimTypeID InTokenType, const IMovieScenePreAnimatedGlobalTokenProducer& Producer)
{
	if (HasActiveCaptureSource() || IsCapturingGlobalState())
	{
		SavePreAnimatedStateDirectly(InTokenType, Producer);
	}
}

void FPreAnimatedStateExtension::SavePreAnimatedStateDirectly(FMovieSceneAnimTypeID InTokenType, const IMovieScenePreAnimatedGlobalTokenProducer& Producer)
{
	using namespace UE::MovieScene;

	TSharedPtr<FAnimTypePreAnimatedStateRootStorage> RootStorage = WeakGenericRootStorage.Pin();
	if (!RootStorage)
	{
		RootStorage = GetOrCreateStorage<FAnimTypePreAnimatedStateRootStorage>();
		WeakGenericRootStorage = RootStorage;
	}

	FPreAnimatedStateEntry   Entry        = RootStorage->MakeEntry(InTokenType);
	FPreAnimatedStorageIndex StorageIndex = Entry.ValueHandle.StorageIndex;

	AddSourceMetaData(Entry);

	EPreAnimatedStorageRequirement StorageRequirement = GetStorageRequirement(Entry);
	if (!RootStorage->IsStorageRequirementSatisfied(StorageIndex, StorageRequirement))
	{
		IMovieScenePreAnimatedGlobalTokenPtr Token = Producer.CacheExistingState();
		if (Token.IsValid())
		{
			const bool bHasEverAninmated = RootStorage->HasEverAnimated(StorageIndex);
			if (!bHasEverAninmated)
			{
				Producer.InitializeForAnimation();
			}

			RootStorage->AssignPreAnimatedValue(StorageIndex, StorageRequirement, MoveTemp(Token));
		}
	}
}

void FPreAnimatedStateExtension::AddReferencedObjects(UMovieSceneEntitySystemLinker*, FReferenceCollector& ReferenceCollector)
{
	for (TPair<FPreAnimatedStorageID, TSharedPtr<IPreAnimatedStorage>>& Pair : StorageImplementations)
	{
		Pair.Value->AddReferencedObjects(ReferenceCollector);
	}
}

void FPreAnimatedStateExtension::OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap)
{
	TSharedPtr<FPreAnimatedObjectGroupManager> ObjectGroupManager = FindGroupManager<FPreAnimatedObjectGroupManager>();
	if (ObjectGroupManager)
	{
		ObjectGroupManager->OnObjectsReplaced(ReplacementMap);
	}
}

} // namespace MovieScene
} // namespace UE
