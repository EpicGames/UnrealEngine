// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateStorage.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/EntityAllocationIterator.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateExtension.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedObjectGroupManager.h"

namespace UE
{
namespace MovieScene
{

FPreAnimatedTrackerParams::FPreAnimatedTrackerParams(FEntityAllocationProxy Item)
{
	Num = Item.GetAllocation()->Num();
	bWantsRestoreState = Item.GetAllocationType().Contains(FBuiltInComponentTypes::Get()->Tags.RestoreState);
}

} // namespace MovieScene
} // namespace UE

