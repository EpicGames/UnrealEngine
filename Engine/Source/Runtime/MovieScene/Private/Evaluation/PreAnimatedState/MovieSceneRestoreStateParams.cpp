// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/PreAnimatedState/MovieSceneRestoreStateParams.h"

#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneInstanceRegistry.h"
#include "EntitySystem/MovieSceneSequenceInstance.h"
#include "IMovieScenePlayer.h"

namespace UE
{
namespace MovieScene
{

IMovieScenePlayer* FRestoreStateParams::GetTerminalPlayer() const
{
	if (Linker && TerminalInstanceHandle.IsValid())
	{
		const FSequenceInstance& Instance = Linker->GetInstanceRegistry()->GetInstance(TerminalInstanceHandle);
		return UE::MovieScene::FPlayerIndexPlaybackCapability::GetPlayer(Instance.GetSharedPlaybackState());
	}

	ensureAlways(false);
	return nullptr;
}

TSharedPtr<FSharedPlaybackState> FRestoreStateParams::GetTerminalPlaybackState() const
{
	if (Linker && TerminalInstanceHandle.IsValid())
	{
		const FSequenceInstance& TerminalInstance = Linker->GetInstanceRegistry()->GetInstance(TerminalInstanceHandle);
		return TerminalInstance.GetSharedPlaybackState().ToSharedPtr();
	}

	ensureAlways(false);
	return nullptr;
}

} // namespace MovieScene
} // namespace UE
