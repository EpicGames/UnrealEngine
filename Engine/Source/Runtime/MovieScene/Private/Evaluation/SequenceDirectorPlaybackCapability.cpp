// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/SequenceDirectorPlaybackCapability.h"

#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "Evaluation/MovieSceneSequenceHierarchy.h"
#include "IMovieScenePlayer.h"
#include "MovieSceneSequence.h"

namespace UE::MovieScene
{

UE_DEFINE_MOVIESCENE_PLAYBACK_CAPABILITY(FSequenceDirectorPlaybackCapability)

FString FSequenceDirectorPlaybackCapability::FDirectorInstanceCache::GetReferencerName() const
{
	return TEXT("FSequenceDirectorPlaybackCapability");
}

void FSequenceDirectorPlaybackCapability::FDirectorInstanceCache::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (auto& Pair : DirectorInstances)
	{
		Collector.AddReferencedObject(Pair.Value);
	}
}

void FSequenceDirectorPlaybackCapability::ResetDirectorInstances()
{
	if (Cache)
	{
		Cache->DirectorInstances.Reset();
	}
	WeakCache.Reset();
}

void FSequenceDirectorPlaybackCapability::InvalidateCachedData(UMovieSceneEntitySystemLinker* Linker)
{
	ResetDirectorInstances();
}

UObject* FSequenceDirectorPlaybackCapability::GetOrCreateDirectorInstance(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, FMovieSceneSequenceIDRef SequenceID)
{
	if (bUseStrongCache && !Cache)
	{
		Cache = MakeUnique<FDirectorInstanceCache>();
	}

	UObject* ExistingDirectorInstance = bUseStrongCache ?
		Cache->DirectorInstances.FindRef(SequenceID).Get() :
		WeakCache.FindRef(SequenceID).Get();
#if WITH_EDITOR
	if (ExistingDirectorInstance)
	{
		// Invalidate our cached director instance if it has been recompiled.
		UClass* DirectorClass = ExistingDirectorInstance->GetClass();
		if (!DirectorClass || DirectorClass->HasAnyClassFlags(CLASS_NewerVersionExists))
		{
			ExistingDirectorInstance = nullptr;
		}
	}
#endif
	if (ExistingDirectorInstance)
	{
		return ExistingDirectorInstance;
	}

	UObject* NewDirectorInstance = nullptr;
	if (SequenceID == MovieSceneSequenceID::Root)
	{
		if (UMovieSceneSequence* Sequence = SharedPlaybackState->GetRootSequence())
		{
			NewDirectorInstance = Sequence->CreateDirectorInstance(SharedPlaybackState, SequenceID);
		}
	}
	else if (const FMovieSceneSequenceHierarchy* Hierarchy = SharedPlaybackState->GetHierarchy())
	{
		const FMovieSceneSubSequenceData* SubData = Hierarchy->FindSubData(SequenceID);
		check(SubData);
		NewDirectorInstance = SubData->GetSequence()->CreateDirectorInstance(SharedPlaybackState, SequenceID);
	}

	if (NewDirectorInstance)
	{
		if (bUseStrongCache)
		{
			Cache->DirectorInstances.Add(SequenceID, NewDirectorInstance);
		}
		else
		{
			WeakCache.Add(SequenceID, NewDirectorInstance);
		}
	}

	return NewDirectorInstance;
}

}  // namespace UE::MovieScene

