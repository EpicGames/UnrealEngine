// Copyright Epic Games, Inc. All Rights Reserved.

#include "EventHandlers/IFolderEventHandler.h"
#include "MovieSceneFolder.h"


namespace UE
{
namespace MovieScene
{

// void IFolderEventHandler::BindTo(UMovieSceneFolder* Folder)
// {
// 	ensureMsgf(!IsLinked(), TEXT("This event handler is already bound - the previous binding will no longer apply. Please call Unlink first."));
// 	Folder->EventHandlers.Link(this);
// }

} // namespace MovieScene
} // namespace UE