// Copyright Epic Games, Inc. All Rights Reserved.

#include "EventHandlers/ISectionEventHandler.h"
#include "MovieSceneSection.h"


namespace UE
{
namespace MovieScene
{

// void ISectionEventHandler::BindTo(UMovieSceneSection* InSection)
// {
// 	ensureMsgf(!IsLinked(), TEXT("This event handler is already bound - the previous binding will no longer apply. Please call Unlink first."));
// 	InSection->EventHandlers.Link(this);
// }

} // namespace MovieScene
} // namespace UE