// Copyright Epic Games, Inc. All Rights Reserved.

#include "EventHandlers/ISequenceDataEventHandler.h"
#include "MovieScene.h"


namespace UE
{
namespace MovieScene
{

// void ISequenceDataEventHandler::BindTo(UMovieScene* MovieScene)
// {
// 	ensureMsgf(!IsLinked(), TEXT("This event handler is already bound - the previous binding will no longer apply. Please call Unlink first."));
// 	MovieScene->EventHandlers.Link(this);
// }

} // namespace MovieScene
} // namespace UE