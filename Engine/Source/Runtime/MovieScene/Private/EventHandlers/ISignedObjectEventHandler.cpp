// Copyright Epic Games, Inc. All Rights Reserved.

#include "EventHandlers/ISignedObjectEventHandler.h"
#include "MovieSceneSignedObject.h"


namespace UE
{
namespace MovieScene
{

// void ISignedObjectEventHandler::BindTo(UMovieSceneSignedObject* Object)
// {
// 	ensureMsgf(!IsLinked(), TEXT("This event handler is already bound - the previous binding will no longer apply. Please call Unlink first."));
// 	Object->EventHandlers.Link(this);
// }

} // namespace MovieScene
} // namespace UE