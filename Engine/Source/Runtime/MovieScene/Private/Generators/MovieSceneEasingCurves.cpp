// Copyright Epic Games, Inc. All Rights Reserved.

#include "Generators/MovieSceneEasingCurves.h"
#include "Curves/CurveFloat.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneEasingCurves)

float IMovieSceneEasingFunction::EvaluateWith(const TScriptInterface<IMovieSceneEasingFunction>& ScriptInterface, float Time)
{
	return ScriptInterface.GetInterface() ? ScriptInterface->Evaluate(Time) : Execute_OnEvaluate(ScriptInterface.GetObject(), Time);
}

UMovieSceneBuiltInEasingFunction::UMovieSceneBuiltInEasingFunction(const FObjectInitializer& Initiailizer)
	: Super(Initiailizer)
{
	Type = EMovieSceneBuiltInEasing::Linear;
}

namespace Impl
{
	float SinOut(float InTime)
	{
		return FMath::Sin(.5f*PI*InTime);
	}
	float SinIn(float InTime)
	{
		return 1.f - SinOut(1.f - InTime);
	}
	float PowIn(float InTime, float Power)
	{
		return FMath::Pow(InTime, Power);
	}
	float PowOut(float InTime, float Power)
	{
		return 1.f - FMath::Pow(1.f - InTime, Power);
	}
	float ExpIn(float InTime)
	{
		return FMath::Pow(2, 10*(InTime - 1.f));
	}
	float ExpOut(float InTime)
	{
		return 1.f - ExpIn(1.f - InTime);
	}
	float CircIn(float InTime)
	{
		return 1.f-FMath::Sqrt(1-InTime*InTime);
	}
	float CircOut(float InTime)
	{
		return 1.f - CircIn(1.f - InTime);
	}
}

float UMovieSceneBuiltInEasingFunction::Evaluate(float Interp) const
{
	using namespace Impl;
	
	// Used for in-out easing
	const float InTime = Interp * 2.f;
	const float OutTime = (Interp - .5f) * 2.f;

	switch (Type)
	{
	case EMovieSceneBuiltInEasing::SinIn: 		return SinIn(Interp);
	case EMovieSceneBuiltInEasing::SinOut: 		return SinOut(Interp);
	case EMovieSceneBuiltInEasing::SinInOut: 	return InTime < 1.f ? .5f*SinIn(InTime) : .5f + .5f*SinOut(OutTime);

	case EMovieSceneBuiltInEasing::QuadIn: 		return PowIn(Interp,2);
	case EMovieSceneBuiltInEasing::QuadOut: 	return PowOut(Interp,2);
	case EMovieSceneBuiltInEasing::QuadInOut: 	return InTime < 1.f ? .5f*PowIn(InTime,2) : .5f + .5f*PowOut(OutTime,2);

	case EMovieSceneBuiltInEasing::Cubic: 		return FMath::Clamp<float>(FMath::CubicInterp<float>(0.f, 0.f, 1.f, 0.f, Interp), 0.f, 1.f);
	case EMovieSceneBuiltInEasing::CubicIn: 	return PowIn(Interp,3);
	case EMovieSceneBuiltInEasing::CubicOut: 	return PowOut(Interp,3);
	case EMovieSceneBuiltInEasing::CubicInOut: 	return InTime < 1.f ? .5f*PowIn(InTime,3) : .5f + .5f*PowOut(OutTime,3);

	case EMovieSceneBuiltInEasing::HermiteCubicInOut: return FMath::Clamp<float>(FMath::SmoothStep(0.0f, 1.0f, Interp), 0.0f, 1.0f);

	case EMovieSceneBuiltInEasing::QuartIn: 	return PowIn(Interp,4);
	case EMovieSceneBuiltInEasing::QuartOut: 	return PowOut(Interp,4);
	case EMovieSceneBuiltInEasing::QuartInOut: 	return InTime < 1.f ? .5f*PowIn(InTime,4) : .5f + .5f*PowOut(OutTime,4);

	case EMovieSceneBuiltInEasing::QuintIn: 	return PowIn(Interp,5);
	case EMovieSceneBuiltInEasing::QuintOut: 	return PowOut(Interp,5);
	case EMovieSceneBuiltInEasing::QuintInOut: 	return InTime < 1.f ? .5f*PowIn(InTime,5) : .5f + .5f*PowOut(OutTime,5);

	case EMovieSceneBuiltInEasing::ExpoIn: 		return ExpIn(Interp);
	case EMovieSceneBuiltInEasing::ExpoOut: 	return ExpOut(Interp);
	case EMovieSceneBuiltInEasing::ExpoInOut: 	return InTime < 1.f ? .5f*ExpIn(InTime) : .5f + .5f*ExpOut(OutTime);

	case EMovieSceneBuiltInEasing::CircIn: 		return CircIn(Interp);
	case EMovieSceneBuiltInEasing::CircOut: 	return CircOut(Interp);
	case EMovieSceneBuiltInEasing::CircInOut: 	return InTime < 1.f ? .5f*CircIn(InTime) : .5f + .5f*CircOut(OutTime);

	case EMovieSceneBuiltInEasing::Linear:
		default:
		return Interp;
	}
}

float UMovieSceneEasingExternalCurve::Evaluate(float InTime) const
{
	return Curve ? Curve->GetFloatValue(InTime) : 0.f;
}

