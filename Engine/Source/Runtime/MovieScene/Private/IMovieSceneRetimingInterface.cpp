// Copyright Epic Games, Inc. All Rights Reserved.

#include "IMovieSceneRetimingInterface.h"
#include "MovieScene.h"

namespace UE::MovieScene
{


FFrameNumber IRetimingInterface::RemapTime(FFrameNumber InTime) const
{
	return RemapTime(FFrameTime(InTime)).RoundToFrame();
}

FFrameRateRetiming::FFrameRateRetiming(FFrameRate InSourceRate, FFrameRate InDestinationRate)
	: SourceRate(InSourceRate)
	, DestinationRate(InDestinationRate)
{}

double FFrameRateRetiming::GetScale() const
{
	return DestinationRate.AsInterval() / SourceRate.AsInterval();
}

FFrameTime FFrameRateRetiming::RemapTime(FFrameTime InTime) const
{
	return ConvertFrameTime(InTime, SourceRate, DestinationRate);
}

TUniquePtr<IRetimingInterface> FFrameRateRetiming::RecurseInto(UMovieScene* InMovieScene) const
{
	return MakeUnique<FFrameRateRetiming>(InMovieScene->GetTickResolution(), DestinationRate);
}

void FFrameRateRetiming::Begin(UMovieScene* InMovieScene) const
{

}

void FFrameRateRetiming::End(UMovieScene* InMovieScene) const
{
	InMovieScene->SetTickResolutionDirectly(DestinationRate);
}


} // namespace UE::MovieScene