// Copyright Epic Games, Inc. All Rights Reserved.
#include "INodeAndChannelMappings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(INodeAndChannelMappings)

UNodeAndChannelMappings::UNodeAndChannelMappings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}
