// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetaData/MovieSceneShotMetaData.h"

#include "AssetRegistry/AssetData.h"
#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneShotMetaData)

#define LOCTEXT_NAMESPACE "MovieSceneShotMetaData"

namespace UE::MovieScene::ShotMetaDataDetail
{
template<typename TTagValue>
static bool ParseAssetTag(const FAssetData& InAssetData, FName InTag, TTagValue& Value)
{
	FString StringValue;
	InAssetData.GetTagValue(InTag, StringValue);
	
	// ExtendAssetRegistryTags sets "Unset" if the optional value was unset. Users could have also messed with the registry.
	if (StringValue.IsNumeric()) 
	{
		Value = static_cast<TTagValue>(FCString::Atoi(*StringValue));
		return true;
	}
	
	return false;
}

static const TCHAR* ConvertBoolToYesNoString(bool bInValue)
{
	return bInValue ? TEXT("Yes") : TEXT("No");
}

/** We need special handling for "bool" tags. We save them as "Yes" / "No" strings. */
template<>
bool ParseAssetTag(const FAssetData& InAssetData, FName InTag, bool& Value)
{
	FString StringValue;
	InAssetData.GetTagValue(InTag, StringValue);

	// Starting 5.7, bool tags are "Yes" and "No"
	if (StringValue == ConvertBoolToYesNoString(true))
	{
		Value = true;
		return true;
	}
	if (StringValue == ConvertBoolToYesNoString(false))
	{
		Value = false;
		return true;
	}

	// Handle pre 5.7, where the tags were stored as strings "1" -> true, "0" -> false
	if (StringValue.IsNumeric()) 
	{
		// Ok to demote int to bool
		Value = static_cast<bool>(FCString::Atoi(*StringValue));
		return true;
	}

	return false;
}
}

const FName UMovieSceneShotMetaData::AssetRegistryTag_bIsNoGood(TEXT("AssetRegistryTag_MovieScene_bIsNoGood"));
const FName UMovieSceneShotMetaData::AssetRegistryTag_bIsFlagged(TEXT("AssetRegistryTag_MovieScene_bIsFlagged"));
const FName UMovieSceneShotMetaData::AssetRegistryTag_bIsSubSequence(TEXT("AssetRegistryTag_IsSubSequence"));
const FName UMovieSceneShotMetaData::AssetRegistryTag_bIsRecorded(TEXT("AssetRegistryTag_IsRecorded"));
const FName UMovieSceneShotMetaData::AssetRegistryTag_FavoriteRating(TEXT("AssetRegistryTag_MovieScene_FavoriteRating"));

bool UMovieSceneShotMetaData::GetIsNoGoodByAssetData(const FAssetData& InAssetData, bool& bOutNoGood)
{
	return UE::MovieScene::ShotMetaDataDetail::ParseAssetTag(InAssetData, AssetRegistryTag_bIsNoGood, bOutNoGood);
}

bool UMovieSceneShotMetaData::GetIsFlaggedByAssetData(const FAssetData& InAssetData, bool& bOutIsFlagged)
{
	return UE::MovieScene::ShotMetaDataDetail::ParseAssetTag(InAssetData, AssetRegistryTag_bIsFlagged, bOutIsFlagged);
}

bool UMovieSceneShotMetaData::GetIsRecordedByAssetData(const FAssetData& InAssetData, bool& bOutIsRecorded)
{
	return UE::MovieScene::ShotMetaDataDetail::ParseAssetTag(InAssetData, AssetRegistryTag_bIsRecorded, bOutIsRecorded);
}

bool UMovieSceneShotMetaData::GetIsSubSequenceByAssetData(const FAssetData& InAssetData, bool& bOutIsSubSequence)
{
	return UE::MovieScene::ShotMetaDataDetail::ParseAssetTag(InAssetData, AssetRegistryTag_bIsSubSequence, bOutIsSubSequence);
}

bool UMovieSceneShotMetaData::GetFavoriteRatingByAssetData(const FAssetData& InAssetData, int32& OutFavoriteRating)
{
	return UE::MovieScene::ShotMetaDataDetail::ParseAssetTag(InAssetData, AssetRegistryTag_FavoriteRating, OutFavoriteRating);
}

void UMovieSceneShotMetaData::ExtendAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	using namespace UE::MovieScene::ShotMetaDataDetail;
	
	Context.AddTag(FAssetRegistryTag(
		AssetRegistryTag_bIsNoGood, bIsNoGood.IsSet() ? ConvertBoolToYesNoString(*bIsNoGood) : TEXT(""),
		FAssetRegistryTag::ETagType::TT_Alphabetical, FAssetRegistryTag::TD_None
	));
	Context.AddTag(FAssetRegistryTag(
		AssetRegistryTag_bIsFlagged, bIsFlagged.IsSet() ? ConvertBoolToYesNoString(*bIsFlagged) : TEXT(""),
		FAssetRegistryTag::ETagType::TT_Alphabetical, FAssetRegistryTag::TD_None
	));
	Context.AddTag(FAssetRegistryTag(
		AssetRegistryTag_bIsRecorded, bIsRecorded.IsSet() ? ConvertBoolToYesNoString(*bIsRecorded) : TEXT(""),
		FAssetRegistryTag::ETagType::TT_Alphabetical, FAssetRegistryTag::TD_None
	));
	Context.AddTag(FAssetRegistryTag(
		AssetRegistryTag_bIsSubSequence, bIsSubSequence.IsSet() ? ConvertBoolToYesNoString(*bIsSubSequence) : TEXT(""),
		FAssetRegistryTag::ETagType::TT_Alphabetical, FAssetRegistryTag::TD_None
	));
	Context.AddTag(FAssetRegistryTag(
		AssetRegistryTag_FavoriteRating, FavoriteRating.IsSet() ? FString::FromInt(*FavoriteRating) : TEXT(""),
		FAssetRegistryTag::ETagType::TT_Alphabetical, FAssetRegistryTag::TD_None
	));
}

#if WITH_EDITOR
void UMovieSceneShotMetaData::ExtendAssetRegistryTagMetaData(TMap<FName, UObject::FAssetRegistryTagMetadata>& OutMetadata) const
{
	OutMetadata.Add(AssetRegistryTag_bIsNoGood, FAssetRegistryTagMetadata()
		.SetDisplayName(LOCTEXT("IsNoGood.Label", "Is No Good"))
		.SetTooltip(LOCTEXT("IsNoGood.Description", "Whether this shot does not fulfill your requirements."))
	);
	OutMetadata.Add(AssetRegistryTag_bIsFlagged, FAssetRegistryTagMetadata()
		.SetDisplayName(LOCTEXT("IsFlagged.Label", "Is Flagged"))
		.SetTooltip(LOCTEXT("IsFlagged.Description", "Whether this shot was highlighted"))
	);
	OutMetadata.Add(AssetRegistryTag_bIsRecorded, FAssetRegistryTagMetadata()
		.SetDisplayName(LOCTEXT("IsRecorded.Label", "Is Recorded"))
		.SetTooltip(LOCTEXT("IsRecorded.Description", "If this sequence was recorded."))
	);
	OutMetadata.Add(AssetRegistryTag_bIsSubSequence, FAssetRegistryTagMetadata()
		.SetDisplayName(LOCTEXT("IsSubSequence.Label", "Is SubSequence"))
		.SetTooltip(LOCTEXT("IsSubSequence.Description", "If this was recorded as a subsequence."))
	);
	OutMetadata.Add(AssetRegistryTag_FavoriteRating, FAssetRegistryTagMetadata()
		.SetDisplayName(LOCTEXT("FavoriteRating.Label", "Rating"))
		.SetTooltip(LOCTEXT("FavoriteRating.Description", "A star rating, usually ranging 1-3 and 0 meaning no rating, yet."))
	);
}
#endif

#undef LOCTEXT_NAMESPACE
