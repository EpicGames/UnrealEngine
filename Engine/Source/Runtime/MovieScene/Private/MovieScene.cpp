// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene.h"

#include "MovieSceneTrack.h"
#include "MovieSceneFolder.h"
#include "MovieSceneSection.h"
#include "MovieSceneSequence.h"
#include "Evaluation/MovieSceneEvaluationCustomVersion.h"
#include "Compilation/MovieSceneSegmentCompiler.h"
#include "UObject/SequencerObjectVersion.h"
#include "Evaluation/IMovieSceneCustomClockSource.h"
#include "CommonFrameRates.h"
#include "EventHandlers/ISequenceDataEventHandler.h"
#include "Misc/FrameRate.h"
#include "Decorations/IMovieSceneDecoration.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/UObjectHash.h"
#include "MovieSceneClock.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScene)
#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneMarkedFrame)

#define LOCTEXT_NAMESPACE "MovieScene"

TOptional<TRangeBound<FFrameNumber>> GetMaxUpperBound(const UMovieSceneTrack* Track)
{
	TOptional<TRangeBound<FFrameNumber>> MaxBound;

	// Find the largest closed upper bound of all the track's sections
	for (UMovieSceneSection* Section : Track->GetAllSections())
	{
		TRangeBound<FFrameNumber> SectionUpper = Section->GetRange().GetUpperBound();
		if (SectionUpper.IsClosed())
		{
			MaxBound = TRangeBound<FFrameNumber>::MaxUpper(MaxBound.Get(SectionUpper), SectionUpper);
		}
	}

	return MaxBound;
}

/* UMovieScene interface
 *****************************************************************************/

#if WITH_EDITOR

UMovieScene::FIsTrackClassAllowedEvent UMovieScene::IsTrackClassAllowedEvent;
UMovieScene::FFixupDynamicBindingPayloadParameterNameEvent UMovieScene::FixupDynamicBindingPayloadParameterNameEvent;
UMovieScene::FIsCustomBindingClassAllowedEvent UMovieScene::IsCustomBindingClassAllowedEvent;
UMovieScene::FIsConditionClassAllowedEvent UMovieScene::IsConditionClassAllowedEvent;
UMovieScene::FFixupDirectorBlueprintConditionPayloadParameterNameEvent UMovieScene::FixupDirectorBlueprintConditionPayloadParameterNameEvent;

bool UMovieScene::IsTrackClassAllowed(UClass* InClass)
{
	if (IsTrackClassAllowedEvent.IsBound() && !IsTrackClassAllowedEvent.Execute(InClass))
	{
		return false;
	}

	return true;
}

bool UMovieScene::IsCustomBindingClassAllowed(UClass* InClass)
{
	if (IsCustomBindingClassAllowedEvent.IsBound() && !IsCustomBindingClassAllowedEvent.Execute(InClass))
	{
		return false;
	}

	return true;
}

bool UMovieScene::IsConditionClassAllowed(const UClass* InClass)
{
	if (IsConditionClassAllowedEvent.IsBound() && !IsConditionClassAllowedEvent.Execute(InClass))
	{
		return false;
	}

	return true;
}

#endif

UMovieScene::UMovieScene(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	EvaluationType = EMovieSceneEvaluationType::WithSubFrames;
	ClockSource = EUpdateClockSource::Tick;

	if (!HasAnyFlags(RF_ClassDefaultObject) && GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::FloatToIntConversion)
	{
		TickResolution = GetLegacyConversionFrameRate();
	}

#if WITH_EDITORONLY_DATA
	bReadOnly = false;
	bPlaybackRangeLocked = false;
	bMarkedFramesLocked = false;
	PlaybackRange.MigrationDefault = FFloatRange::Empty();
	EditorData.WorkingRange_DEPRECATED = EditorData.ViewRange_DEPRECATED = TRange<float>::Empty();

	bForceFixedFrameIntervalPlayback_DEPRECATED = false;
	FixedFrameInterval_DEPRECATED = 0.0f;

	NodeGroupCollection = CreateEditorOnlyDefaultSubobject<UMovieSceneNodeGroupCollection>("NodeGroupCollection");

	InTime_DEPRECATED    =  FLT_MAX;
	OutTime_DEPRECATED   = -FLT_MAX;
	StartTime_DEPRECATED =  FLT_MAX;
	EndTime_DEPRECATED   = -FLT_MAX;
#endif
}

bool UMovieScene::IsPostLoadThreadSafe() const
{
	return true;
}

void UMovieScene::PostInitProperties()
{
	SetFlags(RF_Transactional);

	Super::PostInitProperties();
}

void UMovieScene::PostLoad()
{
	SortMarkedFrames();

#if WITH_EDITORONLY_DATA

	if (!CustomClockSourcePath_DEPRECATED.IsNull())
	{
		UMovieSceneExternalClock* ExternalClock = NewObject<UMovieSceneExternalClock>(this);
		ExternalClock->CustomClockSourcePath = CustomClockSourcePath_DEPRECATED;
		SetCustomClock(ExternalClock);
	}

	for (FMovieSceneMarkedFrame& MarkedFrame : MarkedFrames)
	{
		const FLinearColor DefaultDeprecatedColor(0.f, 1.f, 1.f, 0.4f);

		if (MarkedFrame.Color_DEPRECATED != DefaultDeprecatedColor)
		{
			MarkedFrame.bUseCustomColor = true;
			MarkedFrame.CustomColor = MarkedFrame.Color_DEPRECATED;
		}
	}
#endif

	Super::PostLoad();
}

#if WITH_EDITORONLY_DATA
void UMovieScene::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(TEXT("/Script/MovieScene.MovieSceneNodeGroupCollection")));
}
#endif

void UMovieScene::Serialize( FArchive& Ar )
{
	Ar.UsingCustomVersion(FMovieSceneEvaluationCustomVersion::GUID);
	Ar.UsingCustomVersion(FSequencerObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	// Serialize the MovieScene
	Super::Serialize(Ar);

#if WITH_EDITOR
	if (Ar.IsLoading())
	{
		if (MasterTracks_DEPRECATED.Num())
		{
			Tracks = MasterTracks_DEPRECATED;
			MasterTracks_DEPRECATED.Empty();
		}

		UpgradeTimeRanges();
		RemoveNullTracks();

		const bool bUpgradeSpawnables = Ar.CustomVer(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::SpawnableImprovements;

		for (FMovieSceneSpawnable& Spawnable : Spawnables)
		{
			if (UObject* Template = Spawnable.GetObjectTemplate())
			{
				// Spawnables are no longer marked archetype
				Template->ClearFlags(RF_ArchetypeObject);
				
				FMovieSceneSpawnable::MarkSpawnableTemplate(*Template);
			}

			if (bUpgradeSpawnables)
			{
				Spawnable.AutoSetNetAddressableName();
			}
		}
	}
#endif

#if WITH_EDITORONLY_DATA
	if (Ar.CustomVer(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::FloatToIntConversion)
	{
		if (bForceFixedFrameIntervalPlayback_DEPRECATED)
		{
			EvaluationType = EMovieSceneEvaluationType::FrameLocked;
		}

		// Legacy fixed frame interval conversion to integer play rates
		if      (FixedFrameInterval_DEPRECATED == 1 / 15.0f)  { DisplayRate = FCommonFrameRates::FPS_15();  }
		else if (FixedFrameInterval_DEPRECATED == 1 / 24.0f)  { DisplayRate = FCommonFrameRates::FPS_24();  }
		else if (FixedFrameInterval_DEPRECATED == 1 / 25.0f)  { DisplayRate = FCommonFrameRates::FPS_25();  }
		else if (FixedFrameInterval_DEPRECATED == 1 / 29.97)  { DisplayRate = FCommonFrameRates::NTSC_30(); }
		else if (FixedFrameInterval_DEPRECATED == 1 / 30.0f)  { DisplayRate = FCommonFrameRates::FPS_30();  }
		else if (FixedFrameInterval_DEPRECATED == 1 / 48.0f)  { DisplayRate = FCommonFrameRates::FPS_48();  }
		else if (FixedFrameInterval_DEPRECATED == 1 / 50.0f)  { DisplayRate = FCommonFrameRates::FPS_50();  }
		else if (FixedFrameInterval_DEPRECATED == 1 / 59.94)  { DisplayRate = FCommonFrameRates::NTSC_60(); }
		else if (FixedFrameInterval_DEPRECATED == 1 / 60.0f)  { DisplayRate = FCommonFrameRates::FPS_60();  }
		else if (FixedFrameInterval_DEPRECATED == 1 / 120.0f) { DisplayRate = FCommonFrameRates::FPS_120(); }
		else if (FixedFrameInterval_DEPRECATED != 0.f)
		{
			uint32 Numerator = FMath::RoundToInt(1000.f / FixedFrameInterval_DEPRECATED);
			DisplayRate = FFrameRate(Numerator, 1000);
		}
		else
		{
			// Sequences with 0 FixedFrameInterval used to be assigned a proper interval in SSequencer::OnSequenceInstanceActivated for some reason,
			// But we don't have access to the relevant sequencer settings class here so we just have to make a hacky educated guess based on the class name.
			UObject* Outer = GetOuter();
			if (Outer && Outer->GetClass()->GetFName() == "WidgetAnimation")
			{
				// Widget animations defaulted to 0.05s
				DisplayRate = FFrameRate(20, 1);
			}
			else if (Outer && Outer->GetClass()->GetFName() == "ActorSequence")
			{
				// Actor sequences defaulted to 0.1s
				DisplayRate = FFrameRate(10, 1);
			}
			else
			{
				// Level sequences defaulted to 30fps - this is the fallback default for anything else
				DisplayRate = FFrameRate(30, 1);
			}
		}
	}
#endif

	if (Ar.IsLoading())
	{
		const bool bSortBindings = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::MovieSceneSortedBindings;
		if (bSortBindings)
		{
			Possessables.Sort();
			Spawnables.Sort();
			ObjectBindings.Sort();
		}
	}

#if WITH_EDITORONLY_DATA
	if (Ar.IsSaving())
	{
		MuteNodes_DEPRECATED.Empty();
		SoloNodes_DEPRECATED.Empty();
	}
#endif
}

#if WITH_EDITOR
void UMovieScene::PostEditUndo()
{
	RemoveNullTracks();

	Super::PostEditUndo();
}
#endif

template <typename RangeType, typename ValueType>
auto InsertSorted(RangeType& Range, ValueType&& Value) -> decltype(GetNum(Range))
{
	auto InsertIndex = Algo::LowerBound(Range, Value);
	check(InsertIndex >= 0 && InsertIndex <= GetNum(Range));
	Range.Insert(Value, InsertIndex);
	return InsertIndex;
}

// @todo sequencer: Some of these methods should only be used by tools, and should probably move out of MovieScene!
FGuid UMovieScene::AddSpawnable( const FString& Name, UObject& ObjectTemplate )
{
	Modify();

	FMovieSceneSpawnable NewSpawnable( Name, ObjectTemplate );
	NewSpawnable.AutoSetNetAddressableName();

	// Insert new spawnable sorted into the Spawnables array
	int32 NewSpawnableIndex = InsertSorted(Spawnables, MoveTemp(NewSpawnable));

	check(Spawnables.IsValidIndex(NewSpawnableIndex));

	// Add a new binding (sorted) so that tracks can be added to it
	int32 NewBindingIndex = InsertSorted(ObjectBindings, FMovieSceneBinding(Spawnables[NewSpawnableIndex].GetGuid()));

	EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnBindingAdded, ObjectBindings[NewBindingIndex]);

	return Spawnables[NewSpawnableIndex].GetGuid();
}

void UMovieScene::AddSpawnable(const FMovieSceneSpawnable& InNewSpawnable, const FMovieSceneBinding& InNewBinding)
{
	Modify();

	FMovieSceneSpawnable NewSpawnable;
	NewSpawnable = InNewSpawnable;
	NewSpawnable.AutoSetNetAddressableName();

	// Insert new spawnable sorted into the Spawnables array
	InsertSorted(Spawnables, MoveTemp(NewSpawnable));

	FMovieSceneBinding NewBinding = InNewBinding;
	for (UMovieSceneTrack* Track : NewBinding.GetTracks())
	{
		if (!Track->IsIn(this))
		{
			FName NewName = MakeUniqueObjectName(this, Track->GetClass(), *Track->GetName());
			Track->Rename(*NewName.ToString(), this, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
		}
	}
	int32 NewBindingIndex = InsertSorted(ObjectBindings, MoveTemp(NewBinding));

	for (UMovieSceneTrack* Track : ObjectBindings[NewBindingIndex].GetTracks())
	{
		Track->OnAddedToMovieScene(this);
	}

	EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnBindingAdded, ObjectBindings[NewBindingIndex]);
}

bool UMovieScene::RemoveSpawnable( const FGuid& Guid )
{
	bool bAnythingRemoved = false;
	if( ensure( Guid.IsValid() ) )
	{
		int32 Index = IndexOfSpawnable(Guid);
		if (Index != INDEX_NONE)
		{
			Modify();
			RemoveBinding(Guid);
			Spawnables.RemoveAt(Index);

			EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnBindingRemoved, Guid);

			bAnythingRemoved = true;
		}
	}

	return bAnythingRemoved;
}

FMovieSceneSpawnable* UMovieScene::FindSpawnable( const TFunctionRef<bool(FMovieSceneSpawnable&)>& InPredicate )
{
	return Spawnables.FindByPredicate(InPredicate);
}

FMovieSceneSpawnable& UMovieScene::GetSpawnable(int32 Index)
{
	return Spawnables[Index];
}

int32 UMovieScene::GetSpawnableCount() const
{
	return Spawnables.Num();
}

FMovieSceneSpawnable* UMovieScene::FindSpawnable( const FGuid& Guid )
{
	int32 Index = IndexOfSpawnable(Guid);
	if (Index != INDEX_NONE)
	{
		return &Spawnables[Index];
	}
	return nullptr;
}


FGuid UMovieScene::AddPossessable( const FString& Name, UClass* Class )
{
	Modify();

	int32 NewPossessableIndex = InsertSorted(Possessables, FMovieScenePossessable(Name, Class));
	check(Possessables.IsValidIndex(NewPossessableIndex));
	// Add a new binding so that tracks can be added to it
	int32 NewBindingIndex = InsertSorted(ObjectBindings, FMovieSceneBinding(Possessables[NewPossessableIndex].GetGuid()));
	check(ObjectBindings.IsValidIndex(NewBindingIndex));
	EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnBindingAdded, ObjectBindings[NewBindingIndex]);

	return Possessables[NewPossessableIndex].GetGuid();
}


void UMovieScene::AddPossessable(const FMovieScenePossessable& InNewPossessable, const FMovieSceneBinding& InNewBinding)
{
	const FGuid PossessableID = InNewPossessable.GetGuid();
	const FGuid BindingID     = InNewBinding.GetObjectGuid();

	FGuid FinalID = PossessableID.IsValid() ? PossessableID : BindingID;

	Modify();

	FMovieScenePossessable NewPossessable = InNewPossessable;
	FMovieSceneBinding     NewBinding     = InNewBinding;

	if (!ensureMsgf(FinalID.IsValid() && PossessableID == BindingID, TEXT("GUIDs for possessable and binding objects are Invalid or do not match")))
	{
		if (!FinalID.IsValid())
		{
			FinalID = FGuid::NewGuid();
		}

		NewBinding.SetObjectGuid_Private(FinalID);
		NewPossessable.SetGuid(FinalID);
	}

	InsertSorted(Possessables, MoveTemp(NewPossessable));

	for (UMovieSceneTrack* Track : NewBinding.GetTracks())
	{
		if (!Track->IsIn(this))
		{
			FName NewName = MakeUniqueObjectName(this, Track->GetClass(), *Track->GetName());
			Track->Rename(*NewName.ToString(), this, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
		}
	}
	int32 NewBindingIndex = InsertSorted(ObjectBindings, MoveTemp(NewBinding));
	check(ObjectBindings.IsValidIndex(NewBindingIndex));

	for (UMovieSceneTrack* Track : ObjectBindings[NewBindingIndex].GetTracks())
	{
		Track->OnAddedToMovieScene(this);
	}

	EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnBindingAdded, ObjectBindings[NewBindingIndex]);
}


bool UMovieScene::RemovePossessable( const FGuid& PossessableGuid )
{
	bool bAnythingRemoved = false;

	int32 Index = IndexOfPossessable(PossessableGuid);
	if (Index != INDEX_NONE)
	{
		const FMovieScenePossessable& FoundPossessable = Possessables[Index];
		Modify();

		// Remove the parent-child link for a parent spawnable/child possessable if necessary
		if (FoundPossessable.GetParent().IsValid())
		{
			FMovieSceneSpawnable* ParentSpawnable = FindSpawnable(FoundPossessable.GetParent());
			if (ParentSpawnable)
			{
				ParentSpawnable->RemoveChildPossessable(PossessableGuid);
			}
		}

		// Found it!
		Possessables.RemoveAt(Index);

		RemoveBinding(PossessableGuid);

		EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnBindingRemoved, PossessableGuid);

		bAnythingRemoved = true;
	}

	return bAnythingRemoved;
}


bool UMovieScene::ReplacePossessable( const FGuid& OldGuid, const FMovieScenePossessable& InNewPosessable )
{
	bool bAnythingReplaced = false;

	if (FMovieScenePossessable* OldPossessable = FindPossessable(OldGuid))
	{
		Modify();

		bool bNullPossessedObjectClass = true;
#if WITH_EDITORONLY_DATA
		bNullPossessedObjectClass = InNewPosessable.GetPossessedObjectClass() == nullptr;
#endif

		// Found it!
		if (bNullPossessedObjectClass)
		{
			// @todo: delete this when
			// bool ReplacePossessable(const FGuid& OldGuid, const FGuid& NewGuid, const FString& Name)
			// is removed
			OldPossessable->SetGuid(InNewPosessable.GetGuid());
			OldPossessable->SetName(InNewPosessable.GetName());
		}
		else
		{
			*OldPossessable = InNewPosessable;
		}

		// Replace directly changes the guid, so force a sort here
		Possessables.Sort();

		ReplaceBinding( OldGuid, InNewPosessable.GetGuid(), InNewPosessable.GetName() );
		bAnythingReplaced = true;
	}

	return bAnythingReplaced;
}


FMovieScenePossessable* UMovieScene::FindPossessable( const FGuid& Guid )
{
	int32 Index = IndexOfPossessable(Guid);
	if (Index != INDEX_NONE)
	{
		return &Possessables[Index];
	}
	return nullptr;
}

FMovieScenePossessable* UMovieScene::FindPossessable( const TFunctionRef<bool(FMovieScenePossessable&)>& InPredicate )
{
	return Possessables.FindByPredicate(InPredicate);
}

int32 UMovieScene::GetPossessableCount() const
{
	return Possessables.Num();
}


FMovieScenePossessable& UMovieScene::GetPossessable( const int32 Index )
{
	return Possessables[Index];
}


FText UMovieScene::GetObjectDisplayName(const FGuid& ObjectId) const
{
#if WITH_EDITORONLY_DATA
	const FText* Result = ObjectsToDisplayNames.Find(ObjectId.ToString());

	if (Result && !Result->IsEmpty())
	{
		return *Result;
	}

	int32 SpawnableIndex = IndexOfSpawnable(ObjectId);
	if (SpawnableIndex != INDEX_NONE)
	{
		return FText::FromString(Spawnables[SpawnableIndex].GetName());
	}

	int32 PossessableIndex = IndexOfPossessable(ObjectId);
	if (PossessableIndex != INDEX_NONE)
	{
		return FText::FromString(Possessables[PossessableIndex].GetName());
	}

#endif
	return FText::GetEmpty();
}


void UMovieScene::SetTickResolutionDirectly(FFrameRate InTickResolution)
{
	FFrameRate PreviousTickResolution = TickResolution;
	TickResolution = InTickResolution;
	if (CustomClock)
	{
		CustomClock->HandleTickResolutionChange(PreviousTickResolution, TickResolution);
	}
}

void UMovieScene::AddNewBindingTag(const FName& NewTag)
{
	Modify();

	BindingGroups.Add(NewTag);
}

void UMovieScene::TagBinding(const FName& NewTag, const UE::MovieScene::FFixedObjectBindingID& BindingToTag)
{
	Modify();

	FMovieSceneObjectBindingID SerializedID = BindingToTag;
	BindingGroups.FindOrAdd(NewTag).IDs.AddUnique(SerializedID);
}

void UMovieScene::UntagBinding(const FName& Tag, const UE::MovieScene::FFixedObjectBindingID& Binding)
{
	FMovieSceneObjectBindingID PredicateID = Binding;
	if (FMovieSceneObjectBindingIDs* Array = BindingGroups.Find(Tag))
	{
		Modify();

		Array->IDs.Remove(PredicateID);
		if (Array->IDs.Num() == 0)
		{
			BindingGroups.Remove(Tag);
		}
	}
}

void UMovieScene::RemoveTag(const FName& TagToRemove)
{
	Modify();

	BindingGroups.Remove(TagToRemove);
}

#if WITH_EDITORONLY_DATA
void UMovieScene::SetObjectDisplayName(const FGuid& ObjectId, const FText& DisplayName)
{
	if (DisplayName.IsEmpty())
	{
		ObjectsToDisplayNames.Remove(ObjectId.ToString());
	}
	else
	{
		ObjectsToDisplayNames.Add(ObjectId.ToString(), DisplayName);
	}
}


TArrayView<UMovieSceneFolder* const> UMovieScene::GetRootFolders()
{
	return RootFolders;
}

void UMovieScene::GetRootFolders(TArray<UMovieSceneFolder*>& InRootFolders)
{
	InRootFolders.Append(RootFolders);
}

int32 UMovieScene::GetNumRootFolders() const
{
	return RootFolders.Num();
}

UMovieSceneFolder* UMovieScene::GetRootFolder(int32 FolderIndex) const
{
	return RootFolders[FolderIndex];
}

void UMovieScene::AddRootFolder(UMovieSceneFolder* Folder)
{
	if (!RootFolders.Contains(Folder))
	{
		RootFolders.Add(Folder);
		EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnRootFolderAdded, Folder);
	}
}

int32 UMovieScene::RemoveRootFolder(UMovieSceneFolder* Folder)
{
	const int32 NumRemoved = RootFolders.Remove(Folder);
	if (NumRemoved != 0)
	{
		EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnRootFolderRemoved, Folder);
	}
	return NumRemoved;
}

bool UMovieScene::RemoveRootFolder(int32 FolderIndex)
{
	if (RootFolders.IsValidIndex(FolderIndex))
	{
		UMovieSceneFolder* Folder = RootFolders[FolderIndex];
		RootFolders.RemoveAt(FolderIndex);
		EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnRootFolderRemoved, Folder);
		return true;
	}
	return false;
}

void UMovieScene::EmptyRootFolders()
{
	TArray<TObjectPtr<UMovieSceneFolder>> OldFolders;
	Swap(RootFolders, OldFolders);

	RootFolders.Empty();

	for (UMovieSceneFolder* Folder : OldFolders)
	{
		EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnRootFolderRemoved, Folder);
	}
}


#endif

void UMovieScene::SetPlaybackRange(FFrameNumber Start, int32 Duration, bool bAlwaysMarkDirty)
{
	// Inclusive lower, Exclusive upper bound
	SetPlaybackRange(TRange<FFrameNumber>(Start, Start + Duration), bAlwaysMarkDirty);
}

void UMovieScene::SetPlaybackRange(const TRange<FFrameNumber>& NewRange, bool bAlwaysMarkDirty)
{
	check(NewRange.GetLowerBound().IsClosed() && NewRange.GetUpperBound().IsClosed());

	if (PlaybackRange.Value == NewRange)
	{
		return;
	}

	if (bAlwaysMarkDirty)
	{
		Modify();
	}

	PlaybackRange.Value = NewRange;

#if WITH_EDITORONLY_DATA
	// Update the working and view ranges to encompass the new range
	const double RangeStartSeconds = NewRange.GetLowerBoundValue() / TickResolution;
	const double RangeEndSeconds   = NewRange.GetUpperBoundValue() / TickResolution;

	// Initialize the working and view range with a little bit more space
	const double OutputChange      = (RangeEndSeconds - RangeStartSeconds) * 0.1;

	double ExpandedStart = RangeStartSeconds - OutputChange;
	double ExpandedEnd   = RangeEndSeconds + OutputChange;

	if (EditorData.WorkStart >= EditorData.WorkEnd)
	{
		EditorData.WorkStart = ExpandedStart;
		EditorData.WorkEnd   = ExpandedEnd;
	}

	if (EditorData.ViewStart >= EditorData.ViewEnd)
	{
		EditorData.ViewStart = ExpandedStart;
		EditorData.ViewEnd   = ExpandedEnd;
	}
#endif
}

void UMovieScene::SetWorkingRange(double Start, double End)
{
#if WITH_EDITORONLY_DATA
	EditorData.WorkStart = Start;
	EditorData.WorkEnd   = End;
#endif
}

void UMovieScene::SetViewRange(double Start, double End)
{
#if WITH_EDITORONLY_DATA
	EditorData.ViewStart = Start;
	EditorData.ViewEnd   = End;
#endif
}

#if WITH_EDITORONLY_DATA
bool UMovieScene::IsPlaybackRangeLocked() const
{
	return bPlaybackRangeLocked;
}

void UMovieScene::SetPlaybackRangeLocked(bool bLocked)
{
	bPlaybackRangeLocked = bLocked;
}

bool UMovieScene::AreMarkedFramesLocked() const
{
	return bMarkedFramesLocked;
}

void UMovieScene::SetMarkedFramesLocked(bool bLocked)
{
	bMarkedFramesLocked = bLocked;
}
#endif


#if WITH_EDITORONLY_DATA

bool FMovieSceneSectionGroup::Contains(const UMovieSceneSection& Section) const
{
	return Sections.Contains(&Section);
}

void FMovieSceneSectionGroup::Add(UMovieSceneSection& Section)
{
	Sections.AddUnique(&Section);
}

void FMovieSceneSectionGroup::Append(const FMovieSceneSectionGroup& SectionGroup)
{
	if (SectionGroup == *this)
	{
		return;
	}

	// Append the groups using AddUnique to prevent duplicates
	Sections.Reserve(Sections.Num() + SectionGroup.Num());
	for (TWeakObjectPtr<UMovieSceneSection> Section : SectionGroup)
	{
		if (Section.IsValid())
		{
			Sections.AddUnique(Section);
		}
	}
}

void FMovieSceneSectionGroup::Remove(const UMovieSceneSection& Section)
{
	Sections.RemoveSingle(const_cast<UMovieSceneSection*>(&Section));
}

void FMovieSceneSectionGroup::Clean()
{
	Sections.RemoveAll([](TWeakObjectPtr<UMovieSceneSection>& Section) { return !(Section.IsValid()); });
}

void UMovieSceneNodeGroup::AddNode(const FString& Path)
{
	Modify();

	Nodes.AddUnique(Path);
	OnNodeGroupChangedEvent.Broadcast();
}

void UMovieSceneNodeGroup::RemoveNode(const FString& Path)
{
	Modify();

	Nodes.Remove(Path);
	OnNodeGroupChangedEvent.Broadcast();
}

bool UMovieSceneNodeGroup::ContainsNode(const FString& Path) const
{
	return Nodes.Contains(Path);
}

void UMovieSceneNodeGroup::UpdateNodePath(const FString& OldPath, const FString& NewPath)
{
	if (!OldPath.Equals(NewPath))
	{
		// If the node is in this group, replace it with it's new path
		if (Nodes.Remove(OldPath))
		{
			Nodes.Add(NewPath);
		}

		// Find any nodes with a path that is a child of the changed node, and rename their paths as well
		FString PathPrefix = OldPath + '.';

		TArray<FString> PathsToRename;
		for (const FString& NodePath : Nodes)
		{
			if (NodePath.StartsWith(PathPrefix))
			{
				PathsToRename.Add(NodePath);
			}
		}

		for (const FString& NodePath : PathsToRename)
		{
			FString NewNodePath = NodePath;
			if (NewNodePath.RemoveFromStart(PathPrefix))
			{
				NewNodePath = NewPath + '.' + NewNodePath;
				Nodes.Remove(NodePath);
				Nodes.Add(NewNodePath);
			}
		}

	}
}

void UMovieSceneNodeGroup::SetName(const FName& InName)
{
	Modify();

	Name = InName;
	OnNodeGroupChangedEvent.Broadcast();
}

void UMovieSceneNodeGroup::SetEnableFilter(bool bInEnableFilter)
{
	if (bEnableFilter != bInEnableFilter)
	{
		bEnableFilter = bInEnableFilter;
		OnNodeGroupChangedEvent.Broadcast();
	}
}

void UMovieSceneNodeGroupCollection::Refresh()
{
	bAnyActiveFilter = false;
	for (UMovieSceneNodeGroup* NodeGroup : NodeGroups)
	{
		NodeGroup->OnNodeGroupChanged().AddUObject(this, &UMovieSceneNodeGroupCollection::OnNodeGroupChanged);

		if (NodeGroup->GetEnableFilter())
		{
			bAnyActiveFilter = true;
		}
	}
}

void UMovieSceneNodeGroupCollection::PostLoad()
{
	Refresh();

	Super::PostLoad();
}

void UMovieSceneNodeGroupCollection::PostEditUndo()
{
	Refresh();

	Super::PostEditUndo();
}

void UMovieSceneNodeGroupCollection::AddNodeGroup(UMovieSceneNodeGroup* NodeGroup)
{
	Modify();

	if (!NodeGroups.Contains(NodeGroup))
	{
		NodeGroups.Add(NodeGroup);
		NodeGroup->OnNodeGroupChanged().AddUObject(this, &UMovieSceneNodeGroupCollection::OnNodeGroupChanged);
		
		OnNodeGroupChanged();
	}
}

void UMovieSceneNodeGroupCollection::RemoveNodeGroup(UMovieSceneNodeGroup* NodeGroup)
{
	Modify();

	NodeGroup->OnNodeGroupChanged().RemoveAll(this);

	if (NodeGroups.RemoveSingle(NodeGroup))
	{
		OnNodeGroupChanged();
	}
}

void UMovieSceneNodeGroupCollection::UpdateNodePath(const FString& OldPath, const FString& NewPath)
{
	for (UMovieSceneNodeGroup* NodeGroup : NodeGroups)
	{
		NodeGroup->UpdateNodePath(OldPath, NewPath);
	}
}

void UMovieSceneNodeGroupCollection::OnNodeGroupChanged()
{
	bAnyActiveFilter = false;
	for (const UMovieSceneNodeGroup* NodeGroup : NodeGroups)
	{
		if (NodeGroup->GetEnableFilter())
		{
			bAnyActiveFilter = true;
			break;
		}
	}

	OnNodeGroupCollectionChangedEvent.Broadcast();
}

bool UMovieScene::IsSectionInGroup(const UMovieSceneSection& InSection) const
{
	for (const FMovieSceneSectionGroup& Group : SectionGroups)
	{
		if (Group.Contains(InSection))
		{
			return true;
		}
	}
	return false;
}

void UMovieScene::GroupSections(const TArray<UMovieSceneSection*> InSections)
{
	if (InSections.Num() < 2)
	{
		return;
	}

	Modify();

	FMovieSceneSectionGroup* Group = nullptr;

	// Check if the first section is already in a group, and if so use it as the target group
	for (FMovieSceneSectionGroup& ExistingGroup : SectionGroups)
	{
		if (ExistingGroup.Contains(*InSections[0]))
		{
			Group = &ExistingGroup;
			break;
		}
	}

	// If we didn't find a group, create a new one.
	if (!Group)
	{
		int32 Index = SectionGroups.AddDefaulted(1);
		Group = &SectionGroups[Index];
		Group->Add(*InSections[0]);
	}

	// Add the remaining sections
	for (int32 Index = 1; Index < InSections.Num(); ++Index)
	{
		UMovieSceneSection* Section = InSections[Index];

		// Check if the section is already in a group, and merge if needed
		for (FMovieSceneSectionGroup& ExistingGroup : SectionGroups)
		{
			// Skip checking the working group. If the section is in it, there is nothing to do.
			if (&ExistingGroup == Group)
			{
				continue;
			}

			if (ExistingGroup.Contains(*Section))
			{
				// If the section is already in a group, merge our new group in to it, and work on that group instead
				// This has fewer edge cases than merging the found group in to the working group.
				ExistingGroup.Append(*Group);
				SectionGroups.RemoveSingle(*Group);
				Group = &ExistingGroup;
				break;
			}
		}

		// If the section was not added as part of a merged group, add it now.
		if (!Group->Contains(*Section))
		{
			Group->Add(*Section);
		}
	}
}

void UMovieScene::UngroupSection(const UMovieSceneSection& InSection)
{
	for (FMovieSceneSectionGroup& ExistingGroup : SectionGroups)
	{
		if (ExistingGroup.Contains(InSection))
		{
			Modify();
			ExistingGroup.Remove(InSection);
			break;
		}
	}
	CleanSectionGroups();
}

const FMovieSceneSectionGroup* UMovieScene::GetSectionGroup(const UMovieSceneSection& InSection) const
{
	for (const FMovieSceneSectionGroup& Group : SectionGroups)
	{
		if (Group.Contains(InSection))
		{
			return &Group;
		}
	}
	return nullptr;
}

void UMovieScene::CleanSectionGroups()
{
	for (FMovieSceneSectionGroup& SectionGroup : SectionGroups)
	{
		SectionGroup.Clean();
	}

	SectionGroups.RemoveAll([](FMovieSceneSectionGroup& SectionGroup) { return SectionGroup.Num() < 2; });
}
#endif


TArray<UMovieSceneSection*> UMovieScene::GetAllSections() const
{
	TArray<UMovieSceneSection*> OutSections;

	// Add all type sections 
	for( int32 TrackIndex = 0; TrackIndex < Tracks.Num(); ++TrackIndex )
	{
		OutSections.Append( Tracks[TrackIndex]->GetAllSections() );
	}
	
	// Add all camera cut sections
	if (CameraCutTrack != nullptr)
	{
		OutSections.Append(CameraCutTrack->GetAllSections());
	}

	// Add all object binding sections
	for (const auto& Binding : ObjectBindings)
	{
		for (const auto& Track : Binding.GetTracks())
		{
			OutSections.Append(Track->GetAllSections());
		}
	}

	return OutSections;
}


UMovieSceneTrack* UMovieScene::FindTrack(TSubclassOf<UMovieSceneTrack> TrackClass, const FGuid& ObjectGuid, const FName& TrackName) const
{
	check( ObjectGuid.IsValid() );

	if (const FMovieSceneBinding* Binding = FindBinding(ObjectGuid))
	{
		for (const auto& Track : Binding->GetTracks())
		{
			if (!IsValid(Track))
			{
				continue;
			}

			if (TrackClass.GetDefaultObject() == nullptr || Track->GetClass()->IsChildOf(TrackClass))
			{
				if (TrackName == NAME_None || Track->GetTrackName() == TrackName)
				{
					return Track;
				}
			}
		}
	}
	
	return nullptr;
}

TArray<UMovieSceneTrack*> UMovieScene::FindTracks(TSubclassOf<UMovieSceneTrack> TrackClass, const FGuid& ObjectGuid, const FName& TrackName) const
{
	check(ObjectGuid.IsValid());
	TArray<UMovieSceneTrack*> MovieSceneTracks;
	if (const FMovieSceneBinding* Binding = FindBinding(ObjectGuid))
	{
		for (const auto& Track : Binding->GetTracks())
		{
			if (!IsValid(Track))
			{
				continue;
			}

			if (TrackClass.GetDefaultObject() == nullptr || Track->GetClass()->IsChildOf(TrackClass))
			{
				if (TrackName == NAME_None || Track->GetTrackName() == TrackName)
				{
					MovieSceneTracks.Add(Track);
				}
			}
		}
	}

	return MovieSceneTracks;
}

UMovieSceneTrack* UMovieScene::AddTrack( TSubclassOf<UMovieSceneTrack> TrackClass, const FGuid& ObjectGuid )
{
#if WITH_EDITOR
	if (!IsTrackClassAllowed(TrackClass))
	{
		return nullptr;
	}
#endif

	UMovieSceneTrack* CreatedType = nullptr;

	check(ObjectGuid.IsValid());

	if (FMovieSceneBinding* Binding = FindBinding(ObjectGuid))
	{
		Modify();

		CreatedType = NewObject<UMovieSceneTrack>(this, TrackClass, NAME_None, RF_Transactional);
		check(CreatedType);
		Binding->AddTrack(*CreatedType, this);
	}

	return CreatedType;
}

bool UMovieScene::AddGivenTrack(UMovieSceneTrack* InTrack, const FGuid& ObjectGuid)
{
#if WITH_EDITOR
	if (!IsTrackClassAllowed(InTrack->GetClass()))
	{
		return false;
	}
#endif

	check(ObjectGuid.IsValid());
	check(InTrack);

	Modify();
	if (FMovieSceneBinding* Binding = FindBinding(ObjectGuid))
	{
		InTrack->Rename(nullptr, this, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
		Binding->AddTrack(*InTrack, this);
		return true;
	}

	return false;
}

bool UMovieScene::RemoveTrack(UMovieSceneTrack& Track)
{
	// Remove either a root track or a track from a binding
	if (Tracks.Contains(&Track))
	{
		Modify();

		Track.OnRemovedFromMovieScene();

		const bool bRemoved = Tracks.RemoveSingle(&Track) != 0;
		if (bRemoved)
		{
			EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnTrackRemoved, &Track);
		}
		return bRemoved;
	}
	else
	{
		Modify();

		bool bAnythingRemoved = false;
		
		for (auto& Binding : ObjectBindings)
		{
			if (Binding.RemoveTrack(Track, this))
			{
				bAnythingRemoved = true;

				// The track was removed from the current binding, stop
				// searching now as it cannot exist in any other binding
				break;
			}
		}
	
		return bAnythingRemoved;
	}
}

bool UMovieScene::FindTrackBinding(const UMovieSceneTrack& InTrack, FGuid& OutGuid) const
{
	for (auto& Binding : ObjectBindings)
	{
		for(auto& Track : Binding.GetTracks())
		{
			if(Track == &InTrack)
			{
				OutGuid = Binding.GetObjectGuid();
				return true;
			}
		}
	}

	return false;
}

UMovieSceneTrack* UMovieScene::FindTrack( TSubclassOf<UMovieSceneTrack> TrackClass ) const
{
	UMovieSceneTrack* FoundTrack = nullptr;

	for (const TObjectPtr<UMovieSceneTrack>& Track : Tracks)
	{
		if (!IsValid(Track))
		{
			continue;
		}

		if (Track->GetClass()->IsChildOf(TrackClass))
		{
			FoundTrack = Track;
			break;
		}
	}

	return FoundTrack;
}


UMovieSceneTrack* UMovieScene::AddTrack( TSubclassOf<UMovieSceneTrack> TrackClass )
{
#if WITH_EDITOR
	if (!IsTrackClassAllowed(TrackClass))
	{
		return nullptr;
	}
#endif

	Modify();

	UMovieSceneTrack* CreatedType = NewObject<UMovieSceneTrack>(this, TrackClass, NAME_None, RF_Transactional);
	Tracks.Add( CreatedType );

	CreatedType->OnAddedToMovieScene(this);
	EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnTrackAdded, CreatedType);

	return CreatedType;
}


bool UMovieScene::AddGivenTrack(UMovieSceneTrack* InTrack)
{
#if WITH_EDITOR
	if (!IsTrackClassAllowed(InTrack->GetClass()))
	{
		return false;
	}
#endif

	if (!Tracks.Contains(InTrack))
	{
		Modify();
		Tracks.Add(InTrack);
		InTrack->Rename(nullptr, this, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);

		InTrack->OnAddedToMovieScene(this);
		EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnTrackAdded, InTrack);

		return true;
	}
	return false;
}


bool UMovieScene::ContainsTrack(const UMovieSceneTrack& InTrack) const
{
	for ( const UMovieSceneTrack* Track : Tracks)
	{
		if (&InTrack == Track)
		{
			return true;
		}
	}

	return false;
}


UMovieSceneTrack* UMovieScene::AddCameraCutTrack( TSubclassOf<UMovieSceneTrack> TrackClass )
{
#if WITH_EDITOR
	if (!IsTrackClassAllowed(TrackClass))
	{
		return nullptr;
	}
#endif

	if( !CameraCutTrack )
	{
		Modify();
		CameraCutTrack = NewObject<UMovieSceneTrack>(this, TrackClass, NAME_None, RF_Transactional);

		CameraCutTrack->OnAddedToMovieScene(this);
		EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnTrackAdded, CameraCutTrack);
	}

	return CameraCutTrack;
}


UMovieSceneTrack* UMovieScene::GetCameraCutTrack() const
{
	return CameraCutTrack;
}


void UMovieScene::RemoveCameraCutTrack()
{
	if( CameraCutTrack )
	{
		Modify();
		UMovieSceneTrack* TmpCameraCut = CameraCutTrack;
		CameraCutTrack = nullptr;

		EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnTrackRemoved, TmpCameraCut);
	}
}

void UMovieScene::SetCameraCutTrack(UMovieSceneTrack* InTrack)
{
	if (!InTrack)
	{
		return;
	}

	Modify();
	InTrack->Rename(nullptr, this, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
	UMovieSceneTrack* OldCameraCutTrack = CameraCutTrack;
	CameraCutTrack = InTrack;

	if (OldCameraCutTrack)
	{
		EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnTrackRemoved, OldCameraCutTrack);
	}

	CameraCutTrack->OnAddedToMovieScene(this);
	EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnTrackAdded, CameraCutTrack);
}


void UMovieScene::UpgradeTimeRanges()
{
	// Legacy upgrade for playback ranges:
	// We used to optionally store a start/end and in/out time for sequences.
	// The only 2 uses were UWidgetAnimations and ULevelSequences.
	// Widget animations used to always calculate their length automatically, from the section boundaries, and always started at 0
	// Level sequences defaulted to having a fixed play range.
	// We now expose the playback range more visibly, but we need to upgrade the old data.

	bool bFiniteRangeDefined = false;

#if WITH_EDITORONLY_DATA
	if (InTime_DEPRECATED != FLT_MAX && OutTime_DEPRECATED != -FLT_MAX)
	{
		// Finite range already defined in old data
		PlaybackRange.Value = TRange<FFrameNumber>(
			TickResolution.AsFrameNumber(InTime_DEPRECATED),
			// Prefer exclusive upper bounds for playback ranges so we stop at the next frame
			++TickResolution.AsFrameNumber(OutTime_DEPRECATED)
			);
		bFiniteRangeDefined = true;
	}
#endif
	if (!bFiniteRangeDefined && PlaybackRange.Value.IsEmpty())
	{
		// No range specified, so automatically calculate one by determining the maximum upper bound of the sequence
		// In this instance (UMG), playback always started at 0
		TRangeBound<FFrameNumber> MaxFrame = TRangeBound<FFrameNumber>::Exclusive(0);

		for (const UMovieSceneTrack* Track : Tracks)
		{
			TOptional<TRangeBound<FFrameNumber>> MaxUpper = GetMaxUpperBound(Track);
			if (MaxUpper.IsSet())
			{
				MaxFrame = TRangeBound<FFrameNumber>::MaxUpper(MaxFrame, MaxUpper.GetValue());
			}
		}

		for (const FMovieSceneBinding& Binding : ObjectBindings)
		{
			for (UMovieSceneTrack* Track : Binding.GetTracks())
			{
				TOptional<TRangeBound<FFrameNumber>> MaxUpper = GetMaxUpperBound(Track);
				if (MaxUpper.IsSet())
				{
					MaxFrame = TRangeBound<FFrameNumber>::MaxUpper(MaxFrame, MaxUpper.GetValue());
				}
			}
		}

		// Playback ranges should always have exclusive upper bounds
		if (MaxFrame.IsInclusive())
		{
			MaxFrame = TRangeBound<FFrameNumber>::Exclusive(MaxFrame.GetValue() + 1);
		}

		PlaybackRange.Value = TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(0), MaxFrame);
	}
	else if (PlaybackRange.Value.GetUpperBound().IsInclusive())
	{
		// playback ranges are now always inclusive
		PlaybackRange.Value = TRange<FFrameNumber>(PlaybackRange.Value.GetLowerBound(), TRangeBound<FFrameNumber>::Exclusive(PlaybackRange.Value.GetUpperBoundValue() + 1));
	}

	// PlaybackRange must always be defined to a finite range
	if (!PlaybackRange.Value.HasLowerBound() || !PlaybackRange.Value.HasUpperBound() || PlaybackRange.Value.IsDegenerate())
	{
		PlaybackRange.Value = TRange<FFrameNumber>(FFrameNumber(0), FFrameNumber(0));
	}

#if WITH_EDITORONLY_DATA
	if (GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::FloatToIntConversion)
	{
		EditorData.ViewStart    = EditorData.ViewRange_DEPRECATED.GetLowerBoundValue();
		EditorData.ViewEnd      = EditorData.ViewRange_DEPRECATED.GetUpperBoundValue();

		EditorData.WorkStart    = EditorData.WorkingRange_DEPRECATED.GetLowerBoundValue();
		EditorData.WorkEnd      = EditorData.WorkingRange_DEPRECATED.GetUpperBoundValue();
	}

	// Legacy upgrade for working range
	if (StartTime_DEPRECATED != FLT_MAX && EndTime_DEPRECATED != -FLT_MAX)
	{
		EditorData.WorkStart = StartTime_DEPRECATED;
		EditorData.WorkEnd   = EndTime_DEPRECATED;
	}
	else if (EditorData.WorkStart >= EditorData.WorkEnd)
	{
		EditorData.WorkStart = PlaybackRange.Value.GetLowerBoundValue() / TickResolution;
		EditorData.WorkEnd   = PlaybackRange.Value.GetUpperBoundValue() / TickResolution;
	}

	if (EditorData.ViewStart >= EditorData.ViewEnd)
	{
		// Update the working and view ranges to encompass the new range
		const double RangeStartSeconds = PlaybackRange.Value.GetLowerBoundValue() / TickResolution;
		const double RangeEndSeconds = PlaybackRange.Value.GetUpperBoundValue() / TickResolution;

		// Initialize the working and view range with a little bit more space
		const double OutputChange = (RangeEndSeconds - RangeStartSeconds) * 0.1;

		double ExpandedStart = RangeStartSeconds - OutputChange;
		double ExpandedEnd = RangeEndSeconds + OutputChange;

		EditorData.ViewStart = ExpandedStart;
		EditorData.ViewEnd   = ExpandedEnd;
	}

	if (SelectionRange.Value.GetLowerBound().IsOpen() || SelectionRange.Value.GetUpperBound().IsOpen())
	{
		SelectionRange.Value = TRange<FFrameNumber>::Empty();
	}
#endif
}

bool UMovieScene::RemoveNullTracks()
{
	// Remove any null tracks
	bool bNullTracksFound = false;
	for( int32 TrackIndex = 0; TrackIndex < Tracks.Num(); )
	{
		if (Tracks[TrackIndex] == nullptr)
		{
			bNullTracksFound = true;
			Tracks.RemoveAt(TrackIndex);
		}
		else
		{
			++TrackIndex;
		}
	}

	for (FMovieSceneBinding& Binding : ObjectBindings)
	{
		bNullTracksFound |= Binding.RemoveNullTracks();
		bNullTracksFound |= Binding.RemoveNulls();
	}

#if WITH_EDITORONLY_DATA
	for (int32 RootFolderIndex = 0; RootFolderIndex < RootFolders.Num();)
	{
		if (RootFolders[RootFolderIndex] == nullptr)
		{
			bNullTracksFound |= true;
			RootFolders.RemoveAt(RootFolderIndex);
		}
		else 
		{
			++RootFolderIndex;
		}
	}
#endif

#if WITH_EDITORONLY_DATA
	for (FFrameNumber MarkedFrame : EditorData.MarkedFrames_DEPRECATED)
	{
		MarkedFrames.Add(FMovieSceneMarkedFrame(MarkedFrame));
	}
	EditorData.MarkedFrames_DEPRECATED.Empty();


	// Clean any section groups which might refer to sections which were not serialized
	CleanSectionGroups();
#endif

	return bNullTracksFound;
}

/* UObject interface
 *****************************************************************************/


void UMovieScene::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

#if WITH_EDITORONLY_DATA
	// compress meta data mappings prior to saving
	for (auto It = ObjectsToDisplayNames.CreateIterator(); It; ++It)
	{
		FGuid ObjectId;

		if (!FGuid::Parse(It.Key(), ObjectId) || ((FindPossessable(ObjectId) == nullptr) && (FindSpawnable(ObjectId) == nullptr)))
		{
			It.RemoveCurrent();
		}
	}

	for (auto It = ObjectsToLabels.CreateIterator(); It; ++It)
	{
		FGuid ObjectId;

		if (!FGuid::Parse(It.Key(), ObjectId) || ((FindPossessable(ObjectId) == nullptr) && (FindSpawnable(ObjectId) == nullptr)))
		{
			It.RemoveCurrent();
		}
	}

	// Clean any section groups which contain stale references
	CleanSectionGroups();
#endif
}

void UMovieScene::OnDecorationAdded(UObject* NewDecoration)
{
	if (IMovieSceneDecoration* DecorationInterface = Cast<IMovieSceneDecoration>(NewDecoration))
	{
		DecorationInterface->OnDecorationAdded(this);
	}

	EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnDecorationAdded, NewDecoration);
}

void UMovieScene::OnDecorationRemoved(UObject* Decoration)
{
	if (IMovieSceneDecoration* DecorationInterface = Cast<IMovieSceneDecoration>(Decoration))
	{
		DecorationInterface->OnDecorationRemoved();
	}
	EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnDecorationRemoved, Decoration);
}

/* UMovieScene implementation
 *****************************************************************************/

void UMovieScene::RemoveBinding(const FGuid& Guid)
{
	// WARNING: This function intentionally does not trigger events to ensure
	// that events are triggered when all processing is complete (ie, when removing a spawnable or posessable)

	int32 Index = IndexOfBinding(Guid);
	if (Index != INDEX_NONE)
	{
		// Copy because OnRemovedFromMovieScene may cascade and mutate this binding's tracks.
		TArray<UMovieSceneTrack*> TracksToRemove(ObjectBindings[Index].GetTracks());
		for (UMovieSceneTrack* Track : TracksToRemove)
		{
			if (Track && ObjectBindings[Index].GetTracks().Contains(Track))
			{
				Track->OnRemovedFromMovieScene();
			}
		}
		ObjectBindings.RemoveAt(Index);
	}
}

int32 UMovieScene::IndexOfBinding(const FGuid& ForGuid) const
{
	return Algo::BinarySearch(ObjectBindings, ForGuid);
}

int32 UMovieScene::IndexOfSpawnable(const FGuid& ForGuid) const
{
	return Algo::BinarySearch(Spawnables, ForGuid);
}

int32 UMovieScene::IndexOfPossessable(const FGuid& ForGuid) const
{
	return Algo::BinarySearch(Possessables, ForGuid);
}

void UMovieScene::ReplaceBinding(const FGuid& OldGuid, const FGuid& NewGuid, const FString& Name)
{
	if (FMovieSceneBinding* Binding = FindBinding(OldGuid))
	{
		Binding->SetObjectGuid_Private(NewGuid);

		// Replace directly changes the guid, so force a sort here
		ObjectBindings.Sort();

		// Reget the binding after sorting
		Binding = FindBinding(NewGuid);

		// Changing a binding guid invalidates any tracks contained within the binding
		// Make sure they are written into the transaction buffer by calling modify
		for (UMovieSceneTrack* Track : Binding->GetTracks())
		{
			Track->Modify();
		}

		EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnBindingRemoved, OldGuid);
		EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnBindingAdded, *Binding);
	}
}

void UMovieScene::ReplaceBinding(const FGuid& BindingToReplaceGuid, const FMovieSceneBinding& NewBinding)
{
	Modify();

	if (FMovieSceneBinding* Binding = FindBinding(BindingToReplaceGuid))
	{
		*Binding = NewBinding;
		// We also need to change the track's owners to be the MovieScene.
		for (UMovieSceneTrack* Track : Binding->GetTracks())
		{
			if (!Track->IsIn(this))
			{
				FName NewName = MakeUniqueObjectName(this, Track->GetClass(), *Track->GetName());
				Track->Rename(*NewName.ToString(), this, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
			}
		}

		EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnBindingRemoved, BindingToReplaceGuid);
		EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnBindingAdded, *Binding);
	}
}

void UMovieScene::MoveBindingContents(const FGuid& SourceBindingId, const FGuid& DestinationBindingId)
{
	FMovieSceneBinding* SourceBinding = FindBinding(SourceBindingId);
	FMovieSceneBinding* DestinationBinding = FindBinding(DestinationBindingId);

	if (SourceBinding && DestinationBinding)
	{
		// Swap the tracks round
		DestinationBinding->SetTracks(SourceBinding->StealTracks(this), this);

		// Changing a binding guid invalidates any tracks contained within the binding
		// Make sure they are written into the transaction buffer by calling modify
		for (UMovieSceneTrack* Track : DestinationBinding->GetTracks())
		{
			Track->Modify();
		}
	}

	FMovieSceneSpawnable* DestinationSpawnable = FindSpawnable(DestinationBindingId);

	TArray<FGuid> BindingsToRemove;
	for( auto SourcePossessableIter( Possessables.CreateIterator() ); SourcePossessableIter; ++SourcePossessableIter)
	{
		FMovieScenePossessable& SourcePossessable = *SourcePossessableIter;
		
		// If there is a possessable whose parent is the binding we're moving contents for, 
		// that possessable needs to be remapped to the new destination parent
		if (SourcePossessable.GetParent() == SourceBindingId)
		{
			// Look for an existing destination child possessable
			bool bUseSourcePossessable = true;
			for (auto DestinationPossessableIter(Possessables.CreateIterator()); DestinationPossessableIter; ++DestinationPossessableIter)
			{
				FMovieScenePossessable& DestinationPossessable = *DestinationPossessableIter;

				if (DestinationPossessable.GetName() == SourcePossessable.GetName() &&
					DestinationPossessable.GetParent() == DestinationBindingId)
				{
					// If it's not the same class, we can't use the source possessable because it's going to lead to trouble
#if WITH_EDITORONLY_DATA
					if (SourcePossessable.GetPossessedObjectClass() != DestinationPossessable.GetPossessedObjectClass())
					{
						bUseSourcePossessable = false;
					}
#endif
					if (bUseSourcePossessable)
					{
						BindingsToRemove.Add(DestinationPossessable.GetGuid());
					}
					// Otherwise, discard the source possessable since it's a different class and probably going to be trouble
					else
					{
						BindingsToRemove.Add(SourcePossessable.GetGuid());
					}
					break;
				}
			}
			
			if (bUseSourcePossessable)
			{
				SourcePossessable.SetParent(DestinationBindingId, this);
				if (DestinationSpawnable)
				{
					DestinationSpawnable->AddChildPossessable(SourcePossessable.GetGuid());
				}
			}
		}
	}

	for (FGuid BindingToRemove : BindingsToRemove)
	{
		for (auto PossessableIter(Possessables.CreateIterator()); PossessableIter; ++PossessableIter)
		{
			FMovieScenePossessable& Possessable = *PossessableIter;
			if (Possessable.GetGuid() == BindingToRemove)
			{
				Possessables.RemoveAt(PossessableIter.GetIndex());
				break;
			}
		}

		RemoveBinding(BindingToRemove);
		EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnBindingRemoved, BindingToRemove);
	}
}

FMovieSceneBinding* UMovieScene::FindBinding(const FGuid& ForGuid)
{
	int32 Index = IndexOfBinding(ForGuid);
	if (Index != INDEX_NONE)
	{
		return &ObjectBindings[Index];
	}
	return nullptr;
}

const FMovieSceneBinding* UMovieScene::FindBinding(const FGuid& ForGuid) const
{
	int32 Index = IndexOfBinding(ForGuid);
	if (Index != INDEX_NONE)
	{
		return &ObjectBindings[Index];
	}
	return nullptr;
}

void UMovieScene::SetClockSource(EUpdateClockSource InNewClockSource)
{
	if (ClockSource != InNewClockSource)
	{
		ClockSource = InNewClockSource;
		EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnClockChanged, InNewClockSource, CustomClock);
	}
}

void UMovieScene::SetClockSource(UObject* InNewClockSource)
{
	UMovieSceneExternalClock* Clock = NewObject<UMovieSceneExternalClock>(this);
	Clock->CustomClockSourcePath = InNewClockSource;

	ClockSource = EUpdateClockSource::Custom;
	CustomClock = Clock;
}

void UMovieScene::SetCustomClock(UMovieSceneClock* InNewClockSource)
{
	if (ClockSource == EUpdateClockSource::Custom && CustomClock == InNewClockSource)
	{
		return;
	}

	ClockSource = EUpdateClockSource::Custom;
	if (!InNewClockSource->IsIn(this))
	{
		InNewClockSource = DuplicateObject<UMovieSceneClock>(InNewClockSource, this);
	}
	CustomClock = InNewClockSource;
	EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnClockChanged, EUpdateClockSource::Custom, CustomClock);
}

TSharedPtr<FMovieSceneTimeController> UMovieScene::MakeCustomTimeController(UObject* PlaybackContext)
{
	if (CustomClock != nullptr && ClockSource == EUpdateClockSource::Custom)
	{
		return CustomClock->MakeTimeController(PlaybackContext);
	}
	return nullptr;
}

FMovieSceneTimecodeSource UMovieScene::GetEarliestTimecodeSource() const
{
	FMovieSceneTimecodeSource EarliestTimecodeSource;

#if WITH_EDITORONLY_DATA

	const TArray<UMovieSceneSection*> MovieSceneSections = GetAllSections();

	// Find the first non-default timecode source.
	const FMovieSceneTimecodeSource DefaultTimecodeSource;
	int32 Index = 0;
	while (Index < MovieSceneSections.Num() && EarliestTimecodeSource == DefaultTimecodeSource)
	{
		if (MovieSceneSections[Index])
		{
			EarliestTimecodeSource = MovieSceneSections[Index]->TimecodeSource;
		}

		++Index;
	}

	// Continue searching through the sections where we left off looking for any earlier timecodes.
	// Any subsequently found default timecode source could be considered earlier.
	for (; Index < MovieSceneSections.Num(); ++Index)
	{
		if (!MovieSceneSections[Index])
		{
			continue;
		}

		const FMovieSceneTimecodeSource SectionTimecodeSource = MovieSceneSections[Index]->TimecodeSource;
		const FFrameRate ComparisonFrameRate;
		if (SectionTimecodeSource.Timecode.ToFrameNumber(ComparisonFrameRate) < EarliestTimecodeSource.Timecode.ToFrameNumber(ComparisonFrameRate))
		{
			EarliestTimecodeSource = SectionTimecodeSource;
		}
	}

#endif

	return EarliestTimecodeSource;
}

void UMovieScene::SetMarkedFrame(int32 InMarkIndex, FFrameNumber InFrameNumber)
{
	if (InMarkIndex < 0 && InMarkIndex > MarkedFrames.Num()-1)
	{
		return;
	}

	MarkedFrames[InMarkIndex].FrameNumber = InFrameNumber;
}

int32 UMovieScene::AddMarkedFrame(const FMovieSceneMarkedFrame &InMarkedFrame)
{
	FString NewLabel;

	if (InMarkedFrame.Label.IsEmpty())
	{
		FText Characters = LOCTEXT("MarkedFrameCharacters", "ABCDEFGHIJKLMNOPQRSTUVWXYZ");

		int32 NumCharacters = 1;
		bool bFound = false;
		while (!bFound)
		{
			int32 CharacterIndex = 0;
			for (; CharacterIndex < Characters.ToString().Len() && !bFound; ++CharacterIndex)
			{
				NewLabel = FString::ChrN(NumCharacters, Characters.ToString()[CharacterIndex]);
				bool bExists = false;
				for (const FMovieSceneMarkedFrame& MarkedFrame : MarkedFrames)
				{
					if (MarkedFrame.Label == NewLabel)
					{
						bExists = true;
						break;
					}
				}

				if (!bExists)
				{
					bFound = true;
				}
			}
			++NumCharacters;
		}
	}

	int32 MarkedIndex = MarkedFrames.Add(InMarkedFrame);

	if (!NewLabel.IsEmpty())
	{
		MarkedFrames[MarkedIndex].Label = NewLabel;
	}

	SortMarkedFrames();
	return FindMarkedFrameByFrameNumber(InMarkedFrame.FrameNumber);
}

void UMovieScene::DeleteMarkedFrame(int32 DeleteIndex)
{
	MarkedFrames.RemoveAt(DeleteIndex);
	SortMarkedFrames();
}

void UMovieScene::DeleteMarkedFrames()
{
	MarkedFrames.Empty();
}

void UMovieScene::SortMarkedFrames()
{
	MarkedFrames.Sort([&](const FMovieSceneMarkedFrame& A, const FMovieSceneMarkedFrame& B) { return A.FrameNumber < B.FrameNumber; });
}

int32 UMovieScene::FindMarkedFrameByLabel(const FString& InLabel) const
{
	for (int32 Index = 0; Index < MarkedFrames.Num(); ++Index)
	{
		if (MarkedFrames[Index].Label == InLabel)
		{
			return Index;
		}
	}
	return INDEX_NONE;
}

int32 UMovieScene::FindMarkedFrameByFrameNumber(FFrameNumber InFrameNumber) const
{
	for (int32 Index = 0; Index < MarkedFrames.Num(); ++Index)
	{
		if (MarkedFrames[Index].FrameNumber == InFrameNumber)
		{
			return Index;
		}
	}
	return INDEX_NONE;
}

int32 UMovieScene::FindNextMarkedFrame(FFrameNumber InFrameNumber, bool bForwards)
{
	if (MarkedFrames.Num() == 0)
	{
		return INDEX_NONE;
	}

	MarkedFrames.Sort([&](const FMovieSceneMarkedFrame& A, const FMovieSceneMarkedFrame& B) { return A.FrameNumber < B.FrameNumber; });

	if (bForwards)
	{
		for (int32 Index = MarkedFrames.Num() - 2; Index >= 0; --Index)
		{
			if (InFrameNumber >= MarkedFrames[Index].FrameNumber)
			{
				return Index + 1;
			}
		}
		return 0;
	}
	else
	{
		for (int32 Index = 1; Index < MarkedFrames.Num(); ++Index)
		{
			if (InFrameNumber <= MarkedFrames[Index].FrameNumber)
			{
				return Index - 1;
			}
		}
		return MarkedFrames.Num() - 1;
	}
}

#undef LOCTEXT_NAMESPACE

