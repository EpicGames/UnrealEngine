// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneBinding.h"
#include "MovieSceneTrack.h"

#include "MovieScene.h"
#include "EventHandlers/ISequenceDataEventHandler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneBinding)

/* FMovieSceneBinding interface
 *****************************************************************************/

void FMovieSceneBinding::AddTrack(UMovieSceneTrack& NewTrack, UMovieScene* Owner)
{
#if WITH_EDITOR
	if (!UMovieScene::IsTrackClassAllowed(NewTrack.GetClass()))
	{
		return;
	}
#endif

	Tracks.Add(&NewTrack);

	Owner->EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnTrackAddedToBinding, &NewTrack, ObjectGuid);
}

bool FMovieSceneBinding::RemoveTrack(UMovieSceneTrack& Track, UMovieScene* Owner)
{
	if (Tracks.RemoveSingle(&Track) != 0)
	{
		Track.OnRemovedFromMovieScene();
		Owner->EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnTrackRemovedFromBinding, &Track, ObjectGuid);
		return true;
	}
	return false;
}

bool FMovieSceneBinding::RemoveNullTracks()
{
	bool bNullTracksFound = false;
	for (int32 TrackIndex = Tracks.Num()-1; TrackIndex >= 0; --TrackIndex)
	{
		if (Tracks[TrackIndex] == nullptr)
		{
			bNullTracksFound = true;
			Tracks.RemoveAt(TrackIndex);
			// Don't trigger events for null tracks
		}
	}
	return bNullTracksFound;
}

TArray<UMovieSceneTrack*> FMovieSceneBinding::StealTracks(UMovieScene* Owner)
{
	if (Owner)
	{
		for (UMovieSceneTrack* Track : Tracks)
		{
			Track->OnRemovedFromMovieScene();
			Owner->EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnTrackRemovedFromBinding, Track, ObjectGuid);
		}
	}

	decltype(Tracks) Empty;
	Swap(Empty, Tracks);
	return Empty;
}

void FMovieSceneBinding::SetTracks(TArray<UMovieSceneTrack*>&& InTracks, UMovieScene* Owner)
{
	// Care is taken here to ensure that we trigger the events correctly when
	// InTracks contains tracks that already exist in this binding
	TSet<UMovieSceneTrack*> NewTracks;
	for (UMovieSceneTrack* Track : InTracks)
	{
		NewTracks.Add(Track);
	}

	for (int32 Index = Tracks.Num()-1; Index >= 0; --Index)
	{
		UMovieSceneTrack* Track = Tracks[Index];
		if (!NewTracks.Contains(Track))
		{
			Track->OnRemovedFromMovieScene();
			Tracks.RemoveAt(Index, EAllowShrinking::No);
			if (Owner)
			{
				Owner->EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnTrackRemovedFromBinding, Track, ObjectGuid);
			}
		}
		else
		{
			NewTracks.Remove(Track);
		}
	}

	for (UMovieSceneTrack* Track : NewTracks)
	{
		Tracks.Add(Track);
		if (Owner)
		{
			Track->OnAddedToMovieScene(Owner);
			Owner->EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnTrackAddedToBinding, Track, ObjectGuid);
		}
	}

	Tracks.Shrink();
}
