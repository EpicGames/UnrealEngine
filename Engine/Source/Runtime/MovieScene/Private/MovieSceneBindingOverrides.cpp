// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneBindingOverrides.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneBindingOverrides)

UMovieSceneBindingOverrides::UMovieSceneBindingOverrides(const FObjectInitializer& Init)
	: Super(Init)
	, bLookupDirty(true)
{
}

bool UMovieSceneBindingOverrides::LocateBoundObjects(const FGuid& InBindingId, FMovieSceneSequenceID InSequenceID, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const
{
	if (bLookupDirty)
	{
		RebuildLookupMap();
	}

	bool bAllowDefault = true;

	for (auto It = LookupMap.CreateConstKeyIterator(InBindingId);It;++It)
	{
		const FMovieSceneBindingOverrideData& Data = BindingData[It.Value()];

		// We have fast lookup only on GUID, so be sure to check the sequence ID before allowing overrides
		if (Data.ObjectBindingId.ReinterpretAsFixed().SequenceID != InSequenceID)
		{
			continue;
		}

		UObject* Object = Data.Object.Get();

		if (Data.bOverridesDefault)
		{
			bAllowDefault = false;
		}

		if (Object)
		{
			OutObjects.Add(Object);
		}
	}

	return bAllowDefault;
}

void UMovieSceneBindingOverrides::SetBinding(FMovieSceneObjectBindingID Binding, const TArray<UObject*>& Objects, bool bAllowBindingsFromAsset)
{
	ResetBinding(Binding);

	for (UObject* Object : Objects)
	{
		if (!Object)
		{
			continue;
		}

		LookupMap.Add(Binding.GetGuid(), BindingData.Num());

		FMovieSceneBindingOverrideData NewBinding;
		NewBinding.ObjectBindingId = Binding;
		NewBinding.Object = Object;
		NewBinding.bOverridesDefault = !bAllowBindingsFromAsset;
		BindingData.Add(NewBinding);
	}
}

void UMovieSceneBindingOverrides::AddBinding(FMovieSceneObjectBindingID Binding, UObject* Object, bool bAllowBindingsFromAsset)
{
	if (Object)
	{
		LookupMap.Add(Binding.GetGuid(), BindingData.Num());

		FMovieSceneBindingOverrideData NewBinding;
		NewBinding.ObjectBindingId = Binding;
		NewBinding.Object = Object;
		NewBinding.bOverridesDefault = !bAllowBindingsFromAsset;
		BindingData.Add(NewBinding);
	}
}

void UMovieSceneBindingOverrides::RemoveBinding(FMovieSceneObjectBindingID Binding, UObject* Object)
{
	int32 NumRemoved = BindingData.RemoveAll([=](const FMovieSceneBindingOverrideData& InBindingData){
		return InBindingData.Object == Object && Binding == InBindingData.ObjectBindingId;
	});

	if (NumRemoved)
	{
		bLookupDirty = true;
	}
}

void UMovieSceneBindingOverrides::ResetBinding(FMovieSceneObjectBindingID Binding)
{
	int32 NumRemoved = BindingData.RemoveAll([=](const FMovieSceneBindingOverrideData& InBindingData){
		return Binding == InBindingData.ObjectBindingId;
	});

	if (NumRemoved)
	{
		bLookupDirty = true;
	}
}

void UMovieSceneBindingOverrides::ResetBindings()
{
	if (BindingData.Num())
	{
		BindingData.Reset();
		LookupMap.Reset();
		bLookupDirty = false;
	}
}

void UMovieSceneBindingOverrides::RebuildLookupMap() const
{
	LookupMap.Reset();

	for (int32 Index = 0; Index < BindingData.Num(); ++Index)
	{
		LookupMap.Add(BindingData[Index].ObjectBindingId.GetGuid(), Index);
	}

	bLookupDirty = false;
}

#if WITH_EDITOR

void UMovieSceneBindingOverrides::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	RebuildLookupMap();
}

#endif
