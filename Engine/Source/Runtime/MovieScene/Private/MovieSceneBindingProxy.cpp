// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneBindingProxy.h"
#include "MovieSceneSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneBindingProxy)

UMovieScene* FMovieSceneBindingProxy::GetMovieScene() const
{
	return Sequence ? Sequence->GetMovieScene() : nullptr;
}
