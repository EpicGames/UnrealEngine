// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneBindingReferences.h"
#include "IMovieSceneBoundObjectProxy.h"
#include "UniversalObjectLocatorResolveParams.h"
#include "Engine/World.h"
#include "Evaluation/MovieSceneEvaluationState.h"
#include "UObject/Package.h"
#include "UnrealEngine.h"
#include "Bindings/MovieSceneCustomBinding.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneBindingReferences)

namespace UE::MovieScene
{

UObject* FindBoundObjectProxy(UObject* BoundObject)
{
	if (!BoundObject)
	{
		return nullptr;
	}

	IMovieSceneBoundObjectProxy* RawInterface = Cast<IMovieSceneBoundObjectProxy>(BoundObject);
	if (RawInterface)
	{
		return RawInterface->NativeGetBoundObjectForSequencer(BoundObject);
	}
	else if (BoundObject->GetClass()->ImplementsInterface(UMovieSceneBoundObjectProxy::StaticClass()))
	{
		return IMovieSceneBoundObjectProxy::Execute_BP_GetBoundObjectForSequencer(BoundObject, BoundObject);
	}
	return BoundObject;
}

} // namespace UE::MovieScene

TArrayView<const FMovieSceneBindingReference> FMovieSceneBindingReferences::GetAllReferences() const
{
	return SortedReferences;
}

TArrayView<FMovieSceneBindingReference> FMovieSceneBindingReferences::GetAllReferences() 
{
	return SortedReferences;
}

TArrayView<const FMovieSceneBindingReference> FMovieSceneBindingReferences::GetReferences(const FGuid& ObjectId) const
{
	const int32 Num   = SortedReferences.Num();
	const int32 Index = Algo::LowerBoundBy(SortedReferences, ObjectId, &FMovieSceneBindingReference::ID);

	// Could also use a binary search here, but typically we are only dealing with a single binding
	int32 MatchNum = 0;
	while (Index + MatchNum < Num && SortedReferences[Index + MatchNum].ID == ObjectId)
	{
		++MatchNum;
	}

	return TArrayView<const FMovieSceneBindingReference>(SortedReferences.GetData() + Index, MatchNum);
}

const FMovieSceneBindingReference* FMovieSceneBindingReferences::GetReference(const FGuid& ObjectId, int32 BindingIndex) const
{
	const int32 Index = Algo::LowerBoundBy(SortedReferences, ObjectId, &FMovieSceneBindingReference::ID) + BindingIndex;
	if (SortedReferences.IsValidIndex(Index) && SortedReferences[Index].ID == ObjectId)
	{
		return &SortedReferences[Index];
	}
	return nullptr;
}

bool FMovieSceneBindingReferences::HasBinding(const FGuid& ObjectId) const
{
	const int32 Index = Algo::LowerBoundBy(SortedReferences, ObjectId, &FMovieSceneBindingReference::ID);
	return SortedReferences.IsValidIndex(Index) && SortedReferences[Index].ID == ObjectId;
}

UMovieSceneCustomBinding* FMovieSceneBindingReferences::GetCustomBinding(const FGuid& ObjectId, int32 BindingIndex)
{
	return const_cast<UMovieSceneCustomBinding*>(const_cast<const FMovieSceneBindingReferences*>(this)->GetCustomBinding(ObjectId, BindingIndex));
}

const UMovieSceneCustomBinding* FMovieSceneBindingReferences::GetCustomBinding(const FGuid& ObjectId, int32 BindingIndex) const
{
	const int32 Index = Algo::LowerBoundBy(SortedReferences, ObjectId, &FMovieSceneBindingReference::ID) + BindingIndex;
	if (SortedReferences.IsValidIndex(Index) && SortedReferences[Index].ID == ObjectId)
	{
		return SortedReferences[Index].CustomBinding;
	}
	return nullptr;
}

const FMovieSceneBindingReference* FMovieSceneBindingReferences::AddBinding(const FGuid& ObjectId, FUniversalObjectLocator&& NewLocator)
{
	const int32 Index = Algo::UpperBoundBy(SortedReferences, ObjectId, &FMovieSceneBindingReference::ID);

	FMovieSceneBindingReference& NewBinding = SortedReferences.Insert_GetRef(FMovieSceneBindingReference{ ObjectId, MoveTemp(NewLocator) }, Index);
	NewBinding.InitializeLocatorResolveFlags();
	return &NewBinding;
}

const FMovieSceneBindingReference* FMovieSceneBindingReferences::AddBinding(const FGuid& ObjectId, FUniversalObjectLocator&& NewLocator, ELocatorResolveFlags InResolveFlags, UMovieSceneCustomBinding* CustomBinding/*=nullptr*/)
{
	const int32 Index = Algo::UpperBoundBy(SortedReferences, ObjectId, &FMovieSceneBindingReference::ID);

	FMovieSceneBindingReference& NewBinding = SortedReferences.Insert_GetRef(FMovieSceneBindingReference{ ObjectId, MoveTemp(NewLocator), InResolveFlags, CustomBinding }, Index);

	return &NewBinding;
}

const FMovieSceneBindingReference* FMovieSceneBindingReferences::AddOrReplaceBinding(const FGuid& ObjectId, UMovieSceneCustomBinding* NewCustomBinding, int32 BindingIndex)
{
	const int32 Num = SortedReferences.Num();
	const int32 Index = Algo::LowerBoundBy(SortedReferences, ObjectId, &FMovieSceneBindingReference::ID);

	// Could also use a binary search here, but typically we are only dealing with a single binding
	if (BindingIndex >= 0 && Index + BindingIndex < Num && SortedReferences[Index + BindingIndex].ID == ObjectId)
	{
		// Replace the current binding
		SortedReferences[Index + BindingIndex] = FMovieSceneBindingReference{ ObjectId, FUniversalObjectLocator(), ELocatorResolveFlags::None, NewCustomBinding};
		return &SortedReferences[Index + BindingIndex];
	}
	else
	{
		// Add a new binding instead
		return AddBinding(ObjectId, NewCustomBinding);
	}
}

const FMovieSceneBindingReference* FMovieSceneBindingReferences::AddOrReplaceBinding(const FGuid& ObjectId, FUniversalObjectLocator&& NewLocator, int32 BindingIndex)
{
	const int32 Num = SortedReferences.Num();
	const int32 Index = Algo::LowerBoundBy(SortedReferences, ObjectId, &FMovieSceneBindingReference::ID);

	// Could also use a binary search here, but typically we are only dealing with a single binding
	if (BindingIndex >= 0 && Index + BindingIndex < Num && SortedReferences[Index + BindingIndex].ID == ObjectId)
	{
		// Replace the current binding
		SortedReferences[Index + BindingIndex] = FMovieSceneBindingReference{ ObjectId, MoveTemp(NewLocator), ELocatorResolveFlags::None, nullptr };
		return &SortedReferences[Index + BindingIndex];
	}
	else
	{
		// Add a new binding instead
		return AddBinding(ObjectId, MoveTemp(NewLocator));
	}
}

void FMovieSceneBindingReferences::RemoveBinding(const FGuid& ObjectId)
{
	const int32 StartIndex = Algo::LowerBoundBy(SortedReferences, ObjectId, &FMovieSceneBindingReference::ID);

	if (SortedReferences.IsValidIndex(StartIndex) && SortedReferences[StartIndex].ID == ObjectId)
	{
		const int32 EndIndex = Algo::UpperBoundBy(SortedReferences, ObjectId, &FMovieSceneBindingReference::ID);
		SortedReferences.RemoveAt(StartIndex, EndIndex-StartIndex);
	}
}


UObject* FMovieSceneBindingReferences::ResolveBindingFromLocator(int32 Index, const UE::UniversalObjectLocator::FResolveParams& ResolveParams) const
{
	// Add our resolve param flags
	if (ResolveParams.Context)
	{
		if (UWorld* World = ResolveParams.Context->GetWorld())
		{
			EnumAddFlags(const_cast<UE::UniversalObjectLocator::FResolveParams&>(ResolveParams).Flags, SortedReferences[Index].ResolveFlags);
		}
	}

	UObject* ResolvedObject = SortedReferences[Index].Locator.Resolve(ResolveParams).SyncGet().Object;
	return UE::MovieScene::FindBoundObjectProxy(ResolvedObject);
}

void FMovieSceneBindingReferences::ResolveBindingInternal(const FMovieSceneBindingResolveParams& BindingResolveParams, const UE::UniversalObjectLocator::FResolveParams& LocatorResolveParams, int32 BindingIndex, int32 InternalIndex, TSharedPtr<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const
{
	// If a custom binding is present and we have valid shared playback state, resolve the custom binding
	if (SortedReferences[InternalIndex].CustomBinding && SharedPlaybackState.IsValid())
	{
		TArray<UObject*> ResolvedObjects;

		FMovieSceneBindingResolveResult Result = SortedReferences[InternalIndex].CustomBinding->ResolveBinding(BindingResolveParams, BindingIndex, SharedPlaybackState.ToSharedRef());
		for (TObjectPtr<UObject> ResolvedObject : Result.Objects)
		{
			if (ResolvedObject.Get())
			{
				OutObjects.Add(UE::MovieScene::FindBoundObjectProxy(ResolvedObject.Get()));
			}
		}
	}
	else
	{
		// Otherwise, attempt to resolve via the locator
		if (UObject* ResolvedObject = ResolveBindingFromLocator(InternalIndex, LocatorResolveParams))
		{
			OutObjects.Add(ResolvedObject);
		}
	}
}

void FMovieSceneBindingReferences::ResolveBinding(const FGuid& ObjectId, const UE::UniversalObjectLocator::FResolveParams& ResolveParams, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const
{
#if WITH_EDITORONLY_DATA
	// Sequencer is explicit about providing a resolution context for its bindings. We never want to resolve to objects
	// with a different PIE instance ID, even if the current callstack is being executed inside a different GPlayInEditorID
	// scope. Since ResolveObject will always call FixupForPIE in editor based on GPlayInEditorID, we always override the current
	// GPlayInEditorID to be the current PIE instance of the provided context.
	const int32 ContextPlayInEditorID = ResolveParams.Context ? ResolveParams.Context->GetOutermost()->GetPIEInstanceID() : INDEX_NONE;
	FTemporaryPlayInEditorIDOverride PIEGuard(ContextPlayInEditorID);
#endif

	const int32 StartIndex = Algo::LowerBoundBy(SortedReferences, ObjectId, &FMovieSceneBindingReference::ID);
	const int32 Num = SortedReferences.Num();

	for (int32 Index = StartIndex; Index < Num && SortedReferences[Index].ID == ObjectId; ++Index)
	{
		UObject* ResolvedObject = ResolveBindingFromLocator(Index, ResolveParams);
		if (ResolvedObject)
		{
			OutObjects.Add(ResolvedObject);
		}
	}
}

void FMovieSceneBindingReferences::ResolveBinding(const FMovieSceneBindingResolveParams& BindingResolveParams, const UE::UniversalObjectLocator::FResolveParams& LocatorResolveParams, TSharedPtr<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const
{
	const int32 StartIndex = Algo::LowerBoundBy(SortedReferences, BindingResolveParams.ObjectBindingID, &FMovieSceneBindingReference::ID);
	const int32 Num = SortedReferences.Num();

	for (int32 Index = StartIndex; Index < Num && SortedReferences[Index].ID == BindingResolveParams.ObjectBindingID; ++Index)
	{
		ResolveBindingInternal(BindingResolveParams, LocatorResolveParams, Index - StartIndex, Index, SharedPlaybackState, OutObjects);
	}
}

void FMovieSceneBindingReferences::ResolveSingleBinding(const FMovieSceneBindingResolveParams& BindingResolveParams, int32 BindingIndex, const UE::UniversalObjectLocator::FResolveParams& LocatorResolveParams, TSharedPtr<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const
{
	const int32 Index = Algo::LowerBoundBy(SortedReferences, BindingResolveParams.ObjectBindingID, &FMovieSceneBindingReference::ID) + BindingIndex;
	if (SortedReferences.IsValidIndex(Index) && SortedReferences[Index].ID == BindingResolveParams.ObjectBindingID)
	{
		ResolveBindingInternal(BindingResolveParams, LocatorResolveParams, BindingIndex, Index, SharedPlaybackState, OutObjects);
	}
}

UObject* FMovieSceneBindingReferences::ResolveSingleBinding(const FMovieSceneBindingResolveParams& BindingResolveParams, int32 BindingIndex, const UE::UniversalObjectLocator::FResolveParams& LocatorResolveParams, TSharedPtr<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const
{
	TArray<UObject*, TInlineAllocator<1>> ResolvedObjects;
	ResolveSingleBinding(BindingResolveParams, BindingIndex, LocatorResolveParams, SharedPlaybackState, ResolvedObjects);
	if (ResolvedObjects.Num() > 0)
	{
		return ResolvedObjects[0];
	}
	return nullptr;
}

void FMovieSceneBindingReferences::RemoveObjects(const FGuid& ObjectId, const TArray<UObject*>& InObjects, UObject* InContext)
{
	const int32 StartIndex = Algo::LowerBoundBy(SortedReferences, ObjectId, &FMovieSceneBindingReference::ID);

	for (int32 Index = StartIndex; Index < SortedReferences.Num() && SortedReferences[Index].ID == ObjectId; ++Index)
	{
		UObject* ResolvedObject = SortedReferences[Index].Locator.SyncFind(InContext);
		ResolvedObject = UE::MovieScene::FindBoundObjectProxy(ResolvedObject);

		if (InObjects.Contains(ResolvedObject))
		{
			SortedReferences.RemoveAt(Index);
		}
		else
		{
			++Index;
		}
	}
}

void FMovieSceneBindingReferences::RemoveInvalidObjects(const FGuid& ObjectId, UObject* InContext)
{
	const int32 StartIndex = Algo::LowerBoundBy(SortedReferences, ObjectId, &FMovieSceneBindingReference::ID);

	for (int32 Index = StartIndex; Index < SortedReferences.Num() && SortedReferences[Index].ID == ObjectId; ++Index)
	{
		UObject* ResolvedObject = SortedReferences[Index].Locator.SyncFind(InContext);
		ResolvedObject = UE::MovieScene::FindBoundObjectProxy(ResolvedObject);

		if (!IsValid(ResolvedObject))
		{
			SortedReferences.RemoveAt(Index);
		}
		else
		{
			++Index;
		}
	}
}

FGuid FMovieSceneBindingReferences::FindBindingFromObject(UObject* InObject, UObject* InContext) const
{
	FUniversalObjectLocator Locator(InObject, InContext);

	for (const FMovieSceneBindingReference& Ref : SortedReferences)
	{
		if (Ref.Locator == Locator)
		{
			return Ref.ID;
		}
	}

	return FGuid();
}

void FMovieSceneBindingReferences::RemoveInvalidBindings(const TSet<FGuid>& ValidBindingIDs)
{
	const int32 StartNum = SortedReferences.Num();
	for (int32 Index = StartNum-1; Index >= 0; --Index)
	{
		if (!ValidBindingIDs.Contains(SortedReferences[Index].ID))
		{
			SortedReferences.RemoveAtSwap(Index, EAllowShrinking::No);
		}
	}

	if (SortedReferences.Num() != StartNum)
	{
		Algo::SortBy(SortedReferences, &FMovieSceneBindingReference::ID);
	}
}

void FMovieSceneBindingReference::InitializeLocatorResolveFlags()
{
	using namespace UE::UniversalObjectLocator;
	auto InitializeFlags = [](const EFragmentTypeFlags& FragmentTypeFlags, ELocatorResolveFlags& OutResolveFlags)
	{
		if (EnumHasAllFlags(FragmentTypeFlags, EFragmentTypeFlags::CanBeLoaded | EFragmentTypeFlags::LoadedByDefault))
		{
			EnumAddFlags(OutResolveFlags, ELocatorResolveFlags::Load);
		}
	};

	InitializeFlags(Locator.GetDefaultFlags(), ResolveFlags);
}
