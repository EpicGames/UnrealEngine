// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneClipboard.h"

#define LOCTEXT_NAMESPACE "MovieSceneClipboard"

#if WITH_EDITOR

TMap<FName, TMap<FName, FMovieSceneClipboardKey::FConversionFunction>> FMovieSceneClipboardKey::ConversionMap;

FMovieSceneClipboardKey::FMovieSceneClipboardKey(const FMovieSceneClipboardKey& In)
	: Time(In.Time)
{
	if (In.Data.IsValid())
	{
		In.Data->CopyTo(Data);
	}
}

FMovieSceneClipboardKey& FMovieSceneClipboardKey::operator=(const FMovieSceneClipboardKey& In)
{
	Time = In.Time;
	if (In.Data.IsValid())
	{
		In.Data->CopyTo(Data);
	}
	else
	{
		Data.Reset();
	}
	
	return *this;
}

FFrameNumber FMovieSceneClipboardKey::GetTime() const
{
	return Time;
}

void FMovieSceneClipboardKey::SetTime(FFrameNumber InTime)
{
	Time = InTime;
}

FMovieSceneClipboardKeyTrack::FMovieSceneClipboardKeyTrack(FMovieSceneClipboardKeyTrack&& In)
	: Keys(MoveTemp(In.Keys))
	, TypeName(MoveTemp(In.TypeName))
	, Name(MoveTemp(In.Name))
{}

FMovieSceneClipboardKeyTrack& FMovieSceneClipboardKeyTrack::operator=(FMovieSceneClipboardKeyTrack&& In)
{
	Keys = MoveTemp(In.Keys);
	TypeName = MoveTemp(In.TypeName);
	Name = MoveTemp(In.Name);
	return *this;
}

/** Copy construction/assignment */
FMovieSceneClipboardKeyTrack::FMovieSceneClipboardKeyTrack(const FMovieSceneClipboardKeyTrack& In)
	: Keys(In.Keys)
	, TypeName(In.TypeName)
	, Name(In.Name)
{}

FMovieSceneClipboardKeyTrack& FMovieSceneClipboardKeyTrack::operator=(const FMovieSceneClipboardKeyTrack& In)
{
	Keys = In.Keys;
	TypeName = In.TypeName;
	Name = In.Name;
	return *this;
}

const FName& FMovieSceneClipboardKeyTrack::GetName() const
{
	return Name;
}

FMovieSceneClipboard::FMovieSceneClipboard()
{
}

FMovieSceneClipboard::FMovieSceneClipboard(FMovieSceneClipboard&& In)
	: Environment(In.Environment)
	, KeyTrackGroups(MoveTemp(In.KeyTrackGroups))
{
}

FMovieSceneClipboard& FMovieSceneClipboard::operator=(FMovieSceneClipboard&& In)
{
	Environment = In.Environment;
	KeyTrackGroups = MoveTemp(In.KeyTrackGroups);
	return *this;
}

FMovieSceneClipboard::FMovieSceneClipboard(const FMovieSceneClipboard& In)
	: Environment(In.Environment)
	, KeyTrackGroups(In.KeyTrackGroups)
{
}

FMovieSceneClipboard& FMovieSceneClipboard::operator=(const FMovieSceneClipboard& In)
{
	Environment = In.Environment;
	KeyTrackGroups = In.KeyTrackGroups;
	return *this;
}

const TArray<TArray<FMovieSceneClipboardKeyTrack>>& FMovieSceneClipboard::GetKeyTrackGroups() const
{
	return KeyTrackGroups;
}

FText FMovieSceneClipboard::GetDisplayText() const
{
	if (KeyTrackGroups.Num())
	{
		uint32 NumKeys = 0;
		for (const TArray<FMovieSceneClipboardKeyTrack>& Group : KeyTrackGroups)
		{
			for (const FMovieSceneClipboardKeyTrack& Track : Group)
			{
				Track.IterateKeys([&](const FMovieSceneClipboardKey&){
					++NumKeys;
					return true;
				});
			}
		}

		if (NumKeys == 1)
		{
			return FText::Format(LOCTEXT("KeyDisplayFormat_Single", "Clipboard from {0} (1 key)"), FText::AsTime(Environment.DateTime));
		}
		else
		{
			return FText::Format(LOCTEXT("KeyDisplayFormat_Multiple", "Clipboard from {0} ({1} keys)"), FText::AsTime(Environment.DateTime), FText::AsNumber(NumKeys));
		}
	}
	return FText::Format(LOCTEXT("GenericClipboardFormat", "Clipboard from {0}"), FText::AsTime(Environment.DateTime));
}

const FMovieSceneClipboardEnvironment& FMovieSceneClipboard::GetEnvironment() const
{
	return Environment;
}

FMovieSceneClipboardEnvironment& FMovieSceneClipboard::GetEnvironment()
{
	return Environment;
}

FMovieSceneClipboard FMovieSceneClipboardBuilder::Commit(TOptional<FFrameNumber> CopyRelativeTo)
{
	FMovieSceneClipboard Clipboard;

	// If no relative time was specified, copy relative to the earliest key
	if (!CopyRelativeTo.IsSet())
	{
		for (auto& Pair : TrackIndex)
		{
			for (FMovieSceneClipboardKeyTrack& Track : Pair.Value)
			{
				Track.IterateKeys([&](const FMovieSceneClipboardKey& Key){
					if (!CopyRelativeTo.IsSet())
					{
						CopyRelativeTo = Key.GetTime();
					}
					else
					{
						CopyRelativeTo = FMath::Min(Key.GetTime(), CopyRelativeTo.GetValue());
					}
					return true;
				});
			}
		}
	}

	if (CopyRelativeTo.IsSet())
	{
		for (auto& Pair : TrackIndex)
		{
			for (FMovieSceneClipboardKeyTrack& Track : Pair.Value)
			{
				Track.IterateKeys([&](FMovieSceneClipboardKey& Key){
					Key.SetTime(Key.GetTime() - CopyRelativeTo.GetValue());
					return true;
				});
			}

			Clipboard.KeyTrackGroups.Add(MoveTemp(Pair.Value));
		}

		Clipboard.GetEnvironment().CardinalTime = CopyRelativeTo.GetValue();
	}
	
	TrackIndex.Reset();
	return Clipboard;
}

#endif

#undef LOCTEXT_NAMESPACE
