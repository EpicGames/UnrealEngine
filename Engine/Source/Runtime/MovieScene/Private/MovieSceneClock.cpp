// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneClock.h"

#include "Engine/World.h"
#include "Evaluation/IMovieSceneCustomClockSource.h"
#include "MovieScene.h"
#include "MovieSceneTimeController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneClock)

TSharedPtr<FMovieSceneTimeController> UMovieSceneClock::MakeTimeController(UObject* PlaybackContext) const
{
	return MakeShared<FMovieSceneTimeController_Tick>();
}

TSharedPtr<FMovieSceneTimeController> UMovieSceneExternalClock::MakeTimeController(UObject* PlaybackContext) const
{
	return MakeShared<FMovieSceneTimeController_Custom>(CustomClockSourcePath, PlaybackContext);
}
