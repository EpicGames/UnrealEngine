// Copyright Epic Games, Inc. All Rights Reserved.
#include "MovieSceneCommonHelpers.h"
#include "Channels/MovieSceneChannelEditorData.h"
#include "Channels/MovieSceneChannel.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "EntitySystem/MovieSceneIntermediatePropertyValue.h"
#include "Components/SceneComponent.h"
#include "GameFramework/Actor.h"
#include "Camera/CameraComponent.h"
#include "KeyParams.h"
#include "MovieScene.h"
#include "MovieSceneSection.h"
#include "MovieSceneSequence.h"
#include "MovieSceneSpawnable.h"
#include "Sections/MovieSceneSubSection.h"
#include "Algo/Sort.h"
#include "Sound/SoundWave.h"
#include "Sound/SoundCue.h"
#include "Sound/SoundNodeWavePlayer.h"
#include "MovieSceneTrack.h"
#include "Engine/Engine.h"
#include "UObject/Package.h"
#include "MovieSceneBindingReferences.h"
#include "Bindings/MovieSceneSpawnableBinding.h"
#include "Evaluation/MovieSceneEvaluationState.h"
#include "UObject/UObjectIterator.h"
#include "EntitySystem/MovieSceneSequenceInstance.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "Conditions/MovieSceneGroupCondition.h"
#include "String/ParseTokens.h"

FText MovieSceneHelpers::GetDisplayPathName(const UMovieSceneTrack* Track)
{
	FText DisplayPathName;

	FGuid ObjectGuid = Track->FindObjectBindingGuid();
	if (ObjectGuid.IsValid())
	{
		if (UMovieScene* MovieScene = Cast<UMovieScene>(Track->GetOuter()))
		{
			DisplayPathName = MovieScene->GetObjectDisplayName(ObjectGuid);
		}
	}

#if WITH_EDITOR
	DisplayPathName = DisplayPathName.IsEmpty() ? Track->GetDisplayName() : FText::Join(FText::FromString("."), DisplayPathName, Track->GetDisplayName());
#endif

	return DisplayPathName;
}

FText MovieSceneHelpers::GetDisplayPathName(const UMovieSceneSection* Section)
{
	FText DisplayPathName;

	UMovieSceneTrack* Track = Section->GetTypedOuter<UMovieSceneTrack>();
	if (Track)
	{
		DisplayPathName = GetDisplayPathName(Track);
	}

	FString SectionName;
	if (const UObject* SourceObject = Section->GetSourceObject())
	{
		SectionName = SourceObject->GetName();
	}
	else
	{
		Section->GetName(SectionName);
	}
	
	DisplayPathName = DisplayPathName.IsEmpty() ? FText::FromString(SectionName) : FText::Join(FText::FromString("."), DisplayPathName, FText::FromString(SectionName));
	
	return DisplayPathName;
}

FText MovieSceneHelpers::GetDisplayPathName(const UMovieSceneSection* Section, const FMovieSceneChannel* Channel, const FMovieSceneChannelMetaData& ChannelMetaData)
{
	FText DisplayPathName;

	DisplayPathName = GetDisplayPathName(Section);

#if WITH_EDITOR
	if (Channel)
	{
		DisplayPathName = DisplayPathName.IsEmpty() ? FText::Join(FText::FromString("."), ChannelMetaData.Group, ChannelMetaData.DisplayText) : FText::Join(FText::FromString("."), DisplayPathName, ChannelMetaData.Group, ChannelMetaData.DisplayText);
	}
#endif

	return DisplayPathName;
}

bool MovieSceneHelpers::IsSectionKeyable(const UMovieSceneSection* Section)
{
	if (!Section || !Section->IsActive())
	{
		return false;
	}

	UMovieSceneTrack* Track = Section->GetTypedOuter<UMovieSceneTrack>();
	if (!Track)
	{
		return false;
	}

	if (Track->IsEvalDisabled() || Track->IsRowEvalDisabled(Section->GetRowIndex()))
	{
		return false;
	}

#if WITH_EDITOR
	bool bHasAnyChannels = false;
	bool bHasAnyKeyableChannels = false;
	const FMovieSceneChannelProxy& ChannelProxy = Section->GetChannelProxy();
	for (const FMovieSceneChannelEntry& ChannelEntry : ChannelProxy.GetAllEntries())
	{
		bHasAnyChannels = true;
		for (const FMovieSceneChannelMetaData& MetaData : ChannelEntry.GetMetaData())
		{
			if (MetaData.bEnabled && MetaData.bCanBeKeyed)
			{
				bHasAnyKeyableChannels = true;
				goto DoneCheckingChannels;
			}
		}
	}
DoneCheckingChannels:

	if (!bHasAnyChannels)
	{
		return true;
	}

	if (!bHasAnyKeyableChannels)
	{
		return false;
	}
#endif  // WITH_EDITOR

	return true;
}

UMovieSceneSection* MovieSceneHelpers::FindSectionAtTime( TArrayView<UMovieSceneSection* const> Sections, FFrameNumber Time, int32 RowIndex )
{
	for( int32 SectionIndex = 0; SectionIndex < Sections.Num(); ++SectionIndex )
	{
		UMovieSceneSection* Section = Sections[SectionIndex];

		//@todo sequencer: There can be multiple sections overlapping in time. Returning instantly does not account for that.
		if( (RowIndex == INDEX_NONE || Section->GetRowIndex() == RowIndex) &&
				Section->IsTimeWithinSection( Time ) && IsSectionKeyable(Section) )
		{
			return Section;
		}
	}

	return nullptr;
}

UMovieSceneSection* MovieSceneHelpers::FindNearestSectionAtTime( TArrayView<UMovieSceneSection* const> Sections, FFrameNumber Time, int32 RowIndex )
{
	TArray<UMovieSceneSection*> OverlappingSections, NonOverlappingSections;
	for (UMovieSceneSection* Section : Sections)
	{
		if ((RowIndex == INDEX_NONE || Section->GetRowIndex() == RowIndex) &&
				IsSectionKeyable(Section))
		{
			if (Section->GetRange().Contains(Time))
			{
				OverlappingSections.Add(Section);
			}
			else
			{
				NonOverlappingSections.Add(Section);
			}
		}
	}

	if (OverlappingSections.Num())
	{
		Algo::Sort(OverlappingSections, SortOverlappingSections);
		return OverlappingSections[0];
	}

	if (NonOverlappingSections.Num())
	{
		Algo::SortBy(NonOverlappingSections, Projection(&UMovieSceneSection::GetRange, &TRange<FFrameNumber>::GetUpperBound), SortUpperBounds);

		const int32 PreviousIndex = Algo::UpperBoundBy(NonOverlappingSections, TRangeBound<FFrameNumber>(Time), Projection(&UMovieSceneSection::GetRange, &TRange<FFrameNumber>::GetUpperBound), SortUpperBounds)-1;
		if (NonOverlappingSections.IsValidIndex(PreviousIndex))
		{
			return NonOverlappingSections[PreviousIndex];
		}
		else
		{
			Algo::SortBy(NonOverlappingSections, [](const UMovieSceneSection* A) { return A ? A->GetRange().GetLowerBound() : FFrameNumber(0); }, SortLowerBounds);
			return NonOverlappingSections[0];
		}
	}

	return nullptr;
}

UMovieSceneSection* MovieSceneHelpers::FindNextSection(TArrayView<UMovieSceneSection* const> Sections, FFrameNumber Time)
{
	FFrameNumber MinTime = TNumericLimits<FFrameNumber>::Max();

	TMap<FFrameNumber, int32> StartTimeMap;
	for (int32 SectionIndex = 0; SectionIndex < Sections.Num(); ++SectionIndex)
	{
		UMovieSceneSection* ShotSection = Sections[SectionIndex];

		if (ShotSection && ShotSection->HasStartFrame() && !ShotSection->GetRange().Contains(Time))
		{
			StartTimeMap.Add(ShotSection->GetInclusiveStartFrame(), SectionIndex);
		}
	}

	StartTimeMap.KeySort(TLess<FFrameNumber>());

	int32 NextSectionIndex = -1;
	for (auto StartTimeIt = StartTimeMap.CreateIterator(); StartTimeIt; ++StartTimeIt)
	{
		FFrameNumber StartTime = StartTimeIt->Key;
		if (StartTime > Time)
		{
			FFrameNumber DiffTime = FMath::Abs(StartTime - Time);
			if (DiffTime < MinTime)
			{
				MinTime = DiffTime;
				NextSectionIndex = StartTimeIt->Value;
			}
		}
	}

	if (NextSectionIndex == -1)
	{
		return nullptr;
	}

	return Sections[NextSectionIndex];
}

UMovieSceneSection* MovieSceneHelpers::FindPreviousSection(TArrayView<UMovieSceneSection* const> Sections, FFrameNumber Time)
{
	FFrameNumber MinTime = TNumericLimits<FFrameNumber>::Max();

	TMap<FFrameNumber, int32> StartTimeMap;
	for (int32 SectionIndex = 0; SectionIndex < Sections.Num(); ++SectionIndex)
	{
		UMovieSceneSection* ShotSection = Sections[SectionIndex];

		if (ShotSection && ShotSection->HasStartFrame() && !ShotSection->GetRange().Contains(Time))
		{
			StartTimeMap.Add(ShotSection->GetInclusiveStartFrame(), SectionIndex);
		}
	}

	StartTimeMap.KeySort(TLess<FFrameNumber>());

	int32 PreviousSectionIndex = -1;
	for (auto StartTimeIt = StartTimeMap.CreateIterator(); StartTimeIt; ++StartTimeIt)
	{
		FFrameNumber StartTime = StartTimeIt->Key;
		if (Time >= StartTime)
		{
			FFrameNumber DiffTime = FMath::Abs(StartTime - Time);
			if (DiffTime < MinTime)
			{
				MinTime = DiffTime;
				PreviousSectionIndex = StartTimeIt->Value;
			}
		}
	}

	if (PreviousSectionIndex == -1)
	{
		return nullptr;
	}

	return Sections[PreviousSectionIndex];
}

bool MovieSceneHelpers::SortOverlappingSections(const UMovieSceneSection* A, const UMovieSceneSection* B)
{
	return A->GetRowIndex() == B->GetRowIndex()
		? A->GetOverlapPriority() < B->GetOverlapPriority()
		: A->GetRowIndex() < B->GetRowIndex();
}

void MovieSceneHelpers::SortConsecutiveSections(TArray<UMovieSceneSection*>& Sections)
{
	Algo::SortBy(Sections, [](const UMovieSceneSection* A) { return A ? A->GetRange().GetLowerBound() : FFrameNumber(0); }, SortLowerBounds);
}

bool MovieSceneHelpers::FixupConsecutiveSections(TArray<UMovieSceneSection*>& Sections, UMovieSceneSection& Section, bool bDelete, bool bCleanUp)
{
	// Find the previous section and extend it to take the place of the section being deleted
	int32 SectionIndex = INDEX_NONE;

	const TRange<FFrameNumber> SectionRange = Section.GetRange();

	if (SectionRange.HasLowerBound() && SectionRange.HasUpperBound() && SectionRange.GetLowerBoundValue() >= SectionRange.GetUpperBoundValue())
	{
		return false;
	}

	if (Sections.Find(&Section, SectionIndex))
	{
		int32 PrevSectionIndex = SectionIndex - 1;
		if( Sections.IsValidIndex( PrevSectionIndex ) )
		{
			// Extend the previous section
			UMovieSceneSection* PrevSection = Sections[PrevSectionIndex];

			PrevSection->Modify();

			if (bDelete)
			{
				TRangeBound<FFrameNumber> NewEndFrame = SectionRange.GetUpperBound();

				if (!PrevSection->HasStartFrame() || NewEndFrame.GetValue() > PrevSection->GetInclusiveStartFrame())
				{
					PrevSection->SetEndFrame(NewEndFrame);
				}
			}
			else
			{
				TRangeBound<FFrameNumber> NewEndFrame = TRangeBound<FFrameNumber>::FlipInclusion(SectionRange.GetLowerBound());

				if (!PrevSection->HasStartFrame() || NewEndFrame.GetValue() > PrevSection->GetInclusiveStartFrame())
				{
					PrevSection->SetEndFrame(NewEndFrame);
				}
			}
		}

		if( !bDelete )
		{
			int32 NextSectionIndex = SectionIndex + 1;
			if(Sections.IsValidIndex(NextSectionIndex))
			{
				// Shift the next CameraCut's start time so that it starts when the new CameraCut ends
				UMovieSceneSection* NextSection = Sections[NextSectionIndex];

				NextSection->Modify();

				TRangeBound<FFrameNumber> NewStartFrame = TRangeBound<FFrameNumber>::FlipInclusion(SectionRange.GetUpperBound());

				if (!NextSection->HasEndFrame() || NewStartFrame.GetValue() < NextSection->GetExclusiveEndFrame())
				{
					NextSection->SetStartFrame(NewStartFrame);
				}
			}
		}
	}

	bool bCleanUpDone = false;
	if (bCleanUp)
	{
		const TArray<UMovieSceneSection*> OverlappedSections = Sections.FilterByPredicate([&Section, SectionRange](const UMovieSceneSection* Cur)
				{
					if (Cur != &Section)
					{
						const TRange<FFrameNumber> CurRange = Cur->GetRange();
						return SectionRange.Contains(CurRange);
					}
					return false;
				});
		for (UMovieSceneSection* OverlappedSection : OverlappedSections)
		{
			Sections.Remove(OverlappedSection);
		}
		bCleanUpDone = (OverlappedSections.Num() > 0);
	}

	SortConsecutiveSections(Sections);

	return bCleanUpDone;
}

bool MovieSceneHelpers::FixupConsecutiveBlendingSections(TArray<UMovieSceneSection*>& Sections, UMovieSceneSection& Section, bool bDelete, bool bCleanUp)
{
	int32 SectionIndex = INDEX_NONE;

	TRange<FFrameNumber> SectionRange = Section.GetRange();

	if (SectionRange.HasLowerBound() && SectionRange.HasUpperBound() && SectionRange.GetLowerBoundValue() >= SectionRange.GetUpperBoundValue())
	{
		return false;
	}

	if (Sections.Find(&Section, SectionIndex))
	{
		// Find the previous section and extend it to take the place of the section being deleted
		int32 PrevSectionIndex = SectionIndex - 1;
		if (Sections.IsValidIndex(PrevSectionIndex))
		{
			UMovieSceneSection* PrevSection = Sections[PrevSectionIndex];

			PrevSection->Modify();

			if (bDelete)
			{
				TRangeBound<FFrameNumber> NewEndFrame = SectionRange.GetUpperBound();

				if (!PrevSection->HasStartFrame() || NewEndFrame.GetValue() > PrevSection->GetInclusiveStartFrame())
				{
					// The current section was deleted... extend the previous section to fill the gap.
					PrevSection->SetEndFrame(NewEndFrame);
				}
			}
			else
			{
				const FFrameNumber GapOrOverlap = SectionRange.GetLowerBoundValue() - PrevSection->GetRange().GetUpperBoundValue();
				if (GapOrOverlap > 0)
				{
					// If we made a gap: adjust the previous section's end time so that it ends wherever the current section's ease-in ends.
					TRangeBound<FFrameNumber> NewEndFrame = TRangeBound<FFrameNumber>::Exclusive(SectionRange.GetLowerBoundValue() + Section.Easing.GetEaseInDuration());

					if (!PrevSection->HasStartFrame() || NewEndFrame.GetValue() > PrevSection->GetInclusiveStartFrame())
					{
						// It's a gap!
						PrevSection->SetEndFrame(NewEndFrame);
					}
				}
				else
				{
					// If we created an overlap: calls to UMovieSceneTrack::UpdateEasing will set the easing curves correctly based on overlaps.
					// However, we need to fixup some easing where overlaps don't occur, such as the very first ease-in and the very last ease-out.
					// Don't overlap so far that our ease-out, or the previous section's ease-in, get overlapped. Clamp these easing durations instead.
					if (Section.HasEndFrame() && PrevSection->HasEndFrame())
					{
						const FFrameNumber MaxEaseOutDuration = Section.GetExclusiveEndFrame() - PrevSection->GetExclusiveEndFrame();
						Section.Easing.AutoEaseOutDuration = FMath::Min(FMath::Max(0, MaxEaseOutDuration.Value), Section.Easing.AutoEaseOutDuration);
						Section.Easing.ManualEaseOutDuration = FMath::Min(FMath::Max(0, MaxEaseOutDuration.Value), Section.Easing.ManualEaseOutDuration);
					}
					if (Section.HasStartFrame() && PrevSection->HasStartFrame())
					{
						const FFrameNumber MaxPrevSectionEaseInDuration = Section.GetInclusiveStartFrame() - PrevSection->GetInclusiveStartFrame();
						PrevSection->Easing.AutoEaseInDuration = FMath::Min(FMath::Max(0, MaxPrevSectionEaseInDuration.Value), PrevSection->Easing.AutoEaseInDuration);
						PrevSection->Easing.ManualEaseInDuration = FMath::Min(FMath::Max(0, MaxPrevSectionEaseInDuration.Value), PrevSection->Easing.ManualEaseInDuration);
					}
				}
			}
		}
		else
		{
			if (!bDelete)
			{
				// The given section is the first section. Let's clear its auto ease-in since there's no overlap anymore with a previous section.
				Section.Easing.AutoEaseInDuration = 0;
			}
		}

		// Find the next section and adjust its start time to match the moved/resized section's new end time.
		if (!bDelete)
		{
			int32 NextSectionIndex = SectionIndex + 1;
			if (Sections.IsValidIndex(NextSectionIndex))
			{
				UMovieSceneSection* NextSection = Sections[NextSectionIndex];

				NextSection->Modify();

				const FFrameNumber GapOrOverlap = NextSection->GetRange().GetLowerBoundValue() - SectionRange.GetUpperBoundValue();
				if (GapOrOverlap > 0)
				{
					// If we made a gap: adjust the next section's start time so that it lines up with the current section's end.
					TRangeBound<FFrameNumber> NewStartFrame = TRangeBound<FFrameNumber>::Inclusive(SectionRange.GetUpperBoundValue() - NextSection->Easing.GetEaseInDuration());

					if (!NextSection->HasEndFrame() || NewStartFrame.GetValue() < NextSection->GetExclusiveEndFrame())
					{
						// It's a gap!
						NextSection->SetStartFrame(NewStartFrame);
					}
				}
				else
				{
					// If we created an overlap: calls to UMovieSceneTrack::UpdateEasing will set the easing curves correctly based on overlaps.
					// However, we need to fixup some easing where overlaps don't occur, such as the very first ease-in and the very last ease-out.
					// Don't overlap so far that our ease-in, or the next section's ease-out, get overlapped. Clamp these easing durations instead.
					if (Section.HasStartFrame() && NextSection->HasStartFrame())
					{
						const FFrameNumber MaxEaseInDuration = NextSection->GetInclusiveStartFrame() - Section.GetInclusiveStartFrame();
						Section.Easing.AutoEaseInDuration = FMath::Min(FMath::Max(0, MaxEaseInDuration.Value), Section.Easing.AutoEaseInDuration);
						Section.Easing.ManualEaseInDuration = FMath::Min(FMath::Max(0, MaxEaseInDuration.Value), Section.Easing.ManualEaseInDuration);
					}
					if (Section.HasEndFrame() && NextSection->HasEndFrame())
					{
						const FFrameNumber MaxNextSectionEaseOutDuration = NextSection->GetExclusiveEndFrame() - Section.GetExclusiveEndFrame();
						NextSection->Easing.AutoEaseOutDuration = FMath::Min(FMath::Max(0, MaxNextSectionEaseOutDuration.Value), NextSection->Easing.AutoEaseOutDuration);
						NextSection->Easing.ManualEaseOutDuration = FMath::Min(FMath::Max(0, MaxNextSectionEaseOutDuration.Value), NextSection->Easing.ManualEaseOutDuration);
					}
				}
			}
			else
			{
				// The given section is the last section. Let's clear its auto ease-out since there's no overlap anymore with a next section.
				Section.Easing.AutoEaseOutDuration = 0;
			}
		}
	}

	bool bCleanUpDone = false;
	if (bCleanUp)
	{
		const TArray<UMovieSceneSection*> OverlappedSections = Sections.FilterByPredicate([&Section, SectionRange](const UMovieSceneSection* Cur)
				{
					if (Cur != &Section)
					{
						const TRange<FFrameNumber> CurRange = Cur->GetRange();
						return SectionRange.Contains(CurRange);
					}
					return false;
				});
		for (UMovieSceneSection* OverlappedSection : OverlappedSections)
		{
			Sections.Remove(OverlappedSection);
		}
		bCleanUpDone = (OverlappedSections.Num() > 0);
	}

	SortConsecutiveSections(Sections);

	return bCleanUpDone;
}


void MovieSceneHelpers::GetDescendantMovieScenes(UMovieSceneSequence* InSequence, TArray<UMovieScene*> & InMovieScenes)
{
	UMovieScene* InMovieScene = InSequence->GetMovieScene();
	if (InMovieScene == nullptr || InMovieScenes.Contains(InMovieScene))
	{
		return;
	}

	InMovieScenes.Add(InMovieScene);

	for (auto Section : InMovieScene->GetAllSections())
	{
		UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section);
		if (SubSection != nullptr)
		{
			UMovieSceneSequence* SubSequence = SubSection->GetSequence();
			if (SubSequence != nullptr)
			{
				GetDescendantMovieScenes(SubSequence, InMovieScenes);
			}
		}
	}
}

void MovieSceneHelpers::DisableTrack(UMovieSceneTrack* Track)
{
	Track->Modify();

	// Disable the entire track or each of the rows
	if (Track->GetMaxRowIndex() == 0)
	{
		Track->SetEvalDisabled(true);
	}
	else
	{
		for (int32 RowIndex = 0; RowIndex <= Track->GetMaxRowIndex(); ++RowIndex)
		{
			Track->SetRowEvalDisabled(true, RowIndex);
		}
	}
}

void MovieSceneHelpers::GetDescendantSubSections(const UMovieScene* InMovieScene, TArray<UMovieSceneSubSection*>& InSubSections)
{
	if (!IsValid(InMovieScene))
	{
		return;
	}

	for (UMovieSceneSection* Section : InMovieScene->GetAllSections())
	{
		if (UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section))
		{
			InSubSections.Add(SubSection);
			
			if (const UMovieSceneSequence* SubSequence = SubSection->GetSequence())
			{
				GetDescendantSubSections(SubSequence->GetMovieScene(), InSubSections);
			}
		}
	}
}

UObject* MovieSceneHelpers::ResolveSceneComponentBoundObject(UObject* Object)
{
	return SceneComponentFromRuntimeObject(Object);
}

USceneComponent* MovieSceneHelpers::SceneComponentFromRuntimeObject(UObject* Object)
{
	AActor* Actor = Cast<AActor>(Object);

	USceneComponent* SceneComponent = nullptr;
	if (Actor && Actor->GetRootComponent())
	{
		// If there is an actor, modify its root component
		SceneComponent = Actor->GetRootComponent();
	}
	else
	{
		// No actor was found.  Attempt to get the object as a component in the case that we are editing them directly.
		SceneComponent = Cast<USceneComponent>(Object);
	}
	return SceneComponent;
}

UCameraComponent* MovieSceneHelpers::CameraComponentFromActor(const AActor* InActor)
{
	TArray<UCameraComponent*> CameraComponents;
	InActor->GetComponents(CameraComponents);

	// If there's a camera component that's active, return that one
	for (UCameraComponent* CameraComponent : CameraComponents)
	{
		if (CameraComponent->IsActive())
		{
			return CameraComponent;
		}
	}

	// Otherwise, return the first camera component
	if (CameraComponents.Num() > 0)
	{
		return CameraComponents[0];
	}

	return nullptr;
}

UCameraComponent* MovieSceneHelpers::CameraComponentFromRuntimeObject(UObject* RuntimeObject)
{
	if (RuntimeObject)
	{
		// find camera we want to control
		UCameraComponent* const CameraComponent = Cast<UCameraComponent>(RuntimeObject);
		if (CameraComponent)
		{
			return CameraComponent;
		}

		// see if it's an actor that has a camera component
		AActor* const Actor = Cast<AActor>(RuntimeObject);
		if (Actor)
		{
			return CameraComponentFromActor(Actor);
		}
	}

	return nullptr;
}

float MovieSceneHelpers::GetSoundDuration(USoundBase* Sound)
{
	if (Sound)
	{
		if (Sound->IsProcedurallyGenerated() || !Sound->IsOneShot())
		{
			return INDEFINITELY_LOOPING_DURATION;
		}
		else
		{
			return FMath::Max(0.0f, Sound->GetDuration());
		}
	}
	return 0.0f;
}

float MovieSceneHelpers::CalculateWeightForBlending(UMovieSceneSection* SectionToKey, FFrameNumber Time)
{
	float Weight = 1.0f;
	UMovieSceneTrack* Track = SectionToKey->GetTypedOuter<UMovieSceneTrack>();
	FOptionalMovieSceneBlendType BlendType = SectionToKey->GetBlendType();
	if (Track && BlendType.IsValid() && (( BlendType.Get() == EMovieSceneBlendType::Additive) || 
										 ( BlendType.Get() == EMovieSceneBlendType::Absolute) || 
										 (BlendType.Get() == EMovieSceneBlendType::Override)  ))
	{
		//if additive weight is just the inverse of any weight on it
		if ((BlendType.Get() == EMovieSceneBlendType::Additive) || (BlendType.Get() == EMovieSceneBlendType::Override))
		{
			float TotalWeightValue = SectionToKey->GetTotalWeightValue(Time);
			Weight = !FMath::IsNearlyZero(TotalWeightValue) ? 1.0f / TotalWeightValue : 0.0f;
		}
		else
		{

			const TArray<UMovieSceneSection*>& Sections = Track->GetAllSections();
			TArray<UMovieSceneSection*, TInlineAllocator<4>> OverlappingSections;
			for (UMovieSceneSection* Section : Sections)
			{
				if (MovieSceneHelpers::IsSectionKeyable(Section) && Section->GetRange().Contains(Time))
				{
					OverlappingSections.Add(Section);
				}
			}
			//if absolute need to calculate weight based upon other sections weights (+ implicit absolute weights)
			int TotalNumOfAbsoluteSections = 1;
			for (UMovieSceneSection* Section : OverlappingSections)
			{
				FOptionalMovieSceneBlendType NewBlendType = Section->GetBlendType();

				if (Section != SectionToKey && NewBlendType.IsValid() && NewBlendType.Get() == EMovieSceneBlendType::Absolute)
				{
					++TotalNumOfAbsoluteSections;
				}
			}
			float TotalWeightValue = SectionToKey->GetTotalWeightValue(Time);
			Weight = !FMath::IsNearlyZero(TotalWeightValue) ? float(TotalNumOfAbsoluteSections) / TotalWeightValue : 0.0f;
		}
	}
	return Weight;
}

bool SplitBindingLabel(FString& InOutLabel, int32& OutIdx)
{
	// Look at the label and see if it ends in a number and separate them
	const TArray<TCHAR, FString::AllocatorType>& LabelCharArray = InOutLabel.GetCharArray();
	for (int32 CharIdx = LabelCharArray.Num() - 1; CharIdx >= 0; CharIdx--)
	{
		if (CharIdx == 0 || !FChar::IsDigit(LabelCharArray[CharIdx - 1]))
		{
			FString Idx = InOutLabel.RightChop(CharIdx);
			if (Idx.Len() > 0)
			{
				InOutLabel.LeftInline(CharIdx);
				OutIdx = FCString::Atoi(*Idx);
				return true;
			}
			break;
		}
	}
	return false;
}

FString MovieSceneHelpers::MakeUniqueBindingName(UMovieScene* MovieScene, const FString& InName)
{
	FString Prefix = InName;
	FString ModifiedActorLabel = InName;
	int32   LabelIdx = 0;

	TArray<FString> Names;
	for (int32 Index = 0; Index < MovieScene->GetSpawnableCount(); ++Index)
	{
		Names.Add(MovieScene->GetSpawnable(Index).GetName());
	}
	for (int32 Index = 0; Index < MovieScene->GetPossessableCount(); ++Index)
	{
		Names.Add(MovieScene->GetPossessable(Index).GetName());
	}

	if (Names.Contains(ModifiedActorLabel))
	{
		// See if the current label ends in a number, and try to create a new label based on that
		if (!SplitBindingLabel(Prefix, LabelIdx))
		{
			// If there wasn't a number on there, append a number, starting from 2 (1 before incrementing below)
			LabelIdx = 1;
		}

		// Update the actor label until we find one that doesn't already exist
		while (Names.Contains(ModifiedActorLabel))
		{
			++LabelIdx;
			ModifiedActorLabel = FString::Printf(TEXT("%s%d"), *Prefix, LabelIdx);
		}
	}

	return ModifiedActorLabel;
}

FString MovieSceneHelpers::MakeUniqueSpawnableName(UMovieScene* MovieScene, const FString& InName)
{
	FString Prefix = InName;
	FString ModifiedActorLabel = InName;
	int32   LabelIdx = 0;

	TArray<FString> Names;
	for (int32 Index = 0; Index < MovieScene->GetSpawnableCount(); ++Index)
	{
		Names.Add(MovieScene->GetSpawnable(Index).GetName());
	}

	if (Names.Contains(ModifiedActorLabel))
	{
		// See if the current label ends in a number, and try to create a new label based on that
		if (!SplitBindingLabel(Prefix, LabelIdx))
		{
			// If there wasn't a number on there, append a number, starting from 2 (1 before incrementing below)
			LabelIdx = 1;
		}

		// Update the actor label until we find one that doesn't already exist
		while (Names.Contains(ModifiedActorLabel))
		{
			++LabelIdx;
			ModifiedActorLabel = FString::Printf(TEXT("%s%d"), *Prefix, LabelIdx);
		}
	}

	return ModifiedActorLabel;
}

UObject* MovieSceneHelpers::MakeSpawnableTemplateFromInstance(UObject& InSourceObject, UMovieScene* InMovieScene, FName InName)
{
	// Guard against a stale template object of a different class occupying InName in the same outer.
	if (UObject* ExistingObject = StaticFindObjectFast(UObject::StaticClass(), InMovieScene, InName))
	{
		if (ExistingObject->GetClass() != InSourceObject.GetClass())
		{
			const FName ExpiredName = MakeUniqueObjectName(InMovieScene, ExistingObject->GetClass(), "ExpiredSpawnable");
			ExistingObject->Rename(*ExpiredName.ToString(), nullptr, REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
			ExistingObject->MarkAsGarbage();
		}
	}

	UObject* NewInstance = NewObject<UObject>(InMovieScene, InSourceObject.GetClass(), InName);

	UEngine::FCopyPropertiesForUnrelatedObjectsParams CopyParams;
	CopyParams.bNotifyObjectReplacement = false;
	CopyParams.bPreserveRootComponent = false;
	CopyParams.bPerformDuplication = true;
	UEngine::CopyPropertiesForUnrelatedObjects(&InSourceObject, NewInstance, CopyParams);

	AActor* Actor = CastChecked<AActor>(NewInstance);
	
	// Remove tags that may have gotten stuck on- for spawnables/replaceables these tags will be added after spawning
	static const FName SequencerActorTag("SequencerActor");
	static const FName SequencerPreviewActorTag("SequencerPreviewActor");
	Actor->Tags.Remove(SequencerActorTag);
	Actor->Tags.Remove(SequencerPreviewActorTag);

	if (Actor->GetAttachParentActor() != nullptr)
	{
		// We don't support spawnables and attachments right now
		// @todo: map to attach track?
		Actor->DetachFromActor(FDetachmentTransformRules(FAttachmentTransformRules(EAttachmentRule::KeepRelative, false), false));
	}

	// The spawnable source object was created with RF_Transient. The object generated from that needs its 
	// component flags cleared of RF_Transient so that the template object can be saved to the level sequence.
	for (UActorComponent* Component : Actor->GetComponents())
	{
		if (Component)
		{
			Component->ClearFlags(RF_Transient);
		}
	}

	return NewInstance;
}


bool MovieSceneHelpers::IsBoundToAnySpawnable(UMovieSceneSequence* Sequence, const FGuid& ObjectId, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState)
{
	if (Sequence)
	{
		if (UMovieScene* MovieScene = Sequence->GetMovieScene())
		{
			if (MovieScene->FindSpawnable(ObjectId))
			{
				return true;
			}
		}

		const FMovieSceneBindingReferences* Refs = Sequence->GetBindingReferences();
		if (Refs)
		{
			return Algo::AnyOf(Refs->GetReferences(ObjectId), [&SharedPlaybackState](const FMovieSceneBindingReference& BindingReference) {
				return BindingReference.CustomBinding && BindingReference.CustomBinding->WillSpawnObject(SharedPlaybackState);
			});
		}
	}
	return false;
}

bool MovieSceneHelpers::IsBoundToSpawnable(UMovieSceneSequence* Sequence, const FGuid& ObjectId, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, int32 BindingIndex)
{
	if (Sequence)
	{
		if (UMovieScene* MovieScene = Sequence->GetMovieScene())
		{
			if (MovieScene->FindSpawnable(ObjectId))
			{
				return true;
			}
		}

		const FMovieSceneBindingReferences* Refs = Sequence->GetBindingReferences();
		if (Refs)
		{
			if (const FMovieSceneBindingReference* Ref = Refs->GetReference(ObjectId, BindingIndex))
			{
				return Ref->CustomBinding && Ref->CustomBinding->WillSpawnObject(SharedPlaybackState);
			}
		}
	}
	return false;
}


FGuid MovieSceneHelpers::TryCreateCustomSpawnableBinding(UMovieSceneSequence* Sequence, UObject* CustomBindingObject)
{
	FGuid NewID;
	if (!Sequence)
	{
		return NewID;
	}
	FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences();
	if (!BindingReferences)
	{
		return NewID;
	}
	static TArray<const TSubclassOf<UMovieSceneCustomBinding>> CachedCustomBindingTypes;
	static bool CustomBindingTypesCached = false;
	if (!CustomBindingTypesCached)
	{
		CustomBindingTypesCached = true;
		MovieSceneHelpers::GetPrioritySortedCustomBindingTypes(CachedCustomBindingTypes);
	}

	UMovieSceneCustomBinding* NewCustomBinding = nullptr;

	for (const TSubclassOf<UMovieSceneCustomBinding>& CustomBindingType : CachedCustomBindingTypes)
	{
		// We only want to use children of UMovieSceneSpawnableBindingBase
		if ((CustomBindingType->IsChildOf<UMovieSceneSpawnableBindingBase>()))
		{
			if (UMovieSceneCustomBinding* CustomBindingCDO = CustomBindingType ? CustomBindingType->GetDefaultObject<UMovieSceneCustomBinding>() : nullptr)
			{
				if (CustomBindingObject && CustomBindingCDO->SupportsBindingCreationFromObject(CustomBindingObject))
				{
					// Create a custom binding from this Object
					NewCustomBinding = CustomBindingCDO->CreateNewCustomBinding(CustomBindingObject, *Sequence->GetMovieScene());
					if (NewCustomBinding)
					{
						break;
					}
				}
			}
		}
	}

	if (NewCustomBinding)
	{
		FString DesiredBindingName = NewCustomBinding->GetDesiredBindingName();
		FString CurrentName = DesiredBindingName.IsEmpty() ? FName::NameToDisplayString(CustomBindingObject->GetName(), false) : DesiredBindingName;
		CurrentName = MovieSceneHelpers::MakeUniqueBindingName(Sequence->GetMovieScene(), CurrentName);

		NewID = Sequence->GetMovieScene()->AddPossessable(CurrentName, NewCustomBinding->GetBoundObjectClass());

		// Add the custom binding
		Sequence->GetBindingReferences()->AddOrReplaceBinding(NewID, NewCustomBinding, 0);
	}

	return NewID;
}

UObject* MovieSceneHelpers::GetSingleBoundObject(UMovieSceneSequence* Sequence, const FGuid& ObjectId, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, int32 BindingIndex)
{
	if (FMovieSceneEvaluationState* EvaluationState = SharedPlaybackState->FindCapability<FMovieSceneEvaluationState>())
	{
		FMovieSceneSequenceIDRef SequenceID = EvaluationState->FindSequenceId(Sequence);
		for (UObject* BoundObject : GetBoundObjects(SequenceID, ObjectId, SharedPlaybackState, BindingIndex))
		{
			if (BoundObject)
			{
				return BoundObject;
			}
		}
	}
	return nullptr;
}

TArray<UObject*> MovieSceneHelpers::GetBoundObjects(const FMovieSceneSequenceID& SequenceID, const FGuid& ObjectId, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, int32 BindingIndex)
{
	TArray<UObject*> BoundObjects;
	if (FMovieSceneEvaluationState* EvaluationState = SharedPlaybackState->FindCapability<FMovieSceneEvaluationState>())
	{
		UMovieSceneSequence* Sequence = EvaluationState->FindSequence(SequenceID);
		UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;

		if (MovieScene)
		{
			if (MovieScene->FindSpawnable(ObjectId))
			{
				for (TWeakObjectPtr<> BoundObject : EvaluationState->FindBoundObjects(FMovieSceneEvaluationOperand(SequenceID, ObjectId), SharedPlaybackState))
				{
					if (BoundObject.Get())
					{
						BoundObjects.Add(BoundObject.Get());
					}
				}
			}
			else if (const FMovieScenePossessable* Possessable = MovieScene->FindPossessable(ObjectId))
			{
				const FMovieSceneBindingReferences* Refs = Sequence->GetBindingReferences();
				if (Refs)
				{
					UObject* ResolutionContext = MovieSceneHelpers::GetResolutionContext(Sequence, ObjectId, SequenceID, SharedPlaybackState);

					if (Possessable->GetParent().IsValid() && Sequence->AreParentContextsSignificant())
					{
						TArrayView<TWeakObjectPtr<>> ParentBoundObjects = EvaluationState->FindBoundObjects(FMovieSceneEvaluationOperand(SequenceID, Possessable->GetParent()), SharedPlaybackState);
						for (TWeakObjectPtr<> Parent : ParentBoundObjects)
						{
							ResolutionContext = Parent.Get();
							if (!ResolutionContext)
							{
								continue;
							}
						}
					}

					TArray<UObject*, TInlineAllocator<1>> SingleBoundObjects;
					UE::UniversalObjectLocator::FResolveParams LocatorResolveParams(ResolutionContext);
					FMovieSceneBindingResolveParams BindingResolveParams{ Sequence, ObjectId, SequenceID, ResolutionContext };
					Refs->ResolveSingleBinding(BindingResolveParams, BindingIndex, LocatorResolveParams, SharedPlaybackState, SingleBoundObjects);

					for (UObject* BoundObject : SingleBoundObjects)
					{
						BoundObjects.Add(BoundObject);
					}
				}
			}
		}
	}
	return BoundObjects;
}

UObject* MovieSceneHelpers::GetObjectTemplate(UMovieSceneSequence* Sequence, const FGuid& ObjectId, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, int32 BindingIndex)
{
	if (Sequence)
	{
		UMovieScene* MovieScene = Sequence->GetMovieScene();
		if (!MovieScene)
		{
			return nullptr;
		}
		if (FMovieSceneSpawnable* Spawnable = MovieScene->FindSpawnable(ObjectId))
		{
			return Spawnable->GetObjectTemplate();
		}
		else if (FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences())
		{
			if (UMovieSceneCustomBinding* CustomBinding = BindingReferences->GetCustomBinding(ObjectId, BindingIndex))
			{
				if (UMovieSceneSpawnableBindingBase* SpawnableBinding = CustomBinding->AsSpawnable(SharedPlaybackState))
				{
					return SpawnableBinding->GetObjectTemplate();
				}
			}
		}
	}
	return nullptr;
}

bool MovieSceneHelpers::SetObjectTemplate(UMovieSceneSequence* Sequence, const FGuid& ObjectId, UObject* InSourceObject, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, int32 BindingIndex)
{
	if (Sequence)
	{
		UMovieScene* MovieScene = Sequence->GetMovieScene();
		if (!MovieScene)
		{
			return false;
		}
		if (FMovieSceneSpawnable* Spawnable = MovieScene->FindSpawnable(ObjectId))
		{
			Spawnable->SetObjectTemplate(InSourceObject);
			return true;
		}
		else if (FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences())
		{
			if (UMovieSceneCustomBinding* CustomBinding = BindingReferences->GetCustomBinding(ObjectId, BindingIndex))
			{
				if (UMovieSceneSpawnableBindingBase* SpawnableBinding = CustomBinding->AsSpawnable(SharedPlaybackState))
				{
					SpawnableBinding->SetObjectTemplate(InSourceObject);
					return true;
				}
			}
		}
	}
	return false;
}

bool MovieSceneHelpers::SupportsObjectTemplate(UMovieSceneSequence* Sequence, const FGuid& ObjectId, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, int32 BindingIndex)
{
	if (Sequence)
	{
		UMovieScene* MovieScene = Sequence->GetMovieScene();
		if (!MovieScene)
		{
			return false;
		}
		if (FMovieSceneSpawnable* Spawnable = MovieScene->FindSpawnable(ObjectId))
		{
			return true;
		}
		else if (const FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences())
		{
			if (const UMovieSceneCustomBinding* CustomBinding = BindingReferences->GetCustomBinding(ObjectId, BindingIndex))
			{
				if (const UMovieSceneSpawnableBindingBase* SpawnableBinding = CustomBinding->AsSpawnable(SharedPlaybackState))
				{
					if (SpawnableBinding && SpawnableBinding->SupportsObjectTemplates())
					{
						return true;
					}
				}
			}
		}
	}
	return false;
}

bool MovieSceneHelpers::CopyObjectTemplate(UMovieSceneSequence* Sequence, const FGuid& ObjectId, UObject* InSourceObject, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, int32 BindingIndex )
{
	if (Sequence && InSourceObject)
	{
		UMovieScene* MovieScene = Sequence->GetMovieScene();
		if (!MovieScene)
		{
			return false;
		}
		if (FMovieSceneSpawnable* Spawnable = MovieScene->FindSpawnable(ObjectId))
		{
			Spawnable->CopyObjectTemplate(*InSourceObject, *Sequence);
			return true;
		}
		else if (FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences())
		{
			if (UMovieSceneCustomBinding* CustomBinding = BindingReferences->GetCustomBinding(ObjectId, BindingIndex))
			{
				if (UMovieSceneSpawnableBindingBase* SpawnableBinding = CustomBinding->AsSpawnable(SharedPlaybackState))
				{
					SpawnableBinding->CopyObjectTemplate(InSourceObject, *Sequence);
					return true;
				}
			}
		}
	}
	return false;
}

#if WITH_EDITORONLY_DATA

const UClass* MovieSceneHelpers::GetBoundObjectClass(UMovieSceneSequence* Sequence, const FGuid& ObjectId, int32 BindingIndex)
{
	if (Sequence)
	{
		UMovieScene* MovieScene = Sequence->GetMovieScene();
		if (!MovieScene)
		{
			return nullptr;
		}
		if (FMovieSceneSpawnable* Spawnable = MovieScene->FindSpawnable(ObjectId))
		{
			if (UObject* ObjectTemplate = Spawnable->GetObjectTemplate())
			{
				return ObjectTemplate->GetClass();
			}
		}
		else if (FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences())
		{
			if (UMovieSceneCustomBinding* CustomBinding = BindingReferences->GetCustomBinding(ObjectId, BindingIndex))
			{
				return CustomBinding->GetBoundObjectClass();
			}
		}

		if (FMovieScenePossessable* Possessable = MovieScene->FindPossessable(ObjectId))
		{
			return Possessable->GetPossessedObjectClass();
		}
	}
	return nullptr;
}

#endif

bool MovieSceneHelpers::ResetSectionOrder(UMovieSceneTrack* Track, const TArray<UMovieSceneSection*>& NewSections)
{
	if (Track)
	{
		const TArray<UMovieSceneSection*>& OldSections = Track->GetAllSections();
		//make sure the sections are equivalent but maybe not in same order
		if (NewSections.Num() != OldSections.Num())
		{
			return false;
		}
		//make sure new section have the right track parent and no nullptrs
		for (UMovieSceneSection* NewSection : NewSections)
		{
			if (NewSection == nullptr || NewSection->GetTypedOuter<UMovieSceneTrack>() != Track)
			{
				return false;
			}
		}
		//use set here to handle both duplicates and checking equivalency
		TSet<UMovieSceneSection*> NewSet(NewSections);
		TSet<UMovieSceneSection*> OldSet(OldSections);
		
		if (NewSet.Num() == OldSet.Num() && NewSet.Includes(OldSet))
		{
			//remove all existing
			int32 NumSections = OldSections.Num();
			for (int32 Index = NumSections - 1; Index >= 0; --Index)
			{
				Track->RemoveSectionAt(UMovieSceneTrack::FSectionIndexParameter(Index));
			}
			//add back new copy in order
			for (UMovieSceneSection* Section : NewSections)
			{
				Track->AddSection(UMovieSceneTrack::FSectionParameter(*Section));
			}
			return true;
		}
	}
	return false;
}

void MovieSceneHelpers::GetPrioritySortedCustomBindingTypes(TArray<const TSubclassOf<UMovieSceneCustomBinding>>& OutCustomBindingTypes)
{
	OutCustomBindingTypes.Empty();
	for (TObjectIterator<UClass> It; It; ++It)
	{
		if (It->IsChildOf(UMovieSceneCustomBinding::StaticClass()) && !It->HasAnyClassFlags(CLASS_Abstract))
		{
#if WITH_EDITOR
			if (!UMovieScene::IsCustomBindingClassAllowed(*It))
			{
				continue;
			}
#endif
			// Skip SKEL and REINST classes.
			if (It->GetName().StartsWith(TEXT("SKEL_")) || It->GetName().StartsWith(TEXT("REINST_")))
			{
				continue;
			}
			OutCustomBindingTypes.Add(*It);
		}
	}// Sort by spawner priority to allow disambiguation for similar object types
	OutCustomBindingTypes.Sort([](const TSubclassOf<UMovieSceneCustomBinding>& A, const TSubclassOf<UMovieSceneCustomBinding>& B) {
		return A && B && A->GetDefaultObject<UMovieSceneCustomBinding>()->GetCustomBindingPriority() > B->GetDefaultObject<UMovieSceneCustomBinding>()->GetCustomBindingPriority(); });
}


TSharedRef<UE::MovieScene::FSharedPlaybackState> MovieSceneHelpers::CreateTransientSharedPlaybackState(UObject* WorldContext, UMovieSceneSequence* Sequence)
{
	verify(WorldContext && Sequence);

	using namespace UE::MovieScene;
	FSharedPlaybackStateCreateParams CreateParams;
	CreateParams.PlaybackContext = WorldContext;
	TSharedRef<FSharedPlaybackState> TransientPlaybackState = MakeShared<FSharedPlaybackState>(*Sequence, CreateParams);

	TSharedRef<FMovieSceneEvaluationState> State = MakeShared<FMovieSceneEvaluationState>();
	TransientPlaybackState->AddCapabilityShared(State);
	State->AssignSequence(MovieSceneSequenceID::Root, *Sequence, TransientPlaybackState);

	return TransientPlaybackState;
}

UObject* MovieSceneHelpers::GetResolutionContext(UMovieSceneSequence* Sequence, const FGuid& ObjectId, const FMovieSceneSequenceID& SequenceID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState)
{
	if (!Sequence)
	{
		return nullptr;
	}
	UObject* ResolutionContext = SharedPlaybackState->GetPlaybackContext();
	if (FMovieScenePossessable* Possessable = Sequence->GetMovieScene()->FindPossessable(ObjectId))
	{
		if (Possessable->GetParent().IsValid() && Sequence->AreParentContextsSignificant())
		{
			if (FMovieSceneEvaluationState* EvaluationState = SharedPlaybackState->FindCapability<FMovieSceneEvaluationState>())
			{
				TArrayView<TWeakObjectPtr<>> ParentBoundObjects = EvaluationState->FindBoundObjects(Possessable->GetParent(), SequenceID, SharedPlaybackState);
				for (TWeakObjectPtr<> Parent : ParentBoundObjects)
				{
					ResolutionContext = Parent.Get();
					if (!ResolutionContext)
					{
						continue;
					}
				}
			}
		}
	}
	return ResolutionContext;
}

const UMovieSceneCondition* MovieSceneHelpers::GetSequenceCondition(const UMovieSceneTrack* Track, const UMovieSceneSection* Section, bool bFromCompilation)
{
	TArray<UMovieSceneCondition*, TInlineAllocator<1>> Conditions;

	if (Track)
	{
		// Track Condition
		if (Track->ConditionContainer.Condition)
		{
			Conditions.Add(Track->ConditionContainer.Condition);
		}

		// Track Row Condition
		if (Section)
		{
			if (const FMovieSceneTrackRowMetadata* TrackRowMetadata = Track->FindTrackRowMetadata(Section->GetRowIndex()))
			{
				if (TrackRowMetadata->ConditionContainer.Condition)
				{
					Conditions.Add(TrackRowMetadata->ConditionContainer.Condition);
				}
			}
		}
	}
	
	// Section Condition
	if (Section && Section->ConditionContainer.Condition)
	{
		Conditions.Add(Section->ConditionContainer.Condition);
	}

	if (Conditions.IsEmpty())
	{
		return nullptr;
	}
	else if (Conditions.Num() == 1)
	{
		return Conditions[0];
	}
	else
	{
		// Generate a group condition. During compilation this will get referenced by the entity metadata, otherwise this is considered a temporary and the caller
		// is responsible for holding a reference to this condition.
		UMovieScene* MovieScene = Section ? Section->GetTypedOuter<UMovieScene>() : Track ? Track->GetTypedOuter<UMovieScene>() : nullptr;
		check(MovieScene);
		UMovieSceneGroupCondition* GroupCondition = NewObject<UMovieSceneGroupCondition>(MovieScene);
		for (UMovieSceneCondition* Condition : Conditions)
		{
			FMovieSceneConditionContainer& ConditionContainer = GroupCondition->SubConditions.AddDefaulted_GetRef();
			ConditionContainer.Condition = Condition;
		}
		if (bFromCompilation)
		{
			MovieScene->AddGeneratedCondition(GroupCondition);
		}
		return GroupCondition;
	}
}


bool MovieSceneHelpers::EvaluateSequenceCondition(const FGuid& BindingID, const FMovieSceneSequenceID& SequenceID, const UMovieSceneCondition* Condition, UObject* ConditionOwnerObject, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState)
{
	using namespace UE::MovieScene;
	if (!Condition)
	{
		return true;
	}

	const FSequenceInstance& SequenceInstance = SharedPlaybackState->GetLinker()->GetInstanceRegistry()->GetInstance(SharedPlaybackState->GetRootInstanceHandle());
	return SequenceInstance.EvaluateCondition(BindingID, SequenceID, Condition, ConditionOwnerObject);
}

MovieSceneHelpers::FMovieSceneScopedPackageDirtyGuard::FMovieSceneScopedPackageDirtyGuard(USceneComponent* InComponent)
{
	Component = InComponent;
	if (Component && Component->GetPackage())
	{
		bPackageWasDirty = Component->GetPackage()->IsDirty();
	}
}

MovieSceneHelpers::FMovieSceneScopedPackageDirtyGuard::~FMovieSceneScopedPackageDirtyGuard()
{
	if (Component && Component->GetPackage())
	{
		Component->GetPackage()->SetDirtyFlag(bPackageWasDirty);
	}
}

