// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneDynamicBinding.h"

#include "EngineUtils.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerController.h"
#include "MovieScene.h"
#include "MovieSceneBinding.h"
#include "MovieScenePossessable.h"
#include "MovieSceneSequence.h"
#include "MovieSceneSpawnable.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneDynamicBinding)

FMovieSceneDynamicBindingResolveResult UBuiltInDynamicBindingResolverLibrary::ResolveToPlayerPawn(UObject* WorldContextObject, int32 PlayerControllerIndex)
{
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		const int32 NumPlayerControllers = World->GetNumPlayerControllers();
		if (ensure(PlayerControllerIndex < NumPlayerControllers))
		{
			FConstPlayerControllerIterator It = World->GetPlayerControllerIterator();
			for (int32 Index = 0; Index < PlayerControllerIndex; ++Index)
			{
				++It;
			}
			if (APlayerController* PlayerController = It->Get())
			{
				APawn* Pawn = PlayerController->GetPawn();
				FMovieSceneDynamicBindingResolveResult Result;
				Result.Objects.Add(Pawn);
				Result.Object = Pawn; // Object deprecated 5.7
				Result.bIsPossessedObject = true;
				return Result;
			}
		}
	}

	return FMovieSceneDynamicBindingResolveResult();
}

