// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneDynamicBindingInvoker.h"

#include "CoreMinimal.h"
#include "Engine/World.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Evaluation/SequenceDirectorPlaybackCapability.h"
#include "MovieSceneDynamicBinding.h"
#include "MovieScenePossessable.h"
#include "MovieSceneSequence.h"
#include "MovieSceneSpawnable.h"
#include "UObject/UnrealType.h"
#include "MovieScene.h"

FMovieSceneDynamicBindingResolveResult FMovieSceneDynamicBindingInvoker::ResolveDynamicBinding(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, UMovieSceneSequence* Sequence, const FMovieSceneSequenceID& SequenceID, const FGuid& InGuid, const FMovieSceneDynamicBinding& DynamicBinding)
{
	using namespace UE::MovieScene;

	if (!ensure(Sequence))
	{
		// Sequence is somehow null... fallback to default behavior.
		return FMovieSceneDynamicBindingResolveResult();
	}

	UFunction* DynamicBindingFunc = DynamicBinding.Function.Get();
	if (!DynamicBindingFunc)
	{
		// No dynamic binding, fallback to default behavior.
		return FMovieSceneDynamicBindingResolveResult();
	}

	// Auto-add the director playback capability, which is just really a cache for director instances after
	// they've been created by the sequences in the hierarchy.
	FSequenceDirectorPlaybackCapability* DirectorCapability = SharedPlaybackState->FindCapability<FSequenceDirectorPlaybackCapability>();
	if (!DirectorCapability)
	{
		TSharedRef<FSharedPlaybackState> MutableState = ConstCastSharedRef<FSharedPlaybackState>(SharedPlaybackState);
		DirectorCapability = &MutableState->AddCapability<FSequenceDirectorPlaybackCapability>();
	}

	UObject* DirectorInstance = DirectorCapability->GetOrCreateDirectorInstance(SharedPlaybackState, SequenceID);
	if (!DirectorInstance)
	{
#if !NO_LOGGING
		UE_LOGF(LogMovieScene, Warning,
			"%ls: Failed to resolve dynamic binding '%ls' because no director instance was available.",
			*Sequence->GetName(), *DynamicBindingFunc->GetName());
#endif
		// Fallback to default behavior.
		return FMovieSceneDynamicBindingResolveResult();
	}

	if (!DynamicBindingFunc->IsIn(DirectorInstance->GetClass()))
	{
		UE_LOGF(LogMovieScene, Warning,
			"%ls: Failed to resolve dynamic binding '%ls' because the function is not contained within the director instance.",
			*Sequence->GetName(), *DynamicBindingFunc->GetName());

		return FMovieSceneDynamicBindingResolveResult();
	}

#if WITH_EDITOR
	const static FName NAME_CallInEditor("CallInEditor");

	UWorld* World = DirectorInstance->GetWorld();
	const bool bIsGameWorld = World && World->IsGameWorld();

	if (!bIsGameWorld && !DynamicBindingFunc->HasMetaData(NAME_CallInEditor))
	{
		UE_LOGF(LogMovieScene, Verbose,
			"%ls: Refusing to resolve dynamic binding '%ls' in editor world because function '%ls' has 'Call in Editor' set to false.",
			*Sequence->GetName(), *LexToString(InGuid), *DynamicBindingFunc->GetName());
		// Fallback to default behavior.
		return FMovieSceneDynamicBindingResolveResult();
	}
#endif // WITH_EDITOR

	UE_LOGF(LogMovieScene, VeryVerbose,
		"%ls: Resolving dynamic binding '%ls' with function '%ls'.",
		*Sequence->GetName(), *LexToString(InGuid), *DynamicBindingFunc->GetName());

	FMovieSceneDynamicBindingResolveParams ResolveParams;
	ResolveParams.ObjectBindingID = InGuid;
	ResolveParams.Sequence = Sequence;
	ResolveParams.RootSequence = SharedPlaybackState->GetRootSequence();
	return InvokeDynamicBinding(DirectorInstance, DynamicBinding, ResolveParams);
}

FMovieSceneDynamicBindingResolveResult FMovieSceneDynamicBindingInvoker::InvokeDynamicBinding(UObject* DirectorInstance, const FMovieSceneDynamicBinding& DynamicBinding, const FMovieSceneDynamicBindingResolveParams& ResolveParams)
{
	FMovieSceneDynamicBindingResolveResult Result;

	// Do some basic checks.
	UFunction* DynamicBindingFunc = DynamicBinding.Function.Get();
	if (!ensure(DynamicBindingFunc))
	{
		return Result;
	}

	// Parse all function parameters.
	uint8* Parameters = (uint8*)FMemory_Alloca(DynamicBindingFunc->ParmsSize + DynamicBindingFunc->MinAlignment);
	Parameters = Align(Parameters, DynamicBindingFunc->MinAlignment);

	// Initialize parameters.
	FMemory::Memzero(Parameters, DynamicBindingFunc->ParmsSize);

	FStructProperty* ReturnProp = nullptr;
	for (TFieldIterator<FProperty> It(DynamicBindingFunc); It; ++It)
	{
		FProperty* LocalProp = *It;
		checkSlow(LocalProp);
		if (!LocalProp->HasAnyPropertyFlags(CPF_ZeroConstructor) && LocalProp->HasAllPropertyFlags(CPF_Parm))
		{
			LocalProp->InitializeValue_InContainer(Parameters);
		}

		if (LocalProp->HasAnyPropertyFlags(CPF_ReturnParm))
		{
			ensureMsgf(ReturnProp == nullptr,
				TEXT("Found more than one return parameter in dynamic binding resolver function!"));
			ReturnProp = CastFieldChecked<FStructProperty>(LocalProp);
		}
	}

	// Set the resolve parameter struct if we need to pass it to the function.
	if (FProperty* ResolveParamsProp = DynamicBinding.ResolveParamsProperty.Get())
	{
		ResolveParamsProp->SetValue_InContainer(Parameters, &ResolveParams);
	}

#if WITH_EDITOR
	// In the editor we need to be more forgiving, because we might have temporarily invalid states, such as
	// when undo-ing operations.
	if (ReturnProp != nullptr && ReturnProp->Struct == FMovieSceneDynamicBindingResolveResult::StaticStruct())
#else
	if (ensureMsgf(ReturnProp != nullptr && ReturnProp->Struct == FMovieSceneDynamicBindingResolveResult::StaticStruct(),
		TEXT("The dynamic binding resolver function has no return value of type FMovieSceneDynamicBindingResolveResult")))
#endif
	{
		// Invoke the function.
		DirectorInstance->ProcessEvent(DynamicBindingFunc, Parameters);

		// Grab the result value.
		ReturnProp->GetValue_InContainer(Parameters, static_cast<void*>(&Result));
	}

	// Destroy parameters.
	for (TFieldIterator<FProperty> It(DynamicBindingFunc); It; ++It)
	{
		FProperty* LocalProp = *It;
		checkSlow(LocalProp);

		if (LocalProp->HasAllPropertyFlags(CPF_Parm))
		{
			It->DestroyValue_InContainer(Parameters);
		}
	}

	return Result;
}

