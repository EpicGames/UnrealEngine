// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneFolder.h"
#include "MovieScene.h"
#include "MovieSceneTrack.h"
#include "Algo/Count.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneFolder)

void GetMovieSceneFoldersRecursive(TArrayView<UMovieSceneFolder* const> InFoldersToRecurse, TArray<UMovieSceneFolder*>& OutFolders)
{
	for (UMovieSceneFolder* Folder : InFoldersToRecurse)
	{
		if (Folder)
		{
			OutFolders.Add(Folder);
			GetMovieSceneFoldersRecursive(Folder->GetChildFolders(), OutFolders);
		}
	}
}

UMovieSceneFolder::UMovieSceneFolder( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
#if WITH_EDITORONLY_DATA
	, FolderColor(FColor::White)
	, SortingOrder(-1)
#endif
{
}

FName UMovieSceneFolder::GetFolderName() const
{
	return FolderName;
}


void UMovieSceneFolder::SetFolderName( FName InFolderName )
{
	Modify();

	FolderName = InFolderName;
}


TArrayView<UMovieSceneFolder* const> UMovieSceneFolder::GetChildFolders() const
{
	return ChildFolders;
}


void UMovieSceneFolder::AddChildFolder( UMovieSceneFolder* InChildFolder )
{
	Modify();

#if WITH_EDITORONLY_DATA
	// Ensure the added folder does not belong to any other folder in the same scene.
	UMovieScene* OwningScene = GetTypedOuter<UMovieScene>();
	if (OwningScene)
	{
		TArray<UMovieSceneFolder*> AllFolders;
		GetMovieSceneFoldersRecursive(OwningScene->GetRootFolders(), AllFolders);

		for (UMovieSceneFolder* MovieSceneFolder : AllFolders)
		{
			MovieSceneFolder->RemoveChildFolder(InChildFolder);
		}

		if (OwningScene->GetRootFolders().Contains(InChildFolder))
		{
			OwningScene->Modify();
			OwningScene->RemoveRootFolder(InChildFolder);
		}
	}
#endif

	// Now add it as a child of ourself
	ChildFolders.Add( InChildFolder );

	EventHandlers.Trigger(&UE::MovieScene::IFolderEventHandler::OnChildFolderAdded, InChildFolder);
}


void UMovieSceneFolder::RemoveChildFolder( UMovieSceneFolder* InChildFolder )
{
	Modify();

	ChildFolders.Remove(InChildFolder);

	EventHandlers.Trigger(&UE::MovieScene::IFolderEventHandler::OnChildFolderRemoved, InChildFolder);
}


const TArray<UMovieSceneTrack*>& UMovieSceneFolder::GetChildTracks() const
{
	return ChildTracks;
}


void UMovieSceneFolder::AddChildTrack( UMovieSceneTrack* InTrack )
{
	Modify();

#if WITH_EDITORONLY_DATA
	// Ensure the added track does not belong to any other folder in the same scene.
	UMovieScene* OwningScene = GetTypedOuter<UMovieScene>();
	if (OwningScene)
	{
		TArray<UMovieSceneFolder*> AllFolders;
		GetMovieSceneFoldersRecursive(OwningScene->GetRootFolders(), AllFolders);

		for (UMovieSceneFolder* MovieSceneFolder : AllFolders)
		{
			MovieSceneFolder->RemoveChildTrack(InTrack);
		}
	}
#endif

	ChildTracks.Add( InTrack );

	EventHandlers.Trigger(&UE::MovieScene::IFolderEventHandler::OnTrackAdded, InTrack);
}


void UMovieSceneFolder::RemoveChildTrack( UMovieSceneTrack* InTrack )
{
	Modify();

	if (ChildTracks.Remove( InTrack ) > 0)
	{
		EventHandlers.Trigger(&UE::MovieScene::IFolderEventHandler::OnTrackRemoved, InTrack);
	}
}


void UMovieSceneFolder::ClearChildTracks()
{
	Modify();

	ChildTracks.Empty();
}


const TArray<FGuid>& UMovieSceneFolder::GetChildObjectBindings() const
{
	return ChildObjectBindings;
}


void UMovieSceneFolder::AddChildObjectBinding(const FGuid& InObjectBinding )
{
	Modify();

#if WITH_EDITORONLY_DATA
	// Ensure the added object does not belong to any other folder in the same scene.
	UMovieScene* OwningScene = GetTypedOuter<UMovieScene>();
	if (OwningScene)
	{
		TArray<UMovieSceneFolder*> AllFolders;
		GetMovieSceneFoldersRecursive(OwningScene->GetRootFolders(), AllFolders);

		for (UMovieSceneFolder* MovieSceneFolder : AllFolders)
		{
			MovieSceneFolder->RemoveChildObjectBinding(InObjectBinding);
		}
	}
#endif

	ChildObjectBindings.Add( InObjectBinding );

	EventHandlers.Trigger(&UE::MovieScene::IFolderEventHandler::OnObjectBindingAdded, InObjectBinding);
}


void UMovieSceneFolder::RemoveChildObjectBinding( const FGuid& InObjectBinding )
{
	Modify();

	if (ChildObjectBindings.Remove( InObjectBinding ) > 0)
	{
		EventHandlers.Trigger(&UE::MovieScene::IFolderEventHandler::OnObjectBindingRemoved, InObjectBinding);
	}
}

void UMovieSceneFolder::ClearChildObjectBindings()
{
	Modify();

	ChildObjectBindings.Empty();
}

void UMovieSceneFolder::PostLoad()
{
	// Remove any null folders
	for (int32 ChildFolderIndex = 0; ChildFolderIndex < ChildFolders.Num(); )
	{
		if (ChildFolders[ChildFolderIndex] == nullptr)
		{
			ChildFolders.RemoveAt(ChildFolderIndex);
		}
		else
		{
			++ChildFolderIndex;
		}
	}

#if WITH_EDITORONLY_DATA
	// Historically we've not been very strict about ensuring a folder, track, or object binding existed
	// only in one folder. This is now enforced (via automatically removing the item from other folders
	// when they are added to this folder), and checked (the tree view trips an ensure on invalid children)
	// but all legacy content can still have the invalid children which continuously trips the ensure.
	// Since we now enforce child-only-exists-in-one-folder, we can safely remove any invalid children on
	// load, and be confident that we shouldn't run into situations in the future where an invalid child is
	// left in a folder.
	UMovieScene* OwningScene = GetTypedOuter<UMovieScene>();
	if (OwningScene)
	{
		// Validate child Tracks
		for(int32 ChildTrackIndex = 0; ChildTrackIndex < ChildTracks.Num(); ChildTrackIndex++)
		{
			const UMovieSceneTrack* ChildTrack = ChildTracks[ChildTrackIndex];
			if (!OwningScene->GetTracks().Contains(ChildTrack))
			{
				ChildTracks.RemoveAt(ChildTrackIndex);
				ChildTrackIndex--;

				UE_LOGF(LogMovieScene, Warning, "Folder (%ls) in Sequence (%ls) contained a reference to a Track (%ls) that no longer exists in the sequence, removing.", *GetFolderName().ToString(), *OwningScene->GetPathName(), *GetNameSafe(ChildTrack));
			}
		}

		// Validate child Object Bindings
		for (int32 ChildObjectBindingIndex = 0; ChildObjectBindingIndex < ChildObjectBindings.Num(); ChildObjectBindingIndex++)
		{
			const FGuid& ChildBinding = ChildObjectBindings[ChildObjectBindingIndex];
			if (!OwningScene->FindBinding(ChildBinding))
			{
				UE_LOGF(LogMovieScene, Warning, "Folder (%ls) in Sequence (%ls) contained a reference to an Object Binding (%ls) that no longer exists in the sequence, removing.", *GetFolderName().ToString(), *OwningScene->GetPathName(), *ChildBinding.ToString());

				ChildObjectBindings.RemoveAt(ChildObjectBindingIndex);
				ChildObjectBindingIndex--;
			}
		}

		// A folder should exist in only one place in the tree, as a child of ourself. If they exist in more
		// than one place, two folders point to the same actual UObject, so we'll remove it from ourself. When
		// the that folder is PostLoaded it will search the whole tree again and only find the one reference as
		// our reference will no longer exist.
		TArray<UMovieSceneFolder*> AllFolders;
		GetMovieSceneFoldersRecursive(OwningScene->GetRootFolders(), AllFolders);
		for (int32 ChildFolderIndex = 0; ChildFolderIndex < ChildFolders.Num(); ChildFolderIndex++)
		{
			int32 NumFolderInstances = Algo::Count(AllFolders, ChildFolders[ChildFolderIndex]);
			if (NumFolderInstances > 1)
			{
				UE_LOGF(LogMovieScene, Warning, "Folder (%ls) in Sequence (%ls) contained a reference to an Folder (%ls) that exists in multiple places in the sequence, removing.", *GetFolderName().ToString(), *OwningScene->GetPathName(), *ChildFolders[ChildFolderIndex]->GetFolderName().ToString());

				ChildFolders.RemoveAt(ChildFolderIndex);
				ChildFolderIndex--;
			}
		}
	}
#endif

	Super::PostLoad();
}

UMovieSceneFolder* UMovieSceneFolder::FindFolderContaining(const FGuid& InObjectBinding)
{
	for (FGuid ChildGuid : GetChildObjectBindings())
	{
		if (ChildGuid == InObjectBinding)
		{
			return this;
		}
	}

	for (UMovieSceneFolder* ChildFolder : GetChildFolders())
	{
		UMovieSceneFolder* Folder = ChildFolder->FindFolderContaining(InObjectBinding);
		if (Folder != nullptr)
		{
			return Folder;
		}
	}

	return nullptr;
}

UMovieSceneFolder* UMovieSceneFolder::FindFolderContaining(const UMovieSceneTrack* InTrack)
{
	if (ChildTracks.Contains(InTrack))
	{
		return this;
	}

	for (UMovieSceneFolder* ChildFolder : GetChildFolders())
	{
		UMovieSceneFolder* Folder = ChildFolder->FindFolderContaining(InTrack);
		if (Folder != nullptr)
		{
			return Folder;
		}
	}

	return nullptr;
}

void
TraverseFolder(UMovieSceneFolder* Folder, TMap<UMovieSceneFolder*, UMovieSceneFolder*>& ChildToParentMap)
{
	for (UMovieSceneFolder* Child : Folder->GetChildFolders())
	{
		ChildToParentMap.Add(Child, Folder);

		TraverseFolder(Child, ChildToParentMap);
	}
}

void UMovieSceneFolder::CalculateFolderPath(UMovieSceneFolder* Folder, TArrayView<UMovieSceneFolder* const> RootFolders, TArray<FName>& FolderPath)
{
	TMap<UMovieSceneFolder*, UMovieSceneFolder*> ChildToParentMap;
	for (UMovieSceneFolder* RootFolder : RootFolders)
	{
		TraverseFolder(RootFolder, ChildToParentMap);
	}

	FolderPath.Add(Folder->GetFolderName());

	UMovieSceneFolder* Parent = ChildToParentMap.Contains(Folder) ? ChildToParentMap[Folder] : nullptr;
	while (Parent)
	{
		FolderPath.Insert(Parent->GetFolderName(), 0);

		Parent = ChildToParentMap.Contains(Parent) ? ChildToParentMap[Parent] : nullptr;
	}
}

UMovieSceneFolder* UMovieSceneFolder::GetFolderWithPath(const TArray<FName>& InFolderPath, const TArray<UMovieSceneFolder*>& InFolders, TArrayView<UMovieSceneFolder* const> RootFolders)
{
	for (UMovieSceneFolder* Folder : InFolders)
	{
		TArray<FName> FolderPath;
		UMovieSceneFolder::CalculateFolderPath(Folder, RootFolders, FolderPath);
		if (FolderPath == InFolderPath)
		{
			return Folder;
		}
	}

	return nullptr;
}

void UMovieSceneFolder::Serialize( FArchive& Archive )
{
	if ( Archive.IsLoading() )
	{
		Super::Serialize( Archive );

#if WITH_EDITOR
		if (ChildMasterTracks_DEPRECATED.Num())
		{
			ChildTracks = ChildMasterTracks_DEPRECATED;
			ChildMasterTracks_DEPRECATED.Empty();
		}
#endif

		ChildObjectBindings.Empty();
		for ( const FString& ChildObjectBindingString : ChildObjectBindingStrings )
		{
			FGuid ChildObjectBinding;
			FGuid::Parse( ChildObjectBindingString, ChildObjectBinding );
			ChildObjectBindings.Add( ChildObjectBinding );
		}
	}
	else
	{
		ChildObjectBindingStrings.Empty();
		for ( const FGuid& ChildObjectBinding : ChildObjectBindings )
		{
			ChildObjectBindingStrings.Add( ChildObjectBinding.ToString() );
		}
		Super::Serialize( Archive );
	}
}

FName UMovieSceneFolder::MakeUniqueChildFolderName(FName InName) const
{
	return MakeUniqueChildFolderName(InName, GetChildFolders());
}

FName UMovieSceneFolder::MakeUniqueChildFolderName(FName InName, TArrayView<UMovieSceneFolder* const> InFolders)
{
	bool bFoundExactDuplicate = false;

	int32 NextNameIndex = InName.GetNumber();

	// Iterate all children, finding a new unique name index for any that have the same base name
	for (UMovieSceneFolder* Child : InFolders)
	{
		constexpr bool bCompareNumber = false;
		if (InName.IsEqual(Child->GetFolderName(), ENameCase::IgnoreCase, bCompareNumber))
		{
			NextNameIndex = FMath::Max(NextNameIndex, Child->GetFolderName().GetNumber()) + 1;
		}

		if (InName == Child->GetFolderName())
		{
			bFoundExactDuplicate = true;
		}
	}

	if (bFoundExactDuplicate)
	{
		InName.SetNumber(NextNameIndex);
	}

	return InName;
}

#if WITH_EDITOR

void UMovieSceneFolder::PostEditUndo()
{
	Super::PostEditUndo();

	EventHandlers.Trigger(&UE::MovieScene::IFolderEventHandler::OnPostUndo);
}

void UMovieSceneFolder::PostEditUndo(TSharedPtr<ITransactionObjectAnnotation> TransactionAnnotation)
{
	Super::PostEditUndo(TransactionAnnotation);

	EventHandlers.Trigger(&UE::MovieScene::IFolderEventHandler::OnPostUndo);
}

#endif
