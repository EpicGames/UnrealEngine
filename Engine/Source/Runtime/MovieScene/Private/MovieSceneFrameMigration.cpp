// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneFrameMigration.h"
#include "UObject/PropertyPortFlags.h"
#include "MovieSceneFwd.h"
#include "Misc/FrameRate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneFrameMigration)

bool FMovieSceneFrameRange::Serialize(FArchive& Ar)
{
	Ar << Value;
	return true;
}

TRange<FFrameNumber> FMovieSceneFrameRange::FromFloatRange(const TRange<float>& InFloatRange)
{
	FFrameRate LegacyFrameRate = GetLegacyConversionFrameRate();

	// Always upgrade frame ranges as inclusive since floats will round down to the closest frame number
	TRange<FFrameNumber> NewRange;
	if (InFloatRange.GetLowerBound().IsClosed())
	{
		FFrameNumber ClampedTime = UpgradeLegacyMovieSceneTime(nullptr, LegacyFrameRate, InFloatRange.GetLowerBoundValue());
		NewRange.SetLowerBound(TRangeBound<FFrameNumber>::Inclusive(ClampedTime));
	}

	if (InFloatRange.GetUpperBound().IsClosed())
	{
		FFrameNumber ClampedTime = UpgradeLegacyMovieSceneTime(nullptr, LegacyFrameRate, InFloatRange.GetUpperBoundValue());
		NewRange.SetUpperBound(TRangeBound<FFrameNumber>::Inclusive(ClampedTime));
	}

	return NewRange;
}

bool FMovieSceneFrameRange::SerializeFromMismatchedTag(FPropertyTag const& Tag, FStructuredArchive::FSlot Slot)
{
	static const FName NAME_FloatRange("FloatRange");
	if (Tag.GetType().IsStruct(NAME_FloatRange))
	{
		UScriptStruct* FloatStruct = TBaseStructure<FFloatRange>::Get();

		FFloatRange FloatRange 
#if WITH_EDITORONLY_DATA
			= MigrationDefault
#endif
			;

		FloatStruct->SerializeItem(Slot, &FloatRange, nullptr);

		Value = FromFloatRange(FloatRange);
		return true;
	}

	return false;
}

bool FMovieSceneFrameRange::ExportTextItem(FString& ValueStr, FMovieSceneFrameRange const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	FString String;
	if (Value.GetLowerBound().IsOpen())
	{
		String.Append("Open");
		String.AppendChar(':');
		String.AppendChar('0');
	}
	else if (Value.GetLowerBound().IsInclusive())
	{
		String.Append("Inclusive");
		String.AppendChar(':');
		String.Append(FString::FromInt(Value.GetLowerBoundValue().Value));
	}
	else if (Value.GetLowerBound().IsExclusive())
	{
		String.Append("Exclusive");
		String.AppendChar(':');
		String.Append(FString::FromInt(Value.GetLowerBoundValue().Value));
	}
	
	String.AppendChar(':');
	if (Value.GetUpperBound().IsOpen())
	{
		String.Append("Open");
		String.AppendChar(':');
		String.AppendChar('0');
	}
	else if (Value.GetUpperBound().IsInclusive())
	{
		String.Append("Inclusive");
		String.AppendChar(':');
		String.Append(FString::FromInt(Value.GetUpperBoundValue().Value));
	}
	else if (Value.GetUpperBound().IsExclusive())
	{
		String.Append("Exclusive");
		String.AppendChar(':');
		String.Append(FString::FromInt(Value.GetUpperBoundValue().Value));
	}
	
	ValueStr += String;
	return true;
}

bool FMovieSceneFrameRange::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText)
{
	FString Text;
	const TCHAR* NewBuffer = FPropertyHelpers::ReadToken(Buffer, Text, 1);
	if (NewBuffer)
	{
		TArray<FString> Array;
		Text.ParseIntoArray(Array, TEXT(":"), false);
		if (Array.Num() == 4) {
			if (Array[0] == FString("Open")) 
			{ 
				Value.SetLowerBound(TRangeBound<FFrameNumber>::Open());
			}
			else if (Array[0] == FString("Inclusive"))
			{
				int32 IValue = FCString::Atoi(*(Array[1]));
				Value.SetLowerBound(TRangeBound<FFrameNumber>::Inclusive(IValue));
			}
			else if (Array[0] == FString("Exclusive"))
			{
				int32 IValue = FCString::Atoi(*(Array[1]));
				Value.SetLowerBound(TRangeBound<FFrameNumber>::Exclusive(IValue));
			}
			if (Array[2] == FString("Open"))
			{
				Value.SetUpperBound(TRangeBound<FFrameNumber>::Open());
			}
			else if (Array[2] == FString("Inclusive"))
			{
				int32 IValue = FCString::Atoi(*(Array[3]));
				Value.SetUpperBound(TRangeBound<FFrameNumber>::Inclusive(IValue));
			}
			else if (Array[2] == FString("Exclusive"))
			{
				int32 IValue = FCString::Atoi(*(Array[3]));
				Value.SetUpperBound(TRangeBound<FFrameNumber>::Exclusive(IValue));
			}
			return true;
		}
	}
	return false;
}
