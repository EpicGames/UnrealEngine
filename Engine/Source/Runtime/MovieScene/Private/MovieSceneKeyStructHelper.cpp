// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneKeyStructHelper.h"


void FMovieSceneKeyStructHelper::Add(FMovieSceneChannelValueHelper&& InHelper)
{
	if (!UnifiedKeyTime.IsSet() && InHelper->KeyHandleAndTime.IsSet())
	{
		UnifiedKeyTime = InHelper->KeyHandleAndTime.GetValue().Get<1>();
	}

	Helpers.Add(MoveTemp(InHelper));
}

void FMovieSceneKeyStructHelper::SetStartingValues()
{
	if (UnifiedKeyTime.IsSet())
	{
		for (FMovieSceneChannelValueHelper& Helper : Helpers)
		{
			Helper->SetUserValueFromTime(UnifiedKeyTime.GetValue());
		}
	}
}

TOptional<FFrameNumber> FMovieSceneKeyStructHelper::GetUnifiedKeyTime() const
{
	return UnifiedKeyTime;
}

void FMovieSceneKeyStructHelper::Apply(FFrameNumber InUnifiedTime)
{
	for (FMovieSceneChannelValueHelper& Helper : Helpers)
	{
		Helper->SetKeyFromUserValue(InUnifiedTime);
	}
}