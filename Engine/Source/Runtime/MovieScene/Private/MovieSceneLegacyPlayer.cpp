// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneLegacyPlayer.h"

#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneInstanceRegistry.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "Evaluation/SequenceDirectorPlaybackCapability.h"
#include "IMovieScenePlaybackClient.h"
#include "MovieSceneSpawnRegister.h"

namespace UE::MovieScene
{

UE_DEFINE_MOVIESCENE_PLAYBACK_CAPABILITY(ILegacyPlayerProviderPlaybackCapability)

}  // namespace UE::MovieScene

FMovieSceneLegacyPlayer::FMovieSceneLegacyPlayer()
{
}

FMovieSceneLegacyPlayer::FMovieSceneLegacyPlayer(TSharedRef<UE::MovieScene::FSharedPlaybackState> InSharedPlaybackState)
{
	EvaluationTemplateInstance.Initialize(InSharedPlaybackState);
}

FMovieSceneRootEvaluationTemplateInstance& FMovieSceneLegacyPlayer::GetEvaluationTemplate()
{
	return EvaluationTemplateInstance;
}

UMovieSceneEntitySystemLinker* FMovieSceneLegacyPlayer::ConstructEntitySystemLinker()
{
	ensureMsgf(false, TEXT("This legacy player should never have to construct a linker."));
	return nullptr;
}

UObject* FMovieSceneLegacyPlayer::AsUObject()
{
	return nullptr;
}

EMovieScenePlayerStatus::Type FMovieSceneLegacyPlayer::GetPlaybackStatus() const
{
	using namespace UE::MovieScene;

	if (SharedPlaybackState)
	{
		FRootInstanceHandle RootInstanceHandle = SharedPlaybackState->GetRootInstanceHandle();
		UMovieSceneEntitySystemLinker* Linker = SharedPlaybackState->GetLinker();
		FInstanceRegistry* InstanceRegisty = Linker->GetInstanceRegistry();
		const FSequenceInstance& RootInstance = InstanceRegisty->GetInstance(RootInstanceHandle);
		const FMovieSceneContext& Context = RootInstance.GetContext();
		return Context.GetStatus();
	}
	return EMovieScenePlayerStatus::Stopped;
}

void FMovieSceneLegacyPlayer::SetPlaybackStatus(EMovieScenePlayerStatus::Type InPlaybackStatus)
{
	// Ignored.
	ensureMsgf(false, TEXT("Please don't set playback status from evaluation templates."));
}

IMovieScenePlaybackClient* FMovieSceneLegacyPlayer::GetPlaybackClient()
{
	if (SharedPlaybackState)
	{
		return SharedPlaybackState->FindCapability<IMovieScenePlaybackClient>();
	}
	return nullptr;
}

FMovieSceneSpawnRegister& FMovieSceneLegacyPlayer::GetSpawnRegister()
{
	if (SharedPlaybackState)
	{
		FMovieSceneSpawnRegister* SpawnRegister = SharedPlaybackState->FindCapability<FMovieSceneSpawnRegister>();
		if (SpawnRegister)
		{
			return *SpawnRegister;
		}
	}
	return IMovieScenePlayer::GetSpawnRegister();
}

UObject* FMovieSceneLegacyPlayer::GetPlaybackContext() const
{
	if (SharedPlaybackState)
	{
		return SharedPlaybackState->GetPlaybackContext();
	}
	return nullptr;
}

void FMovieSceneLegacyPlayer::InitializeRootInstance(TSharedRef<UE::MovieScene::FSharedPlaybackState> NewSharedPlaybackState)
{
	ensureMsgf(false, TEXT("The legacy player should never initialize sequences: it only wraps already initialized ones."));
}

