// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneMetaData.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/Package.h"
#include "Engine/Level.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneMetaData)

const FName UMovieSceneMetaData::AssetRegistryTag_Author = "MovieSceneMetaData_Author";
const FName UMovieSceneMetaData::AssetRegistryTag_Created = "MovieSceneMetaData_Created";
const FName UMovieSceneMetaData::AssetRegistryTag_Notes = "MovieSceneMetaData_Notes";

UMovieSceneMetaData::UMovieSceneMetaData(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
	, Created(0)
{
}

UMovieSceneMetaData* UMovieSceneMetaData::GetConfigInstance()
{
	static UMovieSceneMetaData* ConfigInstance = NewObject<UMovieSceneMetaData>(GetTransientPackage(), "DefaultMovieSceneMetaData", RF_MarkAsRootSet);
	return ConfigInstance;
}

UMovieSceneMetaData* UMovieSceneMetaData::CreateFromDefaults(UObject* Outer, FName Name)
{
	if (Name != NAME_None)
	{
		check(!FindObject<UObject>(Outer, *Name.ToString()));
	}

	return CastChecked<UMovieSceneMetaData>(StaticDuplicateObject(GetConfigInstance(), Outer, Name, RF_NoFlags));
}

bool UMovieSceneMetaData::IsEmpty() const
{
	return Author.IsEmpty() && Created == FDateTime(0) && Notes.IsEmpty();
}

FString UMovieSceneMetaData::GetAuthor() const
{
	return Author;
}

FDateTime UMovieSceneMetaData::GetCreated() const
{
	return Created;
}

FString UMovieSceneMetaData::GetNotes() const
{
	return Notes;
}

void UMovieSceneMetaData::SetAuthor(FString InAuthor)
{
	Author = InAuthor;
}

void UMovieSceneMetaData::SetCreated(FDateTime InCreated)
{
	Created = InCreated;
}

void UMovieSceneMetaData::SetNotes(FString InNotes)
{
	Notes = InNotes;
}

void UMovieSceneMetaData::ExtendAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	IMovieSceneMetaDataInterface::ExtendAssetRegistryTags(Context);

	Context.AddTag(FAssetRegistryTag(AssetRegistryTag_Author, Author, FAssetRegistryTag::ETagType::TT_Alphabetical, FAssetRegistryTag::TD_None));
	Context.AddTag(FAssetRegistryTag(AssetRegistryTag_Created, Created.ToString(), FAssetRegistryTag::ETagType::TT_Chronological, FAssetRegistryTag::TD_Date | FAssetRegistryTag::TD_Time));
	Context.AddTag(FAssetRegistryTag(AssetRegistryTag_Notes, Notes, FAssetRegistryTag::ETagType::TT_Alphabetical, FAssetRegistryTag::TD_None));
}

#if WITH_EDITOR

void UMovieSceneMetaData::ExtendAssetRegistryTagMetaData(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const
{
	OutMetadata.Add(AssetRegistryTag_Author, FAssetRegistryTagMetadata()
		.SetDisplayName(NSLOCTEXT("MovieSceneMetaData", "Author_Label", "Author"))
		.SetTooltip(NSLOCTEXT("MovieSceneMetaData", "Author_Tip", "Author of this metadata"))
	);

	OutMetadata.Add(AssetRegistryTag_Created, FAssetRegistryTagMetadata()
		.SetDisplayName(NSLOCTEXT("MovieSceneMetaData", "Created_Label", "Created"))
		.SetTooltip(NSLOCTEXT("MovieSceneMetaData", "Created_Tip", "The date that this metadata was created"))
	);

	OutMetadata.Add(AssetRegistryTag_Notes, FAssetRegistryTagMetadata()
		.SetDisplayName(NSLOCTEXT("MovieSceneMetaData", "Notes_Label", "Notes"))
		.SetTooltip(NSLOCTEXT("MovieSceneMetaData", "Notes_Tip", "Notes for this metadata"))
	);
}

#endif