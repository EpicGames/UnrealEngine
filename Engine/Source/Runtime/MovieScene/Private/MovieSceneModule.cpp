// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "IMovieSceneModule.h"
#include "MovieScene.h"
#include "Compilation/IMovieSceneTemplateGenerator.h"
#include "Evaluation/IMovieScenePlaybackCapability.h"
#include "HAL/IConsoleManager.h"
#include "Logging/MessageLog.h"
#include "Misc/CoreDelegates.h"
#include "Misc/UObjectToken.h"
#include "Modules/VisualizerDebuggingState.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "Decorations/MovieSceneDecorationContainer.h"

DEFINE_LOG_CATEGORY(LogMovieScene);
DEFINE_LOG_CATEGORY(LogMovieSceneECS);

TAutoConsoleVariable<FString> CVarLegacyConversionFrameRate(
	TEXT("MovieScene.LegacyConversionFrameRate"),
	TEXT("60000fps"),
	TEXT("Specifies default tick resolution for UMovieScene data saved before 4.20 (default: 60000fps). Examples: 60000 fps, 120/1 (120 fps), 30000/1001 (29.97), 0.01s (10ms)."),
	ECVF_Default);

struct FCachedLegacyConversionFrameRate
{
	FCachedLegacyConversionFrameRate()
		: FrameRate(60000, 1)
	{
		IConsoleManager::Get().RegisterConsoleVariableSink_Handle(FConsoleCommandDelegate::CreateRaw(this, &FCachedLegacyConversionFrameRate::OnChanged));
		OnChanged();
	}

	void OnChanged()
	{
		TryParseString(FrameRate, *CVarLegacyConversionFrameRate.GetValueOnAnyThread());
	}

	FFrameRate FrameRate;
};

FFrameRate GetLegacyConversionFrameRate()
{
	static FCachedLegacyConversionFrameRate CachedRate;
	return CachedRate.FrameRate;
}

void EmitLegacyOutOfBoundsError(UObject* Object, FFrameRate InFrameRate, double InTime)
{
#if WITH_EDITOR
	static const FName NAME_AssetCheck("AssetCheck");

	FMessageLog AssetCheckLog(NAME_AssetCheck);

	const FText Message = FText::Format(
		NSLOCTEXT("MovieScene", "LegacyOutOfBoundsError", "Encountered time ({0} seconds) that is out of the supported range with a resolution of {1}fps. Saving this asset will cause loss of data. Please reduce MovieScene.LegacyConversionFrameRate and re-load this asset."),
		InTime, InFrameRate.AsDecimal()
	);
	AssetCheckLog.Error()
		->AddToken(FUObjectToken::Create(Object))
		->AddToken(FTextToken::Create(Message));

	AssetCheckLog.Open(EMessageSeverity::Warning);
#endif
}

FFrameNumber UpgradeLegacyMovieSceneTime(UObject* ErrorContext, FFrameRate InFrameRate, double InTime)
{
	double ClampedKeyTime = FMath::Clamp(InTime, -InFrameRate.MaxSeconds(), InFrameRate.MaxSeconds());
	if (InTime != ClampedKeyTime)
	{
		EmitLegacyOutOfBoundsError(ErrorContext, InFrameRate, InTime);
	}
	return InFrameRate.AsFrameNumber(ClampedKeyTime);
}

/**
 * MovieScene module implementation.
 */
class FMovieSceneModule
	: public IMovieSceneModule
	, public TSharedFromThis<FMovieSceneModule>
{
public:

	// IModuleInterface interface
	~FMovieSceneModule()
	{
		ensure(ModuleHandle.IsUnique());
	}

	virtual void StartupModule() override
	{
		using namespace UE::Core;
		using namespace UE::MovieScene;

		FModuleManager::Get().LoadModuleChecked("UniversalObjectLocator");

		// Register the global debug ptrs
		FGuid GEntityManagerForDebuggingGuid(0xaf15add3, 0xd76b4d2c, 0xa79d574b, 0xb4cd9706);
		EVisualizerDebuggingStateResult Result1 = FVisualizerDebuggingState::Assign(GEntityManagerForDebuggingGuid, &GEntityManagerForDebuggingVisualizers);

		FGuid GPlaybackCapabilityIDRegistryForDebuggingGuid(0xf9062bb9, 0xd77c4bc7, 0x861e5d87, 0xd5afb22d);
		EVisualizerDebuggingStateResult Result2 = FVisualizerDebuggingState::Assign(GPlaybackCapabilityIDRegistryForDebuggingGuid, &GPlaybackCapabilityIDRegistryForDebuggingVisualizers);
		GPlaybackCapabilityIDRegistryForDebuggingVisualizers = FPlaybackCapabilityIDRegistry::Get();

		struct FNoopDefaultDeleter
		{
			void operator()(FMovieSceneModule* Object) const {}
		};
		ModuleHandle = MakeShareable(this, FNoopDefaultDeleter());

		FBuiltInComponentTypes::Get();

		FCoreDelegates::OnAllModuleLoadingPhasesComplete.AddStatic(&UMovieSceneEntitySystem::StaticInitialize);
	}

	virtual void ShutdownModule() override
	{
		UE::MovieScene::FBuiltInComponentTypes::Destroy();
	}

	virtual void RegisterEvaluationGroupParameters(FName GroupName, const FMovieSceneEvaluationGroupParameters& GroupParameters) override
	{
		check(!GroupName.IsNone() && GroupParameters.EvaluationPriority != 0);

		for (auto& Pair : EvaluationGroupParameters)
		{
			checkf(Pair.Key != GroupName, TEXT("Cannot add 2 groups of the same name"));
			checkf(Pair.Value.EvaluationPriority != GroupParameters.EvaluationPriority, TEXT("Cannot add 2 groups of the same priority"));
		}

		EvaluationGroupParameters.Add(GroupName, GroupParameters);
	}

	virtual FMovieSceneEvaluationGroupParameters GetEvaluationGroupParameters(FName GroupName) const override
	{
		return EvaluationGroupParameters.FindRef(GroupName);
	}

	virtual TWeakPtr<IMovieSceneModule> GetWeakPtr() override
	{
		return ModuleHandle;
	}
	
	virtual void RegisterCompatibleDecoration(TSubclassOf<UMovieSceneDecorationContainerObject> ContainerClass, TSubclassOf<UObject> DecorationClass) override
	{
		if (!ContainerClass || !DecorationClass)
		{
			return;
		}

		TSet<TSubclassOf<UObject>>& Decorations = CompatibleDecorations.FindOrAdd(ContainerClass);
		Decorations.Add(DecorationClass);
	}

	virtual void UnregisterCompatibleDecoration(TSubclassOf<UMovieSceneDecorationContainerObject> ContainerClass, TSubclassOf<UObject> DecorationClass) override
	{
		// If the engine is exiting, do not do anything as data may have already been trashed before this call.
		if (IsEngineExitRequested())
		{
			return;
		}

		if (!ContainerClass || !DecorationClass)
		{
			return;
		}

		if (TSet<TSubclassOf<UObject>>* Decorations = CompatibleDecorations.Find(ContainerClass))
		{
			Decorations->Remove(DecorationClass);
			if (Decorations->IsEmpty())
			{
				CompatibleDecorations.Remove(ContainerClass);
			}
		}
	}

	virtual void GetCompatibleDecorationsForContainer(const UMovieSceneDecorationContainerObject* Container, TSet<UClass*>& OutClasses) const override
	{
		if (!Container)
		{
			return;
		}

		// First get the decorations from the container's own implementation
		Container->GetCompatibleUserDecorations(OutClasses);

		// Then add any registered decorations for this class and its parent classes
		UClass* ContainerClass = Container->GetClass();
		while (ContainerClass && ContainerClass->IsChildOf(UMovieSceneDecorationContainerObject::StaticClass()))
		{
			if (const TSet<TSubclassOf<UObject>>* RegisteredDecorations = CompatibleDecorations.Find(ContainerClass))
			{
				for (const TSubclassOf<UObject>& DecorationClass : *RegisteredDecorations)
				{
					if (DecorationClass)
					{
						OutClasses.Add(DecorationClass);
					}
				}
			}
			ContainerClass = ContainerClass->GetSuperClass();
		}
	}

	virtual void GetAllRegisteredDecorationClasses(TSet<UClass*>& OutClasses) const override
	{
		for (const TPair<TSubclassOf<UMovieSceneDecorationContainerObject>, TSet<TSubclassOf<UObject>>>& Pair : CompatibleDecorations)
		{
			for (const TSubclassOf<UObject>& DecorationClass : Pair.Value)
			{
				if (DecorationClass)
				{
					OutClasses.Add(DecorationClass);
				}
			}
		}
	}

private:
	TSharedPtr<FMovieSceneModule> ModuleHandle;
	TMap<FName, FMovieSceneEvaluationGroupParameters> EvaluationGroupParameters;

	// Map from container class to set of compatible decoration classes
	TMap<TSubclassOf<UMovieSceneDecorationContainerObject>, TSet<TSubclassOf<UObject>>> CompatibleDecorations;
};


IMPLEMENT_MODULE(FMovieSceneModule, MovieScene);
