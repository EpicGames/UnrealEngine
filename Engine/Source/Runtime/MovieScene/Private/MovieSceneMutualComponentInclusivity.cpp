// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieSceneMutualComponentInclusivity.h"
#include "EntitySystem/MovieSceneMutualComponentInitializer.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneComponentRegistry.h"
#include "EntitySystem/MovieSceneComponentTypeInfo.h"
#include "Misc/StringBuilder.h"
#include "MovieSceneFwd.h"

namespace UE::MovieScene
{

void FMutualComponentInitializers::Add(IMutualComponentInitializer* InitializerToAdd)
{
	// No need to check for uniqueness here because initializers can only
	// exist in the command buffer once
	Initializers.Add(InitializerToAdd);
}

void FMutualComponentInitializers::Reset()
{
	Initializers.Reset();
}

void FMutualComponentInitializers::Execute(const FEntityRange& Range, const FEntityAllocationWriteContext& WriteContext) const
{
	for (IMutualComponentInitializer* Initializer : Initializers)
	{
		Initializer->Run(Range, WriteContext);
	}
}

int32 FMutualInclusivityGraphCommandBuffer::ComputeMutuallyInclusiveComponents(EMutuallyInclusiveComponentType TypesToCompute, const FComponentMask& InMask, FComponentMask& OutMask, FMutualComponentInitializers& OutInitializers) const
{
	int32 NumNewComponents = 0;

	FComponentMask WorkingMask = InMask;

	// NOTE: It is possible to call this function with the same reference for InMask and OutMask
	//       such that populating the mutual components adds them to the same input. This function is written to allow that while still being performant.
	//       This flexibility allows us to compute mutually inclusive components to a separate mask, as well as the same mask.

	// Command buffer will take the form of 1 or more match commands followed by 1 or more include commands.
	// All match commands must be satisfied for subsequent include commands to pass
	// 
	// CommandBuffers adhere to the following schema:
	// [
	//	  { type = mandatory | optional }  // Specifies the type of the commands that follow
	//	  { simple },
	// 
	//	  { match_all[1..N] -> include[1..N] -> (initializer) }
	//            |------------------short circuits to end-----|
	// 
	//	  { match_any[1..N] -> short_circuit -> include[1..N] -> (initializer) }
	//            |                  |-----------|----short circuits to end-----|
	//            |---short circuits to includes-|
	for (int32 Index = 0; Index < Commands.Num(); ++Index)
	{
		FCommand Command = Commands[Index];

		switch (Command.CommandType)
		{
		case ECommandType::ShortCircuit:
			// - 1 because Index will be incremented as part of the loop iteration
			Index = static_cast<int32>(Command.ShortCircuit.ShortCircuitIndex) - 1;
			break;

		case ECommandType::Type:
			// If the type doesn't match, short-circuit the entire stack of commands
			if (!EnumHasAnyFlags(Command.Type.Type, TypesToCompute))
			{
				// - 1 because Index will be incremented as part of the loop iteration
				Index = static_cast<int32>(Command.Type.ShortCircuitIndex) - 1;
			}
			break;

		case ECommandType::MatchAll:
			if (!WorkingMask.Contains(Command.Match.ComponentTypeID))
			{
				// ShortCircuitIndex points to the end of the Include and initializer commands for this stack
				// - 1 because Index will be incremented as part of the loop iteration
				Index = static_cast<int32>(Command.Match.ShortCircuitIndex) - 1;
			}
			break;

		case ECommandType::MatchAny:
			if (WorkingMask.Contains(Command.Match.ComponentTypeID))
			{
				// ShortCircuitIndex points to the first Include command in this stack
				// - 1 because Index will be incremented as part of the loop iteration
				Index = static_cast<int32>(Command.Match.ShortCircuitIndex) - 1;
			}
			break;

		case ECommandType::Simple:
			if (WorkingMask.Contains(Command.Simple.MatchID) && !WorkingMask.Contains(Command.Simple.IncludeID))
			{
				++NumNewComponents;
				OutMask.Set(Command.Simple.IncludeID);
				WorkingMask.Set(Command.Simple.IncludeID);
			}
			break;

		case ECommandType::Include:
			if (!WorkingMask.Contains(Command.Include.ComponentTypeID))
			{
				++NumNewComponents;
				OutMask.Set(Command.Include.ComponentTypeID);
				WorkingMask.Set(Command.Include.ComponentTypeID);
			}
			break;

		case ECommandType::Initialize:
			OutInitializers.Add(Command.Initialize.Initializer);
			break;
		}
	}

	return NumNewComponents;
}


void FMutualInclusivityGraphCommandBuffer::CheckInvariants() const
{
	// Check invariants
#if DO_CHECK
	auto TestShortCircuitIndex = [this](int32 Index)
	{
		checkf(Index != 0 && Index <= this->Commands.Num(),
			TEXT("Invalid short circuit index %d for command buffer length (%d)"), Index, this->Commands.Num());
	};

	for (int32 Index = 0; Index < Commands.Num(); ++Index)
	{
		FCommand Command = Commands[Index];
		switch (Command.CommandType)
		{
		case ECommandType::ShortCircuit:
			TestShortCircuitIndex(Command.ShortCircuit.ShortCircuitIndex);
			break;

		case ECommandType::Type:
			TestShortCircuitIndex(Command.Type.ShortCircuitIndex);
			break;

		case ECommandType::MatchAll:
			TestShortCircuitIndex(Command.Match.ShortCircuitIndex);
			break;

		case ECommandType::MatchAny:
			TestShortCircuitIndex(Command.Match.ShortCircuitIndex);
			checkf(Commands[Command.Match.ShortCircuitIndex-1].CommandType == ECommandType::ShortCircuit,
				TEXT("Expected short circuit command to proceed match any commands"));
			break;

		default:
			break;
		}
	}
#endif
}

void FMutualInclusivityGraph::DefineMutualInclusionRule(FComponentTypeID Predicate, std::initializer_list<FComponentTypeID> Dependents)
{
	DefineMutualInclusionRule(Predicate, Dependents, FMutuallyInclusiveComponentParams());
}

void FMutualInclusivityGraph::DefineMutualInclusionRule(FComponentTypeID Predicate, std::initializer_list<FComponentTypeID> Dependents, FMutuallyInclusiveComponentParams&& Parameters)
{
	bCommandBufferInvalidated = true;

	FIncludePair& IncludePair = SimpleIncludes.FindOrAdd(Predicate);

	FIncludes& Includes = Parameters.Type == EMutuallyInclusiveComponentType::Mandatory
		? IncludePair.MandatoryIncludes
		: IncludePair.OptionalIncludes;

	for (FComponentTypeID Dependent : Dependents)
	{
		Includes.Add(Dependent);
	}

	if (Parameters.CustomInitializer)
	{
		Includes.Initializers.Emplace(MoveTemp(Parameters.CustomInitializer));
	}
}

void FMutualInclusivityGraph::DefineComplexInclusionRule(const FComplexInclusivityFilter& InFilter, std::initializer_list<FComponentTypeID> Dependents)
{
	DefineComplexInclusionRule(InFilter, Dependents, FMutuallyInclusiveComponentParams());
}

void FMutualInclusivityGraph::DefineComplexInclusionRule(const FComplexInclusivityFilter& InFilter, std::initializer_list<FComponentTypeID> Dependents, FMutuallyInclusiveComponentParams&& Parameters)
{
	bCommandBufferInvalidated = true;

	FIncludePair& IncludePair = ComplexIncludes.FindOrAdd(InFilter);

	FIncludes& Includes = Parameters.Type == EMutuallyInclusiveComponentType::Mandatory
		? IncludePair.MandatoryIncludes
		: IncludePair.OptionalIncludes;

	for (FComponentTypeID Dependent : Dependents)
	{
		Includes.Add(Dependent);
	}

	if (Parameters.CustomInitializer)
	{
		Includes.Initializers.Emplace(MoveTemp(Parameters.CustomInitializer));
	}
}

int32 FMutualInclusivityGraph::ComputeMutuallyInclusiveComponents(EMutuallyInclusiveComponentType TypesToCompute, const FComponentMask& InMask, FComponentMask& OutMask, FMutualComponentInitializers& OutInitializers) const
{
	if (bCommandBufferInvalidated)
	{
		ReconstructCommandBuffer();
	}

	return CommandBuffer.ComputeMutuallyInclusiveComponents(TypesToCompute, InMask, OutMask, OutInitializers);
}

int32 FMutualInclusivityGraph::FindPrerequisiteChainLength(const FDirectedGraph& Graph, FComponentTypeID Component, TMap<FComponentTypeID, int32>& InOutCache)
{
	if (const int32* Depth = InOutCache.Find(Component))
	{
		return *Depth;
	}

	TArrayView<const FDirectedGraph::FDirectionalEdge> Edges = Graph.GetEdgesFrom(Component.BitIndex());

	int32 MaxDepth = 0;
	if (Edges.Num() != 0)
	{
		for (FDirectedGraph::FDirectionalEdge Edge : Edges)
		{
			MaxDepth = FMath::Max(MaxDepth, FindPrerequisiteChainLength(Graph, FComponentTypeID::FromBitIndex(Edge.ToNode), InOutCache));
		}
	}

	InOutCache.Add(Component, MaxDepth + 1);
	return MaxDepth + 1;
}

void FMutualInclusivityGraph::ReconstructCommandBuffer() const
{
	using FCommand     = FMutualInclusivityGraphCommandBuffer::FCommand;
	using ECommandType = FMutualInclusivityGraphCommandBuffer::ECommandType;

	FDirectedGraph Graph;

	// Step 1: Allocate nodes that can be produced from mutual includes
	for (const TTuple<FComponentTypeID, FIncludePair>& Pair : SimpleIncludes)
	{
		for (FComponentTypeID Dependent : Pair.Value.MandatoryIncludes.Includes)
		{
			Graph.AllocateNode(Dependent.BitIndex());
		}
		for (FComponentTypeID Dependent : Pair.Value.OptionalIncludes.Includes)
		{
			Graph.AllocateNode(Dependent.BitIndex());
		}
	}
	for (const TTuple<FComplexInclusivityFilter, FIncludePair>& Pair : ComplexIncludes)
	{
		for (FComponentTypeID Dependent : Pair.Value.MandatoryIncludes.Includes)
		{
			Graph.AllocateNode(Dependent.BitIndex());
		}
		for (FComponentTypeID Dependent : Pair.Value.OptionalIncludes.Includes)
		{
			Graph.AllocateNode(Dependent.BitIndex());
		}
	}

	// Step 2: Define edges between dependent nodes. Dependencies point from dependent to predicate so we can walk _up_ the graph
	for (const TTuple<FComponentTypeID, FIncludePair>& Pair : SimpleIncludes)
	{
		if (Graph.IsNodeAllocated(Pair.Key.BitIndex()))
		{
			for (FComponentTypeID Dependent : Pair.Value.MandatoryIncludes.Includes)
			{
				Graph.MakeEdge(Dependent.BitIndex(), Pair.Key.BitIndex());
			}
			for (FComponentTypeID Dependent : Pair.Value.OptionalIncludes.Includes)
			{
				Graph.MakeEdge(Dependent.BitIndex(), Pair.Key.BitIndex());
			}
		}
	}
	for (const TTuple<FComplexInclusivityFilter, FIncludePair>& Pair : ComplexIncludes)
	{
		for (FComponentMaskIterator It(Pair.Key.Mask.Iterate()); It; ++It)
		{
			FComponentTypeID Predicate = FComponentTypeID::FromBitIndex(It.GetIndex());
			if (Graph.IsNodeAllocated(Predicate.BitIndex()))
			{
				for (FComponentTypeID Dependent : Pair.Value.MandatoryIncludes.Includes)
				{
					Graph.MakeEdge(Dependent.BitIndex(), Predicate.BitIndex());
				}
				for (FComponentTypeID Dependent : Pair.Value.OptionalIncludes.Includes)
				{
					Graph.MakeEdge(Dependent.BitIndex(), Predicate.BitIndex());
				}
			}
		}
	}

	if (!ensureMsgf(!Graph.IsCyclic(), TEXT("Mutual component inclusion graph is cyclic! See log for graph output.")))
	{
		FComponentRegistry* ComponentRegistry = UMovieSceneEntitySystemLinker::GetComponents();

		auto EmitLabel = [=](uint16 NodeID, FStringBuilderBase& OutStringBuilder)
		{
			FComponentTypeID ComponentTypeID = FComponentTypeID::FromBitIndex(NodeID);

			const FComponentTypeInfo& ComponentTypeInfo = ComponentRegistry->GetComponentTypeChecked(ComponentTypeID);
		#if UE_MOVIESCENE_ENTITY_DEBUG
			OutStringBuilder.Appendf(TEXT("\t\tnode%d[label=\"%s\"];\n"), ComponentTypeID.BitIndex(), *ComponentTypeInfo.DebugInfo->DebugName);
		#else
			OutStringBuilder.Appendf(TEXT("\t\tnode%d[label=\"%d\"];\n"), ComponentTypeID.BitIndex(), ComponentTypeID.BitIndex());
		#endif
		};

		FDirectedGraphStringParameters Parameters{ TEXT("Mutual Includes") };
		UE_LOGF(LogMovieScene, Warning, "Printing Graph: \n %ls", *Graph.ToString(Parameters, EmitLabel));
	}

	// Step 3: Compute ordering constraints based on depth within the graph
	TMap<FComponentTypeID, int32> DependencyDepths;
	for (TConstSetBitIterator<> SetBitIter(Graph.GetNodeMask()); SetBitIter; ++SetBitIter)
	{
		FindPrerequisiteChainLength(Graph, FComponentTypeID::FromBitIndex(SetBitIter.GetIndex()), DependencyDepths);
	}

	// Step 4: Organize simple inclusion by depth using the predicate component, so that it is ordered after
	//         any other include that might introduce this component
	TArray<TTuple<FComponentTypeID, int32>> OrderedSimpleInclusion;
	for (const TTuple<FComponentTypeID, FIncludePair>& Pair : SimpleIncludes)
	{
		const int32* GraphDepth = DependencyDepths.Find(Pair.Key);

		OrderedSimpleInclusion.Add(MakeTuple(Pair.Key, GraphDepth ? *GraphDepth : 0));
	}
	Algo::SortBy(OrderedSimpleInclusion, [](const TTuple<FComponentTypeID, int32>& In){return In.Get<1>();});

	// Step 5: Organize complex inclusion by depth using the predicate components, so that it is ordered after
	//         any other include that might introduce this component
	TArray<TTuple<FComplexInclusivityFilter, int32>> OrderedComplexInclusion;
	for (const TTuple<FComplexInclusivityFilter, FIncludePair>& Pair : ComplexIncludes)
	{
		int32 Depth = 0;
		for (FComponentMaskIterator It(Pair.Key.Mask.Iterate()); It; ++It)
		{
			FComponentTypeID Predicate = FComponentTypeID::FromBitIndex(It.GetIndex());
			if (const int32* GraphDepth = DependencyDepths.Find(Predicate))
			{
				Depth = FMath::Max(Depth, *GraphDepth);
			}
		}

		OrderedComplexInclusion.Add(MakeTuple(Pair.Key, Depth));
	}
	Algo::SortBy(OrderedComplexInclusion, [](const TTuple<FComplexInclusivityFilter, int32>& In){return In.Get<1>();});

	// Step 6: Generate the command buffer from the two ordered arrays, ensuring simple includes come before complex includes
	CommandBuffer.Commands.Empty();

	int32 SimpleIndex = 0;
	int32 ComplexIndex = 0;

	int32 LastTypeIndex = INDEX_NONE;
	EMutuallyInclusiveComponentType CurrentType = EMutuallyInclusiveComponentType::All;

	auto EmitTypeCommand = [this, &LastTypeIndex, &CurrentType](EMutuallyInclusiveComponentType Type)
	{
		// Emit a type command if this is a different type from the last one added
		if (Type != CurrentType)
		{
			if (LastTypeIndex != INDEX_NONE)
			{
				this->CommandBuffer.Commands[LastTypeIndex].Type.ShortCircuitIndex = this->CommandBuffer.Commands.Num();
			}

			LastTypeIndex = this->CommandBuffer.Commands.Num();
			// Add the type command - we will populate the ShortCircuitIndex later
			this->CommandBuffer.Commands.Emplace(FMutualInclusivityGraphCommandBuffer::FTypeCommand{
				0,
				Type
			});
			CurrentType = Type;
		}
	};

	auto AddSimpleCommand = [this, &EmitTypeCommand](FComponentTypeID PredicateComponent, const FIncludes& Dependencies, EMutuallyInclusiveComponentType Type)
	{
		// Emit a type command if this is a different type from the last one added
		EmitTypeCommand(Type);

		// If the match fails, skip the match command and the includes
		const int32  NumPredicateComponents = 1;
		const int32  NumIncludeComponents   = Dependencies.Includes.Num();
		const int32  NumInitializers        = Dependencies.Initializers.Num();
		const uint16 ShortCircuitIndex      = static_cast<uint16>(this->CommandBuffer.Commands.Num() + NumPredicateComponents + NumIncludeComponents + NumInitializers);

		// Use a simple command if possible to reduce the size of the array
		if (NumIncludeComponents == 1 && NumInitializers == 0)
		{
			this->CommandBuffer.Commands.Emplace(FMutualInclusivityGraphCommandBuffer::FSimpleCommand{
				PredicateComponent,
				Dependencies.Includes[0]
			});
		}
		else
		{
			this->CommandBuffer.Commands.Emplace(FMutualInclusivityGraphCommandBuffer::FMatchCommand{
				PredicateComponent,
				ShortCircuitIndex
			}, EComplexInclusivityFilterMode::AllOf);

			for (FComponentTypeID Dependency : Dependencies.Includes)
			{
				this->CommandBuffer.Commands.Add(FMutualInclusivityGraphCommandBuffer::FIncludeCommand{ Dependency });
			}

			for (const TUniquePtr<IMutualComponentInitializer>& Initializer : Dependencies.Initializers)
			{
				this->CommandBuffer.Commands.Add(FMutualInclusivityGraphCommandBuffer::FInitializeCommand{ Initializer.Get() });
			}
		}
	};

	auto AddSimpleCommands = [this, &AddSimpleCommand](FComponentTypeID Predicate)
	{
		const FIncludePair* Includes = this->SimpleIncludes.Find(Predicate);
		if (Includes)
		{
			// Handle mandatory includes first
			if (Includes->MandatoryIncludes.Includes.Num() != 0)
			{
				AddSimpleCommand(Predicate, Includes->MandatoryIncludes, EMutuallyInclusiveComponentType::Mandatory);
			}
			if (Includes->OptionalIncludes.Includes.Num() != 0)
			{
				AddSimpleCommand(Predicate, Includes->OptionalIncludes, EMutuallyInclusiveComponentType::Optional);
			}
		}
	};
	auto AddComplexCommand = [this, &EmitTypeCommand](const FComplexInclusivityFilter& Predicate, const FIncludes& Dependencies, EMutuallyInclusiveComponentType Type)
	{
		// Emit a type command if this is a different type from the last one added
		EmitTypeCommand(Type);

		// If the match fails, skip the match command and the includes
		const int32  NumPredicateComponents = Predicate.Mask.NumComponents();
		const int32  NumInitializers        = Dependencies.Initializers.Num();

		if (Predicate.Mode == EComplexInclusivityFilterMode::AnyOf)
		{
			// The AllOf command is a stream of matches that jump to the start of the includes on success,
			// followed by a shortcircuit that skips the includes if it is reached
			const uint16 IncludeStartIndex = static_cast<uint16>(this->CommandBuffer.Commands.Num() + NumPredicateComponents + 1);
			const uint16 ShortCircuitIndex = static_cast<uint16>(IncludeStartIndex + Dependencies.Includes.Num() + NumInitializers);

			for (FComponentMaskIterator It(Predicate.Mask.Iterate()); It; ++It)
			{
				FComponentTypeID PredicateComponent = FComponentTypeID::FromBitIndex(It.GetIndex());

				this->CommandBuffer.Commands.Emplace(FMutualInclusivityGraphCommandBuffer::FMatchCommand{
					PredicateComponent,
					IncludeStartIndex
				}, Predicate.Mode);
			}

			this->CommandBuffer.Commands.Emplace(FMutualInclusivityGraphCommandBuffer::FShortCircuitCommand{
				ShortCircuitIndex
			});
		}
		else
		{
			// The Any command is a stream of matches that skip to the end of the command on failure
			const uint16 ShortCircuitIndex = static_cast<uint16>(this->CommandBuffer.Commands.Num() + NumPredicateComponents + Dependencies.Includes.Num() + NumInitializers);

			for (FComponentMaskIterator It(Predicate.Mask.Iterate()); It; ++It)
			{
				FComponentTypeID PredicateComponent = FComponentTypeID::FromBitIndex(It.GetIndex());

				this->CommandBuffer.Commands.Emplace(FMutualInclusivityGraphCommandBuffer::FMatchCommand{
					PredicateComponent,
					ShortCircuitIndex
				}, Predicate.Mode);
			}
		}

		// Add includes
		for (FComponentTypeID Dependency : Dependencies.Includes)
		{
			this->CommandBuffer.Commands.Add(FMutualInclusivityGraphCommandBuffer::FIncludeCommand{ Dependency });
		}

		// Add custom initializers
		for (const TUniquePtr<IMutualComponentInitializer>& Initializer : Dependencies.Initializers)
		{
			this->CommandBuffer.Commands.Add(FMutualInclusivityGraphCommandBuffer::FInitializeCommand{ Initializer.Get() });
		}
	};
	auto AddComplexCommands = [this, &AddComplexCommand](const FComplexInclusivityFilter& Predicate)
	{
		const FIncludePair* Includes = this->ComplexIncludes.Find(Predicate);
		if (Includes)
		{
			// Handle mandatory includes first
			if (Includes->MandatoryIncludes.Includes.Num() != 0)
			{
				AddComplexCommand(Predicate, Includes->MandatoryIncludes, EMutuallyInclusiveComponentType::Mandatory);
			}
			if (Includes->OptionalIncludes.Includes.Num() != 0)
			{
				AddComplexCommand(Predicate, Includes->OptionalIncludes, EMutuallyInclusiveComponentType::Optional);
			}
		}
	};

	// Keep going until we have nothing left
	while (SimpleIndex < OrderedSimpleInclusion.Num() || ComplexIndex < OrderedComplexInclusion.Num())
	{
		const bool bHasSimple  = SimpleIndex < OrderedSimpleInclusion.Num();
		const bool bHasComplex = ComplexIndex < OrderedComplexInclusion.Num();

		if (bHasSimple && bHasComplex)
		{
			// Decide which to add first based on the order
			const int32 SimpleOrder = OrderedSimpleInclusion[SimpleIndex].Get<1>();
			const int32 ComplexOrder = OrderedComplexInclusion[ComplexIndex].Get<1>();

			if (SimpleOrder <= ComplexOrder)
			{
				while (SimpleIndex < OrderedSimpleInclusion.Num() && OrderedSimpleInclusion[SimpleIndex].Get<1>() == SimpleOrder)
				{
					FComponentTypeID Predicate = OrderedSimpleInclusion[SimpleIndex].Get<0>();
					AddSimpleCommands(Predicate);

					++SimpleIndex;
				}
			}
			else
			{
				while (ComplexIndex < OrderedComplexInclusion.Num() && OrderedComplexInclusion[ComplexIndex].Get<1>() == ComplexOrder)
				{
					const FComplexInclusivityFilter& Predicate = OrderedComplexInclusion[ComplexIndex].Get<0>();
					AddComplexCommands(Predicate);

					++ComplexIndex;
				}
			}
		}
		else if (bHasSimple)
		{
			while (SimpleIndex < OrderedSimpleInclusion.Num())
			{
				FComponentTypeID Predicate = OrderedSimpleInclusion[SimpleIndex].Get<0>();
				AddSimpleCommands(Predicate);

				++SimpleIndex;
			}
		}
		else
		{
			while (ComplexIndex < OrderedComplexInclusion.Num())
			{
				const FComplexInclusivityFilter& Predicate = OrderedComplexInclusion[ComplexIndex].Get<0>();
				AddComplexCommands(Predicate);

				++ComplexIndex;
			}
		}
	}

	// Populate the most recent type command if one was added
	if (LastTypeIndex != INDEX_NONE)
	{
		CommandBuffer.Commands[LastTypeIndex].Type.ShortCircuitIndex = CommandBuffer.Commands.Num();
	}

	bCommandBufferInvalidated = false;
	CommandBuffer.CheckInvariants();
}

} // namespace UE::MovieScene