// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneNameableTrack.h"
#include "UObject/NameTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneNameableTrack)

#define LOCTEXT_NAMESPACE "MovieSceneNameableTrack"


/* UMovieSceneNameableTrack interface
 *****************************************************************************/

#if WITH_EDITORONLY_DATA

void UMovieSceneNameableTrack::SetDisplayName(const FText& NewDisplayName)
{
	if (NewDisplayName.EqualTo(DisplayName))
	{
		return;
	}

	SetFlags(RF_Transactional);
	Modify();

	DisplayName = NewDisplayName;
}

void UMovieSceneNameableTrack::SetTrackRowDisplayName(const FText& NewDisplayName, int32 TrackRowIndex)
{
	if (TrackRowIndex >= TrackRowDisplayNames.Num())
	{
		TrackRowDisplayNames.AddDefaulted(TrackRowIndex+1);
	}

	if (NewDisplayName.EqualTo(TrackRowDisplayNames[TrackRowIndex]))
	{
		return;
	}

	SetFlags(RF_Transactional);
	Modify();

	TrackRowDisplayNames[TrackRowIndex] = NewDisplayName;
}

bool UMovieSceneNameableTrack::ValidateDisplayName(const FText& NewDisplayName, FText& OutErrorMessage) const
{
	if (NewDisplayName.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("RenameFailed_LeftBlank", "Labels cannot be left blank");
		return false;
	}
	else if (NewDisplayName.ToString().Len() >= NAME_SIZE)
	{
		OutErrorMessage = FText::Format(LOCTEXT("RenameFailed_TooLong", "Names must be less than {0} characters long"), NAME_SIZE);
		return false;
	}
	return true;
}

#endif


/* UMovieSceneTrack interface
 *****************************************************************************/

#if WITH_EDITORONLY_DATA

FText UMovieSceneNameableTrack::GetDisplayName() const
{
	if (DisplayName.IsEmpty())
	{
		return GetDefaultDisplayName();
	}

	return DisplayName;
}

FText UMovieSceneNameableTrack::GetTrackRowDisplayName(int32 TrackRowIndex) const
{
	if (TrackRowIndex < TrackRowDisplayNames.Num() && !TrackRowDisplayNames[TrackRowIndex].IsEmpty())
	{
		return TrackRowDisplayNames[TrackRowIndex];
	}

	return GetDisplayName();
}

FText UMovieSceneNameableTrack::GetDefaultDisplayName() const
{ 
	return LOCTEXT("UnnamedTrackName", "Unnamed Track"); 
}

void UMovieSceneNameableTrack::OnRowIndicesChanged(const TMap<int32, int32>& NewToOldRowIndices)
{
	Super::OnRowIndicesChanged(NewToOldRowIndices);

	TArray<FText> OriginalTrackRowDisplayNames = TrackRowDisplayNames;

	Modify();
	TrackRowDisplayNames.Empty();
	for (int32 RowIndex = 0, RowMax = GetMaxRowIndex(); RowIndex <= RowMax; ++RowIndex)
	{
		const int32* OldIndex = NewToOldRowIndices.Find(RowIndex);
		int32 RemappedIndex = OldIndex ? *OldIndex : RowIndex;
		if (RemappedIndex < OriginalTrackRowDisplayNames.Num())
		{
			TrackRowDisplayNames.Add(OriginalTrackRowDisplayNames[RemappedIndex]);
		}
	}
}

#endif


#undef LOCTEXT_NAMESPACE

