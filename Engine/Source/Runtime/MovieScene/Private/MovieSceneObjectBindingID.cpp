// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneObjectBindingID.h"
#include "Evaluation/MovieSceneRootOverridePath.h"
#include "Evaluation/MovieSceneSequenceHierarchy.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "EntitySystem/MovieSceneSequenceInstance.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "IMovieScenePlayer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneObjectBindingID)



namespace UE
{
namespace MovieScene
{


FMovieSceneSequenceID ResolveExternalSequenceID(FMovieSceneSequenceID SourceSequenceID, int32 RemapSourceParentIndex, FMovieSceneSequenceID TargetSequenceID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	ensureMsgf(RemapSourceParentIndex >= 0, TEXT("Invalid parent index specified"));

	if (SourceSequenceID == MovieSceneSequenceID::Root)
	{
		return TargetSequenceID;
	}

	const FMovieSceneSequenceHierarchy* Hierarchy = SharedPlaybackState->GetHierarchy();
	if (!ensureMsgf(Hierarchy, TEXT("Sequence being evaluated as a sub sequence without any hierarchy. This indicates a bug with either the player or the compiled data.")))
	{
		return TargetSequenceID;
	}

	using namespace UE::MovieScene;

	FSubSequencePath PathFromSource(SourceSequenceID, Hierarchy);
	PathFromSource.PopGenerations(RemapSourceParentIndex);
	return PathFromSource.ResolveChildSequenceID(TargetSequenceID);
}


FMovieSceneSequenceID ResolveExternalSequenceID(FMovieSceneSequenceID SourceSequenceID, int32 RemapSourceParentIndex, FMovieSceneSequenceID TargetSequenceID, const FMovieSceneSequenceHierarchy* Hierarchy)
{
	ensureMsgf(RemapSourceParentIndex >= 0, TEXT("Invalid parent index specified"));

	if (SourceSequenceID == MovieSceneSequenceID::Root)
	{
		return TargetSequenceID;
	}

	if (!ensureMsgf(Hierarchy, TEXT("Sequence being evaluated as a sub sequence without any hierarchy. This indicates a bug with either the player or the compiled data.")))
	{
		return TargetSequenceID;
	}

	using namespace UE::MovieScene;

	FSubSequencePath PathFromSource(SourceSequenceID, Hierarchy);
	PathFromSource.PopGenerations(RemapSourceParentIndex);
	return PathFromSource.ResolveChildSequenceID(TargetSequenceID);
}


FRelativeObjectBindingID::FRelativeObjectBindingID(FMovieSceneSequenceID SourceSequenceID, FMovieSceneSequenceID TargetSequenceID, const FGuid& TargetGuid, UMovieSceneSequence* RootSequence)
{
	UMovieSceneCompiledDataManager* CompiledDataManager = UMovieSceneCompiledDataManager::GetPrecompiledData();

	FMovieSceneSequenceHierarchy TempHierarchy;
	UMovieSceneCompiledDataManager::CompileHierarchy(RootSequence, &TempHierarchy, EMovieSceneServerClientMask::All);

	ConstructInternal(SourceSequenceID, TargetSequenceID, TargetGuid, &TempHierarchy);
}


FRelativeObjectBindingID::FRelativeObjectBindingID(FMovieSceneSequenceID SourceSequenceID, FMovieSceneSequenceID TargetSequenceID, const FGuid& TargetGuid, IMovieScenePlayer& Player)
{
	ConstructInternal(SourceSequenceID, TargetSequenceID, TargetGuid, Player.GetSharedPlaybackState()->GetHierarchy());
}

FRelativeObjectBindingID::FRelativeObjectBindingID(FMovieSceneSequenceID SourceSequenceID, FMovieSceneSequenceID TargetSequenceID, const FGuid& TargetGuid, TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	ConstructInternal(SourceSequenceID, TargetSequenceID, TargetGuid, SharedPlaybackState->GetHierarchy());
}


FRelativeObjectBindingID::FRelativeObjectBindingID(FMovieSceneSequenceID SourceSequenceID, FMovieSceneSequenceID TargetSequenceID, const FGuid& TargetGuid, const FMovieSceneSequenceHierarchy* Hierarchy)
{
	ConstructInternal(SourceSequenceID, TargetSequenceID, TargetGuid, Hierarchy);
}

void FRelativeObjectBindingID::ConstructInternal(FMovieSceneSequenceID SourceSequenceID, FMovieSceneSequenceID TargetSequenceID, const FGuid& TargetGuid, const FMovieSceneSequenceHierarchy* Hierarchy)
{
	FSubSequencePath PathFromOwner(SourceSequenceID, Hierarchy);
	FSubSequencePath PathFromTarget(TargetSequenceID, Hierarchy);

	FMovieSceneSequenceID CommonParent = FSubSequencePath::FindCommonParent(PathFromOwner, PathFromTarget);

	Guid = TargetGuid;
	SequenceID = PathFromTarget.MakeLocalSequenceID(CommonParent);
	ResolveParentIndex = PathFromOwner.NumGenerationsFromLeaf(CommonParent);
}

FRelativeObjectBindingID FFixedObjectBindingID::ConvertToRelative(FMovieSceneSequenceID SourceSequenceID, IMovieScenePlayer& InPlayer) const
{
	return ConvertToRelative(SourceSequenceID, InPlayer.GetSharedPlaybackState());
}

FRelativeObjectBindingID FFixedObjectBindingID::ConvertToRelative(FMovieSceneSequenceID SourceSequenceID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState) const
{
	return FRelativeObjectBindingID(SourceSequenceID, SequenceID, Guid, SharedPlaybackState);
}

FRelativeObjectBindingID FFixedObjectBindingID::ConvertToRelative(FMovieSceneSequenceID SourceSequenceID, const FMovieSceneSequenceHierarchy* Hierarchy) const
{
	return FRelativeObjectBindingID(SourceSequenceID, SequenceID, Guid, Hierarchy);
}


} // namespace MovieScene
} // namespace UE




#if WITH_EDITORONLY_DATA

void FMovieSceneObjectBindingID::PostSerialize(const FArchive& Ar)
{
	if (Space_DEPRECATED != EMovieSceneObjectBindingSpace::Unused)
	{
		ResolveParentIndex = (Space_DEPRECATED == EMovieSceneObjectBindingSpace::Local) ? 0 : FixedRootSequenceParentIndex;
		Space_DEPRECATED = EMovieSceneObjectBindingSpace::Unused;
	}
}

#endif // WITH_EDITORONLY_DATA

FMovieSceneSequenceID FMovieSceneObjectBindingID::ResolveSequenceID(FMovieSceneSequenceID LocalSequenceID, IMovieScenePlayer& Player) const
{
	return ResolveSequenceID(LocalSequenceID, Player.GetSharedPlaybackState());
}

FMovieSceneSequenceID FMovieSceneObjectBindingID::ResolveSequenceID(FMovieSceneSequenceID LocalSequenceID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState) const
{
	FMovieSceneSequenceID TargetSequenceID = FMovieSceneSequenceID(SequenceID);

	if (ResolveParentIndex == FixedRootSequenceParentIndex)
	{
		return TargetSequenceID;
	}
	
	return UE::MovieScene::ResolveExternalSequenceID(LocalSequenceID, ResolveParentIndex, TargetSequenceID, SharedPlaybackState);
}

FMovieSceneSequenceID FMovieSceneObjectBindingID::ResolveSequenceID(FMovieSceneSequenceID LocalSequenceID, const FMovieSceneSequenceHierarchy* Hierarchy) const
{
	FMovieSceneSequenceID TargetSequenceID = FMovieSceneSequenceID(SequenceID);

	if (ResolveParentIndex == FixedRootSequenceParentIndex)
	{
		return TargetSequenceID;
	}

	return UE::MovieScene::ResolveExternalSequenceID(LocalSequenceID, ResolveParentIndex, TargetSequenceID, Hierarchy);

}
UE::MovieScene::FFixedObjectBindingID FMovieSceneObjectBindingID::ResolveToFixed(FMovieSceneSequenceID RuntimeSequenceID, IMovieScenePlayer& Player) const
{
	return ResolveToFixed(RuntimeSequenceID, Player.GetSharedPlaybackState());
}

UE::MovieScene::FFixedObjectBindingID FMovieSceneObjectBindingID::ResolveToFixed(FMovieSceneSequenceID RuntimeSequenceID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState) const
{
	FMovieSceneSequenceID ThisSequenceID = FMovieSceneSequenceID(SequenceID);

	if (ResolveParentIndex == FixedRootSequenceParentIndex)
	{
		return UE::MovieScene::FFixedObjectBindingID(Guid, ThisSequenceID);
	}
	else if (ensure(RuntimeSequenceID != MovieSceneSequenceID::Invalid))
	{
		ThisSequenceID = UE::MovieScene::ResolveExternalSequenceID(RuntimeSequenceID, ResolveParentIndex, ThisSequenceID, SharedPlaybackState);

		return UE::MovieScene::FFixedObjectBindingID(Guid, ThisSequenceID);
	}

	return UE::MovieScene::FFixedObjectBindingID(Guid, MovieSceneSequenceID::Invalid);
}

TArrayView<TWeakObjectPtr<>> FMovieSceneObjectBindingID::ResolveBoundObjects(FMovieSceneSequenceID LocalSequenceID, IMovieScenePlayer& Player) const
{
	return ResolveBoundObjects(LocalSequenceID, Player.GetSharedPlaybackState());
}

TArrayView<TWeakObjectPtr<>> FMovieSceneObjectBindingID::ResolveBoundObjects(FMovieSceneSequenceID LocalSequenceID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState) const
{
	if (IsValid())
	{
		if (FMovieSceneEvaluationState* State = SharedPlaybackState->FindCapability<FMovieSceneEvaluationState>())
		{
			FMovieSceneSequenceID ResolvedSequenceID = ResolveSequenceID(LocalSequenceID, SharedPlaybackState);
			return State->FindBoundObjects(Guid, ResolvedSequenceID, SharedPlaybackState);
		}
	}

	return TArrayView<TWeakObjectPtr<>>();
}

TArrayView<TWeakObjectPtr<>> FMovieSceneObjectBindingID::ResolveBoundObjects(const UE::MovieScene::FSequenceInstance& SequenceInstance) const
{
	if (IsValid())
	{
		return ResolveBoundObjects(SequenceInstance.GetSequenceID(), SequenceInstance.GetSharedPlaybackState());
	}

	return TArrayView<TWeakObjectPtr<>>();
}
