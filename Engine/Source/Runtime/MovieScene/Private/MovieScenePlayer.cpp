// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "IMovieScenePlayer.h"
#include "Evaluation/EventContextsPlaybackCapability.h"
#include "Evaluation/EventTriggerControlPlaybackCapability.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "EntitySystem/MovieSceneSequenceInstance.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "IMovieScenePlaybackClient.h"
#include "UniversalObjectLocatorResolveParams.h"
#include "Misc/ScopeRWLock.h"
#include "MovieSceneFwd.h"
#include "MovieSceneSequence.h"
#include "MovieSceneSequenceID.h"
#include "UniversalObjectLocatorResolveParameterBuffer.inl"
#include "MovieSceneBindingReferences.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/TransactionallySafeRWLock.h"

namespace UE
{
namespace MovieScene
{

static FTransactionallySafeRWLock       GGlobalPlayerRegistryLock;
static TSparseArray<IMovieScenePlayer*> GGlobalPlayerRegistry;
static TBitArray<> GGlobalPlayerUpdateFlags;

UE_DEFINE_MOVIESCENE_PLAYBACK_CAPABILITY(FPlayerIndexPlaybackCapability)

IMovieScenePlayer* FPlayerIndexPlaybackCapability::GetPlayer(TSharedRef<const FSharedPlaybackState> Owner)
{
	if (FPlayerIndexPlaybackCapability* Cap = Owner->FindCapability<FPlayerIndexPlaybackCapability>())
	{
		return IMovieScenePlayer::Get(Cap->PlayerIndex);
	}
	return nullptr;
}

uint16 FPlayerIndexPlaybackCapability::GetPlayerIndex(TSharedRef<const FSharedPlaybackState> Owner)
{
	if (FPlayerIndexPlaybackCapability* Cap = Owner->FindCapability<FPlayerIndexPlaybackCapability>())
	{
		return Cap->PlayerIndex;
	}
	return (uint16)-1;
}

} // namespace MovieScene
} // namespace UE

UE_DEFINE_MOVIESCENE_PLAYBACK_CAPABILITY(IMovieScenePlaybackClient)

PRAGMA_DISABLE_DEPRECATION_WARNINGS
IMovieScenePlayer::IMovieScenePlayer()
	: BindingOverrides(StaticBindingOverrides.BindingOverrides)
{
	UE::TWriteScopeLock ScopeLock(UE::MovieScene::GGlobalPlayerRegistryLock);

	UE::MovieScene::GGlobalPlayerRegistry.Shrink();
	UniqueIndex = UE::MovieScene::GGlobalPlayerRegistry.Add(this);

	UE::MovieScene::GGlobalPlayerUpdateFlags.PadToNum(UniqueIndex + 1, false);
	UE::MovieScene::GGlobalPlayerUpdateFlags[UniqueIndex] = 0;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
IMovieScenePlayer::~IMovieScenePlayer()
{	
	UE::TWriteScopeLock ScopeLock(UE::MovieScene::GGlobalPlayerRegistryLock);

	UE::MovieScene::GGlobalPlayerUpdateFlags[UniqueIndex] = 0;
	UE::MovieScene::GGlobalPlayerRegistry.RemoveAt(UniqueIndex, 1);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

IMovieScenePlayer* IMovieScenePlayer::Get(uint16 InUniqueIndex)
{
	UE::TReadScopeLock ScopeLock(UE::MovieScene::GGlobalPlayerRegistryLock);
	check(UE::MovieScene::GGlobalPlayerRegistry.IsValidIndex(InUniqueIndex));
	return UE::MovieScene::GGlobalPlayerRegistry[InUniqueIndex];
}

void IMovieScenePlayer::Get(TArray<IMovieScenePlayer*>& OutPlayers, bool bOnlyUnstoppedPlayers)
{
	UE::TReadScopeLock ScopeLock(UE::MovieScene::GGlobalPlayerRegistryLock);
	for (auto It = UE::MovieScene::GGlobalPlayerRegistry.CreateIterator(); It; ++It)
	{
		if (IMovieScenePlayer* Player = *It)
		{
			if (!bOnlyUnstoppedPlayers || Player->GetPlaybackStatus() != EMovieScenePlayerStatus::Stopped)
			{
				OutPlayers.Add(*It);
			}
		}
	}
}

void IMovieScenePlayer::SetIsEvaluatingFlag(uint16 InUniqueIndex, bool bIsUpdating)
{
	check(UE::MovieScene::GGlobalPlayerUpdateFlags.IsValidIndex(InUniqueIndex));
	UE::MovieScene::GGlobalPlayerUpdateFlags[InUniqueIndex] = bIsUpdating;
}

bool IMovieScenePlayer::IsEvaluating() const
{
	return UE::MovieScene::GGlobalPlayerUpdateFlags[UniqueIndex];
}

void IMovieScenePlayer::PopulateUpdateFlags(UE::MovieScene::ESequenceInstanceUpdateFlags& OutFlags)
{
	using namespace UE::MovieScene;

	OutFlags |= ESequenceInstanceUpdateFlags::NeedsPreEvaluation | ESequenceInstanceUpdateFlags::NeedsPostEvaluation;
}

void IMovieScenePlayer::ResolveBoundObjects(const FGuid& InBindingId, FMovieSceneSequenceID SequenceID, UMovieSceneSequence& Sequence, UObject* ResolutionContext, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const
{
	// This deprecated version of ResolveBoundObjects no longer gets called directly by the FMovieSceneObjectCache- that uses the ResolveParams overload below.
	// In order to ensure things continue to work properly for anyone that may have been calling this directly rather than FindBoundObjects, we direct
	// this towards FindBoundObjects below.

	TArrayView<TWeakObjectPtr<>> BoundObjects = const_cast<IMovieScenePlayer*>(this)->FindBoundObjects(InBindingId, SequenceID);
	for (TWeakObjectPtr<> BoundObject : BoundObjects)
	{
		if (UObject* Obj = BoundObject.Get())
		{
			OutObjects.Add(Obj);
		}
	}
}

void IMovieScenePlayer::ResolveBoundObjects(UE::UniversalObjectLocator::FResolveParams& LocatorResolveParams, const FGuid& InBindingId, FMovieSceneSequenceID SequenceID, UMovieSceneSequence& InSequence, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const
{
	using namespace UE::UniversalObjectLocator;
	using namespace UE::MovieScene;

	const IMovieScenePlaybackClient* PlaybackClient = GetPlaybackClient();

	bool bAllowDefault = PlaybackClient ? PlaybackClient->RetrieveBindingOverrides(InBindingId, SequenceID, OutObjects) : true;

	if (bAllowDefault)
	{
		if (const FMovieSceneBindingReferences* BindingReferences = InSequence.GetBindingReferences())
		{
			FMovieSceneBindingResolveParams BindingResolveParams{ &InSequence, InBindingId, SequenceID, LocatorResolveParams.Context };
			BindingReferences->ResolveBinding(BindingResolveParams, LocatorResolveParams, FindSharedPlaybackState(), OutObjects);
		}
		else
		{
			InSequence.LocateBoundObjects(InBindingId, LocatorResolveParams, FindSharedPlaybackState(), OutObjects);
		}
	}
}

TArrayView<TWeakObjectPtr<>> IMovieScenePlayer::FindBoundObjects(const FGuid& ObjectBindingID, FMovieSceneSequenceIDRef SequenceID)
{
	using namespace UE::MovieScene;

	if (TSharedPtr<const FSharedPlaybackState> SharedPlaybackState = FindSharedPlaybackState())
	{
		FMovieSceneEvaluationState* ActualState = GetEvaluationState();
		return ActualState->FindBoundObjects(ObjectBindingID, SequenceID, SharedPlaybackState.ToSharedRef());
	}

	return TArrayView<TWeakObjectPtr<>>();
}

void IMovieScenePlayer::InvalidateCachedData()
{
	FMovieSceneRootEvaluationTemplateInstance& Template = GetEvaluationTemplate();

	UE::MovieScene::FSequenceInstance* RootInstance = Template.FindInstance(MovieSceneSequenceID::Root);
	if (RootInstance)
	{
		RootInstance->InvalidateCachedData();
	}
}

TSharedPtr<UE::MovieScene::FSharedPlaybackState> IMovieScenePlayer::FindSharedPlaybackState()
{
	return GetEvaluationTemplate().GetSharedPlaybackState();
}

TSharedPtr<const UE::MovieScene::FSharedPlaybackState> IMovieScenePlayer::FindSharedPlaybackState() const
{
	return ConstCastSharedPtr<const UE::MovieScene::FSharedPlaybackState>(const_cast<IMovieScenePlayer*>(this)->GetEvaluationTemplate().GetSharedPlaybackState());
}

TSharedRef<UE::MovieScene::FSharedPlaybackState> IMovieScenePlayer::GetSharedPlaybackState()
{
	// ToSharedRef will assert if evaluation template isn't initialized
	return GetEvaluationTemplate().GetSharedPlaybackState().ToSharedRef();
}

TSharedRef<const UE::MovieScene::FSharedPlaybackState> IMovieScenePlayer::GetSharedPlaybackState() const
{
	// ToSharedRef will assert if evaluation template isn't initialized
	return ConstCastSharedRef<const UE::MovieScene::FSharedPlaybackState>(const_cast<IMovieScenePlayer*>(this)->GetEvaluationTemplate().GetSharedPlaybackState().ToSharedRef());
}

FMovieSceneEvaluationOperand* IMovieScenePlayer::GetBindingOverride(const FMovieSceneEvaluationOperand& InOperand)
{
	if (UE::MovieScene::IStaticBindingOverridesPlaybackCapability* ActualOverrides = GetStaticBindingOverrides())
	{
		return ActualOverrides->GetBindingOverride(InOperand);
	}
	return nullptr;
}

void IMovieScenePlayer::AddBindingOverride(const FMovieSceneEvaluationOperand& InOperand, const FMovieSceneEvaluationOperand& InOverrideOperand)
{
	if (UE::MovieScene::IStaticBindingOverridesPlaybackCapability* ActualOverrides = GetStaticBindingOverrides())
	{
		ActualOverrides->AddBindingOverride(InOperand, InOverrideOperand);
	}
}

void IMovieScenePlayer::RemoveBindingOverride(const FMovieSceneEvaluationOperand& InOperand)
{
	if (UE::MovieScene::IStaticBindingOverridesPlaybackCapability* ActualOverrides = GetStaticBindingOverrides())
	{
		ActualOverrides->RemoveBindingOverride(InOperand);
	}
}

void IMovieScenePlayer::ResetDirectorInstances()
{
	using namespace UE::MovieScene;

	TSharedPtr<const FSharedPlaybackState> SharedPlaybackState = FindSharedPlaybackState();
	if (!SharedPlaybackState)
	{
		return;
	}

	FSequenceDirectorPlaybackCapability* Cap = SharedPlaybackState->FindCapability<FSequenceDirectorPlaybackCapability>();
	if (Cap)
	{
		Cap->ResetDirectorInstances();
	}
}

UObject* IMovieScenePlayer::GetOrCreateDirectorInstance(TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, FMovieSceneSequenceIDRef SequenceID)
{
	using namespace UE::MovieScene;

	FSequenceDirectorPlaybackCapability* Cap = SharedPlaybackState->FindCapability<FSequenceDirectorPlaybackCapability>();
	if (Cap)
	{
		return Cap->GetOrCreateDirectorInstance(SharedPlaybackState, SequenceID);
	}
	return nullptr;
}

TArray<UObject*> IMovieScenePlayer::GetEventContexts() const
{
	using namespace UE::MovieScene;

	// By default, look for the playback capability, for backwards compatibility.
	IMovieScenePlayer* This = const_cast<IMovieScenePlayer*>(this);
	if (TSharedPtr<const FSharedPlaybackState> SharedPlaybackState = This->FindSharedPlaybackState())
	{
		if (IEventContextsPlaybackCapability* EventContextsCapability = SharedPlaybackState->FindCapability<IEventContextsPlaybackCapability>())
		{
			return EventContextsCapability->GetEventContexts();
		}
	}
	return TArray<UObject*>();
}

bool IMovieScenePlayer::IsDisablingEventTriggers(FFrameTime& DisabledUntilTime) const
{
	using namespace UE::MovieScene;

	// By default, look for the playback capability, for backwards compatibility.
	IMovieScenePlayer* This = const_cast<IMovieScenePlayer*>(this);
	if (TSharedPtr<const FSharedPlaybackState> SharedPlaybackState = This->FindSharedPlaybackState())
	{
		if (FEventTriggerControlPlaybackCapability* TriggerControlCapability = SharedPlaybackState->FindCapability<FEventTriggerControlPlaybackCapability>())
		{
			return TriggerControlCapability->IsDisablingEventTriggers(DisabledUntilTime);
		}
	}
	return false;
}


FGuid IMovieScenePlayer::CreateBinding(UMovieSceneSequence* InSequence, UObject* InObject)
{
	if (InSequence && InObject)
	{
		return InSequence->CreatePossessable(InObject);
	}
	return FGuid();
}

FMovieSceneEvaluationState* IMovieScenePlayer::GetEvaluationState()
{
	using namespace UE::MovieScene;

	// Return the playback capability, which is generally the same as the State member variable if
	// InitializeRootInstance has been called, but that member variable will be removed in the future.
	// In addition to this, if our underlying type is FMovieSceneLegacyPlayer, the playback capability
	// is NOT the same as our State member variable.
	if (TSharedPtr<FSharedPlaybackState> SharedPlaybackState = FindSharedPlaybackState())
	{
		return SharedPlaybackState->FindCapability<FMovieSceneEvaluationState>();
	}

	return nullptr;
}

UE::MovieScene::IStaticBindingOverridesPlaybackCapability* IMovieScenePlayer::GetStaticBindingOverrides()
{
	using namespace UE::MovieScene;

	// Return the playback capability, which is generally the same as "this" if InitializeRootInstance 
	// has been called, but we want to deprecate the BindingOverrides member field in the future.
	// In addition to this, if our underlying type is FMovieSceneLegacyPlayer, the playback capability
	// is NOT the same as "this".
	if (TSharedPtr<FSharedPlaybackState> SharedPlaybackState = FindSharedPlaybackState())
	{
		return SharedPlaybackState->FindCapability<IStaticBindingOverridesPlaybackCapability>();
	}

	return nullptr;
}

void IMovieScenePlayer::InitializeRootInstance(TSharedRef<UE::MovieScene::FSharedPlaybackState> NewSharedPlaybackState)
{
	using namespace UE::MovieScene;

	NewSharedPlaybackState->AddCapability<FPlayerIndexPlaybackCapability>(UniqueIndex);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	NewSharedPlaybackState->AddCapabilityRaw(&State);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Only add the spawnregister if it is different from the default 'null' register (which does nothing)
	FMovieSceneSpawnRegister* SpawnRegister = &GetSpawnRegister();
	if (SpawnRegister != &IMovieScenePlayer::GetSpawnRegister())
	{
		NewSharedPlaybackState->AddCapabilityRaw(SpawnRegister);
	}
	NewSharedPlaybackState->AddCapabilityRaw((IObjectBindingNotifyPlaybackCapability*)this);
	NewSharedPlaybackState->AddCapabilityRaw(&StaticBindingOverrides);

	if (IMovieScenePlaybackClient* PlaybackClient = GetPlaybackClient())
	{
		NewSharedPlaybackState->AddCapabilityRaw(PlaybackClient);
	}

	UMovieSceneEntitySystemLinker* Linker = NewSharedPlaybackState->GetLinker();

	if (ensure(Linker))
	{
		FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();

		if (ensure(InstanceRegistry))
		{
			FSequenceInstance& RootInstance = InstanceRegistry->MutateInstance(NewSharedPlaybackState->GetRootInstanceHandle());
			RootInstance.Initialize();
		}
	}
}

