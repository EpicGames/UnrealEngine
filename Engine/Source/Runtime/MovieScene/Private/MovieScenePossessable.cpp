// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScenePossessable.h"
#include "MovieSceneSpawnableAnnotation.h"
#include "MovieSceneSequence.h"
#include "MovieSceneSequenceID.h"
#include "MovieScene.h"
#include "IMovieScenePlayer.h"
#include "EventHandlers/ISequenceDataEventHandler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScenePossessable)

bool FMovieScenePossessable::BindSpawnableObject(FMovieSceneSequenceID SequenceID, UObject* Object, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState)
{
	TOptional<FMovieSceneSpawnableAnnotation> Spawnable = FMovieSceneSpawnableAnnotation::Find(Object);
	if (Spawnable.IsSet())
	{
		// Check whether the spawnable is underneath the current sequence, if so, we can remap it to a local sequence ID
		SetSpawnableObjectBindingID(UE::MovieScene::FRelativeObjectBindingID(SequenceID, Spawnable->SequenceID, Spawnable->ObjectBindingID, SharedPlaybackState));
		return true;
	}

	return false;
}

bool FMovieScenePossessable::BindSpawnableObject(FMovieSceneSequenceID SequenceID, UObject* Object, IMovieScenePlayer* Player)
{
	return BindSpawnableObject(SequenceID, Object, Player->GetSharedPlaybackState());
}

void FMovieScenePossessable::SetParent(const FGuid& InParentGuid)
{
	ParentGuid = InParentGuid;
}

void FMovieScenePossessable::SetParent(const FGuid& InParentGuid, UMovieScene* Owner)
{
	if (ParentGuid != InParentGuid)
	{
		ParentGuid = InParentGuid;
		Owner->EventHandlers.Trigger(&UE::MovieScene::ISequenceDataEventHandler::OnBindingParentChanged, Guid, InParentGuid);
	}
}

#if WITH_EDITORONLY_DATA

void FMovieScenePossessable::FixupPossessedObjectClass(UMovieSceneSequence* InSequence, UObject* Context)
{
	TArray<UObject*, TInlineAllocator<1>> BoundObjects;
	InSequence->LocateBoundObjects(Guid, UE::UniversalObjectLocator::FResolveParams(Context), nullptr, BoundObjects);

	TArray<UClass*> Classes;
	for (UObject* BoundObject : BoundObjects)
	{
		if (BoundObject)
		{
			Classes.Add(BoundObject->GetClass());
		}
	}

	if (Classes.Num() > 0)
	{
		const UClass* ObjectClass = UClass::FindCommonBase(Classes);
		if (ObjectClass)
		{
			PossessedObjectClass = ObjectClass;
		}
	}
}

#endif