// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/MovieScenePropertyBinding.h"
#include "UObject/NameTypes.h"
#include "Misc/StringBuilder.h"
#include "Containers/StringView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScenePropertyBinding)

FMovieScenePropertyBinding::FMovieScenePropertyBinding(FName InPropertyName, const FString& InPropertyPath)
	: PropertyName(InPropertyName), PropertyPath(*InPropertyPath)
{
	bCanUseClassLookup = !(InPropertyPath.Contains(TEXT("/")) || InPropertyPath.Contains(TEXT("\\")) || InPropertyPath.Contains(TEXT("[")));
}

FMovieScenePropertyBinding FMovieScenePropertyBinding::FromPath(const FString& InPropertyPath)
{
	FName PropertyName;

	int32 NamePos = INDEX_NONE;
	if (InPropertyPath.FindLastChar('.', NamePos) || InPropertyPath.FindLastChar('/', NamePos) || InPropertyPath.FindLastChar('\\', NamePos))
	{
		PropertyName = FName(FStringView(*InPropertyPath + NamePos, InPropertyPath.Len() - NamePos));
	}
	else
	{
		PropertyName = *InPropertyPath;
	}
	return FMovieScenePropertyBinding(PropertyName, InPropertyPath);
}

#if WITH_EDITORONLY_DATA
void FMovieScenePropertyBinding::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		const FNameEntry* NameEntry = PropertyPath.GetComparisonNameEntry();
		if (NameEntry)
		{
			if (NameEntry->IsWide())
			{
				TWideStringBuilder<128> Path;
				NameEntry->AppendNameToString(Path);
				TStringView<WIDECHAR> PathView = Path.ToView();

				int Unused = 0;
				bCanUseClassLookup = !(PathView.FindChar('/', Unused) || PathView.FindChar('\\', Unused) || PathView.FindChar('[', Unused));
			}
			else
			{
				TAnsiStringBuilder<128> Path;
				NameEntry->AppendAnsiNameToString(Path);
				TStringView<ANSICHAR> PathView = Path.ToView();

				int Unused  = 0;
				bCanUseClassLookup = !(PathView.FindChar('/', Unused) || PathView.FindChar('\\', Unused) || PathView.FindChar('[', Unused));
			}
		}
	}
}
#endif
