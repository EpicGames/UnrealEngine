// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneSection.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneTimeHelpers.h"
#include "Decorations/IMovieSceneSectionDecoration.h"
#include "Decorations/IMovieSceneSectionProviderDecoration.h"
#include "Decorations/IMovieSceneLifetimeDecoration.h"
#include "Evaluation/MovieSceneEvalTemplate.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "Generators/MovieSceneEasingCurves.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneBlenderSystem.h"
#include "EntitySystem/IMovieSceneBlenderSystemSupport.h"
#include "Containers/ArrayView.h"
#include "Channels/MovieSceneChannel.h"
#include "UObject/SequencerObjectVersion.h"
#include "Misc/FeedbackContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSection)

#if WITH_EDITORONLY_DATA
FOnMovieSceneSectionAddedToTrack UMovieSceneSection::OnAddedToTrackEvent;
FOnMovieSceneSectionChanged UMovieSceneSection::OnRemovedFromTrackEvent;
#endif

UMovieSceneSection::UMovieSceneSection(const FObjectInitializer& ObjectInitializer)
	: Super( ObjectInitializer )
	, PreRollFrames(0)
	, PostRollFrames(0)
	, RowIndex(0)
	, OverlapPriority(0)
	, bIsActive(true)
	, bIsLocked(false)
	, StartTime_DEPRECATED(0.f)
	, EndTime_DEPRECATED(0.f)
	, PreRollTime_DEPRECATED(0.f)
	, PostRollTime_DEPRECATED(0.f)
	, bIsInfinite_DEPRECATED(0)
	, bSupportsInfiniteRange(false)
{
	SectionRange.Value = TRange<FFrameNumber>(0);

	UMovieSceneBuiltInEasingFunction* DefaultEaseIn = ObjectInitializer.CreateDefaultSubobject<UMovieSceneBuiltInEasingFunction>(this, "EaseInFunction");
	DefaultEaseIn->SetFlags(RF_Public); //@todo Need to be marked public. GLEO occurs when transform sections are added to actor sequence blueprints. Are these not being duplicated properly?
	DefaultEaseIn->Type = EMovieSceneBuiltInEasing::CubicInOut;
	Easing.EaseIn = DefaultEaseIn;

	UMovieSceneBuiltInEasingFunction* DefaultEaseOut = ObjectInitializer.CreateDefaultSubobject<UMovieSceneBuiltInEasingFunction>(this, "EaseOutFunction");
	DefaultEaseOut->SetFlags(RF_Public); //@todo Need to be marked public. GLEO occurs when transform sections are added to actor sequence blueprints. Are these not being duplicated properly?
	DefaultEaseOut->Type = EMovieSceneBuiltInEasing::CubicInOut;
	Easing.EaseOut = DefaultEaseOut;

	ChannelProxyType = EMovieSceneChannelProxyType::Static;

#if WITH_EDITORONLY_DATA
	ColorTint = FColor(0, 0, 0, 0);
#endif
}

void UMovieSceneSection::PostInitProperties()
{
	SetFlags(RF_Transactional);

	// Propagate sub object flags from our outer (track) to ourselves. This is required for sections that are stored on blueprints (archetypes) so that they can be referenced in worlds.
	if (GetOuter()->HasAnyFlags(RF_ClassDefaultObject|RF_ArchetypeObject))
	{
		SetFlags(GetOuter()->GetMaskedFlags(RF_PropagateToSubObjects));
	}
	
	Super::PostInitProperties();
}

bool UMovieSceneSection::IsPostLoadThreadSafe() const
{
	return true;
}

void UMovieSceneSection::PostEditImport()
{
	if (ChannelProxyType == EMovieSceneChannelProxyType::Dynamic)
	{
		ChannelProxy = nullptr;
	}
	Super::PostEditImport();
}

void UMovieSceneSection::Serialize(FArchive& Ar)
{
	using namespace UE::MovieScene;

	if (Ar.IsLoading() && ChannelProxyType == EMovieSceneChannelProxyType::Dynamic)
	{
		ChannelProxy = nullptr;
	}

	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FSequencerObjectVersion::GUID);

	if (Ar.CustomVer(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::FloatToIntConversion)
	{
		const FFrameRate LegacyFrameRate = GetLegacyConversionFrameRate();

		if (bIsInfinite_DEPRECATED && bSupportsInfiniteRange)
		{
			SectionRange = TRange<FFrameNumber>::All();
		}
		else
		{
			FFrameNumber StartFrame = UpgradeLegacyMovieSceneTime(this, LegacyFrameRate, StartTime_DEPRECATED);
			FFrameNumber LastFrame  = UpgradeLegacyMovieSceneTime(this, LegacyFrameRate, EndTime_DEPRECATED);

			// Exclusive upper bound so we want the upper bound to be exclusively the next frame after LastFrame
			SectionRange = TRange<FFrameNumber>(StartFrame, LastFrame + 1);
		}

		// All these times are offsets from the start/end time so it's highly unlikely that they'll be out-of bounds
		PreRollFrames                = LegacyFrameRate.AsFrameNumber(PreRollTime_DEPRECATED).Value;
		PostRollFrames               = LegacyFrameRate.AsFrameNumber(PostRollTime_DEPRECATED).Value;
#if WITH_EDITORONLY_DATA
		Easing.AutoEaseInDuration    = (Easing.AutoEaseInTime_DEPRECATED * LegacyFrameRate).RoundToFrame().Value;
		Easing.AutoEaseOutDuration   = (Easing.AutoEaseOutTime_DEPRECATED * LegacyFrameRate).RoundToFrame().Value;
		Easing.ManualEaseInDuration  = (Easing.ManualEaseInTime_DEPRECATED * LegacyFrameRate).RoundToFrame().Value;
		Easing.ManualEaseOutDuration = (Easing.ManualEaseOutTime_DEPRECATED * LegacyFrameRate).RoundToFrame().Value;
#endif
	}
}

#if WITH_EDITORONLY_DATA

FOnMovieSceneSectionAddedToTrack& UMovieSceneSection::GetOnSectionAddedToTrack()
{
	return OnAddedToTrackEvent;
}

FOnMovieSceneSectionChanged& UMovieSceneSection::GetOnSectionRemovedFromTrack()
{
	return OnRemovedFromTrackEvent;
}

#endif  // WITH_EDITORONLY_DATA

void UMovieSceneSection::OnAddedToTrack(UMovieSceneTrack* Track)
{
	OnAddedToTrackImpl(Track);

	if (UMovieScene* MovieScene = Track->GetTypedOuter<UMovieScene>())
	{
		for (UObject* Decoration : GetDecorations())
		{
			if (IMovieSceneLifetimeDecoration* Lifetime = Cast<IMovieSceneLifetimeDecoration>(Decoration))
			{
				Lifetime->OnReconstruct(MovieScene);
			}
		}
	}

#if WITH_EDITORONLY_DATA
	OnAddedToTrackEvent.Broadcast(Track, this);
#endif
}

void UMovieSceneSection::OnRemovedFromTrack()
{
	OnRemovedFromTrackImpl();

	if (UMovieScene* MovieScene = GetTypedOuter<UMovieScene>())
	{
		for (UObject* Decoration : GetDecorations())
		{
			if (IMovieSceneLifetimeDecoration* Lifetime = Cast<IMovieSceneLifetimeDecoration>(Decoration))
			{
				Lifetime->OnDestroy(MovieScene);
			}
		}
	}

#if WITH_EDITORONLY_DATA
	OnRemovedFromTrackEvent.Broadcast(this);
#endif
}

void UMovieSceneSection::OnDecorationAdded(UObject* NewDecoration)
{
	UMovieScene* MovieScene = GetTypedOuter<UMovieScene>();
	IMovieSceneLifetimeDecoration* Lifetime = Cast<IMovieSceneLifetimeDecoration>(NewDecoration);

	if (MovieScene && Lifetime)
	{
		Lifetime->OnReconstruct(MovieScene);
	}
	if (IMovieSceneSectionDecoration* DecorationInterface = Cast<IMovieSceneSectionDecoration>(NewDecoration))
	{
		DecorationInterface->OnDecorationAdded(this);
	}

	EventHandlers.Trigger(&UE::MovieScene::ISectionEventHandler::OnDecorationAdded, NewDecoration);
}

void UMovieSceneSection::OnDecorationRemoved(UObject* Decoration)
{
	if (IMovieSceneSectionDecoration* DecorationInterface = Cast<IMovieSceneSectionDecoration>(Decoration))
	{
		DecorationInterface->OnDecorationRemoved();
	}

	UMovieScene* MovieScene = GetTypedOuter<UMovieScene>();
	IMovieSceneLifetimeDecoration* Lifetime = Cast<IMovieSceneLifetimeDecoration>(Decoration);

	if (MovieScene && Lifetime)
	{
		Lifetime->OnDestroy(MovieScene);
	}
	EventHandlers.Trigger(&UE::MovieScene::ISectionEventHandler::OnDecorationRemoved, Decoration);
}

#if WITH_EDITORONLY_DATA
void UMovieSceneSection::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(UMovieSceneBuiltInEasingFunction::StaticClass()));
}
#endif


void UMovieSceneSection::PostDuplicate(bool bDuplicateForPIE)
{
	if (ChannelProxyType == EMovieSceneChannelProxyType::Dynamic)
	{
		ChannelProxy = nullptr;
	}

	Super::PostDuplicate(bDuplicateForPIE);
}

void UMovieSceneSection::PostRename(UObject* OldOuter, const FName OldName)
{
	if (ChannelProxyType == EMovieSceneChannelProxyType::Dynamic)
	{
		ChannelProxy = nullptr;
	}

	UMovieScene* NewMovieScene = GetTypedOuter<UMovieScene>();
	if (NewMovieScene && NewMovieScene != OldOuter->GetTypedOuter<UMovieScene>())
	{
		for (UObject* Decoration : GetDecorations())
		{
			// If this is being renamed into a movie scene structure, call reconstruct on its decorations
			if (IMovieSceneLifetimeDecoration* DecorationInterface = Cast<IMovieSceneLifetimeDecoration>(Decoration))
			{
				DecorationInterface->OnReconstruct(NewMovieScene);
			}
		}
	}

	Super::PostRename(OldOuter, OldName);
}

void UMovieSceneSection::SetStartFrame(TRangeBound<FFrameNumber> NewStartFrame)
{
	if (TryModify())
	{
		bool bIsValidStartFrame = ensureMsgf(SectionRange.Value.GetUpperBound().IsOpen() || NewStartFrame.IsOpen() || SectionRange.Value.GetUpperBound().GetValue() >= NewStartFrame.GetValue(),
			TEXT("Invalid start frame specified; will be clamped to current end frame."));

		if (bIsValidStartFrame)
		{
			SectionRange.Value.SetLowerBound(NewStartFrame);
		}
		else
		{
			SectionRange.Value.SetLowerBound(TRangeBound<FFrameNumber>::FlipInclusion(SectionRange.Value.GetUpperBound()));
		}
	}
}

void UMovieSceneSection::SetEndFrame(TRangeBound<FFrameNumber> NewEndFrame)
{
	if (TryModify())
	{
		bool bIsValidEndFrame = ensureMsgf(SectionRange.Value.GetLowerBound().IsOpen() || NewEndFrame.IsOpen() || SectionRange.Value.GetLowerBound().GetValue() <= NewEndFrame.GetValue(),
			TEXT("Invalid end frame specified; will be clamped to current start frame."));

		if (bIsValidEndFrame)
		{
			SectionRange.Value.SetUpperBound(NewEndFrame);
		}
		else
		{
			SectionRange.Value.SetUpperBound(TRangeBound<FFrameNumber>::FlipInclusion(SectionRange.Value.GetLowerBound()));
		}
	}
}

void UMovieSceneSection::InvalidateChannelProxy()
{
	ChannelProxyType = EMovieSceneChannelProxyType::Dynamic;
	ChannelProxy = nullptr;
}

FMovieSceneChannelProxy& UMovieSceneSection::GetChannelProxy() const
{
	if (!ChannelProxy.IsValid())
	{
		ChannelProxyType = const_cast<UMovieSceneSection*>(this)->CacheChannelProxy();
	}

	FMovieSceneChannelProxy* Proxy = ChannelProxy.Get();
	check(Proxy);
	return *Proxy;
}

EMovieSceneChannelProxyType UMovieSceneSection::CacheChannelProxy()
{
	ChannelProxy = MakeShared<FMovieSceneChannelProxy>();
	return EMovieSceneChannelProxyType::Static;
}

TSharedPtr<FStructOnScope> UMovieSceneSection::GetKeyStruct(TArrayView<const FKeyHandle> KeyHandles)
{
	return nullptr;
}

void UMovieSceneSection::MoveSectionImpl(FFrameNumber DeltaFrame)
{
	if (TryModify())
	{
		TRange<FFrameNumber> NewRange = SectionRange.Value;
		if (SectionRange.Value.GetLowerBound().IsClosed())
		{
			SectionRange.Value.SetLowerBoundValue(SectionRange.Value.GetLowerBoundValue() + DeltaFrame);
		}
		if (SectionRange.Value.GetUpperBound().IsClosed())
		{
			SectionRange.Value.SetUpperBoundValue(SectionRange.Value.GetUpperBoundValue() + DeltaFrame);
		}
#if WITH_EDITOR
		const bool bHasStartFrame = HasStartFrame();

		for (const FMovieSceneChannelEntry& Entry : GetChannelProxy().GetAllEntries())
		{
			TArrayView<FMovieSceneChannel* const>        Channels = Entry.GetChannels();
			TArrayView<const FMovieSceneChannelMetaData> MetaData = Entry.GetMetaData();
			for (int32 Index = 0; Index < Channels.Num(); ++Index)
			{
				if (!MetaData[Index].bRelativeToSection || !bHasStartFrame)
				{
					Channels[Index]->Offset(DeltaFrame);
				}
			}
		}
#else
		for (const FMovieSceneChannelEntry& Entry : GetChannelProxy().GetAllEntries())
		{
			for (FMovieSceneChannel* Channel : Entry.GetChannels())
			{
				Channel->Offset(DeltaFrame);
			}
		}
#endif
	}
}

void UMovieSceneSection::FixupRelativeKeyframes(FFrameNumber Offset)
{
#if WITH_EDITOR
	if (!TryModify() || !HasStartFrame())
	{
		return;
	}

	for (const FMovieSceneChannelEntry& Entry : GetChannelProxy().GetAllEntries())
	{
		TArrayView<FMovieSceneChannel* const>        Channels = Entry.GetChannels();
		TArrayView<const FMovieSceneChannelMetaData> MetaData = Entry.GetMetaData();
		for (int32 Index = 0; Index < Channels.Num(); ++Index)
		{
			if (MetaData[Index].bRelativeToSection)
			{
				UObject* OwningObject = MetaData[Index].WeakOwningObject.Get();
				if (OwningObject && OwningObject != this)
				{
					OwningObject->Modify();
				}

				Channels[Index]->Offset(-Offset);
			}
		}
	}
#endif
}

void UMovieSceneSection::MoveSection(FFrameNumber DeltaFrame)
{
	MoveSectionImpl(DeltaFrame);
}

TRange<FFrameNumber> UMovieSceneSection::ComputeEffectiveRange() const
{
	if (!SectionRange.Value.GetLowerBound().IsOpen() && !SectionRange.Value.GetUpperBound().IsOpen())
	{
		return GetRange();
	}

	TRange<FFrameNumber> EffectiveRange = TRange<FFrameNumber>::Empty();

	for (const FMovieSceneChannelEntry& Entry : GetChannelProxy().GetAllEntries())
	{
		for (const FMovieSceneChannel* Channel : Entry.GetChannels())
		{
			EffectiveRange = TRange<FFrameNumber>::Hull(EffectiveRange, Channel->ComputeEffectiveRange());
		}
	}

	return TRange<FFrameNumber>::Intersection(EffectiveRange, SectionRange.Value);
}


TOptional<TRange<FFrameNumber> > UMovieSceneSection::GetAutoSizeRange() const
{
	TRange<FFrameNumber> EffectiveRange = TRange<FFrameNumber>::Empty();
	
	for (const FMovieSceneChannelEntry& Entry : GetChannelProxy().GetAllEntries())
	{
		for (const FMovieSceneChannel* Channel : Entry.GetChannels())
		{
			EffectiveRange = TRange<FFrameNumber>::Hull(EffectiveRange, Channel->ComputeEffectiveRange());
		}
	}

	if (!EffectiveRange.IsEmpty())
	{
		return EffectiveRange;
	}

	return TOptional<TRange<FFrameNumber> >();
}

FMovieSceneBlendTypeField UMovieSceneSection::GetSupportedBlendTypes() const
{
	UMovieSceneTrack* Track = GetTypedOuter<UMovieSceneTrack>();
	return Track ? Track->GetSupportedBlendTypes() : FMovieSceneBlendTypeField::None();
}

int32 UMovieSceneSection::GetBlendingOrder() const
{
	//currently needs to support overrides and have one present
	int32 BlendingOrder = INDEX_NONE;
	bool bHasOverride = false;
	if (GetBlendType().IsValid() && GetBlendType().BlendType == EMovieSceneBlendType::Absolute)
	{
		return BlendingOrder;
	}

	if (UMovieSceneTrack* Track = GetTypedOuter<UMovieSceneTrack>())
	{
		if (Track->GetSupportedBlendTypes().Contains(EMovieSceneBlendType::Override))
		{
			const TArray<UMovieSceneSection*>& Sections = Track->GetAllSections();
			for (int32 Index = 0; Index < Sections.Num(); ++Index)
			{
				if (bHasOverride == false && Sections[Index]->GetBlendType().IsValid() && Sections[Index]->GetBlendType() == EMovieSceneBlendType::Override)
				{
					bHasOverride = true;
				}
				if (Sections[Index] == this)
				{
					BlendingOrder = Index;
				}
				if (bHasOverride && BlendingOrder != INDEX_NONE)
				{
					break;
				}
			}
		}
	}
	return bHasOverride ? BlendingOrder : INDEX_NONE;
}

void UMovieSceneSection::BuildDefaultComponents(UMovieSceneEntitySystemLinker* EntityLinker, const UE::MovieScene::FEntityImportParams& Params, UE::MovieScene::FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();

	FComponentTypeID BlendTag;
	if (BlendType.IsValid())
	{
		if (BlendType.Get() == EMovieSceneBlendType::Absolute)
		{
			BlendTag = Components->Tags.AbsoluteBlend;
		}
		else if (BlendType.Get() == EMovieSceneBlendType::Relative)
		{
			BlendTag = Components->Tags.RelativeBlend;
		}
		else if (BlendType.Get() == EMovieSceneBlendType::Additive)
		{
			BlendTag = Components->Tags.AdditiveBlend;
		}
		else if (BlendType.Get() == EMovieSceneBlendType::AdditiveFromBase)
		{
			BlendTag = Components->Tags.AdditiveFromBaseBlend;
		}
		else if (BlendType.Get() == EMovieSceneBlendType::Override)
		{
			BlendTag = Components->Tags.OverrideBlend;
		}
	}

	int32 BlendingOrder = GetBlendingOrder();

	const bool bHasEasing = (Easing.GetEaseInDuration() > 0 || Easing.GetEaseOutDuration() > 0);

	// Should restore state if we're not forcing keep state and any one of the following:
	// - We're forcing restore state
	// - This section is set to restore state
	// - This section is set to the default, and the default is restore state
	const bool bForceKeepState     = EnumHasAnyFlags(Params.Sequence.SubSectionFlags, EMovieSceneSubSectionFlags::OverrideKeepState);
	const bool bShouldRestoreState = bForceKeepState == false &&
		(
			EnumHasAnyFlags(Params.Sequence.SubSectionFlags, EMovieSceneSubSectionFlags::OverrideRestoreState) ||
			(EvalOptions.CompletionMode == EMovieSceneCompletionMode::RestoreState) ||
			(EvalOptions.CompletionMode == EMovieSceneCompletionMode::ProjectDefault && Params.Sequence.DefaultCompletionMode == EMovieSceneCompletionMode::RestoreState)
		);

	TComponentTypeID<FEasingComponentData> EasingComponentID = Components->Easing;
	FComponentTypeID RestoreStateTag = Components->Tags.RestoreState;

	const bool bHasForcedTime      = Params.EntityMetaData && Params.EntityMetaData->ForcedTime != TNumericLimits<int32>::Lowest();
	const bool bHasSectionPreRoll  = Params.EntityMetaData && EnumHasAnyFlags(Params.EntityMetaData->Flags, ESectionEvaluationFlags::PreRoll | ESectionEvaluationFlags::PostRoll);
	const bool bHasSequencePreRoll = Params.Sequence.bPreRoll || Params.Sequence.bPostRoll;

	TSubclassOf<UMovieSceneBlenderSystem> BlenderSystemClass = nullptr;

	// Try and find a blender system to use
	{
		IMovieSceneBlenderSystemSupport* BlenderSystemSupport = Cast<IMovieSceneBlenderSystemSupport>(this);
		if (!BlenderSystemSupport)
		{
			BlenderSystemSupport = GetImplementingOuter<IMovieSceneBlenderSystemSupport>();
		}
		if (BlenderSystemSupport)
		{
			BlenderSystemClass = BlenderSystemSupport->GetBlenderSystem();
		}
	}

	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.AddConditional(Components->BlenderType,                    BlenderSystemClass, BlenderSystemClass.Get() != nullptr)
		.AddConditional(Components->BlendingOrder,					BlendingOrder, BlendingOrder != INDEX_NONE)
		.AddConditional(Components->Easing,                         FEasingComponentData{ decltype(FEasingComponentData::Section)(this) }, bHasEasing)
		.AddConditional(Components->HierarchicalBias,               Params.Sequence.HierarchicalBias, Params.Sequence.HierarchicalBias != 0)
		.AddConditional(Components->Interrogation.InputKey,         Params.InterrogationKey, Params.InterrogationKey.IsValid())
		.AddConditional(Components->Interrogation.Instance,         Params.InterrogationInstance, Params.InterrogationInstance.IsValid())
		.AddConditional(Components->EvalTime,                       Params.EntityMetaData ? Params.EntityMetaData->ForcedTime : 0, bHasForcedTime)
		.AddTagConditional(Components->Tags.RestoreState,           bShouldRestoreState)
		.AddTagConditional(Components->Tags.IgnoreHierarchicalBias, EnumHasAnyFlags(Params.Sequence.SubSectionFlags, EMovieSceneSubSectionFlags::IgnoreHierarchicalBias))
		.AddTagConditional(Components->Tags.BlendHierarchicalBias,  EnumHasAnyFlags(Params.Sequence.SubSectionFlags, EMovieSceneSubSectionFlags::BlendHierarchicalBias))
		.AddTagConditional(Components->Tags.FixedTime,              bHasForcedTime)
		.AddTagConditional(Components->Tags.SectionPreRoll,         bHasSectionPreRoll)
		.AddTagConditional(Components->Tags.PreRoll,                bHasSequencePreRoll || bHasSectionPreRoll)
		.AddTagConditional(BlendTag,                                BlendTag != FComponentTypeID::Invalid())
	);

	if (BlendTag == Components->Tags.AdditiveFromBaseBlend)
	{
		const UMovieScene* MovieScene = GetTypedOuter<UMovieScene>();
		const TRange<FFrameNumber> PlaybackRange = MovieScene->GetPlaybackRange();
		const TRange<FFrameNumber> TrueRange = GetTrueRange();
		const FFrameNumber BaseValueEvalTime = TrueRange.HasLowerBound() ? 
			TrueRange.GetLowerBoundValue() : 
			(PlaybackRange.HasLowerBound() ? PlaybackRange.GetLowerBoundValue() : FFrameNumber(0));
		OutImportedEntity->AddBuilder(
			FEntityBuilder().Add(Components->BaseValueEvalTime, BaseValueEvalTime)
		);
	}
}

bool UMovieSceneSection::TryModify(bool bAlwaysMarkDirty)
{
	if (IsReadOnly())
	{
		return false;
	}

	Modify(bAlwaysMarkDirty);

	return true;
}

bool UMovieSceneSection::IsReadOnly() const
{
	if (IsLocked())
	{
		return true;
	}

#if WITH_EDITORONLY_DATA
	if (UMovieScene* OuterScene = GetTypedOuter<UMovieScene>())
	{
		if (OuterScene->IsReadOnly())
		{
			return true;
		}
	}
#endif

	return false;
}

void UMovieSceneSection::SetRowIndex(int32 NewRowIndex)
{
	const int32 OldRowIndex = RowIndex;
	RowIndex = NewRowIndex;

	if (OldRowIndex != RowIndex)
	{
		EventHandlers.Trigger(&UE::MovieScene::ISectionEventHandler::OnRowChanged, this);
	}
}

void UMovieSceneSection::GetOverlappingSections(TArray<UMovieSceneSection*>& OutSections, bool bSameRow, bool bIncludeThis)
{
	UMovieSceneTrack* Track = GetTypedOuter<UMovieSceneTrack>();
	if (!Track)
	{
		return;
	}

	TRange<FFrameNumber> ThisRange = GetRange();
	for (UMovieSceneSection* Section : Track->GetAllSections())
	{
		if (!Section || (!bIncludeThis && Section == this))
		{
			continue;
		}

		if (bSameRow && Section->GetRowIndex() != GetRowIndex())
		{
			continue;
		}

		if (Section->GetRange().Overlaps(ThisRange))
		{
			OutSections.Add(Section);
		}
	}
}

/* Returns whether this section can have an open lower bound. This will generally be false if sections of this type cannot be blended and there is another section on the same row before this one.*/
bool UMovieSceneSection::CanHaveOpenLowerBound() const
{
	if (!GetBlendType().IsValid())
	{
		UMovieSceneTrack* Track = GetTypedOuter<UMovieSceneTrack>();
		if (!Track)
		{
			return true;
		}

		TRange<FFrameNumber> ThisRange = GetRange();

		if (!ThisRange.HasLowerBound())
		{
			return true;
		}

		for (UMovieSceneSection* Section : Track->GetAllSections())
		{
			if (!Section || (Section == this))
			{
				continue;
			}

			if (Section->GetRowIndex() != GetRowIndex())
			{
				continue;
			}

			if (Section->GetRange().Overlaps(ThisRange) || (Section->GetRange().HasUpperBound() && Section->GetRange().GetUpperBoundValue() <= ThisRange.GetLowerBoundValue()))
			{
				return false;
			}
		}
	}
	return true;
}

/* Returns whether this section can have an open upper bound. This will generally be false if sections of this type cannot be blended and there is another section on the same row after this one.*/
bool UMovieSceneSection::CanHaveOpenUpperBound() const
{
	if (!GetBlendType().IsValid())
	{
		UMovieSceneTrack* Track = GetTypedOuter<UMovieSceneTrack>();
		if (!Track)
		{
			return true;
		}

		TRange<FFrameNumber> ThisRange = GetRange();

		if (!ThisRange.HasUpperBound())
		{
			return true;
		}

		for (UMovieSceneSection* Section : Track->GetAllSections())
		{
			if (!Section || (Section == this))
			{
				continue;
			}

			if (Section->GetRowIndex() != GetRowIndex())
			{
				continue;
			}

			if (Section->GetRange().Overlaps(ThisRange) || (Section->GetRange().HasLowerBound() && Section->GetRange().GetLowerBoundValue() >= ThisRange.GetUpperBoundValue()))
			{
				return false;
			}
		}
	}
	return true;
}


const UMovieSceneSection* UMovieSceneSection::OverlapsWithSections(const TArray<UMovieSceneSection*>& Sections, int32 TrackDelta, int32 TimeDelta) const
{
	// Check overlaps with exclusive ranges so that sections can butt up against each other
	int32 NewTrackIndex = RowIndex + TrackDelta;

	// @todo: sequencer-timecode: is this correct? it seems like we should just use the section's ranges directly rather than fiddling with the bounds
	// TRange<FFrameNumber> NewSectionRange;
	// if (SectionRange.GetLowerBound().IsClosed())
	// {
	// 	NewSectionRange = TRange<FFrameNumber>(
	// 		TRangeBound<FFrameNumber>::Exclusive(SectionRange.GetLowerBoundValue() + TimeDelta),
	// 		NewSectionRange.GetUpperBound()
	// 		);
	// }

	// if (SectionRange.GetUpperBound().IsClosed())
	// {
	// 	NewSectionRange = TRange<FFrameNumber>(
	// 		NewSectionRange.GetLowerBound(),
	// 		TRangeBound<FFrameNumber>::Exclusive(SectionRange.GetUpperBoundValue() + TimeDelta)
	// 		);
	// }

	TRange<FFrameNumber> ThisRange = SectionRange.Value;

	for (const auto Section : Sections)
	{
		check(Section);
		if ((this != Section) && (Section->GetRowIndex() == NewTrackIndex))
		{
			//TRange<float> ExclusiveSectionRange = TRange<float>(TRange<float>::BoundsType::Exclusive(Section->GetRange().GetLowerBoundValue()), TRange<float>::BoundsType::Exclusive(Section->GetRange().GetUpperBoundValue()));
			if (ThisRange.Overlaps(Section->GetRange()))
			{
				return Section;
			}
		}
	}

	return nullptr;
}


void UMovieSceneSection::InitialPlacement(const TArray<UMovieSceneSection*>& Sections, FFrameNumber InStartTime, int32 Duration, bool bAllowMultipleRows)
{
	check(Duration >= 0);

	// Inclusive lower, exclusive upper bounds
	SectionRange = TRange<FFrameNumber>(InStartTime, InStartTime + Duration);
	RowIndex = 0;

	for (UMovieSceneSection* OtherSection : Sections)
	{
		OverlapPriority = FMath::Max(OtherSection->GetOverlapPriority()+1, OverlapPriority);
	}

	if (bAllowMultipleRows)
	{
		while (OverlapsWithSections(Sections) != nullptr)
		{
			++RowIndex;
		}
	}
	else
	{
		for (;;)
		{
			const UMovieSceneSection* OverlappedSection = OverlapsWithSections(Sections);
			if (OverlappedSection == nullptr)
			{
				break;
			}

			TRange<FFrameNumber> OtherRange = OverlappedSection->GetRange();
			if (OtherRange.GetUpperBound().IsClosed())
			{
				MoveSectionImpl(OtherRange.GetUpperBoundValue() - InStartTime);
			}
			else
			{
				++OverlapPriority;
				break;
			}
		}
	}

	UMovieSceneTrack* Track = GetTypedOuter<UMovieSceneTrack>();
	if (Track)
	{
		Track->UpdateEasing();
	}
}

void UMovieSceneSection::InitialPlacementOnRow(const TArray<UMovieSceneSection*>& Sections, FFrameNumber InStartTime, int32 Duration, int32 InRowIndex)
{
	check(Duration >= 0);

	// Inclusive lower, exclusive upper bounds
	SectionRange = TRange<FFrameNumber>(InStartTime, InStartTime + Duration);
	RowIndex = InRowIndex;

	// If no given row index, put it on the next available row
	if (RowIndex == INDEX_NONE)
	{
		RowIndex = 0;
		while (OverlapsWithSections(Sections) != nullptr)
		{
			++RowIndex;
		}
	}

	for (UMovieSceneSection* OtherSection : Sections)
	{
		OverlapPriority = FMath::Max(OtherSection->GetOverlapPriority()+1, OverlapPriority);
	}

	// If this overlaps with any sections, move out all the sections that are beyond this row
	if (OverlapsWithSections(Sections))
	{
		for (UMovieSceneSection* OtherSection : Sections)
		{
			if (OtherSection != nullptr && OtherSection != this && OtherSection->GetRowIndex() >= RowIndex)
			{
				OtherSection->SetRowIndex(OtherSection->GetRowIndex()+1);
			}
		}
	}

	UMovieSceneTrack* Track = GetTypedOuter<UMovieSceneTrack>();
	if (Track)
	{
		Track->UpdateEasing();
	}
}

void UMovieSceneSection::SetColorTint(const FColor& InColorTint)
{
#if WITH_EDITORONLY_DATA
	if (TryModify())
	{
		ColorTint = InColorTint;
	}
#endif
}

FColor UMovieSceneSection::GetColorTint() const
{
#if WITH_EDITORONLY_DATA
	return ColorTint;
#else
	return FColor(0, 0, 0, 0);
#endif
}

UMovieSceneSection* UMovieSceneSection::SplitSection(FQualifiedFrameTime SplitTime, bool bDeleteKeys)
{
	if (!SectionRange.Value.Contains(SplitTime.Time.GetFrame()))
	{
		return nullptr;
	}

	SetFlags(RF_Transactional);

	if (TryModify())
	{
		// A section is owned either directly by a track, or by a decoration that hosts its own
		// sections (e.g. animation mask sections on a mixer layer). Resolve whichever owner this
		// section belongs to so the duplicated right-hand section can be registered with it.
		UObject* Owner = GetOuter();
		UMovieSceneTrack* OwningTrack = Cast<UMovieSceneTrack>(Owner);
		IMovieSceneSectionProviderDecoration* SectionProvider = OwningTrack ? nullptr : Cast<IMovieSceneSectionProviderDecoration>(Owner);

		if (!OwningTrack && !SectionProvider)
		{
			return nullptr;
		}

		Owner->Modify();

		// Duplicate the current section to be the section on the right side of the trim point
		UMovieSceneSection* NewSection = DuplicateObject<UMovieSceneSection>(this, Owner);
		check(NewSection);

		// Tracks register the new section before trimming. Section provider decorations may
		// auto-resolve overlapping ranges when a section is added, which would collapse the
		// freshly-duplicated section while it still spans the full range - so for those we add
		// only after both halves have been trimmed below.
		if (OwningTrack)
		{
			OwningTrack->AddSection(*NewSection);
		}

		TrimSection(SplitTime, false, bDeleteKeys);
		Easing.AutoEaseOutDuration = 0;
		Easing.bManualEaseOut = false;
		Easing.ManualEaseOutDuration = 0;

		NewSection->TrimSection(SplitTime, true, bDeleteKeys);
		NewSection->Easing.AutoEaseInDuration = 0;
		NewSection->Easing.bManualEaseIn = false;
		NewSection->Easing.ManualEaseInDuration = 0;

		if (SectionProvider)
		{
			SectionProvider->AddSection(NewSection);
		}

		return NewSection;
	}

	return nullptr;
}

UObject* UMovieSceneSection::GetImplicitObjectOwner()
{
	if (UMovieSceneTrack* Track = GetTypedOuter<UMovieSceneTrack>())
	{
		if (UMovieScene* MovieScene = Track->GetTypedOuter<UMovieScene>())
		{
			FGuid Guid;
			if (MovieScene->FindTrackBinding(*Track, Guid))
			{
				if (FMovieSceneSpawnable* MovieSceneSpanwable = MovieScene->FindSpawnable(Guid))
				{
					return MovieSceneSpanwable->GetObjectTemplate();
				}
				else if (FMovieScenePossessable* MovieScenePossessable = MovieScene->FindPossessable(Guid))
				{
#if WITH_EDITORONLY_DATA
					if (MovieScenePossessable->GetPossessedObjectClass() && MovieScenePossessable->GetPossessedObjectClass()->GetDefaultObject())
					{
						return MovieScenePossessable->GetPossessedObjectClass()->GetDefaultObject();
					}
#endif
				}
			}
		}
	}
	return nullptr;
}


void UMovieSceneSection::TrimSection(FQualifiedFrameTime TrimTime, bool bTrimLeft, bool bDeleteKeys)
{
	if (SectionRange.Value.Contains(TrimTime.Time.GetFrame()))
	{
		SetFlags(RF_Transactional);
		if (TryModify())
		{
			if (bTrimLeft)
			{
				SectionRange.Value.SetLowerBound(TRangeBound<FFrameNumber>::Inclusive(TrimTime.Time.GetFrame()));
			}
			else
			{
				SectionRange.Value.SetUpperBound(TRangeBound<FFrameNumber>::Exclusive(TrimTime.Time.GetFrame()));
			}

			if (bDeleteKeys)
			{
				for (const FMovieSceneChannelEntry& Entry : GetChannelProxy().GetAllEntries())
				{
					for (FMovieSceneChannel* Channel : Entry.GetChannels())
					{
						Channel->DeleteKeysFrom(TrimTime.Time.GetFrame(), bTrimLeft);
					}
				}
			}
		}
	}
}


float UMovieSceneSection::EvaluateEasing(FFrameTime InTime) const
{
	float EaseInValue = 1.f;
	float EaseOutValue = 1.f;

	if (HasStartFrame() && Easing.GetEaseInDuration() > 0 && Easing.EaseIn.GetObject())
	{
		const int32  EaseFrame    = (InTime.FrameNumber - GetInclusiveStartFrame()).Value;
		const double EaseInInterp = (double(EaseFrame) + InTime.GetSubFrame()) / Easing.GetEaseInDuration();

		if (EaseInInterp < 0.0)
		{
			EaseInValue = 0.0;
		}
		else if (EaseInInterp > 1.0)
		{
			EaseInValue = 1.0;
		}
		else
		{
			EaseInValue = IMovieSceneEasingFunction::EvaluateWith(Easing.EaseIn, EaseInInterp);
		}
	}


	if (HasEndFrame() && Easing.GetEaseOutDuration() > 0 && Easing.EaseOut.GetObject())
	{
		const int32  EaseFrame     = (InTime.FrameNumber - GetExclusiveEndFrame() + Easing.GetEaseOutDuration()).Value;
		const double EaseOutInterp = (double(EaseFrame) + InTime.GetSubFrame()) / Easing.GetEaseOutDuration();

		if (EaseOutInterp < 0.0)
		{
			EaseOutValue = 1.0;
		}
		else if (EaseOutInterp > 1.0)
		{
			EaseOutValue = 0.0;
		}
		else
		{
			EaseOutValue = 1.f - IMovieSceneEasingFunction::EvaluateWith(Easing.EaseOut, EaseOutInterp);
		}
	}

	return EaseInValue * EaseOutValue;
}


void UMovieSceneSection::EvaluateEasing(FFrameTime InTime, TOptional<float>& OutEaseInValue, TOptional<float>& OutEaseOutValue, float* OutEaseInInterp, float* OutEaseOutInterp) const
{
	if (HasStartFrame() && Easing.EaseIn.GetObject() && GetEaseInRange().Contains(InTime.FrameNumber))
	{
		const int32  EaseFrame    = (InTime.FrameNumber - GetInclusiveStartFrame()).Value;
		const double EaseInInterp = (double(EaseFrame) + InTime.GetSubFrame()) / Easing.GetEaseInDuration();

		OutEaseInValue = IMovieSceneEasingFunction::EvaluateWith(Easing.EaseIn, EaseInInterp);

		if (OutEaseInInterp)
		{
			*OutEaseInInterp = EaseInInterp;
		}
	}

	if (HasEndFrame() && Easing.EaseOut.GetObject() && GetEaseOutRange().Contains(InTime.FrameNumber))
	{
		const int32  EaseFrame     = (InTime.FrameNumber - GetExclusiveEndFrame() + Easing.GetEaseOutDuration()).Value;
		const double EaseOutInterp = (double(EaseFrame) + InTime.GetSubFrame()) / Easing.GetEaseOutDuration();

		OutEaseOutValue = 1.f - IMovieSceneEasingFunction::EvaluateWith(Easing.EaseOut, EaseOutInterp);

		if (OutEaseOutInterp)
		{
			*OutEaseOutInterp = EaseOutInterp;
		}
	}
}


TRange<FFrameNumber> UMovieSceneSection::GetEaseInRange() const
{
	if (HasStartFrame() && Easing.GetEaseInDuration() > 0)
	{
		TRangeBound<FFrameNumber> LowerBound = TRangeBound<FFrameNumber>::Inclusive(GetInclusiveStartFrame());
		TRangeBound<FFrameNumber> UpperBound = TRangeBound<FFrameNumber>::Inclusive(GetInclusiveStartFrame() + Easing.GetEaseInDuration());

		UpperBound = TRangeBound<FFrameNumber>::MinUpper(UpperBound, SectionRange.Value.GetUpperBound());
		return TRange<FFrameNumber>(LowerBound, UpperBound);
	}

	return TRange<FFrameNumber>::Empty();
}


TRange<FFrameNumber> UMovieSceneSection::GetEaseOutRange() const
{
	if (HasEndFrame() && Easing.GetEaseOutDuration() > 0)
	{
		TRangeBound<FFrameNumber> UpperBound = TRangeBound<FFrameNumber>::Inclusive(GetExclusiveEndFrame());
		TRangeBound<FFrameNumber> LowerBound = TRangeBound<FFrameNumber>::Inclusive(GetExclusiveEndFrame() - Easing.GetEaseOutDuration());

		LowerBound = TRangeBound<FFrameNumber>::MaxLower(LowerBound, SectionRange.Value.GetLowerBound());
		return TRange<FFrameNumber>(LowerBound, UpperBound);
	}

	return TRange<FFrameNumber>::Empty();
}


bool UMovieSceneSection::ShouldUpgradeEntityData(FArchive& Ar, FMovieSceneEvaluationCustomVersion::Type UpgradeVersion) const
{
	return Ar.IsLoading() && !Ar.HasAnyPortFlags(PPF_Duplicate | PPF_DuplicateForPIE) && GetLinkerCustomVersion(FMovieSceneEvaluationCustomVersion::GUID) < UpgradeVersion;
}

#if WITH_EDITOR

bool UMovieSceneSection::IsLocalEvalDisabled() const
{
	if (bIsLocalEvalDisabled)
	{
		return true;
	}
	if (UMovieSceneTrack* Track = GetTypedOuter<UMovieSceneTrack>())
	{
		return Track->IsRowEvalDisabled(GetRowIndex(), /*bInCheckLocal=*/true);
	}
	return false;
}

void UMovieSceneSection::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName NAME_SectionRange = GET_MEMBER_NAME_CHECKED(UMovieSceneSection, SectionRange);

	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property != nullptr && PropertyChangedEvent.Property->GetFName() == NAME_SectionRange)
	{
		if (UMovieSceneTrack* Track = GetTypedOuter<UMovieSceneTrack>())
		{
			Track->UpdateEasing();
		}
	}
}


void UMovieSceneSection::PostPaste()
{
	if (UObject* DefaultEaseIn = Easing.EaseIn.GetObject())
	{
		DefaultEaseIn->ClearFlags(RF_Transient);
	}
	if (UObject* DefaultEaseOut = Easing.EaseOut.GetObject())
	{
		DefaultEaseOut->ClearFlags(RF_Transient);
	}

	for (UObject* Decoration : GetDecorations())
	{
		if (!Decoration->GetClass() || !Decoration->GetClass()->HasAnyClassFlags(CLASS_Transient))
		{
			Decoration->ClearFlags(RF_Transient);
		}
	}
}

ECookOptimizationFlags UMovieSceneSection::GetCookOptimizationFlags() const
{
	UMovieSceneTrack* Track = GetTypedOuter<UMovieSceneTrack>();

	if (UMovieSceneTrack::RemoveMutedTracksOnCook() && Track && Track->IsRowEvalDisabled(GetRowIndex()))
	{
		return ECookOptimizationFlags::RemoveSection;
	}
	return ECookOptimizationFlags::None; 
}

void UMovieSceneSection::RemoveForCook()
{
	for (const FMovieSceneChannelEntry& Entry : GetChannelProxy().GetAllEntries())
	{
		for (FMovieSceneChannel* Channel : Entry.GetChannels())
		{
			if (Channel)
			{
				Channel->Reset();
			}
		}
	}
}

#endif
