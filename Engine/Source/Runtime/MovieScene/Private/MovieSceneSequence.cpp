// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneSequence.h"

#include "Evaluation/MovieSceneEvaluationCustomVersion.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "MovieScene.h"
#include "MovieSceneBindingReferences.h"
#include "UObject/EditorObjectVersion.h"
#include "UObject/ReleaseObjectVersion.h"
#include "UObject/ObjectSaveContext.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "Sections/MovieSceneSubSection.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "Interfaces/ITargetPlatform.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "UniversalObjectLocator.h"
#include "Bindings/MovieSceneSpawnableBinding.h"
#include "MovieSceneCommonHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSequence)


UMovieSceneSequence::UMovieSceneSequence(const FObjectInitializer& Init)
	: Super(Init)
{
	bParentContextsAreSignificant = false;
	bPlayableDirectly = true;
	SequenceFlags = EMovieSceneSequenceFlags::None;
	CompiledData = nullptr;

	// Ensure that the precompiled data is set up when constructing the CDO. This guarantees that we do not try and create it for the first time when collecting garbage
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		UMovieSceneCompiledDataManager::GetPrecompiledData();

#if WITH_EDITOR
		UMovieSceneCompiledDataManager::GetPrecompiledData(EMovieSceneServerClientMask::Client);
		UMovieSceneCompiledDataManager::GetPrecompiledData(EMovieSceneServerClientMask::Server);
#endif
	}
}

bool UMovieSceneSequence::MakeLocatorForObject(UObject* Object, UObject* Context, FUniversalObjectLocator& OutLocator) const
{
	if (CanPossessObject(*Object, Context))
	{
		OutLocator.Reset(Object, Context);
		return true;
	}

	return false;
}

const FMovieSceneBindingReferences* UMovieSceneSequence::GetBindingReferences() const
{
	return nullptr;
}

FMovieSceneBindingReferences* UMovieSceneSequence::GetBindingReferences()
{
	const FMovieSceneBindingReferences* Result = const_cast<const UMovieSceneSequence*>(this)->GetBindingReferences();
	return const_cast<FMovieSceneBindingReferences*>(Result);
}

void UMovieSceneSequence::LocateBoundObjects(const FGuid& ObjectId, const UE::UniversalObjectLocator::FResolveParams& ResolveParams, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const
{
	LocateBoundObjects(ObjectId, ResolveParams, nullptr, OutObjects);
}

void UMovieSceneSequence::LocateBoundObjects(const FGuid& ObjectId, const UE::UniversalObjectLocator::FResolveParams& ResolveParams, TSharedPtr<const FSharedPlaybackState> SharedPlaybackState, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const
{
	const FMovieSceneBindingReferences* Refs = GetBindingReferences();
	if (Refs)
	{
		Refs->ResolveBinding(ObjectId, ResolveParams, OutObjects);
	}
	else
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		LocateBoundObjects(ObjectId, const_cast<UObject*>(ResolveParams.Context), OutObjects);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

FGuid UMovieSceneSequence::FindBindingFromObject(UObject* InObject, UObject* Context) const
{
	if (InObject && Context)
	{
		TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState = MovieSceneHelpers::CreateTransientSharedPlaybackState(Context, const_cast<UMovieSceneSequence*>(this));

		return FindBindingFromObject(InObject, SharedPlaybackState);
	}

	return FGuid();
}

void UMovieSceneSequence::PostLoad()
{
	UMovieSceneCompiledDataManager* PrecompiledData = UMovieSceneCompiledDataManager::GetPrecompiledData();

#if WITH_EDITORONLY_DATA
	// Wipe compiled data on editor load to ensure we don't try and iteratively compile previously saved content. In a cooked game, this will contain our up-to-date compiled template.
	PrecompiledData->Reset(this);
#endif

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		PrecompiledData->LoadCompiledData(this);

#if !WITH_EDITOR
		// Don't need this any more - allow it to be GC'd so it doesn't take up memory
		CompiledData = nullptr;
#else
		// Wipe out in -game as well
		if (!GIsEditor)
		{
			CompiledData = nullptr;
		}
#endif
	}

#if DO_CHECK
	if (FPlatformProperties::RequiresCookedData() && !EnumHasAnyFlags(SequenceFlags, EMovieSceneSequenceFlags::Volatile) && !HasAnyFlags(RF_ClassDefaultObject|RF_ArchetypeObject))
	{
		ensureAlwaysMsgf(PrecompiledData->FindDataID(this).IsValid(), TEXT("No precompiled movie scene data is present for sequence '%s'. This should have been generated and saved during cook."), *GetName());
	}
#endif

	Super::PostLoad();
}

void UMovieSceneSequence::BeginDestroy()
{
	Super::BeginDestroy();

	if (!GExitPurge && !HasAnyFlags(RF_ClassDefaultObject))
	{
		UMovieSceneCompiledDataManager::ReportSequenceDestroyed(this);
	}
}

void UMovieSceneSequence::PostDuplicate(bool bDuplicateForPIE)
{
	if (bDuplicateForPIE)
	{
		UMovieSceneCompiledDataManager::GetPrecompiledData()->Compile(this);
	}

	Super::PostDuplicate(bDuplicateForPIE);
}

EMovieSceneServerClientMask UMovieSceneSequence::OverrideNetworkMask(EMovieSceneServerClientMask InDefaultMask) const
{
	return InDefaultMask;
}

void UMovieSceneSequence::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
#if WITH_EDITOR
	if (!HasAnyFlags(RF_ClassDefaultObject|RF_ArchetypeObject))
	{
		const ITargetPlatform* TargetPlatform = ObjectSaveContext.GetTargetPlatform();
		if (TargetPlatform && TargetPlatform->RequiresCookedData())
		{
			EMovieSceneServerClientMask NetworkMask = EMovieSceneServerClientMask::All;
			if (TargetPlatform->IsClientOnly())
			{
				NetworkMask = EMovieSceneServerClientMask::Client;
			}
			else if (!TargetPlatform->AllowAudioVisualData())
			{
				NetworkMask = EMovieSceneServerClientMask::Server;
			}
			NetworkMask = OverrideNetworkMask(NetworkMask);

			if (ObjectSaveContext.IsCooking())
			{
				OptimizeForCook();
			}
	
			UMovieSceneCompiledDataManager::GetPrecompiledData(NetworkMask)->CopyCompiledData(this);
		}
		else if (CompiledData)
		{
			// Don't save template data unless we're cooking
			CompiledData->Reset();
		}
	}
#endif
	Super::PreSave(ObjectSaveContext);
}

void UMovieSceneSequence::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FMovieSceneEvaluationCustomVersion::GUID);
	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);
	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);

	Super::Serialize(Ar);
}

#if WITH_EDITOR

void UMovieSceneSequence::OptimizeForCook()
{
	// Suppress any change to signature GUIDs, because that could cause cooking indeterminism.
	UE::MovieScene::FScopedSignedObjectModifySuppress SignatureChangeSupression(true);

	UMovieScene* MovieScene = GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	// Go through the root tracks.
	for (int32 TrackIndex = 0; TrackIndex < MovieScene->GetTracks().Num(); )
	{
		UMovieSceneTrack* Track = MovieScene->GetTracks()[TrackIndex];
		if (Track && Track->GetCookOptimizationFlags() == ECookOptimizationFlags::RemoveTrack)
		{				
			Track->RemoveForCook();
			MovieScene->RemoveTrack(*Track);
			UE_LOGF(LogMovieScene, Display, "Removing muted track: %ls from: %ls", *Track->GetDisplayName().ToString(), *GetPathName());
			continue;
		}
		++TrackIndex;
	}

	// Go through the root tracks again and look at sections.
	// If a section points to a sub-sequence, also optimize that sub-sequence. We might end up optimizing
	// some of these sub-sequences multiple times, if they're used in more than one place, but any 
	// subsequent times should not do anything.
	for (int32 TrackIndex = 0; TrackIndex < MovieScene->GetTracks().Num(); ++TrackIndex)
	{
		UMovieSceneTrack* Track =  MovieScene->GetTracks()[TrackIndex];
		if (Track)
		{
			for (int32 SectionIndex = 0; SectionIndex < Track->GetAllSections().Num(); )
			{
				UMovieSceneSection* Section = Track->GetAllSections()[SectionIndex];
				if (Section && Section->GetCookOptimizationFlags() == ECookOptimizationFlags::RemoveSection)
				{
					Section->RemoveForCook();
					Track->RemoveSection(*Section);
					UE_LOGF(LogMovieScene, Display, "Removing muted section: %ls from: %ls", *Section->GetPathName(), *Track->GetDisplayName().ToString());
					continue;
				}
				if (UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section))
				{
					if (UMovieSceneSequence* SubSequence = SubSection->GetSequence())
					{
						SubSequence->OptimizeForCook();
					}
				}
				++SectionIndex;
			}
		}
	}

	// Go through object bindings.
	const TArray<FMovieSceneBinding>& Bindings = ((const UMovieScene*)MovieScene)->GetBindings();
	for (int32 ObjectBindingIndex = 0; ObjectBindingIndex < Bindings.Num(); )
	{
		bool bRemoveObject = false;

		// First, see if we need to remove the object.
		for (int32 TrackIndex = 0; TrackIndex < Bindings[ObjectBindingIndex].GetTracks().Num(); ++TrackIndex)
		{
			UMovieSceneTrack* Track = Bindings[ObjectBindingIndex].GetTracks()[TrackIndex];
			if (Track && Track->GetCookOptimizationFlags() == ECookOptimizationFlags::RemoveObject)
			{
				bRemoveObject = true;
				break;
			}
		}

		// Then, remove any appropriate tracks, or all tracks if we decided to remove the object altogether.
		for (int32 TrackIndex = 0; TrackIndex < Bindings[ObjectBindingIndex].GetTracks().Num(); )
		{
			UMovieSceneTrack* Track = Bindings[ObjectBindingIndex].GetTracks()[TrackIndex];
			if (Track && (Track->GetCookOptimizationFlags() == ECookOptimizationFlags::RemoveTrack || bRemoveObject))
			{
				Track->RemoveForCook();
				MovieScene->RemoveTrack(*Track);
				UE_LOGF(LogMovieScene, Display, "Removing muted track: %ls from: %ls", *Track->GetDisplayName().ToString(), *GetPathName());
				continue;
			}
			++TrackIndex;
		}

		// Go through the tracks again and look at sections.
		// Once again, we recurse into sub-sequences if needed (see previous comment).
		for (int32 TrackIndex = 0; TrackIndex < Bindings[ObjectBindingIndex].GetTracks().Num(); ++TrackIndex)
		{
			UMovieSceneTrack* Track = Bindings[ObjectBindingIndex].GetTracks()[TrackIndex];
			if (Track)
			{
				for (int32 SectionIndex = 0; SectionIndex < Track->GetAllSections().Num(); )
				{
					UMovieSceneSection* Section = Track->GetAllSections()[SectionIndex];
					if (Section && (Section->GetCookOptimizationFlags() == ECookOptimizationFlags::RemoveSection || bRemoveObject))
					{
						Section->RemoveForCook();
						Track->RemoveSection(*Section);
						UE_LOGF(LogMovieScene, Display, "Removing muted section: %ls from: %ls", *Section->GetPathName(), *Track->GetDisplayName().ToString());
						continue;
					}
					if (UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section))
					{
						if (UMovieSceneSequence* SubSequence = SubSection->GetSequence())
						{
							SubSequence->OptimizeForCook();
						}
					}
					++SectionIndex;
				}
			}
		}
		
		if (bRemoveObject)
		{
			FGuid GuidToRemove = Bindings[ObjectBindingIndex].GetObjectGuid();
			FText ObjectDisplayName = MovieScene->GetObjectDisplayName(GuidToRemove);
			UE_LOGF(LogMovieScene, Display, "Removing muted object: %ls from: %ls", *ObjectDisplayName.ToString(), *GetPathName());
			MovieScene->RemoveSpawnable(GuidToRemove);
			MovieScene->RemovePossessable(GuidToRemove);
		}
		else
		{
			++ObjectBindingIndex;
		}
	}
}

#endif

UMovieSceneCompiledData* UMovieSceneSequence::GetCompiledData() const
{
	return CompiledData;
}

UMovieSceneCompiledData* UMovieSceneSequence::GetOrCreateCompiledData()
{
	if (!CompiledData)
	{
		CompiledData = FindObject<UMovieSceneCompiledData>(this, TEXT("CompiledData"));
		if (CompiledData)
		{
			CompiledData->Reset();
		}
		else
		{
			CompiledData = NewObject<UMovieSceneCompiledData>(this, "CompiledData");
		}
	}
	return CompiledData;
}

FGuid UMovieSceneSequence::FindPossessableObjectId(UObject& Object, UObject* Context) const
{
	using namespace UE::MovieScene;
	UMovieSceneSequence* ThisSequence = const_cast<UMovieSceneSequence*>(this);
	TSharedRef<UE::MovieScene::FSharedPlaybackState> TransientPlaybackState = MovieSceneHelpers::CreateTransientSharedPlaybackState(Context, ThisSequence);
	if (FMovieSceneEvaluationState* EvaluationState = TransientPlaybackState->FindCapability<FMovieSceneEvaluationState>())
	{
		FGuid ExistingID = EvaluationState->FindObjectId(Object, MovieSceneSequenceID::Root, TransientPlaybackState);
		return ExistingID;
	}
	return FGuid();
}

FMovieSceneObjectBindingID UMovieSceneSequence::FindBindingByTag(FName InBindingName) const
{
	if (const TArray<FMovieSceneObjectBindingID>& Bindings = FindBindingsByTag(InBindingName); Bindings.Num() > 0)
	{
		return Bindings[0];
	}

	FMessageLog("PIE")
		.Warning(NSLOCTEXT("UMovieSceneSequence", "FindNamedBinding_Warning", "Attempted to find a named binding that did not exist"))
		->AddToken(FUObjectToken::Create(this));

	return FMovieSceneObjectBindingID();
}

const TArray<FMovieSceneObjectBindingID>& UMovieSceneSequence::FindBindingsByTag(FName InBindingName) const
{
	const UMovieScene* MovieScene = GetMovieScene();
	if (MovieScene)
	{
		const FMovieSceneObjectBindingIDs* BindingIDs = MovieScene->AllTaggedBindings().Find(InBindingName);
		if (BindingIDs)
		{
			return BindingIDs->IDs;
		}
	}

	static TArray<FMovieSceneObjectBindingID> EmptyBindings;
	return EmptyBindings;
}

FMovieSceneTimecodeSource UMovieSceneSequence::GetEarliestTimecodeSource() const
{
	const UMovieScene* MovieScene = GetMovieScene();
	if (!MovieScene)
	{
		return FMovieSceneTimecodeSource();
	}

	return MovieScene->GetEarliestTimecodeSource();
}

UObject* UMovieSceneSequence::CreateDirectorInstance(IMovieScenePlayer& Player, FMovieSceneSequenceID SequenceID)
{
	return CreateDirectorInstance(Player.GetSharedPlaybackState(), SequenceID);
}

#if WITH_EDITOR

ETrackSupport UMovieSceneSequence::IsTrackSupported(TSubclassOf<class UMovieSceneTrack> InTrackClass) const
{
	if (!UMovieScene::IsTrackClassAllowed(InTrackClass))
	{
		return ETrackSupport::NotSupported;
	}

	return IsTrackSupportedImpl(InTrackClass); 
}

bool UMovieSceneSequence::IsFilterSupported(const FString& InFilterName) const
{
	return IsFilterSupportedImpl(InFilterName); 
}

#endif

bool UMovieSceneSequence::IsSubSequenceCompatible(const UMovieSceneSequence& SubSequence) const
{
	if (IsCompatibleSubSequence(SubSequence))
	{
		return SubSequence.IsCompatibleAsSubSequence(*this);
	}
	return false;
}
