// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/MovieSceneSequenceInstanceData.h"
#include "Evaluation/MovieSceneEvalTemplateBase.h"
#include "Evaluation/MovieSceneEvalTemplateSerializer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSequenceInstanceData)

FMovieSceneSequenceInstanceDataPtr::FMovieSceneSequenceInstanceDataPtr(const FMovieSceneSequenceInstanceDataPtr& RHS)
{
	*this = RHS;
}

FMovieSceneSequenceInstanceDataPtr& FMovieSceneSequenceInstanceDataPtr::operator=(const FMovieSceneSequenceInstanceDataPtr& RHS)
{
	if (RHS.IsValid())
	{
		UScriptStruct::ICppStructOps& StructOps = *RHS->GetScriptStruct().GetCppStructOps();

		void* Allocation = Reserve(StructOps.GetSize(), StructOps.GetAlignment());
		StructOps.Construct(Allocation);
		StructOps.Copy(Allocation, &RHS.GetValue(), 1);
	}

	return *this;
}

bool FMovieSceneSequenceInstanceDataPtr::Serialize(FArchive& Ar)
{
	bool bShouldWarn = !WITH_EDITORONLY_DATA;
	return SerializeInlineValue(*this, Ar, bShouldWarn);
}

