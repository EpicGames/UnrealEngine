// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneSequencePlayer.h"
#include "MovieScene.h"
#include "MovieSceneFwd.h"
#include "MovieSceneTimeHelpers.h"
#include "MovieSceneSequence.h"
#include "MovieSceneSequenceTickManager.h"
#include "Channels/MovieSceneTimeWarpChannel.h"
#include "Engine/Engine.h"
#include "UObject/Stack.h"
#include "Internationalization/Text.h"
#include "GameFramework/WorldSettings.h"
#include "Misc/RuntimeErrors.h"
#include "Net/UnrealNetwork.h"
#include "Engine/NetDriver.h"
#include "Engine/NetConnection.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "Evaluation/EventTriggerControlPlaybackCapability.h"
#include "Evaluation/MovieSceneSequenceWeights.h"
#include "UniversalObjectLocatorResolveParams.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/PlayerState.h"
#include "Algo/BinarySearch.h"
#include "UniversalObjectLocatorResolveParameterBuffer.inl"

#include "Iris/ReplicationSystem/ReplicationFragmentUtil.h"
#include "Net/Iris/ReplicationSystem/ReplicationSystemUtil.h"
#include "MovieSceneObjectBindingID.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSequencePlayer)

DEFINE_LOG_CATEGORY_STATIC(LogMovieSceneRepl, Log, All);

DECLARE_STATS_GROUP(TEXT("MovieSceneRepl"), STATGROUP_MovieSceneRepl, STATCAT_Advanced);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumServerSamples"),MovieSceneRepl_NumServerSamples,STATGROUP_MovieSceneRepl);
DECLARE_FLOAT_COUNTER_STAT(TEXT("SmoothedServerTime"),MovieSceneRepl_SmoothedServerTime,STATGROUP_MovieSceneRepl);

float GSequencerNetSyncThresholdMS = 200;
static FAutoConsoleVariableRef CVarSequencerNetSyncThresholdMS(
	TEXT("Sequencer.NetSyncThreshold"),
	GSequencerNetSyncThresholdMS,
	TEXT("(Default: 200ms. Defines the threshold at which clients and servers must be forcibly re-synced during playback.")
	);

int32 GSequencerMaxSmoothedNetSyncSampleAge = 5000;
static FAutoConsoleVariableRef CVarSequencerMaxSmoothedNetSyncSampleAge(
	TEXT("Sequencer.SmoothedMaxNetSyncSampleAge"),
	GSequencerMaxSmoothedNetSyncSampleAge,
	TEXT("(Default: 5000. Defines the range of samples (in milliseconds) required to perform smoothed net sync. Use 0 to disable smoothing.")
	);

int32 GSequencerMaxSmoothedNetSyncSampleCount = 50;
static FAutoConsoleVariableRef CVarSequencerMaxSmoothedNetSyncSampleCount(
	TEXT("Sequencer.SmoothedMaxNetSyncSampleCount"),
	GSequencerMaxSmoothedNetSyncSampleCount,
	TEXT("(Default: 50. The maximum number of samples to keep in memory.")
	);

float GSequencerSmoothedNetSyncDeviationThreshold = 200;
static FAutoConsoleVariableRef CVarSequencerSmoothedNetSyncDeviationThreshold(
	TEXT("Sequencer.SmoothedNetSyncDeviationThreshold"),
	GSequencerSmoothedNetSyncDeviationThreshold,
	TEXT("(Default: 200ms. Defines the acceptable deviation for smoothed net sync samples. Samples outside this deviation will be discarded.")
	);

int32 GSequencerApplyDisplayRateToDynResFrameTimeBudget = 0;
static FAutoConsoleVariableRef CVarSequencerApplyDisplayRateToDynResFrameTimeBudget(
	TEXT("Sequencer.ApplyDisplayRateToDynamicResolutionFrameTimeBudget"),
	GSequencerApplyDisplayRateToDynResFrameTimeBudget,
	TEXT("(Whether to override r.DynamicRes.FrameTimeBudget based on sequence display rate when using 'Lock to Display Rate at Runtime'.")
);

bool FMovieSceneSequenceLoopCount::SerializeFromMismatchedTag( const FPropertyTag& Tag, FStructuredArchive::FSlot Slot )
{
	if (Tag.Type == NAME_IntProperty)
	{
		Slot << Value;
		return true;
	}

	return false;
}

bool FMovieSceneSequencePlaybackSettings::SerializeFromMismatchedTag( const FPropertyTag& Tag, FStructuredArchive::FSlot Slot )
{
	if (Tag.GetType().IsStruct("LevelSequencePlaybackSettings"))
	{
		StaticStruct()->SerializeItem(Slot, this, nullptr);
		return true;
	}

	return false;
}

FFrameTime FMovieSceneSequencePlaybackParams::GetPlaybackPosition(UMovieSceneSequencePlayer* Player) const
{
	FFrameTime PlaybackPosition = Player->GetCurrentTime().Time;

	if (PositionType == EMovieScenePositionType::Frame)
	{
		PlaybackPosition = Frame;
	}
	else if (PositionType == EMovieScenePositionType::Time)
	{
		PlaybackPosition = Time * Player->GetFrameRate();
	}
	else if (PositionType == EMovieScenePositionType::MarkedFrame)
	{
		UMovieScene* MovieScene = Player->GetSequence() ? Player->GetSequence()->GetMovieScene() : nullptr;

		if (MovieScene)
		{
			int32 MarkedIndex = MovieScene->FindMarkedFrameByLabel(MarkedFrame);

			if (MarkedIndex != INDEX_NONE)
			{
				PlaybackPosition = ConvertFrameTime(MovieScene->GetMarkedFrames()[MarkedIndex].FrameNumber, MovieScene->GetTickResolution(), MovieScene->GetDisplayRate());
			}
		}
	}
	else if (PositionType == EMovieScenePositionType::Timecode)
	{
		PlaybackPosition = Timecode.ToFrameNumber(Player->GetFrameRate());
	}

	return PlaybackPosition;
}

FFrameTime FMovieSceneSequencePlaybackParams::GetPlaybackPosition(UMovieSceneSequence* Sequence) const
{
	FFrameTime PlaybackPosition;

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return PlaybackPosition;
	}

	FFrameRate DisplayRate = MovieScene->GetDisplayRate();
	FFrameRate TickResolution = MovieScene->GetTickResolution();

	if (PositionType == EMovieScenePositionType::Frame)
	{
		PlaybackPosition = Frame;
	}
	else if (PositionType == EMovieScenePositionType::Time)
	{
		PlaybackPosition = Time * DisplayRate;
	}
	else if (PositionType == EMovieScenePositionType::MarkedFrame)
	{
		int32 MarkedIndex = MovieScene->FindMarkedFrameByLabel(MarkedFrame);
		
		if (MarkedIndex != INDEX_NONE)
		{
			PlaybackPosition = ConvertFrameTime(MovieScene->GetMarkedFrames()[MarkedIndex].FrameNumber, TickResolution, DisplayRate);
		}
	}
	else if (PositionType == EMovieScenePositionType::Timecode)
	{
		PlaybackPosition = Timecode.ToFrameNumber(DisplayRate);
	}

	return PlaybackPosition;
}

UMovieSceneSequencePlayer::UMovieSceneSequencePlayer(const FObjectInitializer& Init)
	: Super(Init)
	, Status(EMovieScenePlayerStatus::Stopped)
	, bReversePlayback(false)
	, bPendingOnStartedPlaying(false)
	, TimeControllerState(ETimeControllerState::ReadyToPlay)
	, bIsAsyncUpdate(false)
	, bSkipNextUpdate(false)
	, bUpdateNetSync(false)
	, bWarnZeroDuration(true)
	, Sequence(nullptr)
	, StartTime(0)
	, DurationFrames(0)
	, DurationSubFrames(0.f)
	, CurrentNumLoops(0)
	, SerialNumber(0)
	, CurrentRunner(nullptr)
{
	PlayPosition.Reset(FFrameTime(0));

	NetSyncProps.LastKnownPosition = FFrameTime(0);
	NetSyncProps.LastKnownStatus   = Status;
}

UMovieSceneSequencePlayer::~UMovieSceneSequencePlayer()
{
	if (GEngine && OldMaxTickRate.IsSet())
	{
		GEngine->SetMaxFPS(OldMaxTickRate.GetValue());
	}

	if (bOverridingDynResFrameTimeBudget)
	{
		static IConsoleVariable* CVarDynResFrameTimeBudget = IConsoleManager::Get().FindConsoleVariable(TEXT("r.DynamicRes.FrameTimeBudget"));
		CVarDynResFrameTimeBudget->Unset(ECVF_SetByCode);

		bOverridingDynResFrameTimeBudget = false;
	}
}

void UMovieSceneSequencePlayer::UpdateNetworkSyncProperties()
{
	if (HasAuthority())
	{
		NetSyncProps.LastKnownPosition = PlayPosition.GetCurrentPosition();
		NetSyncProps.LastKnownStatus   = Status;
		NetSyncProps.LastKnownNumLoops = CurrentNumLoops;
		NetSyncProps.LastKnownSerialNumber = SerialNumber;
	}
}

void UMovieSceneSequencePlayer::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	DOREPLIFETIME(UMovieSceneSequencePlayer, NetSyncProps);
	DOREPLIFETIME(UMovieSceneSequencePlayer, bReversePlayback);
	DOREPLIFETIME(UMovieSceneSequencePlayer, StartTime);
	DOREPLIFETIME(UMovieSceneSequencePlayer, DurationFrames);
	DOREPLIFETIME(UMovieSceneSequencePlayer, DurationSubFrames);
	DOREPLIFETIME(UMovieSceneSequencePlayer, PlaybackSettings);
	DOREPLIFETIME(UMovieSceneSequencePlayer, Observer);
}

EMovieScenePlayerStatus::Type UMovieSceneSequencePlayer::GetPlaybackStatus() const
{
	return Status;
}

FMovieSceneSpawnRegister& UMovieSceneSequencePlayer::GetSpawnRegister()
{
	return SpawnRegister.IsValid() ? *SpawnRegister : IMovieScenePlayer::GetSpawnRegister();
}

void UMovieSceneSequencePlayer::ResolveBoundObjects(UE::UniversalObjectLocator::FResolveParams& ResolveParams, const FGuid& InBindingId, FMovieSceneSequenceID SequenceID, UMovieSceneSequence& InSequence, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const
{
	using namespace UE::MovieScene;
	bool bAllowDefault = PlaybackClient ? PlaybackClient->RetrieveBindingOverrides(InBindingId, SequenceID, OutObjects) : true;

	if (bAllowDefault)
	{
		InSequence.LocateBoundObjects(InBindingId, ResolveParams, FindSharedPlaybackState(), OutObjects);
	}
}

void UMovieSceneSequencePlayer::Play()
{
	if (NeedsQueueLatentAction())
	{
		QueueLatentAction(FMovieSceneSequenceLatentActionDelegate::CreateUObject(this, &UMovieSceneSequencePlayer::Play));
		return;
	}

	bReversePlayback = false;
	PlayInternal();
}

void UMovieSceneSequencePlayer::PlayReverse()
{
	if (NeedsQueueLatentAction())
	{
		QueueLatentAction(FMovieSceneSequenceLatentActionDelegate::CreateUObject(this, &UMovieSceneSequencePlayer::PlayReverse));
		return;
	}

	bReversePlayback = true;
	PlayInternal();
}

void UMovieSceneSequencePlayer::ChangePlaybackDirection()
{
	if (NeedsQueueLatentAction())
	{
		QueueLatentAction(FMovieSceneSequenceLatentActionDelegate::CreateUObject(this, &UMovieSceneSequencePlayer::ChangePlaybackDirection));
		return;
	}

	bReversePlayback = !bReversePlayback;
	PlayInternal();
}

void UMovieSceneSequencePlayer::PlayLooping(int32 NumLoops)
{
	if (NeedsQueueLatentAction())
	{
		QueueLatentAction(FMovieSceneSequenceLatentActionDelegate::CreateUObject(this, &UMovieSceneSequencePlayer::PlayLooping, NumLoops));
		return;
	}

	// PlayLooping should only override num loops temporarily, so cache the current settings here for restoring later when playback stops.
	if (!CachedPlaybackSettings.IsSet())
	{
		CachedPlaybackSettings = PlaybackSettings;
	}

	PlaybackSettings.LoopCount.Value = NumLoops;
	PlayInternal();
}

void UMovieSceneSequencePlayer::PlayInternal()
{
	if (Observer && !Observer->CanObserveSequence())
	{
		return;
	}

	if (NeedsQueueLatentAction())
	{
		QueueLatentAction(FMovieSceneSequenceLatentActionDelegate::CreateUObject(this, &UMovieSceneSequencePlayer::PlayInternal));
		return;
	}

	if (!IsPlaying() && Sequence && CanPlay() && TimeControllerState == ETimeControllerState::ReadyToPlay)
	{
		const FString SequenceName = GetSequenceName(true);
		UE_LOGF(LogMovieScene, Verbose, "PlayInternal - %ls (current status: %ls)", *SequenceName, *UEnum::GetValueAsString(Status));

		// Set playback status to playing before any calls to update the position
		Status = EMovieScenePlayerStatus::Playing;

		float PlayRate = bReversePlayback ? -PlaybackSettings.PlayRate : PlaybackSettings.PlayRate;

		// If at the end and playing forwards, rewind to beginning
		if (GetCurrentTime().Time == GetLastValidTime())
		{
			if (PlayRate > 0.f)
			{
				SetPlaybackPosition(FMovieSceneSequencePlaybackParams(FFrameTime(StartTime), EUpdatePositionMethod::Jump));
			}
		}
		else if (GetCurrentTime().Time == FFrameTime(StartTime))
		{
			if (PlayRate < 0.f)
			{
				SetPlaybackPosition(FMovieSceneSequencePlaybackParams(GetLastValidTime(), EUpdatePositionMethod::Jump));
			}
		}

		// Update now
		if (PlaybackSettings.FinishCompletionStateOverride == EMovieSceneCompletionModeOverride::ForceRestoreState)
		{
			RootTemplateInstance.EnableGlobalPreAnimatedStateCapture();
		}

		TimeController->PrepareToPlay(GetCurrentTime());
		if (TimeController->IsReadyToPlay())
		{
			StartTimeControllerAndBroadcastPlayState();
			TimeControllerState = ETimeControllerState::ReadyToPlay;
		}
		else
		{
			TimeControllerState = ETimeControllerState::PreparingToPlay;
		}
	}
}

void UMovieSceneSequencePlayer::Pause()
{
	if (Observer && !Observer->CanObserveSequence())
	{
		return;
	}

	if (NeedsQueueLatentAction())
	{
		QueueLatentAction(FMovieSceneSequenceLatentActionDelegate::CreateUObject(this, &UMovieSceneSequencePlayer::Pause));
		return;
	}

	const FString SequenceName = GetSequenceName(true);
	UE_LOGF(LogMovieScene, Verbose, "Pause - %ls (current status: %ls)", *SequenceName, *UEnum::GetValueAsString(Status));

	if (IsPlaying())
	{
		Status = EMovieScenePlayerStatus::Paused;
		TimeController->StopPlaying(GetCurrentTime());

		PauseOnFrame.Reset();
		LastTickGameTimeSeconds.Reset();

		auto FinishPause = [this]
		{
			this->RunLatentActions();
			this->UpdateNetworkSyncProperties();

			const FString SequenceName = this->GetSequenceName(true);
			UE_LOGF(LogMovieScene, Verbose, "Paused - %ls", *SequenceName);

			if (this->OnPause.IsBound())
			{
				this->OnPause.Broadcast();
			}
		};

		// Evaluate the sequence at its current time, with a status of 'stopped' to ensure that animated state pauses correctly. (ie. audio sounds should stop/pause)
		if (TSharedPtr<FMovieSceneEntitySystemRunner> Runner = RootTemplateInstance.GetRunner())
		{
			FMovieSceneEvaluationRange CurrentTimeRange = PlayPosition.GetCurrentPositionAsRange();

			if (PlaybackClient)
			{
				PlaybackClient->WarpEvaluationRange(CurrentTimeRange);
			}
			
			const FMovieSceneContext Context(CurrentTimeRange, EMovieScenePlayerStatus::Stopped);
			
			Runner->QueueUpdate(Context, RootTemplateInstance.GetRootInstanceHandle(), FSimpleDelegate::CreateWeakLambda(this, FinishPause));
		}
		else
		{
			FinishPause();
		}
	}
}

void UMovieSceneSequencePlayer::Scrub()
{
	Status = EMovieScenePlayerStatus::Scrubbing;
	TimeController->StopPlaying(GetCurrentTime());

	UpdateNetworkSyncProperties();
}

void UMovieSceneSequencePlayer::Stop()
{
	StopInternal(bReversePlayback ? GetLastValidTime() : FFrameTime(StartTime));
}

void UMovieSceneSequencePlayer::StopAtCurrentTime()
{
	StopInternal(PlayPosition.GetCurrentPosition());
}

void UMovieSceneSequencePlayer::StopInternal(FFrameTime TimeToResetTo)
{
	if (Observer && !Observer->CanObserveSequence())
	{
		return;
	}

	if (NeedsQueueLatentAction())
	{
		QueueLatentAction(FMovieSceneSequenceLatentActionDelegate::CreateUObject(this, &UMovieSceneSequencePlayer::StopInternal, TimeToResetTo));
		return;
	}

	const FString SequenceName = GetSequenceName(true);
	UE_LOGF(LogMovieScene, Verbose, "StopInternal - %ls (at: %ls, current status: %ls)", *SequenceName, *LexToString(TimeToResetTo), *UEnum::GetValueAsString(Status));

	// We may have been waiting for our time controller to tell us it was ready to play,
	// so TimeControllerState may be ETimeControllerState::PreparingToPlay. Here we reset that back to its
	// initialized state of ETimeControllerState::ReadyToPlay.
	TimeControllerState = ETimeControllerState::ReadyToPlay;

	if (IsPlaying() || IsPaused())
	{
		Status = EMovieScenePlayerStatus::Stopped;

		// Put the cursor at the specified position
		PlayPosition.Reset(TimeToResetTo);
		if (TimeController.IsValid())
		{
			TimeController->StopPlaying(GetCurrentTime());
		}

		CurrentNumLoops = 0;
		PauseOnFrame.Reset();
		LastTickGameTimeSeconds.Reset();

		// Restore playback settings to cached state to clear any temporary overrides.
		if (CachedPlaybackSettings.IsSet())
		{
			PlaybackSettings = CachedPlaybackSettings.GetValue();
			CachedPlaybackSettings.Reset();
		}

		if (PlaybackSettings.FinishCompletionStateOverride == EMovieSceneCompletionModeOverride::ForceRestoreState)
		{
			RestorePreAnimatedState();
		}
		else if (PlaybackSettings.FinishCompletionStateOverride == EMovieSceneCompletionModeOverride::ForceKeepState)
		{
			DiscardPreAnimatedState();
		}

		// Lambda that is invoked when the request to finish this sequence has been fulfilled
		auto OnFlushed = [this, TimeToResetTo]
		{
			if (this->OldMaxTickRate.IsSet())
			{
				GEngine->SetMaxFPS(OldMaxTickRate.GetValue());
				this->OldMaxTickRate.Reset();
			}

			if (bOverridingDynResFrameTimeBudget)
			{
				static IConsoleVariable* CVarDynResFrameTimeBudget = IConsoleManager::Get().FindConsoleVariable(TEXT("r.DynamicRes.FrameTimeBudget"));
				CVarDynResFrameTimeBudget->Unset(ECVF_SetByCode);

				bOverridingDynResFrameTimeBudget = false;
			}

			this->UpdateNetworkSyncProperties();

			const FString SequenceName = this->GetSequenceName(true);
			UE_LOGF(LogMovieScene, Verbose, "Stopped - %ls", *SequenceName);

			if (this->HasAuthority())
			{
				// Explicitly handle Stop() events through an RPC call
				this->RPC_OnStopEvent(TimeToResetTo, SerialNumber + 1);
			}

			this->OnStopped();

			if (this->OnStop.IsBound())
			{
				this->OnStop.Broadcast();
			}

			this->RunLatentActions();
		};

		TSharedPtr<FMovieSceneEntitySystemRunner> Runner = RootTemplateInstance.GetRunner();

		if (Runner)
		{
			// Finish but do not destroy
			if (Runner->QueueFinalUpdate(RootTemplateInstance.GetRootInstanceHandle(), FSimpleDelegate::CreateWeakLambda(this, OnFlushed)))
			{
				Runner->Flush();
			}
		}
	}
	else if (RootTemplateInstance.IsValid() && RootTemplateInstance.HasEverUpdated())
	{
		if (PlaybackSettings.FinishCompletionStateOverride == EMovieSceneCompletionModeOverride::ForceRestoreState)
		{
			RestorePreAnimatedState();
		}
		else if (PlaybackSettings.FinishCompletionStateOverride == EMovieSceneCompletionModeOverride::ForceKeepState)
		{
			DiscardPreAnimatedState();
		}

		TSharedPtr<FMovieSceneEntitySystemRunner> Runner = RootTemplateInstance.GetRunner();
		if (Runner)
		{
			// Finish but do not destroy
			if (Runner->QueueFinalUpdate(RootTemplateInstance.GetRootInstanceHandle()))
			{
				Runner->Flush();
			}
		}
	}
}

void UMovieSceneSequencePlayer::FinishPlaybackInternal(FFrameTime TimeToFinishAt)
{
	if (PlaybackSettings.bPauseAtEnd)
	{
		Pause();
	}
	else
	{
		StopInternal(TimeToFinishAt);
	}

	TimeController->StopPlaying(GetCurrentTime());

	if (OnFinished.IsBound())
	{
		OnFinished.Broadcast();
	}

	OnNativeFinished.ExecuteIfBound();
}

void UMovieSceneSequencePlayer::GoToEndAndStop()
{
	FFrameTime LastValidTime = GetLastValidTime();

	if (PlayPosition.GetCurrentPosition() == LastValidTime && Status == EMovieScenePlayerStatus::Stopped)
	{
		return;
	}

	Status = EMovieScenePlayerStatus::Playing;
	SetPlaybackPosition(FMovieSceneSequencePlaybackParams(LastValidTime, EUpdatePositionMethod::Jump));
	StopInternal(LastValidTime);
}

FQualifiedFrameTime UMovieSceneSequencePlayer::GetCurrentTime() const
{
	FFrameTime Time = PlayPosition.GetCurrentPosition();
	return FQualifiedFrameTime(Time, PlayPosition.GetInputRate());
}

FQualifiedFrameTime UMovieSceneSequencePlayer::GetDuration() const
{
	return FQualifiedFrameTime(FFrameTime(DurationFrames, DurationSubFrames), PlayPosition.GetInputRate());
}

int32 UMovieSceneSequencePlayer::GetFrameDuration() const
{
	return DurationFrames;
}

void UMovieSceneSequencePlayer::SetFrameRate(FFrameRate FrameRate)
{
	if (!FrameRate.IsValid() || FrameRate.Numerator <= 0)
	{
		const FString SequenceName = GetSequenceName(true);
		UE_LOGF(LogMovieScene, Error, "Attempting to set sequence %ls with an invalid frame rate: %ls", *SequenceName, *FrameRate.ToPrettyText().ToString());
		return;
	}

	UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;
	if (MovieScene)
	{
		if (MovieScene->GetEvaluationType() == EMovieSceneEvaluationType::FrameLocked && !FrameRate.IsMultipleOf(MovieScene->GetTickResolution()))
		{
			UE_LOGF(LogMovieScene, Warning, "Attempting to play back a sequence with tick resolution of %f ticks per second frame locked to %f fps, which is not a multiple of the resolution.", MovieScene->GetTickResolution().AsDecimal(), FrameRate.AsDecimal());
		}
	}

	FFrameRate CurrentInputRate = PlayPosition.GetInputRate();

	StartTime      = ConvertFrameTime(StartTime,                    CurrentInputRate, FrameRate).FloorToFrame();
	DurationFrames = ConvertFrameTime(FFrameNumber(DurationFrames), CurrentInputRate, FrameRate).RoundToFrame().Value;

	PlayPosition.SetTimeBase(FrameRate, PlayPosition.GetOutputRate(), PlayPosition.GetEvaluationType());
}

void UMovieSceneSequencePlayer::SetFrameRange( int32 NewStartTime, int32 Duration, float SubFrames )
{
	Duration = FMath::Max(Duration, 0);

	StartTime      = NewStartTime;
	DurationFrames = Duration;
	DurationSubFrames = SubFrames;

	TOptional<FFrameTime> CurrentTime = PlayPosition.GetCurrentPosition();
	if (CurrentTime.IsSet())
	{
		FFrameTime LastValidTime = GetLastValidTime();

		if (CurrentTime.GetValue() < StartTime)
		{
			PlayPosition.Reset(StartTime);
		}
		else if (CurrentTime.GetValue() > LastValidTime)
		{
			PlayPosition.Reset(LastValidTime);
		}
	}

	if (TimeController.IsValid())
	{
		TimeController->Reset(GetCurrentTime());
	}

	UpdateNetworkSyncProperties();
}

void UMovieSceneSequencePlayer::SetTimeRange( float StartTimeSeconds, float DurationSeconds )
{
	const FFrameRate Rate = PlayPosition.GetInputRate();

	const FFrameNumber StartFrame = ( StartTimeSeconds * Rate ).FloorToFrame();
	const FFrameNumber Duration   = ( DurationSeconds  * Rate ).RoundToFrame();

	SetFrameRange(StartFrame.Value, Duration.Value);
}

void UMovieSceneSequencePlayer::PlayTo(FMovieSceneSequencePlaybackParams InPlaybackParams, FMovieSceneSequencePlayToParams PlayToParams)
{
	FPauseOnArgs Args;
	Args.Time = InPlaybackParams.GetPlaybackPosition(this);
	Args.bExclusive = PlayToParams.bExclusive;
	PauseOnFrame = Args;

	if (GetCurrentTime().Time < Args.Time)
	{
		Play();
	}
	else
	{
		PlayReverse();
	}
}

void UMovieSceneSequencePlayer::SetPlaybackPosition(FMovieSceneSequencePlaybackParams InPlaybackParams)
{
	if (Observer && !Observer->CanObserveSequence())
	{
		return;
	}

	if (!Sequence)
	{
		return;
	}

	if (NeedsQueueLatentAction())
	{
		QueueLatentAction(FMovieSceneSequenceLatentActionDelegate::CreateUObject(this, &UMovieSceneSequencePlayer::SetPlaybackPosition, InPlaybackParams));
		return;
	}

	FFrameTime NewPosition = InPlaybackParams.GetPlaybackPosition(this);

	UpdateTimeCursorPosition(NewPosition, InPlaybackParams.UpdateMethod, InPlaybackParams.bHasJumped);

	TimeController->Reset(GetCurrentTime());

	if (HasAuthority())
	{
		RPC_ExplicitServerUpdateEvent(InPlaybackParams.UpdateMethod, NewPosition, SerialNumber + 1);
	}
}

void UMovieSceneSequencePlayer::RestoreState()
{
	if (PlaybackSettings.FinishCompletionStateOverride != EMovieSceneCompletionModeOverride::ForceRestoreState)
	{
		UE_LOGF(LogMovieScene, Warning, "Attempting to restore pre-animated state for a player that was not set to capture pre-animated state. Please set PlaybackSettings.FinishCompletionStateOverride to ForceRestoreState");
	}

	RestorePreAnimatedState();
}

void UMovieSceneSequencePlayer::SetCompletionModeOverride(EMovieSceneCompletionModeOverride CompletionModeOverride)
{
	if (IsPlaying() && PlaybackSettings.FinishCompletionStateOverride != EMovieSceneCompletionModeOverride::ForceRestoreState && CompletionModeOverride == EMovieSceneCompletionModeOverride::ForceRestoreState)
	{
		UE_LOGF(LogMovieScene, Warning, "Attempting to set completion mode override to force restore state while the sequence is already playing. Force restore state must be set before starting playback.");
	}

	PlaybackSettings.FinishCompletionStateOverride = CompletionModeOverride;
}

EMovieSceneCompletionModeOverride UMovieSceneSequencePlayer::GetCompletionModeOverride() const
{
	return PlaybackSettings.FinishCompletionStateOverride;
}

bool UMovieSceneSequencePlayer::IsPlaying() const
{
	return Status == EMovieScenePlayerStatus::Playing;
}

bool UMovieSceneSequencePlayer::IsPaused() const
{
	return Status == EMovieScenePlayerStatus::Paused;
}

bool UMovieSceneSequencePlayer::IsReversed() const
{
	return bReversePlayback;
}

float UMovieSceneSequencePlayer::GetPlayRate() const
{
	return PlaybackSettings.PlayRate;
}

void UMovieSceneSequencePlayer::SetPlayRate(float PlayRate)
{
	PlaybackSettings.PlayRate = PlayRate;
}

bool UMovieSceneSequencePlayer::GetHideHud() const
{
	return PlaybackSettings.bHideHud;
}

void UMovieSceneSequencePlayer::SetHideHud(bool HideHud)
{
	PlaybackSettings.bHideHud = HideHud;
}

FFrameTime UMovieSceneSequencePlayer::GetLastValidTime() const
{
	if (DurationFrames > 0)
	{
		if (DurationSubFrames > 0.f)
		{
			return FFrameTime(StartTime + DurationFrames, DurationSubFrames);
		}
		else
		{
			return FFrameTime(StartTime + DurationFrames - 1, 0.99999994f);
		}
	}
	else
	{
		return FFrameTime(StartTime);
	}
}

FFrameRate UMovieSceneSequencePlayer::GetDisplayRate() const
{
	return Sequence && Sequence->GetMovieScene() ? Sequence->GetMovieScene()->GetDisplayRate() : FFrameRate();
}

bool UMovieSceneSequencePlayer::ShouldStopOrLoop(FFrameTime NewPosition) const
{
	bool bShouldStopOrLoop = false;
	if (IsPlaying())
	{
		if (!bReversePlayback)
		{
			bShouldStopOrLoop = NewPosition >= FFrameTime(StartTime + GetFrameDuration(), DurationSubFrames);
		}
		else
		{
			bShouldStopOrLoop = NewPosition.FrameNumber < StartTime;
		}
	}
	return bShouldStopOrLoop;
}

TOptional<TRange<FFrameTime>> UMovieSceneSequencePlayer::GetPauseRange(const FFrameTime& NewPosition) const
{
	if (IsPlaying() && PauseOnFrame.IsSet())
	{
		TRange<FFrameTime> OutRange;

		// NewPosition and PauseOnFrame are stored in Display Rate, but we need to convert to tick resolution for the actual ranges.
		FFrameTime PauseTime = FFrameRate::TransformTime(PauseOnFrame->Time, PlayPosition.GetInputRate(), PlayPosition.GetOutputRate());
		FFrameTime CurrentTime = FFrameRate::TransformTime(PlayPosition.GetCurrentPosition(), PlayPosition.GetInputRate(), PlayPosition.GetOutputRate());
		FFrameTime NewTime = FFrameRate::TransformTime(NewPosition, PlayPosition.GetInputRate(), PlayPosition.GetOutputRate());

		// This logic is a little complicated due to inclusive/exclusive range times for pausing. Inclusive/Exclusive only have any meaning
		// with direction (as we need direction to decide which bound to make inclusive/exclusive).
		if (bReversePlayback)
		{

			// If we're going in reverse then the start frame is the exclusive one
			OutRange = TRange<FFrameTime>(PauseOnFrame->bExclusive
				? TRangeBound<FFrameTime>::Inclusive(PauseTime + FFrameTime(FFrameNumber(1)))
				: TRangeBound<FFrameTime>::Inclusive(PauseTime),
				TRangeBound<FFrameTime>::Inclusive(CurrentTime));
			
		}
		else
		{
			// If we're playing forward then the upper bound is (potentially) exclusive
			OutRange = TRange<FFrameTime>(TRangeBound<FFrameTime>::Inclusive(CurrentTime),
				PauseOnFrame->bExclusive
				? TRangeBound<FFrameTime>::Exclusive(PauseTime)
				: TRangeBound<FFrameTime>::Exclusive(PauseTime + FFrameTime(FFrameNumber(1))));

		}

		// If the new time is outside of bounds, then we should pause.
		if (!OutRange.Contains(NewTime))
		{
			return OutRange;
		}
	}

	return TOptional<TRange<FFrameTime>>();
}

TOptional<FFrameTime> UMovieSceneSequencePlayer::GetPauseTimeForNewPosition(const FFrameTime& NewPosition) const
{
	TOptional<TRange<FFrameTime>> PauseRange = GetPauseRange(NewPosition);

	if (PauseRange.IsSet())
	{
		if (bReversePlayback)
		{
			return PauseRange.GetValue().GetLowerBoundValue();
		}
		else
		{
			return PauseRange.GetValue().GetUpperBoundValue();
		}
	}

	return TOptional<FFrameTime>();
}

UMovieSceneEntitySystemLinker* UMovieSceneSequencePlayer::ConstructEntitySystemLinker()
{
	if (ensure(TickManager && RegisteredTickInterval.IsSet()) && !EnumHasAnyFlags(Sequence->GetFlags(), EMovieSceneSequenceFlags::BlockingEvaluation))
	{
		return TickManager->GetLinker(RegisteredTickInterval.GetValue());
	}

	return UMovieSceneEntitySystemLinker::CreateLinker(GetPlaybackContext(), UE::MovieScene::EEntitySystemLinkerRole::Standalone);
}

void UMovieSceneSequencePlayer::InitializeForTick(UObject* Context)
{
	// Store a reference to the global tick manager to keep it alive while there are sequence players active.
	if (ensure(Context))
	{
		TickManager = UMovieSceneSequenceTickManager::Get(Context);
	}
}


void UMovieSceneSequencePlayer::SetPlaybackSettings(const FMovieSceneSequencePlaybackSettings& InSettings)
{
	PlaybackSettings = InSettings;
}

void UMovieSceneSequencePlayer::Initialize(UMovieSceneSequence* InSequence, const FMovieSceneSequencePlaybackSettings& InSettings)
{
	PlaybackSettings = InSettings;
	Initialize(InSequence);
}

void UMovieSceneSequencePlayer::Initialize(UMovieSceneSequence* InSequence)
{
	check(InSequence);
	check(!IsEvaluating());

	// If we have a valid sequence that may have been played back,
	// Explicitly stop and tear down the template instance before 
	// reinitializing it with the new sequence. Care should be taken
	// here that Stop is not called on the first Initialization as this
	// may be called during PostLoad.
	if (Sequence)
	{
		StopAtCurrentTime();
	}

	Sequence = InSequence;

	FFrameTime StartTimeWithOffset = StartTime;

	EUpdateClockSource ClockToUse = EUpdateClockSource::Tick;

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (MovieScene)
	{
		EMovieSceneEvaluationType EvaluationType    = MovieScene->GetEvaluationType();
		FFrameRate                TickResolution    = MovieScene->GetTickResolution();
		FFrameRate                DisplayRate       = MovieScene->GetDisplayRate();

		UE_LOGF(LogMovieScene, Verbose, "Initialize - MovieSceneSequence: %ls, TickResolution: %ls, DisplayRate: %ls", *InSequence->GetName(), *TickResolution.ToPrettyText().ToString(), *DisplayRate.ToPrettyText().ToString());

		if (!TickResolution.IsValid() || TickResolution.Numerator <= 0)
		{
			const FString SequenceName = GetSequenceName(true);
			const FFrameRate DefaultTickResolution(60000, 1);
			UE_LOGF(LogMovieScene, Error, "Attempting to set sequence %ls with an invalid tick resolution: %ls, defaulting to: %ls", *SequenceName, *TickResolution.ToPrettyText().ToString(), *DefaultTickResolution.ToPrettyText().ToString());
			TickResolution = DefaultTickResolution;
		}

		if (!DisplayRate.IsValid() || DisplayRate.Numerator <= 0)
		{
			const FString SequenceName = GetSequenceName(true);
			const FFrameRate DefaultDisplayRate(30, 1);
			UE_LOGF(LogMovieScene, Error, "Attempting to set sequence %ls with an invalid display rate: %ls, defaulting to: %ls", *SequenceName, *DisplayRate.ToPrettyText().ToString(), *DefaultDisplayRate.ToPrettyText().ToString());
			DisplayRate = DefaultDisplayRate;
		}

		// We set the play position in terms of the display rate,
		// but want evaluation ranges in the moviescene's tick resolution
		PlayPosition.SetTimeBase(DisplayRate, TickResolution, EvaluationType);

		{
			// Set up the default frame range from the sequence's play range
			TRange<FFrameNumber> PlaybackRange   = MovieScene->GetPlaybackRange();

			const FFrameNumber SrcStartFrame = UE::MovieScene::DiscreteInclusiveLower(PlaybackRange);
			const FFrameNumber SrcEndFrame   = UE::MovieScene::DiscreteExclusiveUpper(PlaybackRange);

			const FFrameTime EndingTime = ConvertFrameTime(SrcEndFrame, TickResolution, DisplayRate);

			const FFrameNumber StartingFrame = ConvertFrameTime(SrcStartFrame, TickResolution, DisplayRate).FloorToFrame();
			const FFrameNumber EndingFrame   = EndingTime.FloorToFrame();

			SetFrameRange(StartingFrame.Value, (EndingFrame - StartingFrame).Value, EndingTime.GetSubFrame());
		}

		// Reset the play position based on the user-specified start offset, or a random time
		FFrameTime SpecifiedStartOffset = PlaybackSettings.StartTime * DisplayRate;

		// Setup the starting time
		FFrameTime StartingTimeOffset = PlaybackSettings.bRandomStartTime
			? FFrameTime(FMath::Rand() % GetFrameDuration())
			: FMath::Clamp<FFrameTime>(SpecifiedStartOffset, 0, GetFrameDuration()-1);
			
		StartTimeWithOffset = StartTime + StartingTimeOffset;

		ClockToUse = MovieScene->GetClockSource();

		if (ClockToUse == EUpdateClockSource::Custom)
		{
			TimeController = MovieScene->MakeCustomTimeController(GetPlaybackContext());
		}
	}

	if (!TimeController.IsValid())
	{
		switch (ClockToUse)
		{
		case EUpdateClockSource::Audio:    TimeController = MakeShared<FMovieSceneTimeController_AudioClock>();    break;
		case EUpdateClockSource::Platform: TimeController = MakeShared<FMovieSceneTimeController_PlatformClock>(); break;
		case EUpdateClockSource::RelativeTimecode: TimeController = MakeShared<FMovieSceneTimeController_RelativeTimecodeClock>(); break;
		case EUpdateClockSource::Timecode: TimeController = MakeShared<FMovieSceneTimeController_TimecodeClock>(); break;
		case EUpdateClockSource::PlayEveryFrame: TimeController = MakeShared<FMovieSceneTimeController_PlayEveryFrame>(); break;
		default:                           TimeController = MakeShared<FMovieSceneTimeController_Tick>();          break;
		}

		if (!ensureMsgf(TimeController.IsValid(), TEXT("No time controller specified for sequence playback. Falling back to Engine Tick clock source.")))
		{
			TimeController = MakeShared<FMovieSceneTimeController_Tick>();
		}
	}

	if (!TickManager)
	{
		InitializeForTick(GetPlaybackContext());
	}

	FMovieSceneSequenceTickInterval TickInterval = PlaybackSettings.bInheritTickIntervalFromOwner
		? FMovieSceneSequenceTickInterval::GetInheritedInterval(this)
		: PlaybackSettings.TickInterval;

	// If we haven't registered with the tick manager yet, register directly
	if (!RegisteredTickInterval.IsSet())
	{
		TickManager->RegisterTickClient(TickInterval, this);
	}
	// If we were already registered with a different Tick Interval we need to re-register with the new one, which involves tearing everything down and setting up a new instance
	else if (RegisteredTickInterval.GetValue() != TickInterval)
	{
		RootTemplateInstance.TearDown();
		TickManager->UnregisterTickClient(this);

		TickManager->RegisterTickClient(RegisteredTickInterval.GetValue(), this);
	}

	RegisteredTickInterval = TickInterval;

	RootTemplateInstance.Initialize(*Sequence, *this, nullptr);

	if (!PlaybackSettings.bDynamicWeighting)
	{
		UMovieSceneCompiledDataManager* CompiledDataManager = RootTemplateInstance.GetCompiledDataManager();
		FMovieSceneCompiledDataID       CompiledDataID      = RootTemplateInstance.GetCompiledDataID();
		if (CompiledDataManager && CompiledDataID.IsValid())
		{
			PlaybackSettings.bDynamicWeighting = EnumHasAnyFlags(CompiledDataManager->GetEntryRef(CompiledDataID).AccumulatedFlags, EMovieSceneSequenceFlags::DynamicWeighting);
		}
	}

	LatentActionManager.ClearLatentActions();

	// Set up playback position (with offset) after Stop(), which will reset the starting time to StartTime
	PlayPosition.Reset(StartTimeWithOffset);
	TimeController->Reset(GetCurrentTime());

	// Update the sync properties on the server.
	UpdateNetworkSyncProperties();
	// On the client, we also update LastKnownPosition. This is because our first PostNetReceive
	// could be called with an incomplete set of replicated values in very rare cases... so for instance we might
	// get the proper LastKnownStatus from the server, but, say, not the proper LastKnownPosition. If the sequence does
	// not start at frame 0, we would see LastKnownPosition left at 0, while our own client-side position is the first
	// frame of the sequence, as initialized above (SetFrameRange). At this point, we would incorrectly assume that the server 
	// jumped to frame 0 and we would do the same, when really the server hasn't moved and it's just that the correct
	// LastKnownPosition value is coming in a later net packet.
	NetSyncProps.LastKnownPosition = PlayPosition.GetCurrentPosition();
}

void UMovieSceneSequencePlayer::Update(const float DeltaSeconds)
{
	UWorld* World = GetPlaybackWorld();
	float CurrentWorldTime = 0.f;
	if (World)
	{
		CurrentWorldTime = World->GetTimeSeconds();
	}

	UpdateNetworkSync();

	if (TimeControllerState == ETimeControllerState::PreparingToPlay && TimeController->IsReadyToPlay())
	{
		StartTimeControllerAndBroadcastPlayState();
		TimeControllerState = ETimeControllerState::ReadyToPlay;
	}

	if (TimeControllerState == ETimeControllerState::ReadyToPlay && IsPlaying())
	{
		// Delta seconds has already been multiplied by GetEffectiveTimeDilation at this point, so don't pass that through to Tick
		float PlayRate = bReversePlayback ? -PlaybackSettings.PlayRate : PlaybackSettings.PlayRate;

		float DeltaTimeForFunction = DeltaSeconds;

		TimeController->Tick(DeltaTimeForFunction, PlayRate);

		if (World)
		{
			PlayRate *= World->GetWorldSettings()->GetEffectiveTimeDilation();
		}

		if (!bSkipNextUpdate)
		{
			check(!IsEvaluating());

			FFrameTime NewTime = TimeController->RequestCurrentTime(GetCurrentTime(), PlayRate, GetDisplayRate());
			UpdateTimeCursorPosition(NewTime, EUpdatePositionMethod::Play);
		}

		bSkipNextUpdate = false;

		// CAREFUL with stateful changes after this... in 95% of cases, the sequence evaluation was
		// only queued up, and hasn't run yet!
	}

	if (World)
	{
		LastTickGameTimeSeconds = CurrentWorldTime;
	}
}

void UMovieSceneSequencePlayer::TickFromSequenceTickManager(float DeltaSeconds, FMovieSceneEntitySystemRunner* InRunner)
{
	TGuardValue<FMovieSceneEntitySystemRunner*> RunnerGuard(CurrentRunner, InRunner);
	UpdateAsync(DeltaSeconds);
}

void UMovieSceneSequencePlayer::UpdateAsync(const float DeltaSeconds)
{
	check(!bIsAsyncUpdate);
	bIsAsyncUpdate = true;

	Update(DeltaSeconds);

	bIsAsyncUpdate = false;
}

void UMovieSceneSequencePlayer::UpdateTimeCursorPosition(FFrameTime NewPosition, EUpdatePositionMethod Method, bool bHasJumpedOverride)
{
	if (ensure(!IsEvaluating()))
	{
		UpdateTimeCursorPosition_Internal(NewPosition, Method, bHasJumpedOverride);
	}
}

EMovieScenePlayerStatus::Type UpdateMethodToStatus(EUpdatePositionMethod Method, EMovieScenePlayerStatus::Type SequenceStatus)
{
	switch(Method)
	{
	case EUpdatePositionMethod::Scrub: return EMovieScenePlayerStatus::Scrubbing;
	case EUpdatePositionMethod::Jump:  return SequenceStatus;						// If jumping, the new status after the jump should be whatever the current status of the sequence is
	case EUpdatePositionMethod::Play:  return EMovieScenePlayerStatus::Playing;
	default:                           return EMovieScenePlayerStatus::Stopped;
	}
}

FMovieSceneEvaluationRange UpdatePlayPosition(FMovieScenePlaybackPosition& InOutPlayPosition, FFrameTime NewTime, EUpdatePositionMethod Method, bool bPreferReversePlayback)
{
	EPlayDirection PreferredDirection = bPreferReversePlayback ? EPlayDirection::Backwards : EPlayDirection::Forwards;
	if (Method == EUpdatePositionMethod::Play)
	{
		return InOutPlayPosition.PlayTo(NewTime, PreferredDirection);
	}

	return InOutPlayPosition.JumpTo(NewTime, PreferredDirection);
}

void UMovieSceneSequencePlayer::StartTimeControllerAndBroadcastPlayState()
{
	bPendingOnStartedPlaying = true;
	Status = EMovieScenePlayerStatus::Playing;
	TimeController->StartPlaying(GetCurrentTime());

	if (PlayPosition.GetEvaluationType() == EMovieSceneEvaluationType::FrameLocked)
	{
		if (!OldMaxTickRate.IsSet())
		{
			OldMaxTickRate = GEngine->GetMaxFPS();
		}

		GEngine->SetMaxFPS(1.f / PlayPosition.GetInputRate().AsInterval());

		if (GSequencerApplyDisplayRateToDynResFrameTimeBudget)
		{
			const float DyResFrameTimeBudget = PlayPosition.GetInputRate().AsInterval() * 1000.0f;

			static IConsoleVariable* CVarDynResFrameTimeBudget = IConsoleManager::Get().FindConsoleVariable(TEXT("r.DynamicRes.FrameTimeBudget"));
			CVarDynResFrameTimeBudget->Set(DyResFrameTimeBudget, ECVF_SetByCode);

			bOverridingDynResFrameTimeBudget = true;
		}
	}

	if (!PlayPosition.GetLastPlayEvalPostition().IsSet() || PlayPosition.GetLastPlayEvalPostition() != PlayPosition.GetCurrentPosition())
	{
		EPlayDirection PreferredDirection = bReversePlayback ? EPlayDirection::Backwards : EPlayDirection::Forwards;
		UpdateMovieSceneInstance(PlayPosition.PlayTo(PlayPosition.GetCurrentPosition(), PreferredDirection), EMovieScenePlayerStatus::Playing);
	}

	RunLatentActions();
	UpdateNetworkSyncProperties();

	if (bReversePlayback)
	{
		if (OnPlayReverse.IsBound())
		{
			OnPlayReverse.Broadcast();
		}
	}
	else
	{
		if (OnPlay.IsBound())
		{
			OnPlay.Broadcast();
		}
	}
}

void UMovieSceneSequencePlayer::UpdateTimeCursorPosition_Internal(FFrameTime NewPosition, EUpdatePositionMethod Method, bool bHasJumpedOverride)
{
	bHasJumpedOverride = bHasJumpedOverride || (Method == EUpdatePositionMethod::Jump);
	EMovieScenePlayerStatus::Type StatusOverride = UpdateMethodToStatus(Method, Status);

	const int32 Duration = DurationFrames;
	if (Duration == 0 && DurationSubFrames == 0.f)
	{
		if (bWarnZeroDuration)
		{
			bWarnZeroDuration = false;
			const FString SequenceName = GetSequenceName(true);
			UE_LOGF(LogMovieScene, Warning, "Attempting to play back sequence %ls with zero duration", *SequenceName);
		}
		return;
	}
	bWarnZeroDuration = true;
	
	if (bPendingOnStartedPlaying)
	{
		OnStartedPlaying();
		bPendingOnStartedPlaying = false;
	}

	if (Method == EUpdatePositionMethod::Play)
	{
		const FMovieSceneSequenceHierarchy* Hierarchy = RootTemplateInstance.GetHierarchy();

		if (Hierarchy && Hierarchy->GetRootTransform().FindFirstWarpDomain() == UE::MovieScene::ETimeWarpChannelDomain::PlayRate)
		{
			NewPosition = ConvertFrameTime(NewPosition, PlayPosition.GetInputRate(), PlayPosition.GetOutputRate());
			NewPosition = Hierarchy->GetRootTransform().TransformTime(NewPosition);
			NewPosition = ConvertFrameTime(NewPosition, PlayPosition.GetOutputRate(), PlayPosition.GetInputRate());
		}
	}

	// If we should pause during this evaluation, we'll handle that below.
	const TOptional<FFrameTime> PauseAtTime = GetPauseTimeForNewPosition(NewPosition);
	if (!PauseAtTime.IsSet() && Method == EUpdatePositionMethod::Play && ShouldStopOrLoop(NewPosition))
	{
		// The actual start time taking into account reverse playback
		FFrameTime StartTimeWithReversed = bReversePlayback ? GetLastValidTime() : StartTime;

		// The actual end time taking into account reverse playback
		FFrameTime EndTimeWithReversed = bReversePlayback ? StartTime : GetLastValidTime();

		// Operate in tick resolution (for subframes)
		const double DurationWithSubFrames   = FMath::Max<double>(UE_SMALL_NUMBER, GetDuration().Time.AsDecimal());
		const double PositionRelativeToStart = (NewPosition - StartTimeWithReversed).AsDecimal();

		const int32 NumTimesLooped    = FMath::Abs(FMath::TruncToInt32(PositionRelativeToStart / DurationWithSubFrames));
		const bool  bLoopIndefinitely = PlaybackSettings.LoopCount.Value < 0;

		// loop playback
		if (bLoopIndefinitely || CurrentNumLoops + NumTimesLooped <= PlaybackSettings.LoopCount.Value)
		{
			CurrentNumLoops += NumTimesLooped;
			if (NumTimesLooped > 0)
			{
				// Reset server time samples when this player has looped. This ensures that
				// smoothed playback (if enabled) does not result in a smoothed frame in the previous
				// loop.
				ServerTimeSamples.Reset();
			}

			// Finish evaluating any frames left in the current loop in case they have events attached
			FFrameTime CurrentPosition = PlayPosition.GetCurrentPosition();
			if ((bReversePlayback && CurrentPosition > EndTimeWithReversed) ||
				(!bReversePlayback && CurrentPosition < EndTimeWithReversed))
			{
				FMovieSceneEvaluationRange Range = PlayPosition.PlayTo(EndTimeWithReversed);
				UpdateMovieSceneInstance(Range, StatusOverride, bHasJumpedOverride);
			}

			const FFrameTime Overplay = FFrameTime::FromDecimal(FMath::Fmod(PositionRelativeToStart, DurationWithSubFrames));
			FFrameTime NewFrameOffset;
			
			if (bReversePlayback)
			{
				NewFrameOffset = (Overplay > 0) ?  FFrameTime(Duration) + Overplay : Overplay;
			}
			else
			{
				NewFrameOffset = (Overplay < 0) ? FFrameTime(Duration) + Overplay : Overplay;
			}

			if (SpawnRegister.IsValid())
			{
				SpawnRegister->ForgetExternallyOwnedSpawnedObjects(GetSharedPlaybackState());
			}

			// Reset the play position, and generate a new range that gets us to the new frame time
			if (bReversePlayback)
			{
				PlayPosition.Reset(Overplay > 0 ? GetLastValidTime() : StartTimeWithReversed);
			}
			else
			{
				PlayPosition.Reset(Overplay < 0 ? GetLastValidTime() : StartTimeWithReversed);
			}

			FMovieSceneEvaluationRange Range = PlayPosition.PlayTo(StartTimeWithReversed + NewFrameOffset);

			const bool bHasJumped = true;
			const bool bHasLooped = true;
			UpdateMovieSceneInstance(Range, StatusOverride, bHasJumped, bHasLooped);

			// Use the exact time here rather than a frame locked time to ensure we don't skip the amount that was overplayed in the time controller
			FQualifiedFrameTime ExactCurrentTime(StartTimeWithReversed + NewFrameOffset, PlayPosition.GetInputRate());
			TimeController->Reset(ExactCurrentTime);

			UpdateNetworkSyncProperties();

			OnLooped();
		}

		// We reached the end of playback
		else
		{
			// Clamp the position to the duration
			NewPosition = FMath::Clamp(NewPosition, FFrameTime(StartTime), GetLastValidTime());

			FMovieSceneEvaluationRange Range = UpdatePlayPosition(PlayPosition, NewPosition, Method, bReversePlayback);
			UpdateMovieSceneInstance(Range, StatusOverride, bHasJumpedOverride);

			// We have authority to finish playback if:
			// 1. There's no playback replication (standalone sequence)
			// 2. We are the server side of a replicated sequence
			// 3. We are the client side of a replicated sequence, but playing is only happening on our side (i.e. the Play() method was
			//    called only on the client, and the server sequence is stopped)
			const bool bHasAuthorityToFinish = (
				(!PlaybackClient || !PlaybackClient->GetIsReplicatedPlayback()) ||
				HasAuthority() ||
				NetSyncProps.LastKnownStatus == EMovieScenePlayerStatus::Stopped);
			const FString SequenceName = GetSequenceName(true);
			if (bHasAuthorityToFinish)
			{
				UE_LOGF(LogMovieScene, Verbose,
						"Finishing sequence %ls at frame %ls since we have authority.",
						*SequenceName, *LexToString(NewPosition));
				FinishPlaybackInternal(NewPosition);

				// Explicitly tell the clients to finish their playback. They won't have called FinishPlaybackInternal
				// because it's in the line right above, only for sequence players with some authority
				// (client only or server).
				RPC_OnFinishPlaybackEvent(NewPosition, SerialNumber + 1);
			}
			else
			{
				UE_LOGF(LogMovieScene, Verbose, 
						"Keeping sequence %ls at frame %ls while waiting for playback finish from server.",
						*SequenceName, *LexToString(NewPosition));
			}

			UpdateNetworkSyncProperties();
		}
	}
	else
	{
		// If the desired evaluation will take us past where we want to go we need to use a clipped range provided by PauseAtTime, otherwise use the normal one.
		FFrameTime ActualNewPosition = PauseAtTime.IsSet() ?
			FFrameRate::TransformTime(PauseAtTime.GetValue(), PlayPosition.GetOutputRate(), PlayPosition.GetInputRate()) :
			NewPosition;
		FMovieSceneEvaluationRange Range = UpdatePlayPosition(
				PlayPosition,
				ActualNewPosition,
				Method,
				bReversePlayback);

		UMovieSceneSequence* MovieSceneSequence = RootTemplateInstance.GetSequence(MovieSceneSequenceID::Root);
		const bool bIsSequenceBlocking = MovieSceneSequence ? EnumHasAnyFlags(MovieSceneSequence->GetFlags(), EMovieSceneSequenceFlags::BlockingEvaluation) : false;
		
		// Just update the time and sequence... if we are in the main level update we want, if possible,
		// to only queue this sequence's update, so everything updates in parallel. If not possible, or if
		// not in the main level update, we run the evaluation synchronously.
		FMovieSceneUpdateArgs Args;
		Args.bIsAsync = (bIsAsyncUpdate && !bIsSequenceBlocking);
		Args.bHasJumped = bHasJumpedOverride;

		PostEvaluationCallbacks.Add(FOnEvaluationCallback::CreateUObject(this, &UMovieSceneSequencePlayer::UpdateNetworkSyncProperties));

		UpdateMovieSceneInstance(Range, StatusOverride, Args);

		// Now that the evaluation has taken place we call Pause (to trigger delegates, etc.), however if we're running the evaluation
		// as async we actually need to queue a latent action to do it so that it happens after the data is actually updated. We can't
		// use the QueueLatentAction inside of Pause() because IsEvaluating() is only set during actual evaluation so it won't be set yet.
		if (PauseAtTime.IsSet())
		{
			if (Args.bIsAsync)
			{
				QueueLatentAction(FMovieSceneSequenceLatentActionDelegate::CreateUObject(this, &UMovieSceneSequencePlayer::Pause));
			}
			else
			{
				Pause();
			}
		}
	}

	// WARNING: DO NOT CHANGE PLAYER STATE ANYMORE HERE!
	// The code path above (in the "else" statement) queues an asynchronous evaluation, so any further 
	// state change must be moved in the first first block, with a post-evaluation callback in the second 
	// block... see `UpdateNetworkSyncProperties` as an example.
}

void UMovieSceneSequencePlayer::UpdateMovieSceneInstance(FMovieSceneEvaluationRange InRange, EMovieScenePlayerStatus::Type PlayerStatus, bool bHasJumped, bool bHasLooped)
{
	FMovieSceneUpdateArgs Args;
	Args.bHasJumped = bHasJumped;
	Args.bHasLooped = bHasLooped;
	UpdateMovieSceneInstance(InRange, PlayerStatus, Args);
}

void UMovieSceneSequencePlayer::UpdateMovieSceneInstance(FMovieSceneEvaluationRange InRange, EMovieScenePlayerStatus::Type PlayerStatus, const FMovieSceneUpdateArgs& Args)
{
	if (Observer && !Observer->CanObserveSequence())
	{
		UE_LOGF(LogMovieScene, Error, "Refusing to update an unobservable sequence! Did it become unobservable during playback?");
		return;
	}

	UMovieSceneSequence* MovieSceneSequence = RootTemplateInstance.GetSequence(MovieSceneSequenceID::Root);
	if (!MovieSceneSequence)
	{
		return;
	}

#if !NO_LOGGING
	if (UE_LOG_ACTIVE(LogMovieScene, VeryVerbose))
	{
		const FQualifiedFrameTime CurrentTime = GetCurrentTime();
		const FString SequenceName = GetSequenceName(true);
		UE_LOGF(LogMovieScene, VeryVerbose, "Evaluating sequence %ls at frame %d, subframe %f (%f fps).", *SequenceName, CurrentTime.Time.FrameNumber.Value, CurrentTime.Time.GetSubFrame(), CurrentTime.Rate.AsDecimal());
	}
#endif

	if (PlaybackClient)
	{
		PlaybackClient->WarpEvaluationRange(InRange);
	}

	// Once we have updated we must no longer skip updates
	bSkipNextUpdate = false;

	// We shouldn't be asked to run an async update if we have a blocking sequence.
	check(!Args.bIsAsync || !EnumHasAnyFlags(MovieSceneSequence->GetFlags(), EMovieSceneSequenceFlags::BlockingEvaluation));
	// We shouldn't be asked to run an async update if we don't have a tick manager.
	check(!Args.bIsAsync || TickManager != nullptr);

	FMovieSceneContext Context(InRange, PlayerStatus);
	Context.SetHasJumped(Args.bHasJumped);
	Context.SetHasLooped(Args.bHasLooped);

	TSharedPtr<FMovieSceneEntitySystemRunner> Runner = RootTemplateInstance.GetRunner();
	if (Runner)
	{
		Runner->QueueUpdate(Context, RootTemplateInstance.GetRootInstanceHandle());
		if (Runner == SynchronousRunner || !Args.bIsAsync)
		{
			Runner->Flush();
		}
	}
}

void UMovieSceneSequencePlayer::TearDown()
{
	RootTemplateInstance.TearDown();

	if (TickManager)
	{
		RegisteredTickInterval.Reset();
		TickManager->UnregisterTickClient(this);
		TickManager = nullptr;
	}

	Status = EMovieScenePlayerStatus::Stopped;
}

bool UMovieSceneSequencePlayer::IsValid() const
{
	return RootTemplateInstance.IsValid();
}

bool UMovieSceneSequencePlayer::HasDynamicWeighting() const
{
	return PlaybackSettings.bDynamicWeighting;
}

void UMovieSceneSequencePlayer::PreEvaluation(const FMovieSceneContext& Context)
{
	RunPreEvaluationCallbacks();
}

void UMovieSceneSequencePlayer::PostEvaluation(const FMovieSceneContext& Context)
{
	FFrameTime CurrentTime  = ConvertFrameTime(Context.GetTime(),         Context.GetFrameRate(), PlayPosition.GetInputRate());
	FFrameTime PreviousTime = ConvertFrameTime(Context.GetPreviousTime(), Context.GetFrameRate(), PlayPosition.GetInputRate());
	OnMovieSceneSequencePlayerUpdate.Broadcast(*this, CurrentTime, PreviousTime);

	RunPostEvaluationCallbacks();
}

void UMovieSceneSequencePlayer::RunPreEvaluationCallbacks()
{
	for (const FOnEvaluationCallback& Callback : PreEvaluationCallbacks)
	{
		Callback.ExecuteIfBound();
	}
	PreEvaluationCallbacks.Reset();
}

void UMovieSceneSequencePlayer::RunPostEvaluationCallbacks()
{
	for (const FOnEvaluationCallback& Callback : PostEvaluationCallbacks)
	{
		Callback.ExecuteIfBound();
	}
	PostEvaluationCallbacks.Reset();
}

FString UMovieSceneSequencePlayer::GetSequenceName(bool bAddClientInfo) const
{
	if (Sequence)
	{
		FString SequenceName = GetNameSafe(Sequence);
		if (bAddClientInfo)
		{
			AActor* Actor = GetTypedOuter<AActor>();
			if (Actor && Actor->GetWorld() && Actor->GetWorld()->GetNetMode() == NM_Client)
			{
				SequenceName += FString::Printf(TEXT(" (client %d)"), UE::GetPlayInEditorID() - 1);
			}
		}
		return SequenceName;
	}
	else
	{
		return LexToString(NAME_None);
	}
}

void UMovieSceneSequencePlayer::SetPlaybackClient(TScriptInterface<IMovieScenePlaybackClient> InPlaybackClient)
{
	PlaybackClient = InPlaybackClient;
}

TSharedPtr<FMovieSceneTimeController> UMovieSceneSequencePlayer::GetTimeController() const
{
	return TimeController;
}

void UMovieSceneSequencePlayer::SetTimeController(TSharedPtr<FMovieSceneTimeController> InTimeController)
{
	SetTimeControllerDirectly(InTimeController);
	if (TimeController.IsValid())
	{
		TimeController->Reset(GetCurrentTime());
	}
}

void UMovieSceneSequencePlayer::SetTimeControllerDirectly(TSharedPtr<FMovieSceneTimeController> InTimeController)
{
	TimeController = InTimeController;
}

void UMovieSceneSequencePlayer::SetIgnorePlaybackReplication(bool bState)
{
	bIgnorePlaybackReplication = bState;
}

TArray<UObject*> UMovieSceneSequencePlayer::GetBoundObjects(FMovieSceneObjectBindingID ObjectBinding)
{
	TArray<UObject*> Objects;
	TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState = GetSharedPlaybackState();

	for (TWeakObjectPtr<> WeakObject : ObjectBinding.ResolveBoundObjects(MovieSceneSequenceID::Root, SharedPlaybackState))
	{
		if (UObject* Object = WeakObject.Get())
		{
			Objects.Add(Object);
		}
	}
	return Objects;
}

TArray<FMovieSceneObjectBindingID> UMovieSceneSequencePlayer::GetObjectBindings(UObject* InObject)
{
	TArray<FMovieSceneObjectBindingID> Bindings;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	State.FilterObjectBindings(InObject, GetSharedPlaybackState(), &Bindings);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	return Bindings;
}

void UMovieSceneSequencePlayer::RequestInvalidateBinding(FMovieSceneObjectBindingID ObjectBinding)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	State.Invalidate(ObjectBinding.GetGuid(), ObjectBinding.GetRelativeSequenceID());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

UWorld* UMovieSceneSequencePlayer::GetPlaybackWorld() const
{
	UObject* PlaybackContext = GetPlaybackContext();
	return PlaybackContext ? PlaybackContext->GetWorld() : nullptr;
}

bool UMovieSceneSequencePlayer::HasAuthority() const
{
	AActor* Actor = GetTypedOuter<AActor>();
	return Actor && Actor->HasAuthority() && IsValidChecked(this) && !IsUnreachable();
}

FFrameTime UMovieSceneSequencePlayer::UpdateServerTimeSamples()
{
	// Attempt to estimate the server time based on our samples.
	// We need to reproject the samples to the current wall-clock time, based on when they were taken
	const double CurrentWallClock = FPlatformTime::Seconds();
	const double Lifetime         = CurrentWallClock - float(GSequencerMaxSmoothedNetSyncSampleAge) / 1000.f;
	const float PlaybackMultiplier = bReversePlayback ? -PlaybackSettings.PlayRate : PlaybackSettings.PlayRate;

	float TimeDilation = 1.0f;
	if (const UWorld* World = GetPlaybackWorld())
	{
		if (AWorldSettings* WorldSettings = World->GetWorldSettings())
		{
			TimeDilation = WorldSettings->GetEffectiveTimeDilation();
		}
	}

	// Cull any old samples that were taken more than GSequencerMaxSmoothedNetSyncSampleAge ms ago by
	// Finding the index of the first sample younger than this time
	const int32 FirstValidSample = Algo::LowerBoundBy(ServerTimeSamples, Lifetime, &FServerTimeSample::ReceivedTime);
	if (FirstValidSample >= ServerTimeSamples.Num())
	{
		// Never found a sample that is recent enough, all samples are too old
		ServerTimeSamples.Reset();
	}
	else if (FirstValidSample > 0)
	{
		// Remove from the front of the array up until the first valid sample
		ServerTimeSamples.RemoveAt(0, FirstValidSample);
	}

	// If we have too many samples, uniformly cull intermediate samples by compacting them into the MaxNumSamples range
	// Making sure to always keep the most recent sample
	const int32 MaxNumSamples = GSequencerMaxSmoothedNetSyncSampleCount;
	if (ServerTimeSamples.Num() > MaxNumSamples)
	{
		float Step = FMath::Max(ServerTimeSamples.Num() / float(MaxNumSamples), 1.f);

		for (int Index = 1; Index < MaxNumSamples-1; ++Index)
		{
			const int32 RemappedIndex = ServerTimeSamples.Num() - int(Step*Index) - 1;
			ServerTimeSamples[Index] = ServerTimeSamples[RemappedIndex];
		}
		ServerTimeSamples[MaxNumSamples-1] = ServerTimeSamples.Last();
		ServerTimeSamples.RemoveAt(MaxNumSamples, ServerTimeSamples.Num() - MaxNumSamples, EAllowShrinking::Yes);
	}

	auto UpdateSamplesForChangedTimeDilation = [&]()
	{
		// Project all server time samples back based on the new play-rate and time dilation so future updates will be accurate
		if (LastEffectiveTimeDilation != TimeDilation)
		{
			for (FServerTimeSample& Sample : ServerTimeSamples)
			{
				const double ThisSample = Sample.ServerTime + (CurrentWallClock - Sample.ReceivedTime) * PlaybackMultiplier * LastEffectiveTimeDilation;
				Sample.ReceivedTime = CurrentWallClock - (ThisSample - Sample.ServerTime) / (PlaybackMultiplier * TimeDilation);
			}
			LastEffectiveTimeDilation = TimeDilation;
		}
	};

	if (ServerTimeSamples.Num() < 10)
	{
		// Fallback to the current time if there are not enough samples
		UpdateSamplesForChangedTimeDilation();
		return PlayPosition.GetCurrentPosition();
	}

	// Compute the Standard Deviation so we can understand the variance in the samples
	double MeanTime = 0;
	for (const FServerTimeSample& Sample : ServerTimeSamples)
	{
		const double ThisSample = Sample.ServerTime + (CurrentWallClock - Sample.ReceivedTime) * PlaybackMultiplier * LastEffectiveTimeDilation;
		MeanTime += ThisSample;
	}
	MeanTime = MeanTime / ServerTimeSamples.Num();

	
	double StandardDeviation = 0;
	for (const FServerTimeSample& Sample : ServerTimeSamples)
	{
		const double ThisSample = Sample.ServerTime + (CurrentWallClock - Sample.ReceivedTime) * PlaybackMultiplier * LastEffectiveTimeDilation;
		StandardDeviation += FMath::Square(ThisSample - MeanTime);
	}
	StandardDeviation = StandardDeviation / ServerTimeSamples.Num();
	StandardDeviation = FMath::Sqrt(StandardDeviation);

	const int32 OriginalNum = ServerTimeSamples.Num();

	// Possibly need to recompute the mean if we discard any samples
	double NewMeanTime = MeanTime;

	// If the deviation is greater than our threshold, we start culling samples that lie outside it
	const double DeviationThreshold = ((GSequencerSmoothedNetSyncDeviationThreshold * 0.001f) * PlayPosition.GetInputRate()).AsDecimal();
	if (StandardDeviation > DeviationThreshold)
	{
		// Discard anything outside the standard deviation in the hopes that future samples will converge
		for (int32 SampleIndex = ServerTimeSamples.Num()-1; SampleIndex >= 0; --SampleIndex)
		{
			const double ThisSample = ServerTimeSamples[SampleIndex].ServerTime + (CurrentWallClock - ServerTimeSamples[SampleIndex].ReceivedTime) * PlaybackMultiplier * LastEffectiveTimeDilation;
			if (FMath::Abs(ThisSample - MeanTime) > StandardDeviation)
			{
				ServerTimeSamples.RemoveAt(SampleIndex, EAllowShrinking::No);
			}
			else
			{
				NewMeanTime += ThisSample;
			}
		}
		NewMeanTime = NewMeanTime / ServerTimeSamples.Num();
	}

	UpdateSamplesForChangedTimeDilation();

	// If we didn't cull too many samples, we have confidence in the data set
	if (ServerTimeSamples.Num() >= OriginalNum/2)
	{
		return NewMeanTime * PlayPosition.GetInputRate();
	}
	else
	{
		// Not enough confidence in the data
		return PlayPosition.GetCurrentPosition();
	}
}

void UMovieSceneSequencePlayer::AdvanceClientSerialNumberTo(int32 NewSerialNumber)
{
	if (ensureAlwaysMsgf(!HasAuthority(), TEXT("Trying to advance the serial number on a server player!")))
	{
		if (ensureAlwaysMsgf(NewSerialNumber >= SerialNumber, TEXT("Advancing to an older serial number!")))
		{
			SerialNumber = NewSerialNumber;
		}
	}
}

void UMovieSceneSequencePlayer::RPC_ExplicitServerUpdateEvent_Implementation(EUpdatePositionMethod EventMethod, FFrameTime MarkerTime, int32 NewSerialNumber)
{
	// ~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
	// Handle an explicit jump/play/scrub command from the server.

	if (HasAuthority())
	{
		// Never run network sync operations on authoritative players
		ensure(NewSerialNumber > SerialNumber);
		SerialNumber = NewSerialNumber;
		return;
	}

	if (!Sequence || bIgnorePlaybackReplication)
	{
		// Never run network sync operations on players that have not been initialized yet
		return;
	}

	// Explicit RPC call - empty our smoothed server samples
	ServerTimeSamples.Reset();

#if !NO_LOGGING
	// Log the sync event if necessary
	if (UE_LOG_ACTIVE(LogMovieScene, Verbose))
	{
		const FFrameTime CurrentTime = PlayPosition.GetCurrentPosition();
		const FString SequenceName = GetSequenceName(true);
		UE_LOGF(LogMovieScene, Verbose, "Explicit update event for sequence %ls %ls @ %ls. Server has moved to %ls @ %ls.",
			*SequenceName,
			*UEnum::GetValueAsString(Status.GetValue()), *LexToString(CurrentTime),
			*UEnum::GetValueAsString(NetSyncProps.LastKnownStatus.GetValue()), *LexToString(MarkerTime));
	}
#endif

	// Update our serial number
	AdvanceClientSerialNumberTo(NewSerialNumber);

	// Explicitly repeat the authoritative update event on this client.

	// Note: in the case of PlayToFrame this will not necessarily sweep the exact same range as the server did
	// because this client player is unlikely to be at exactly the same time that the server was at when it performed the operation.
	// This is irrelevant for jumps and scrubs as only the new time is meaningful.
	SetPlaybackPosition(FMovieSceneSequencePlaybackParams(MarkerTime, EventMethod));
}

void UMovieSceneSequencePlayer::RPC_OnStopEvent_Implementation(FFrameTime StoppedTime, int32 NewSerialNumber)
{
	// ~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
	// Handle an explicit Stop command from the server.

	if (HasAuthority())
	{
		// Never run network sync operations on authoritative players
		ensure(NewSerialNumber > SerialNumber);
		SerialNumber = NewSerialNumber;
		return;
	}

	if (!Sequence || bIgnorePlaybackReplication)
	{
		// Never run network sync operations on players that have not been initialized yet
		return;
	}

	// Explicit RPC call - empty our smoothed server samples
	ServerTimeSamples.Reset();

#if !NO_LOGGING
	if (UE_LOG_ACTIVE(LogMovieSceneRepl, Verbose))
	{
		const FFrameTime CurrentTime = PlayPosition.GetCurrentPosition();
		const FString SequenceName = GetSequenceName(true);
		UE_LOGF(LogMovieSceneRepl, Verbose, "Explicit Stop() event for sequence %ls %ls @ frame %d, subframe %f. Server has stopped at frame %d, subframe %f.",
			*SequenceName, *UEnum::GetValueAsString(Status.GetValue()),
			CurrentTime.FrameNumber.Value, CurrentTime.GetSubFrame(),
			StoppedTime.FrameNumber.Value, StoppedTime.GetSubFrame());
	}
#endif

	// Update our serial number
	AdvanceClientSerialNumberTo(NewSerialNumber);

	EUpdatePositionMethod UpdatePositionMethod;
	switch (Status.GetValue())
	{
		case EMovieScenePlayerStatus::Playing:
			UpdatePositionMethod = EUpdatePositionMethod::Play;
			break;
		case EMovieScenePlayerStatus::Scrubbing:
			UpdatePositionMethod = EUpdatePositionMethod::Scrub;
			break;
		default:
			UpdatePositionMethod = EUpdatePositionMethod::Jump;
			break;
	}

	// Catch up with any loops we are missing compared to the server. This is generally just 0 or 1 loops.
	// When it's 1 loop, it's generally because we are very close to the end, and the server somehow stopped
	// near the beginning of the next loop, so we have just a little bit of catching up to do.
	const int32 LoopOffset = (NetSyncProps.LastKnownNumLoops - CurrentNumLoops);
	const FFrameTime LoopEndTime = (bReversePlayback ? StartTime : GetLastValidTime());
	for (int32 LoopIndex = 0; LoopIndex < LoopOffset; ++LoopIndex)
	{
		SetPlaybackPosition(FMovieSceneSequencePlaybackParams(LoopEndTime, UpdatePositionMethod));
	}

	// Now do the last bit of catch-up for the current loop.
	if (PlayPosition.GetCurrentPosition() < StoppedTime)
	{
		UE_LOGF(LogMovieSceneRepl, Verbose, "Catching up to explicit stop time %ls", *LexToString(StoppedTime));
		SetPlaybackPosition(FMovieSceneSequencePlaybackParams(StoppedTime, UpdatePositionMethod));
	}

	StopInternal(StoppedTime);
}

void UMovieSceneSequencePlayer::RPC_OnFinishPlaybackEvent_Implementation(FFrameTime StoppedTime, int32 NewSerialNumber)
{
	if (HasAuthority())
	{
		// Never run network sync operations on authoritative players
		ensure(NewSerialNumber > SerialNumber);
		SerialNumber = NewSerialNumber;
		return;
	}

	if (!Sequence || bIgnorePlaybackReplication)
	{
		// Never run network sync operations on players that have not been initialized yet
		return;
	}

	const FString SequenceName = GetSequenceName(true);
	UE_LOGF(LogMovieScene, Verbose, "Received RPC event to finish sequence %ls at frame %ls.", *SequenceName, *LexToString(StoppedTime));

	// Update our serial number
	AdvanceClientSerialNumberTo(NewSerialNumber);

	FinishPlaybackInternal(StoppedTime);
}

void UMovieSceneSequencePlayer::PostNetReceive()
{
	// ~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
	// Handle a passive update of the replicated status and time properties of the player.

	Super::PostNetReceive();

	if (!ensure(!HasAuthority()) || !Sequence || bIgnorePlaybackReplication)
	{
		// Never run network sync operations on authoritative players or players that have not been initialized yet
		return;
	}

	// Very occasionally a stray network update can come late, and we need to discard it. One such situation
	// is when the server invokes an RPC to stop and finish the sequence, but late network updates arrive
	// after that for the last few frames, and the client player ends up restarting the sequence to evaluate
	// these last few frames even though it has already stopped from the RPCs.
	if (NetSyncProps.LastKnownSerialNumber < SerialNumber)
	{
#if !NO_LOGGING
		const FFrameTime CurrentTime = PlayPosition.GetCurrentPosition();
		const FString SequenceName = GetSequenceName(true);
		UE_LOGF(LogMovieSceneRepl, Verbose, 
				"Ignoring network update with old serial (%d < %d) for sequence %ls %ls @ %ls. Server was %ls @ %ls.",
				NetSyncProps.LastKnownSerialNumber, SerialNumber, *SequenceName,
				*UEnum::GetValueAsString(Status.GetValue()), *LexToString(CurrentTime),
				*UEnum::GetValueAsString(NetSyncProps.LastKnownStatus.GetValue()), *LexToString(NetSyncProps.LastKnownPosition));
#endif
		return;
	}

	const bool bHasStartedPlaying = NetSyncProps.LastKnownStatus == EMovieScenePlayerStatus::Playing && Status != EMovieScenePlayerStatus::Playing;
	const bool bHasChangedStatus  = NetSyncProps.LastKnownStatus   != Status;
	const bool bHasChangedTime    = NetSyncProps.LastKnownPosition != PlayPosition.GetCurrentPosition();

	// We need to take play-rate into account when determining how many frames we can lag behind the server.
	// For instance, if we play 3 times faster than normal (play-rate = 3), we should be able to lag 3 times as
	// many frames behind as normal before we force a re-sync.
	const float PlayRate = PlaybackSettings.PlayRate;

	float TimeDilation = 1.0f;
	if (const UWorld* World = GetPlaybackWorld())
	{
		if (AWorldSettings* WorldSettings = World->GetWorldSettings())
		{
			TimeDilation = WorldSettings->GetEffectiveTimeDilation();
		}
	}

	const float PingMs            = GetPing();
	const FFrameTime PingLag      = (PingMs/1000.f) * PlayPosition.GetInputRate() * PlayRate * TimeDilation;
	//const FFrameTime LagThreshold = 0.2f * PlayPosition.GetInputRate();
	//const FFrameTime LagDisparity = FMath::Abs(PlayPosition.GetCurrentPosition() - NetSyncProps.LastKnownPosition);

	const FFrameTime LagThreshold = (GSequencerNetSyncThresholdMS * 0.001f) * PlayPosition.GetInputRate() * PlayRate * TimeDilation;

	if (!bHasChangedStatus && !bHasChangedTime)
	{
		// Nothing to do
		return;
	}

	if (Observer && !Observer->CanObserveSequence())
	{
		// We shouldn't do anything.
#if !NO_LOGGING
		if (UE_LOG_ACTIVE(LogMovieSceneRepl, Verbose))
		{
			const FFrameTime CurrentTime = PlayPosition.GetCurrentPosition();
			const FString SequenceName = GetSequenceName(true);
			UE_LOGF(LogMovieSceneRepl, Verbose, "Ignoring network update for unobservable sequence %ls %ls @ %ls. Server is %ls @ %ls.",
				*SequenceName,
				*UEnum::GetValueAsString(Status.GetValue()), *LexToString(CurrentTime),
				*UEnum::GetValueAsString(NetSyncProps.LastKnownStatus.GetValue()), *LexToString(NetSyncProps.LastKnownPosition));
		}
#endif
		return;
	}

#if !NO_LOGGING
	if (UE_LOG_ACTIVE(LogMovieSceneRepl, VeryVerbose))
	{
		const FFrameTime CurrentTime = PlayPosition.GetCurrentPosition();
		const FString SequenceName = GetSequenceName(true);
		UE_LOGF(LogMovieSceneRepl, VeryVerbose, "Network sync for sequence %ls %ls @ %ls. Server is %ls @ %ls.",
			*SequenceName,
			*UEnum::GetValueAsString(Status.GetValue()), *LexToString(CurrentTime),
			*UEnum::GetValueAsString(NetSyncProps.LastKnownStatus.GetValue()), *LexToString(NetSyncProps.LastKnownPosition));
	}
#endif

	// Deal with changes of state from stopped <-> playing separately, as they require slightly different considerations
	if (bHasStartedPlaying)
	{
		// Note: when starting playback, we assume that the client and server were at the same time prior to the server initiating playback
		ServerTimeSamples.Reset();

		// Initiate playback from our current position
		PlayInternal();

		const FFrameTime LagDisparity = FMath::Abs(PlayPosition.GetCurrentPosition() - (NetSyncProps.LastKnownPosition + PingLag));
		if (LagDisparity > LagThreshold)
		{
			// Synchronize to the server time as best we can if there is a large disparity
			SetPlaybackPosition(FMovieSceneSequencePlaybackParams(NetSyncProps.LastKnownPosition + PingLag, EUpdatePositionMethod::Play));
		}
	}
	else
	{
		if (bHasChangedTime)
		{
			// Treat all net updates as the main level update - this ensures they get evaluated as part of the 
			// main tick manager
			bIsAsyncUpdate = true;

			// Make sure the client time matches the server according to the client's current status
			if (Status == EMovieScenePlayerStatus::Playing)
			{
				if (bHasChangedStatus)
				{
					// If the status has changed forcibly play to the server position before setting the new status
					SetPlaybackPosition(FMovieSceneSequencePlaybackParams(NetSyncProps.LastKnownPosition + PingLag, EUpdatePositionMethod::Play));
				}
				else
				{
					// Delay net synchronization until next Update call to ensure that we only issue
					// one desync correction per tick.
					bUpdateNetSync = true;
				}
			}
			else if (Status == EMovieScenePlayerStatus::Scrubbing)
			{
				// Scrub to the new position.
				SetPlaybackPosition(FMovieSceneSequencePlaybackParams(NetSyncProps.LastKnownPosition, EUpdatePositionMethod::Scrub));
			}
			else if (Status == EMovieScenePlayerStatus::Stopped)
			{
				// Both client and server are stopped so just update our (client) position to match the server's.
				UpdatePlayPosition(PlayPosition, NetSyncProps.LastKnownPosition, EUpdatePositionMethod::Jump, bReversePlayback);
				TimeController->Reset(GetCurrentTime());
			}

			bIsAsyncUpdate = false;
		}

		if (bHasChangedStatus)
		{
			ServerTimeSamples.Reset();

			switch (NetSyncProps.LastKnownStatus)
			{
			case EMovieScenePlayerStatus::Paused:    Pause(); break;
			case EMovieScenePlayerStatus::Playing:   Play();  break;
			case EMovieScenePlayerStatus::Scrubbing: Scrub(); break;
			}
		}
	}
}

void UMovieSceneSequencePlayer::UpdateNetworkSync()
{
	using namespace UE::MovieScene;

	if (!bUpdateNetSync)
	{
		return;
	}
	bUpdateNetSync = false;

	// Only process net playback synchronization if we are still Playing.
	if (Status == EMovieScenePlayerStatus::Playing)
	{
		const float PingMs = GetPing();

		// We need to take play-rate into account when determining how many frames we can lag behind the server.
		// For instance, if we play 3 times faster than normal (play-rate = 3), we should be able to lag 3 times as
		// many frames behind as normal before we force a re-sync.
		const float PlayRate = PlaybackSettings.PlayRate;

		float TimeDilation = 1.0f;
		if (const UWorld* World = GetPlaybackWorld())
		{
			if (AWorldSettings* WorldSettings = World->GetWorldSettings())
			{
				TimeDilation = WorldSettings->GetEffectiveTimeDilation();
			}
		}

		const FFrameTime PingLag      = (PingMs/1000.f) * PlayPosition.GetInputRate() * PlayRate * TimeDilation;
		const FFrameTime LagThreshold = (GSequencerNetSyncThresholdMS * 0.001f) * PlayPosition.GetInputRate() * PlayRate * TimeDilation;
		
		// When the server has looped back to the start but a client is near the end (and is thus about to loop), we don't want to forcibly synchronize the time unless
		// the *real* difference in time is above the threshold. We compute the real-time difference by adding SequenceDuration*LoopCountDifference to the server position:
		//		start	srv_time																																clt_time		end
		//		0		1		2		3		4		5		6		7		8		9		10		11		12		13		14		15		16		17		18		19		20
		//		|		|																																		|				|
		//
		//		Let NetSyncProps.LastKnownNumLoops = 1, CurrentNumLoops = 0, bReversePlayback = false
		//			=> LoopOffset = 1
		//			   OffsetServerTime = srv_time + FrameDuration*LoopOffset = 1 + 20*1 = 21
		//			   Difference = 21 - 18 = 3 frames
		const int32        LoopOffset       = (NetSyncProps.LastKnownNumLoops - CurrentNumLoops) * (bReversePlayback ? -1 : 1);
		const FFrameTime   OffsetServerTime = (NetSyncProps.LastKnownPosition + PingLag) + GetFrameDuration() * LoopOffset;

		if (LoopOffset != 0)
		{
			// If we crossed a loop boundary, reset the samples
			ServerTimeSamples.Reset();
		}

		const bool bUseSmoothing = GSequencerMaxSmoothedNetSyncSampleAge != 0;
		if (bUseSmoothing)
		{
			ServerTimeSamples.Add(FServerTimeSample{ OffsetServerTime / PlayPosition.GetInputRate(), FPlatformTime::Seconds() });
		}

		const FFrameTime SmoothedServerTime = bUseSmoothing ? UpdateServerTimeSamples() : OffsetServerTime;
		const FFrameTime Difference         = FMath::Abs(PlayPosition.GetCurrentPosition() - SmoothedServerTime);

		SET_DWORD_STAT(MovieSceneRepl_NumServerSamples, ServerTimeSamples.Num());
		SET_FLOAT_STAT(MovieSceneRepl_SmoothedServerTime, SmoothedServerTime.AsDecimal());
		
		if (Difference > LagThreshold + PingLag)
		{
#if !NO_LOGGING
			if (UE_LOG_ACTIVE(LogMovieSceneRepl, Log))
			{
				const FFrameTime CurrentTime = PlayPosition.GetCurrentPosition();
				const FString SequenceName = GetSequenceName(true);
				UE_LOGF(LogMovieSceneRepl, Log, "Correcting de-synced play position for sequence %ls %ls @ %ls. Server is %ls @ %ls, (smoothed: %ls). Client ping is %.2fms.",
					*SequenceName, 
					*UEnum::GetValueAsString(Status.GetValue()), *LexToString(CurrentTime),
					*UEnum::GetValueAsString(NetSyncProps.LastKnownStatus.GetValue()), *LexToString(NetSyncProps.LastKnownPosition),
					*LexToString(SmoothedServerTime), PingMs);
			}
#endif
			// We're drastically out of sync with the server so we need to forcibly set the time.
			const FFrameTime LastPosition = FFrameRate::TransformTime(
					PlayPosition.GetCurrentPosition(), PlayPosition.GetInputRate(), PlayPosition.GetOutputRate());

			// Play to the time only if it is further on in the sequence (in our play direction).
			// Otherwise, jump backwards in time (in our play direction).
			const bool bPlayToFrame = bReversePlayback ? SmoothedServerTime < PlayPosition.GetCurrentPosition() : SmoothedServerTime > PlayPosition.GetCurrentPosition();
			if (bPlayToFrame)
			{
				FMovieSceneSequencePlaybackParams Params(SmoothedServerTime, EUpdatePositionMethod::Play);
				// Indicate that the sequence may have jumped a considerable distance.
				// This especially helps the audio track to stay in-sync after a correction
				Params.bHasJumped = true;
				SetPlaybackPosition(Params);
			}
			else
			{
				SetPlaybackPosition(FMovieSceneSequencePlaybackParams(SmoothedServerTime, EUpdatePositionMethod::Jump));
			}

			// When playing back we skip this sequence's ticked update to avoid queuing 2 updates this frame
			bSkipNextUpdate = true;

			// Also skip all events up to the last known position, otherwise if we skipped back in time we
			// will re-trigger events again.
			TSharedRef<FSharedPlaybackState> SharedPlaybackState = GetSharedPlaybackState();
			FEventTriggerControlPlaybackCapability& TriggerControlCapability = SharedPlaybackState->SetOrAddCapability<FEventTriggerControlPlaybackCapability>();
			TriggerControlCapability.DisableEventTriggersUntilTime = LastPosition;
		}
	}
}

float UMovieSceneSequencePlayer::GetPing() const
{
	float PingMs = 0.0f;
	if (const UWorld* PlayWorld = GetPlaybackWorld())
	{
		const UNetDriver* NetDriver = PlayWorld->GetNetDriver();
		if (NetDriver && NetDriver->ServerConnection && NetDriver->ServerConnection->PlayerController && NetDriver->ServerConnection->PlayerController->PlayerState)
		{
			PingMs = NetDriver->ServerConnection->PlayerController->PlayerState->ExactPing * (bReversePlayback ? -1.f : 1.f);
		}
	}
	return PingMs;
}

void UMovieSceneSequencePlayer::BeginDestroy()
{
	RootTemplateInstance.TearDown();

	TearDown();

	Super::BeginDestroy();
}

int32 UMovieSceneSequencePlayer::GetFunctionCallspace(UFunction* Function, FFrame* Stack)
{
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		// Try to use the same logic as function libraries for static functions, will try to use the global context to check authority only/cosmetic
		return GEngine->GetGlobalFunctionCallspace(Function, this, Stack);
	}

	check(GetOuter());
	return GetOuter()->GetFunctionCallspace(Function, Stack);
}

bool UMovieSceneSequencePlayer::CallRemoteFunction(UFunction* Function, void* Parameters, FOutParmRec* OutParms, FFrame* Stack)
{
	check(!HasAnyFlags(RF_ClassDefaultObject));

	AActor*     Actor     = GetTypedOuter<AActor>();
	UNetDriver* NetDriver = Actor ? Actor->GetNetDriver() : nullptr;
	if (NetDriver)
	{
		NetDriver->ProcessRemoteFunction(Actor, Function, Parameters, OutParms, Stack, this);
		return true;
	}

	return false;
}

void UMovieSceneSequencePlayer::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
	UE::Net::FReplicationFragmentUtil::CreateAndRegisterFragmentsForObject(this, Context, RegistrationFlags);
}

bool UMovieSceneSequencePlayer::NeedsQueueLatentAction() const
{
	return IsEvaluating();
}

void UMovieSceneSequencePlayer::QueueLatentAction(FMovieSceneSequenceLatentActionDelegate Delegate)
{
	if (ensure(TickManager) && !EnumHasAnyFlags(Sequence->GetFlags(), EMovieSceneSequenceFlags::BlockingEvaluation))
	{
		// Queue latent actions on the global tick manager.
		TickManager->AddLatentAction(Delegate);
	}
	else
	{
		// Queue latent actions locally.
		LatentActionManager.AddLatentAction(Delegate);
	}
}

void UMovieSceneSequencePlayer::RunLatentActions()
{
	if (!Sequence)
	{
		return;
	}

	if (SynchronousRunner)
	{
		LatentActionManager.RunLatentActions([this]
		{
			this->SynchronousRunner->Flush();
		});
	}
	else if (ensure(TickManager) && !EnumHasAnyFlags(Sequence->GetFlags(), EMovieSceneSequenceFlags::BlockingEvaluation))
	{
		TickManager->RunLatentActions();
	}
}

void UMovieSceneSequencePlayer::SetWeight(double InWeight)
{
	SetWeight(InWeight, MovieSceneSequenceID::Root);
}

void UMovieSceneSequencePlayer::SetWeight(double InWeight, FMovieSceneSequenceID SequenceID)
{
	UMovieSceneEntitySystemLinker* Linker = RootTemplateInstance.GetEntitySystemLinker();
	if (Linker)
	{
		if (!SequenceWeights)
		{
			SequenceWeights = MakeUnique<UE::MovieScene::FSequenceWeights>(Linker, RootTemplateInstance.GetRootInstanceHandle());

			if (!PlaybackSettings.bDynamicWeighting && Sequence)
			{
				FText Text = NSLOCTEXT("UMovieSceneSequencePlayer", "SetWeightWarning", "Attempting to set a weight on sequence {0} with PlaybackSettings.bDynamicWeighting disabled. This may lead to undesireable blending artifacts or broken in/out blends.");
				FFrame::KismetExecutionMessage(*FText::Format(Text, FText::FromString(Sequence->GetName())).ToString(), ELogVerbosity::Warning);
			}
		}

		SequenceWeights->SetWeight(SequenceID, InWeight);
	}
}

void UMovieSceneSequencePlayer::RemoveWeight()
{
	RemoveWeight(MovieSceneSequenceID::Root);
}

void UMovieSceneSequencePlayer::RemoveWeight(FMovieSceneSequenceID SequenceID)
{
	UMovieSceneEntitySystemLinker* Linker = RootTemplateInstance.GetEntitySystemLinker();
	if (Linker && SequenceWeights)
	{
		SequenceWeights->RemoveWeight(SequenceID);
	}
}
