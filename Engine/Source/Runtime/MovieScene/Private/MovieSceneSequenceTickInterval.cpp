// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneSequenceTickInterval.h"
#include "Engine/Level.h"
#include "GameFramework/Actor.h"
#include "Components/ActorComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSequenceTickInterval)

namespace UE::MovieScene
{

	int32 GMovieSceneTickIntervalResolutionMs = 50;
	static FAutoConsoleVariableRef CVarMovieSceneTickIntervalResolutionMs(
		TEXT("Sequencer.TickIntervalGroupingResolutionMs"),
		GMovieSceneTickIntervalResolutionMs,
		TEXT("Defines the maximum resolution for actor tick interval groupings. Bigger numbers will group more actors together when they have custom tick intervals, but will lead to less accurate intervals.\n"),
		ECVF_Default
	);

} // namespace UE::MovieScene


FMovieSceneSequenceTickInterval::FMovieSceneSequenceTickInterval(const AActor* InActor)
{
	check(InActor);

	TickIntervalSeconds = InActor->PrimaryActorTick.TickInterval;
	bTickWhenPaused = InActor->PrimaryActorTick.bTickEvenWhenPaused;
	bAllowRounding = true;
}

FMovieSceneSequenceTickInterval::FMovieSceneSequenceTickInterval(const UActorComponent* InActorComponent)
{
	check(InActorComponent);

	TickIntervalSeconds = InActorComponent->PrimaryComponentTick.TickInterval;
	bTickWhenPaused = InActorComponent->PrimaryComponentTick.bTickEvenWhenPaused;
	bAllowRounding = true;
}

int32 FMovieSceneSequenceTickInterval::RoundTickIntervalMs() const
{
	using namespace UE::MovieScene;

	// We round actor tick intervals to the nearest GMovieSceneTickIntervalResolutionMs to prevent excessive fragmentation.
	// TickInterval is in seconds, so we multiply by 1000/<rounding> to round it as ms in integer space
	if (bAllowRounding)
	{
		return FMath::CeilToInt(TickIntervalSeconds * (1000.f/GMovieSceneTickIntervalResolutionMs)) * GMovieSceneTickIntervalResolutionMs;
	}

	return FMath::CeilToInt(TickIntervalSeconds * 1000.f);
}

FMovieSceneSequenceTickInterval FMovieSceneSequenceTickInterval::GetInheritedInterval(UObject* ContextObject)
{
	UObject* Current = ContextObject;
	while (Current && !Current->IsA<ULevel>())
	{
		if (USceneComponent* SceneComponent = Cast<USceneComponent>(Current))
		{
			return FMovieSceneSequenceTickInterval(SceneComponent);
		}
		if (AActor* Actor = Cast<AActor>(Current))
		{
			return FMovieSceneSequenceTickInterval(Actor);
		}

		Current = Current->GetOuter();
	}

	return FMovieSceneSequenceTickInterval();
}
