// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneSequenceTickManager.h"
#include "MovieSceneSequencePlayer.h"
#include "Engine/World.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "MovieSceneSequenceTickManagerClient.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "ProfilingDebugging/CountersTrace.h"

#include "Logging/MessageLog.h"
#include "Logging/TokenizedMessage.h"
#include "Misc/UObjectToken.h"
#include "Algo/IndexOf.h"
#include "ProfilingDebugging/CsvProfiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSequenceTickManager)

#define LOCTEXT_NAMESPACE "MovieSceneSequenceTickManager"

DECLARE_CYCLE_STAT(TEXT("Sequence Tick Manager"), MovieSceneEval_SequenceTickManager, STATGROUP_MovieSceneEval);
DECLARE_CYCLE_STAT(TEXT("Tick Clients"), MovieSceneEval_TickClients, STATGROUP_MovieSceneEval);

namespace UE::MovieScene
{

int32 GMovieSceneMaxLatentActionLoops = 100;
static FAutoConsoleVariableRef CVarMovieSceneMaxLatentActionLoops(
	TEXT("Sequencer.MaxLatentActionLoops"),
	GMovieSceneMaxLatentActionLoops,
	TEXT("Defines the maximum number of latent action loops that can be run in one frame.\n"),
	ECVF_Default
);

} // namespace UE::MovieScene

UMovieSceneSequenceTickManager* UMovieSceneSequenceTickManager::Get(UObject* PlaybackContext)
{
	check(PlaybackContext != nullptr && PlaybackContext->GetWorld() != nullptr);
	UWorld* World = PlaybackContext->GetWorld();


	UMovieSceneSequenceTickManager* TickManager = FindObject<UMovieSceneSequenceTickManager>(World, TEXT("GlobalMovieSceneSequenceTickManager"));
	if (!TickManager)
	{
		TickManager = NewObject<UMovieSceneSequenceTickManager>(World, TEXT("GlobalMovieSceneSequenceTickManager"));

		FDelegateHandle Handle = World->AddMovieSceneSequenceTickHandler(
				FOnMovieSceneSequenceTick::FDelegate::CreateUObject(TickManager, &UMovieSceneSequenceTickManager::TickSequenceActors));
		check(Handle.IsValid());
		TickManager->WorldTickDelegateHandle = Handle;
	}
	return TickManager;
}


UMovieSceneSequenceTickManager::UMovieSceneSequenceTickManager(const FObjectInitializer& Init)
	: Super(Init)
{
	PendingActorOperations = nullptr;
}

void UMovieSceneSequenceTickManager::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UMovieSceneSequenceTickManager* This = CastChecked<UMovieSceneSequenceTickManager>(InThis);
	for (FLinkerGroup& Group : This->LinkerGroups)
	{
		Collector.AddReferencedObject(Group.Linker);
	}
}

UMovieSceneEntitySystemLinker* UMovieSceneSequenceTickManager::GetLinker(const FMovieSceneSequenceTickInterval& TickInterval)
{
	const int32 RoundedTickIntervalMs = TickInterval.RoundTickIntervalMs();
	for (const FLinkerGroup& Group : LinkerGroups)
	{
		if (Group.RoundedTickIntervalMs == RoundedTickIntervalMs)
		{
			return Group.Linker;
		}
	}
	return nullptr;
}

TSharedPtr<FMovieSceneEntitySystemRunner> UMovieSceneSequenceTickManager::GetRunner(const FMovieSceneSequenceTickInterval& TickInterval)
{
	const int32 RoundedTickIntervalMs = TickInterval.RoundTickIntervalMs();
	for (FLinkerGroup& Group : LinkerGroups)
	{
		if (Group.RoundedTickIntervalMs == RoundedTickIntervalMs)
		{
			return Group.Runner;
		}
	}
	return nullptr;
}

void UMovieSceneSequenceTickManager::BeginDestroy()
{
	if (WorldTickDelegateHandle.IsValid())
	{
		UWorld* World = GetTypedOuter<UWorld>();
		if (ensure(World != nullptr))
		{
			World->RemoveMovieSceneSequenceTickHandler(WorldTickDelegateHandle);
			WorldTickDelegateHandle.Reset();
		}
	}

	Super::BeginDestroy();
}

void UMovieSceneSequenceTickManager::RegisterTickClient(const FMovieSceneSequenceTickInterval& ResolvedTickInterval, TScriptInterface<IMovieSceneSequenceTickManagerClient> InTickInterface)
{
	using namespace UE::MovieScene;

	UObject* InterfaceObject = InTickInterface.GetObject();
	IMovieSceneSequenceTickManagerClient* ClientInterface = InTickInterface.GetInterface();
	if (!ensure(InterfaceObject && ClientInterface))
	{
		return;
	}

	// If PendingActorOperations is non-null, it means we cannot mutate either TickableClients or LinkerGroups so we have to defer until afterwards
	if (PendingActorOperations)
	{
		PendingActorOperations->Add(FPendingOperation{ InTickInterface, InterfaceObject, ResolvedTickInterval, FPendingOperation::EType::Register });
		return;
	}

#if DO_GUARD_SLOW
	UWorld* ThisWorld   = GetWorld();
	UWorld* ObjectWorld = InterfaceObject->GetWorld();
	checkfSlow(ObjectWorld == ThisWorld, TEXT("Attempting to register object %s that is a part of world %s to a sequence tick manager contained within world %s."),
		*InterfaceObject->GetName(), ObjectWorld ? *ObjectWorld->GetName() : TEXT("nullptr"), ThisWorld ? *ThisWorld->GetName() : TEXT("nullptr"));
#endif

	// Remove the client from any existing groups if we already know about it
	UnregisterTickClientImpl(InTickInterface.GetInterface());

	const int32 DesiredTickIntervalMs = ResolvedTickInterval.RoundTickIntervalMs();

	// Find a linker group with the specified tick interval
	// Would be nice to use Algo::IndexOfBy here but that cannot work on TSparseArray since it is a non-contiguous range
	int32 LinkerIndex = 0;
	for (; LinkerIndex < LinkerGroups.GetMaxIndex(); ++LinkerIndex)
	{
		if (LinkerGroups.IsAllocated(LinkerIndex) && LinkerGroups[LinkerIndex].RoundedTickIntervalMs == DesiredTickIntervalMs)
		{
			break;
		}
	}

	const float DesiredBudgetMs = ResolvedTickInterval.EvaluationBudgetMicroseconds / 1000.f;
	if (!LinkerGroups.IsValidIndex(LinkerIndex))
	{
		LinkerIndex = LinkerGroups.Emplace();

		TStringBuilder<64> LinkerName;
		LinkerName.Append(TEXT("MovieSceneSequencePlayerEntityLinker"));
		if (DesiredTickIntervalMs != 0)
		{
			LinkerName.Appendf(TEXT("_%i_ms"), DesiredTickIntervalMs);
		}

		// With support for multi-frame evaluations, it is possible for the linker group
		// to be torn down mid evaluation which can leave the linker in a bad state. Use a unique
		// linker name to avoid reusing those linkers.
		const FName UniqueLinkerName = MakeUniqueObjectName(GetWorld(), UMovieSceneEntitySystemLinker::StaticClass(), LinkerName.ToString());
		UMovieSceneEntitySystemLinker* Linker = UMovieSceneEntitySystemLinker::FindOrCreateLinker(GetWorld(), EEntitySystemLinkerRole::LevelSequences, *UniqueLinkerName.ToString());
		check(Linker);

		FLinkerGroup& NewGroup = LinkerGroups[LinkerIndex];
		NewGroup.Linker = Linker;
		NewGroup.Runner = Linker->GetRunner();
		NewGroup.RoundedTickIntervalMs = DesiredTickIntervalMs;
		NewGroup.FrameBudgetMs = DesiredBudgetMs;
		NewGroup.NumClients = 1;
	}
	else
	{
		FLinkerGroup& ExistingGroup = LinkerGroups[LinkerIndex];
		++ExistingGroup.NumClients;

		// If we're attempting to play sequences at the same tick interval with different budgets,
		// we have to reconcile those. We emit a warning for mismatches of budgeted vs. not-budgeted
		// as it will actually change behavior (non-budgeted will always eval in one frame, budgeted
		// will sometimes eval over multiple frames).
		if (ExistingGroup.FrameBudgetMs != DesiredBudgetMs)
		{
			if (ExistingGroup.FrameBudgetMs == 0.f)
			{
				ExistingGroup.FrameBudgetMs = DesiredBudgetMs;

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
				TSharedRef<FTokenizedMessage> Message = FMessageLog("PIE")
				.Warning(LOCTEXT("BudgetMismatch", "Attempting to play multiple sequences with the same tick interval but different budgets. Lowest budget will be used, but may result in undesired multi-frame effects."))
				->AddToken(FTextToken::Create(LOCTEXT("ClientList", "Initiating tick client ")))
				->AddToken(FUObjectToken::Create(InterfaceObject))
				->AddToken(FTextToken::Create(FText::Format(LOCTEXT("ExistingClients", " with budget of {0}μs into group with an interval of {1}ms and budget of {2}μs. Existing clients: "),
					FMath::RoundToInt(DesiredBudgetMs/1000.f), ExistingGroup.RoundedTickIntervalMs, FMath::RoundToInt(ExistingGroup.FrameBudgetMs/1000.f))));
				for (const FTickableClientData& Client : TickableClients)
				{
					if (Client.LinkerIndex == LinkerIndex)
					{
						UObject* Object = Client.WeakObject.Get();
						while (Object && !(Object->IsA<AActor>() || Object->IsA<USceneComponent>()))
						{
							Object = Object->GetOuter();
						}
						Message->AddToken(FUObjectToken::Create(Object));
					}
				}
#endif
			}
			else if (DesiredBudgetMs != 0.f)
			{
				ExistingGroup.FrameBudgetMs = FMath::Min(ExistingGroup.FrameBudgetMs, DesiredBudgetMs);
			}
		}
	}

	checkSlow(LinkerIndex >= 0 && LinkerIndex < uint16(-1));

	TickableClients.Add(FTickableClientData{
		ClientInterface,
		InterfaceObject,
		static_cast<uint16>(LinkerIndex),
		ResolvedTickInterval.bTickWhenPaused
	});
}

void UMovieSceneSequenceTickManager::UnregisterTickClient(TScriptInterface<IMovieSceneSequenceTickManagerClient> InTickInterface)
{
	using namespace UE::MovieScene;

	IMovieSceneSequenceTickManagerClient* ClientInterface = InTickInterface.GetInterface();
	if (!ensure(ClientInterface))
	{
		return;
	}

	// If PendingActorOperations is non-null, it means we cannot mutate either TickableClients or LinkerGroups so we have to defer until afterwards
	if (PendingActorOperations)
	{
		PendingActorOperations->Add(FPendingOperation{ InTickInterface, InTickInterface.GetObject(), FMovieSceneSequenceTickInterval(),  FPendingOperation::EType::Unregister });
		return;
	}
	
	// Remove any latent actions tied to the given client.
	UObject* Object = InTickInterface.GetObject();
	if (Object)
	{
		ClearLatentActions(Object);
	}

	UnregisterTickClientImpl(ClientInterface);
}

void UMovieSceneSequenceTickManager::UnregisterTickClientImpl(IMovieSceneSequenceTickManagerClient* InClientInterface)
{
	const int32 ClientIndex = Algo::IndexOfBy(TickableClients, InClientInterface, &FTickableClientData::Interface);
	if (!TickableClients.IsValidIndex(ClientIndex))
	{
		return;
	}

	const int32 LinkerIndex = TickableClients[ClientIndex].LinkerIndex;
	checkSlow(LinkerGroups.IsValidIndex(LinkerIndex));

	TickableClients.RemoveAtSwap(ClientIndex);

	if (--LinkerGroups[LinkerIndex].NumClients == 0)
	{
		LinkerGroups.RemoveAt(LinkerIndex);
	}
}

void UMovieSceneSequenceTickManager::TickSequenceActors(float DeltaSeconds)
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(MovieSceneEval);
	SCOPE_CYCLE_COUNTER(MovieSceneEval_SequenceTickManager);

	if (IsUnreachable() || HasAnyFlags(RF_BeginDestroyed))
	{
		return;
	}

	// Let all tickable clients update. Some of them won't do anything, others will do synchronous
	// things (e.g. start/stop, loop, etc.), but in 95% of cases, they will just queue up a normal evaluation
	// request...
	//
	UWorld* World = GetTypedOuter<UWorld>();
	checkSlow(World != nullptr);
#if DO_GUARD_SLOW
	ensure(LatentActionManager.IsEmpty());
#endif

	const bool  bIsPaused                  = World->IsPaused();
	const float CurrentUnpausedTimeSeconds = World->GetUnpausedTimeSeconds();
	const float CurrentTimeSeconds         = World->GetTimeSeconds();

	struct FUpdatedDeltaTimes
	{
		float UnpausedDeltaTime;
		float DeltaTime;
	};

	using InlineSparseAllocator = TSparseArrayAllocator<TInlineAllocator<16>, TInlineAllocator<4>>;

	// Sparse array where an allocated entry maps to the FLinkerGroup within LinkerGroups
	TSparseArray<FUpdatedDeltaTimes, InlineSparseAllocator> UpdatedDeltaTimes;

	TBitArray<> OutstandingLinkers;

	// -----------------------------------------------------------------------------
	// Step 1: Check the tick intervals of all our linker groups, and mark any whose
	//         interval has passed for update
	for (int32 Index = 0; Index < LinkerGroups.GetMaxIndex(); ++Index)
	{
		if (LinkerGroups.IsAllocated(Index))
		{
			// Check the delta time against the last update time for this group
			FLinkerGroup& Group = LinkerGroups[Index];

			float UnpausedDeltaTime = DeltaSeconds;
			float DeltaTime = DeltaSeconds;

			// If we're currently evaluating this linker group,
			// skip ticking it until we're completely finished
			if (Group.Runner->IsCurrentlyEvaluating())
			{
				OutstandingLinkers.PadToNum(Index + 1, false);
				OutstandingLinkers[Index] = true;
				continue;
			}

			if (Group.LastUnpausedTimeSeconds >= 0.f)
			{
				UnpausedDeltaTime = CurrentUnpausedTimeSeconds - Group.LastUnpausedTimeSeconds;
				DeltaTime = CurrentTimeSeconds - Group.LastTimeSeconds;

				if (UnpausedDeltaTime < Group.RoundedTickIntervalMs * 0.001f)
				{
					// If the unpaused time is less than the required tick interval, leave this group alone this frame
					// We don't need to check the paused delta-time because that will always be >= unpaused
					continue;
				}
			}

			// We know the unpaused delta-time is >= our interval, and thus so will the paused time
			// Add this delta-time to the sparse array to indicate the group with the corresponding index needs updating
			Group.LastUnpausedTimeSeconds = CurrentUnpausedTimeSeconds;
			Group.LastTimeSeconds = CurrentTimeSeconds;

			UpdatedDeltaTimes.Insert(Index, FUpdatedDeltaTimes{ UnpausedDeltaTime, DeltaTime });
		}
	}

	// Skip any work if there are no updates scheduled
	if (UpdatedDeltaTimes.Num() == 0 && OutstandingLinkers.Num() == 0)
	{
		return;
	}

	TArray<FPendingOperation> CurrentPendingActorOperations;
	{
		// Guard against any mutation while the entries are being ticked - anything added this tick cycle will have to wait until next tick
		TGuardValue<TArray<FPendingOperation>*> Guard(PendingActorOperations, &CurrentPendingActorOperations);

		// Step 2: Tick unbudgeted clients for any with updated delta times
		// 
		if (UpdatedDeltaTimes.Num() != 0)
		{
			SCOPE_CYCLE_COUNTER(MovieSceneEval_TickClients);
			for (const FTickableClientData& Client : TickableClients)
			{
				const bool bCanTick = (Client.bTickWhenPaused || !bIsPaused) && Client.WeakObject.Get() != nullptr;
				if (bCanTick && UpdatedDeltaTimes.IsValidIndex(Client.LinkerIndex))
				{
					const FUpdatedDeltaTimes Delta = UpdatedDeltaTimes[Client.LinkerIndex];
					const float DeltaTime = Client.bTickWhenPaused ? Delta.UnpausedDeltaTime : Delta.DeltaTime;

					TSharedPtr<FMovieSceneEntitySystemRunner> Runner = LinkerGroups[Client.LinkerIndex].Runner;
					Client.Interface->TickFromSequenceTickManager(DeltaTime, Runner.Get());
				}
			}
		}

		// Step 3: Flush any budgeted evaluations that are still pending
		// 
		for (TConstSetBitIterator<> LinkerIndex(OutstandingLinkers); LinkerIndex; ++LinkerIndex)
		{
			FLinkerGroup& Group = LinkerGroups[LinkerIndex.GetIndex()];

			if (Group.Linker == nullptr || Group.Linker->IsUnreachable() || Group.Linker->HasAnyFlags(RF_BeginDestroyed))
			{
				continue;
			}

			check(!UpdatedDeltaTimes.IsValidIndex(LinkerIndex.GetIndex()));

			Group.Runner->Flush(Group.FrameBudgetMs);
		}

		// Step 4: Update and flush linkers as needed
		//
		for (int32 Index = 0; Index < UpdatedDeltaTimes.GetMaxIndex(); ++Index)
		{
			if (UpdatedDeltaTimes.IsAllocated(Index))
			{
				FLinkerGroup& Group = LinkerGroups[Index];

				if (Group.Linker == nullptr || Group.Linker->IsUnreachable() || Group.Linker->HasAnyFlags(RF_BeginDestroyed))
				{
					continue;
				}

				// Hitting this check would indicate that the loop above that processes OutstandingLinkers either failed, or some other partial flush happened between then and now.
				ensureMsgf(!Group.Runner->IsCurrentlyEvaluating(), TEXT("Linker is part-way thorugh a flush when a new flush is being instigated. This is undefined behavior."));

				if (Group.Runner->HasQueuedUpdates())
				{
					Group.Runner->Flush(Group.FrameBudgetMs);
				}
			}
		}
	}

	// Process any pending operations that were added while we were updating
	ProcessPendingOperations(CurrentPendingActorOperations);
	// Run latent actions now we have finished flushing everything
	RunLatentActions();
}

void UMovieSceneSequenceTickManager::ProcessPendingOperations(TArrayView<const FPendingOperation> InOperations)
{
	// Process any pending operations that were added while we were updating
	for (const FPendingOperation& PendingOperation : InOperations)
	{
		// Check the object is still alive
		if (PendingOperation.WeakObject.Get() != nullptr)
		{
			if (PendingOperation.Type == FPendingOperation::EType::Register)
			{
				RegisterTickClient(PendingOperation.TickInterval, PendingOperation.Interface);
			}
			else
			{
				UnregisterTickClient(PendingOperation.Interface);
			}
		}
	}
}

void UMovieSceneSequenceTickManager::ClearLatentActions(UObject* Object)
{
	LatentActionManager.ClearLatentActions(Object);
}

void UMovieSceneSequenceTickManager::AddLatentAction(FMovieSceneSequenceLatentActionDelegate Delegate)
{
	LatentActionManager.AddLatentAction(Delegate);
}

void UMovieSceneSequenceTickManager::RunLatentActions()
{
	LatentActionManager.RunLatentActions([this]
	{
		this->FlushRunners();
	});
}

void UMovieSceneSequenceTickManager::FlushRunners()
{
	TArray<FPendingOperation> CurrentPendingActorOperations;
	{
		// Guard against any mutation while the entries are being ticked - anything added this tick cycle will have to wait until next tick
		TGuardValue<TArray<FPendingOperation>*> Guard(PendingActorOperations, &CurrentPendingActorOperations);
		for (FLinkerGroup& LinkerGroup : LinkerGroups)
		{
			if (LinkerGroup.Linker == nullptr || LinkerGroup.Linker->IsUnreachable() || LinkerGroup.Linker->HasAnyFlags(RF_BeginDestroyed))
			{
				continue;
			}

			LinkerGroup.Runner->Flush();
		}
	}

	// Process any pending operations that were added while we were updating
	ProcessPendingOperations(CurrentPendingActorOperations);
}

void FMovieSceneLatentActionManager::AddLatentAction(FMovieSceneSequenceLatentActionDelegate Delegate)
{
	check(Delegate.GetUObject() != nullptr);
	LatentActions.Add(Delegate);
}

void FMovieSceneLatentActionManager::ClearLatentActions(UObject* Object)
{
	check(Object);

	for (FMovieSceneSequenceLatentActionDelegate& Action : LatentActions)
	{
		// Rather than remove actions, we simply unbind them, to ensure that we do not
		// shuffle the array if it is already being processed higher up the call-stack
		if (Action.IsBound() && Action.GetUObject() == Object)
		{
			Action.Unbind();
		}
	}
}

void FMovieSceneLatentActionManager::ClearLatentActions()
{
	if (ensureMsgf(!bIsRunningLatentActions, TEXT("Can't clear latent actions while they are running!")))
	{
		LatentActions.Reset();
	}
}

void FMovieSceneLatentActionManager::RunLatentActions(TFunctionRef<void()> FlushCallback)
{
	if (bIsRunningLatentActions)
	{
		// If someone is asking to run latent actions while we are running latent actions, we
		// can just safely bail out... if they have just queued more latent actions, we will 
		// naturally get to them as we make our way through the list.
		return;
	}

	TGuardValue<bool> IsRunningLatentActionsGuard(bIsRunningLatentActions, true);

	int32 NumLoopsLeft = UE::MovieScene::GMovieSceneMaxLatentActionLoops;
	while (LatentActions.Num() > 0)
	{
		// We can run *one* latent action per sequence player before having to flush the linker again.
		// This way, if we have 42 sequence players with 2 latent actions each, we only flush the linker
		// twice, as opposed to 42*2=84 times.
		int32 Index = 0;
		TSet<UObject*> ExecutedDelegateOwners;
		while (Index < LatentActions.Num())
		{
			const FMovieSceneSequenceLatentActionDelegate& Delegate = LatentActions[Index];
			if (!Delegate.IsBound())
			{
				LatentActions.RemoveAt(Index);
				continue;
			}

			UObject* BoundObject = Delegate.GetUObject();
			if (ensure(BoundObject) && !ExecutedDelegateOwners.Contains(BoundObject))
			{
				UMovieSceneSequencePlayer* Player = Cast<UMovieSceneSequencePlayer>(BoundObject);
				if (Player && Player->IsEvaluating())
				{
					// If our player is still evaluating, defer all latent actions for this
					// sequence player to the next pass.
					++Index;
				}
				else
				{
					Delegate.ExecuteIfBound();
                	LatentActions.RemoveAt(Index);
				}
				ExecutedDelegateOwners.Add(BoundObject);
			}
			else
			{
				++Index;
			}
		}

		FlushCallback();

		--NumLoopsLeft;
		if (!ensureMsgf(NumLoopsLeft > 0,
					TEXT("Detected possible infinite loop! Are you requeuing the same latent action over and over?")))
		{
			break;
		}
	}
}

#undef LOCTEXT_NAMESPACE
