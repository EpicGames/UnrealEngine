// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneSequenceVisitor.h"
#include "MovieSceneSequence.h"
#include "MovieScene.h"
#include "MovieSceneTrack.h"
#include "MovieSceneSection.h"
#include "MovieSceneTransformTypes.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "Sections/MovieSceneSubSection.h"
#include "Evaluation/MovieSceneRootOverridePath.h"

namespace UE
{
namespace MovieScene
{

FSubSequenceSpace::FSubSequenceSpace()
	: RootToSequenceTransform()
	, SequenceID(MovieSceneSequenceID::Root)
	, RootClampRange(TRange<FFrameNumber>::All())
	, LocalClampRange(TRange<FFrameNumber>::All())
	, HierarchicalBias(0)
{}

void VisitTrackImpl(const FSequenceVisitParams& InParams, UMovieSceneTrack* InTrack, ISequenceVisitor& InVisitor, const FGuid& ObjectBinding, FSubSequencePath* InOutRootPath, FSubSequenceSpace* SubSequenceSpace);
void VisitSequenceImpl(UMovieSceneSequence* Sequence, const FSequenceVisitParams& InParams, ISequenceVisitor& InVisitor, FSubSequencePath* InOutRootPath, FSubSequenceSpace* SubSequenceSpace);

void VisitSubTrackImpl(const FSequenceVisitParams& InParams, UMovieSceneSubTrack* SubTrack, ISequenceVisitor& InVisitor, const FGuid& ObjectBinding, FSubSequencePath* InOutRootPath, FSubSequenceSpace* SubSequenceSpace)
{
	if (SubTrack->IsEvalDisabled() && !InParams.bVisitDisabledSubSequences)
	{
		return;
	}

	for (const FMovieSceneTrackEvaluationFieldEntry& Entry : SubTrack->GetEvaluationField().Entries)
	{
		if (EnumHasAnyFlags(Entry.Flags, ESectionEvaluationFlags::PreRoll) || EnumHasAnyFlags(Entry.Flags, ESectionEvaluationFlags::PostRoll))
		{
			continue;
		}

		UMovieSceneSubSection* SubSection  = Cast<UMovieSceneSubSection>(Entry.Section);
	
		if (SubSection && SubTrack->IsRowEvalDisabled(SubSection->GetRowIndex()) && !InParams.bVisitDisabledSubSequences)
		{
			continue;
		}

		UMovieSceneSequence*   SubSequence = SubSection ? SubSection->GetSequence() : nullptr;
		if (SubSequence == nullptr)
		{
			continue;
		}

		InVisitor.VisitStartSubSequence(SubSection, ObjectBinding, *SubSequenceSpace);

		auto VisitRange = [&InOutRootPath, &InVisitor, &InParams, SubSection, &ObjectBinding, SubSequence, SubSequenceSpace](TRange<FFrameTime> Range)
		{
			TRange<FFrameNumber> EffectiveRange = ConvertToDiscreteRange(TRange<FFrameTime>::Intersection(Range, ConvertToFrameTimeRange(SubSequenceSpace->RootClampRange)));
			if (!EffectiveRange.IsEmpty())
			{
				FMovieSceneSequenceTransform OuterToInnerTransform = SubSection->OuterToInnerTransform();
				FMovieSceneSequenceID        LocalSequenceID       = SubSection->GetSequenceID();
				const FMovieSceneSequenceID  SubSequenceID         = InOutRootPath->ResolveChildSequenceID(LocalSequenceID);

				FSubSequenceSpace LocalSpace;
				LocalSpace.RootToSequenceTransform = OuterToInnerTransform * SubSequenceSpace->RootToSequenceTransform;
				LocalSpace.StartBreadcrumbs = SubSequenceSpace->StartBreadcrumbs;
				LocalSpace.EndBreadcrumbs = SubSequenceSpace->EndBreadcrumbs;
				LocalSpace.SequenceID = InOutRootPath->ResolveChildSequenceID(LocalSequenceID);
				LocalSpace.RootClampRange = EffectiveRange;
				LocalSpace.LocalClampRange = ConvertToDiscreteRange(LocalSpace.RootToSequenceTransform.ComputeTraversedHull(EffectiveRange));
				LocalSpace.HierarchicalBias = SubSequenceSpace->HierarchicalBias + SubSection->Parameters.HierarchicalBias;

				if (OuterToInnerTransform.NeedsBreadcrumbs())
				{
					OuterToInnerTransform.TransformTime(DiscreteInclusiveLower(EffectiveRange.GetLowerBoundValue()), FTransformTimeParams().AppendBreadcrumbs(LocalSpace.StartBreadcrumbs));
					OuterToInnerTransform.TransformTime(DiscreteExclusiveUpper(EffectiveRange.GetUpperBoundValue()), FTransformTimeParams().AppendBreadcrumbs(LocalSpace.EndBreadcrumbs));
				}

				// Recurse into the sub sequence
				InOutRootPath->PushGeneration(SubSequenceID, LocalSequenceID);
				{
					InVisitor.VisitSubSequence(SubSequence, ObjectBinding, LocalSpace);
					VisitSequenceImpl(SubSequence, InParams, InVisitor, InOutRootPath, &LocalSpace);
				}
				InOutRootPath->PopGenerations(1);
			}
			return true;
		};

		FMovieSceneInverseSequenceTransform SequenceToRootTransform = SubSequenceSpace->RootToSequenceTransform.Inverse();
		SequenceToRootTransform.TransformFiniteRangeWithinRange(ConvertToFrameTimeRange(Entry.Range), VisitRange, SubSequenceSpace->StartBreadcrumbs, SubSequenceSpace->EndBreadcrumbs);

		InVisitor.VisitEndSubSequence(SubSection, ObjectBinding, *SubSequenceSpace);
	}
}

void VisitTrackImpl(const FSequenceVisitParams& InParams, UMovieSceneTrack* InTrack, ISequenceVisitor& InVisitor, const FGuid& ObjectBinding, FSubSequencePath* InOutRootPath, FSubSequenceSpace* SubSequenceSpace)
{
	if (InParams.bVisitSubSequences)
	{
		if (UMovieSceneSubTrack* SubTrack = Cast<UMovieSceneSubTrack>(InTrack))
		{
			VisitSubTrackImpl(InParams, SubTrack, InVisitor, ObjectBinding, InOutRootPath, SubSequenceSpace);
		}
	}

	if (InParams.bVisitTracks || InParams.bVisitRootTracks)
	{
		InVisitor.VisitTrack(InTrack, ObjectBinding, *SubSequenceSpace);
	}

	if (InParams.bVisitSections)
	{
		for (UMovieSceneSection* Section : InTrack->GetAllSections())
		{
			if(Section->IsActive() || InParams.bVisitDisabledSections)
			{
				InVisitor.VisitSection(InTrack, Section, ObjectBinding, *SubSequenceSpace);
			}
		}
	}
}

void VisitSequenceImpl(UMovieSceneSequence* Sequence, const FSequenceVisitParams& InParams, ISequenceVisitor& InVisitor, FSubSequencePath* InOutRootPath, FSubSequenceSpace* SubSequenceSpace)
{
	check(Sequence);

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	if (InParams.bVisitRootTracks)
	{
		if (UMovieSceneTrack* Track = MovieScene->GetCameraCutTrack())
		{
			VisitTrackImpl(InParams, Track, InVisitor, FGuid(), InOutRootPath, SubSequenceSpace);
		}

		for (UMovieSceneTrack* Track : MovieScene->GetTracks())
		{
			VisitTrackImpl(InParams, Track, InVisitor, FGuid(), InOutRootPath, SubSequenceSpace);
		}
	}

	if (InParams.bVisitObjectBindings)
	{
		for (const FMovieSceneBinding& ObjectBinding : ((const UMovieScene*)MovieScene)->GetBindings())
		{
			InVisitor.VisitObjectBinding(ObjectBinding, *SubSequenceSpace);

			if (!InParams.CanVisitTracksOrSections())
			{
				continue;
			}

			for (UMovieSceneTrack* Track : ObjectBinding.GetTracks())
			{
				VisitTrackImpl(InParams, Track, InVisitor, ObjectBinding.GetObjectGuid(), InOutRootPath, SubSequenceSpace);
			}
		}
	}
}

void VisitSequence(UMovieSceneSequence* Sequence, const FSequenceVisitParams& InParams, ISequenceVisitor& Visitor)
{
	FSubSequenceSpace RootSpace;
	FSubSequencePath RootPath;

	VisitSequenceImpl(Sequence, InParams, Visitor, &RootPath, &RootSpace);
}

} // namespace MovieScene
} // namespace UE



