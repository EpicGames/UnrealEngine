// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneSignedObject.h"
#include "Templates/Casts.h"
#include "MovieSceneSequence.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "UObject/Package.h"
#include "CoreGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSignedObject)

bool GMovieSceneSuppressSignatureChangeInCook = true;
static FAutoConsoleVariableRef CVarMovieSceneSuppressSignatureChangeInCook(
	TEXT("Sequencer.SuppressSignatureChangeInCook"),
	GMovieSceneSuppressSignatureChangeInCook,
	TEXT("(Default: true) Globally suppress signature GUID changes inside cook commandlets."),
	ECVF_Default
);

namespace UE
{
namespace MovieScene
{

static TSet<TWeakPtr<IDeferredSignedObjectFlushSignal>> GDeferredSignedObjectFlushSignals;
static TWeakPtr<IDeferredSignedObjectChangeHandler> GDeferredSignedObjectChangeHandler;
static uint32 GScopedSignedObjectDeferCount = 0;
static uint32 GScopedSignedObjectSuppressCount = 0;
static uint32 GScopedSignedObjectWarnModifyCompiledData = 0;

void SignalScopedSignedObjectModifyFlush()
{
	// Copy to a temporary set to guard against re-entrancy
	TSet<TWeakPtr<IDeferredSignedObjectFlushSignal>> Temp;
	Swap(Temp, GDeferredSignedObjectFlushSignals);

	for (TWeakPtr<UE::MovieScene::IDeferredSignedObjectFlushSignal> WeakSignal : Temp)
	{
		if (TSharedPtr<UE::MovieScene::IDeferredSignedObjectFlushSignal> Signal = WeakSignal.Pin())
		{
			Signal->OnDeferredModifyFlush();
		}
	}
}

FScopedSignedObjectModifyDefer::FScopedSignedObjectModifyDefer(bool bInForceFlush)
{
	++GScopedSignedObjectDeferCount;
	bForceFlush = bInForceFlush;
}

FScopedSignedObjectModifyDefer::~FScopedSignedObjectModifyDefer()
{
	--GScopedSignedObjectDeferCount;

	if (GScopedSignedObjectDeferCount == 0 || bForceFlush)
	{
		if (GDeferredSignedObjectChangeHandler.IsValid())
		{
			GDeferredSignedObjectChangeHandler.Pin()->Flush();
		}
		SignalScopedSignedObjectModifyFlush();
	}
}

FScopedSignedObjectModifySuppress::FScopedSignedObjectModifySuppress(bool bInWarnModifyCompiledData)
	: bWarnModifyCompiledData(bInWarnModifyCompiledData)
{
	++GScopedSignedObjectSuppressCount;

	if (bWarnModifyCompiledData)
	{
		++GScopedSignedObjectWarnModifyCompiledData;
	}
}

FScopedSignedObjectModifySuppress::~FScopedSignedObjectModifySuppress()
{
	--GScopedSignedObjectSuppressCount;
	check(GScopedSignedObjectSuppressCount >= 0);

	if (bWarnModifyCompiledData)
	{
		--GScopedSignedObjectWarnModifyCompiledData;
		check(GScopedSignedObjectWarnModifyCompiledData >= 0);
	}
}

} // namespace MovieScene
} // namespace UE


UMovieSceneSignedObject::UMovieSceneSignedObject(const FObjectInitializer& Init)
	: Super(Init)
{
	using namespace UE::MovieScene;

 	if (HasAnyFlags(RF_Transactional) && GDeferredSignedObjectChangeHandler.IsValid() 
#if WITH_EDITOR
		&& GIsTransacting
#endif
	)
	{
		GDeferredSignedObjectChangeHandler.Pin()->DeferMarkAsChanged(this);
	}
}

TWeakPtr<UE::MovieScene::IDeferredSignedObjectChangeHandler> UMovieSceneSignedObject::GetDeferredHandler()
{
	return UE::MovieScene::GDeferredSignedObjectChangeHandler;
}

void UMovieSceneSignedObject::SetDeferredHandler(TWeakPtr<UE::MovieScene::IDeferredSignedObjectChangeHandler>&& InHandler)
{
	UE::MovieScene::GDeferredSignedObjectChangeHandler = MoveTemp(InHandler);
}

void UMovieSceneSignedObject::AddFlushSignal(TWeakPtr<UE::MovieScene::IDeferredSignedObjectFlushSignal> Signal)
{
	UE::MovieScene::GDeferredSignedObjectFlushSignals.Add(Signal);
}

void UMovieSceneSignedObject::ResetImplicitScopedModifyDefer()
{
	using namespace UE::MovieScene;
	if (GDeferredSignedObjectChangeHandler.IsValid())
	{
		GDeferredSignedObjectChangeHandler.Pin()->ResetImplicitScopedModifyDefer();
	}
}

void UMovieSceneSignedObject::PostInitProperties()
{
	Super::PostInitProperties();

	// Always seed newly created objects with a new signature
	// (CDO and archetypes always have a zero GUID)
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject | RF_NeedLoad | RF_LoadCompleted) && Signature == GetDefault<UMovieSceneSignedObject>()->Signature)
	{
		Signature = FGuid::NewGuid();
	}
}

void UMovieSceneSignedObject::PostLoad()
{
	Super::PostLoad();
}

void UMovieSceneSignedObject::MarkAsChanged()
{
	using namespace UE::MovieScene;

	const bool bGlobalCookSuppression = (GMovieSceneSuppressSignatureChangeInCook && IsRunningCookCommandlet());

	if (GScopedSignedObjectWarnModifyCompiledData > 0 || bGlobalCookSuppression)
	{
		// During cooking, we don't want to regenerate new GUIDs, since they would be different
		// every time and trigger non-detrministic cooking errors. We therefore bail out early
		// and leave the original GUID.
		//
		// However, we also check that nobody is trying to modify data after we have compiled the
		// sequence, as this would lead to a mismatch between the source data and the compiled
		// data we use to run it. We therefore check with the compiled data manager whether our
		// parent sequence has already been compiled or not.
		UMovieSceneSequence* OuterSequence = GetTypedOuter<UMovieSceneSequence>();
		if (OuterSequence)
		{
			UMovieSceneCompiledDataManager* CompiledDataManager = UMovieSceneCompiledDataManager::GetPrecompiledData();
			check(CompiledDataManager);
			const bool bCanMarkAsChanged = CompiledDataManager->CanMarkSignedObjectAsChangedDuringCook(OuterSequence);
			ensureAlwaysMsgf(bCanMarkAsChanged,
					TEXT("This object's signature was locked after its sequence was compiled, "
						 "but someone attempted to mark it, or one of its inner objects, as changed! "
						 "Current object: %s ; Current sequence: %s"),
					*GetPathName(), *OuterSequence->GetPathName());
		}
	}

	// Completely skip signature changes and notifications if we are suppressing things.
	if (GScopedSignedObjectSuppressCount > 0 || bGlobalCookSuppression)
	{
		return;
	}

	// We always change the signature immediately to ensure that any external code that wants
	// to directly check our signature (eg, to clear caches) can still do so even while there
	// is an outstanding signal pending for this object
	Signature = FGuid::NewGuid();

	// Regenerate the signature for all parents of this class
	UObject* Outer = GetOuter();
	while (Outer)
	{
		UMovieSceneSignedObject* TypedOuter = Cast<UMovieSceneSignedObject>(Outer);
		if (TypedOuter)
		{
			TypedOuter->Signature = FGuid::NewGuid();
		}
		Outer = Outer->GetOuter();
	}

	// Give the change handler an opportunity to create an implicit scope if there's
	// no explicit one active.
	if (GScopedSignedObjectDeferCount == 0 && GDeferredSignedObjectChangeHandler.IsValid())
	{
		GDeferredSignedObjectChangeHandler.Pin()->CreateImplicitScopedModifyDefer();
	}

	if (GScopedSignedObjectDeferCount == 0 || !GDeferredSignedObjectChangeHandler.IsValid())
	{
		BroadcastChanged();
		SignalScopedSignedObjectModifyFlush();
	}
	else
	{
		GDeferredSignedObjectChangeHandler.Pin()->DeferMarkAsChanged(this);
	}
}

void UMovieSceneSignedObject::BroadcastChanged()
{
	using namespace UE::MovieScene;

	EventHandlers.Trigger(&ISignedObjectEventHandler::OnModifiedDirectly, this);
	OnSignatureChangedEvent.Broadcast();

	UObject* Outer = GetOuter();
	while (Outer)
	{
		UMovieSceneSignedObject* TypedOuter = Cast<UMovieSceneSignedObject>(Outer);
		if (TypedOuter)
		{
			TypedOuter->EventHandlers.Trigger(&ISignedObjectEventHandler::OnModifiedIndirectly, this);
			TypedOuter->OnSignatureChangedEvent.Broadcast();
		}
		Outer = Outer->GetOuter();
	}
}

#if WITH_EDITOR
bool UMovieSceneSignedObject::Modify(bool bAlwaysMarkDirty)
{
	using namespace UE::MovieScene;

	bool bModified = Super::Modify(bAlwaysMarkDirty);
	if ( bAlwaysMarkDirty )
	{
		MarkAsChanged();
	}
	return bModified;
}

void UMovieSceneSignedObject::PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	MarkAsChanged();
}

void UMovieSceneSignedObject::PostEditUndo()
{
	using namespace UE::MovieScene;

	Super::PostEditUndo();
	MarkAsChanged();

	EventHandlers.Trigger(&ISignedObjectEventHandler::OnPostUndo);
}

void UMovieSceneSignedObject::PostEditUndo(TSharedPtr<ITransactionObjectAnnotation> TransactionAnnotation)
{
	using namespace UE::MovieScene;

	Super::PostEditUndo(TransactionAnnotation);
	MarkAsChanged();

	EventHandlers.Trigger(&ISignedObjectEventHandler::OnPostUndo);
}
#endif


