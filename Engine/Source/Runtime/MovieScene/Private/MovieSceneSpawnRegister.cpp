// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneSpawnRegister.h"
#include "MovieScene.h"
#include "IMovieScenePlayer.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "MovieSceneSpawnableAnnotation.h"
#include "MovieSceneBindingReferences.h"
#include "Bindings/MovieSceneSpawnableBinding.h"

UE_DEFINE_MOVIESCENE_PLAYBACK_CAPABILITY(FMovieSceneSpawnRegister)

FMovieSceneSpawnRegister::FMovieSceneSpawnRegister() = default;
FMovieSceneSpawnRegister::FMovieSceneSpawnRegister(const FMovieSceneSpawnRegister&) = default;
FMovieSceneSpawnRegister::~FMovieSceneSpawnRegister() = default;

TWeakObjectPtr<> FMovieSceneSpawnRegister::FindSpawnedObject(const FGuid& BindingId, FMovieSceneSequenceIDRef TemplateID, int BindingIndex/* = 0*/) const
{
	FMovieSceneSpawnRegisterKey Key(TemplateID, BindingId, BindingIndex);

	const FSpawnedObject* Existing = Register.Find(Key);
	return Existing ? Existing->Object : TWeakObjectPtr<>();
}

UObject* FMovieSceneSpawnRegister::SpawnObject(const FGuid& BindingId, UMovieScene& MovieScene, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState, int BindingIndex/* = 0*/)
{
	TWeakObjectPtr<> WeakObjectInstance = FindSpawnedObject(BindingId, TemplateID, BindingIndex);
	UObject*         ObjectInstance     = WeakObjectInstance.Get();

	if (ObjectInstance)
	{
		return ObjectInstance;
	}

	// Find the spawnable definition

	UObject* SpawnedActor = nullptr; 
	ESpawnOwnership SpawnOwnership = ESpawnOwnership::InnerSequence;

	UMovieSceneSequence* Sequence = SharedPlaybackState->GetSequence(TemplateID);
	if (!ensure(Sequence))
	{
		return nullptr;
	}

	// First check if we're using the old-style FMovieSceneSpawnable
	FMovieSceneSpawnable* Spawnable = MovieScene.FindSpawnable(BindingId);
	if (Spawnable)
	{
		if (WeakObjectInstance.IsStale() && !Spawnable->bContinuouslyRespawn)
		{
			return nullptr;
		}

		SpawnOwnership = Spawnable->GetSpawnOwnership();

		// Call through to the list of spawners to see who can spawn something from this FMovieSceneSpawnable
		SpawnedActor = SpawnObject(*Spawnable, TemplateID, SharedPlaybackState);
	}
	else if (UMovieSceneSequence* MovieSceneSequence = MovieScene.GetTypedOuter<UMovieSceneSequence>())
	{
		if (FMovieSceneBindingReferences* BindingReferences = MovieSceneSequence->GetBindingReferences())
		{
			if (UMovieSceneCustomBinding* CustomBinding = BindingReferences->GetCustomBinding(BindingId, BindingIndex))
			{
				if (UMovieSceneSpawnableBindingBase* SpawnableBinding = CustomBinding->AsSpawnable(SharedPlaybackState))
				{
					if (WeakObjectInstance.IsStale() && !SpawnableBinding->bContinuouslyRespawn)
					{
						return nullptr;
					}

					SpawnOwnership = SpawnableBinding->SpawnOwnership;

					// Call the Spawnable binding itself to spawn the object
					SpawnedActor = SpawnableBinding->SpawnObject(BindingId, BindingIndex, MovieScene, TemplateID, SharedPlaybackState);
				}
			}
		}
	}
	
	if (SpawnedActor)
	{
		FMovieSceneSpawnableAnnotation::Add(SpawnedActor, BindingId, TemplateID, Sequence);

		FMovieSceneSpawnRegisterKey Key(TemplateID, BindingId, BindingIndex);
		Register.Add(Key, FSpawnedObject(BindingId, *SpawnedActor, SpawnOwnership));

		if (FMovieSceneEvaluationState* State = SharedPlaybackState->FindCapability<FMovieSceneEvaluationState>())
		{
			State->Invalidate(BindingId, TemplateID);
		}
	}

	return SpawnedActor;
}

void FMovieSceneSpawnRegister::PreDestroyObject(UObject & Object, const FGuid & BindingId, FMovieSceneSequenceIDRef TemplateID)
{
	PreDestroyObject(Object, BindingId, 0, TemplateID);
}

bool FMovieSceneSpawnRegister::DestroySpawnedObject(const FGuid& BindingId, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState, int BindingIndex/* = 0*/)
{
	TGuardValue<bool> CleaningUp(bCleaningUp, true);

	FMovieSceneSpawnRegisterKey Key(TemplateID, BindingId, BindingIndex);
	
	FSpawnedObject* Existing = Register.Find(Key);
	UObject* SpawnedObject = Existing ? Existing->Object.Get() : nullptr;
	if (SpawnedObject)
	{
		PreDestroyObject(*SpawnedObject, BindingId, BindingIndex, TemplateID);

		bool bCustomBinding = false;
		// If we have a custom binding, it will handle object destruction
		if (FMovieSceneEvaluationState* State = SharedPlaybackState->FindCapability<FMovieSceneEvaluationState>())
		{
			if (UMovieSceneSequence* MovieSceneSequence = State->FindSequence(TemplateID))
			{
				if (FMovieSceneBindingReferences* BindingReferences = MovieSceneSequence->GetBindingReferences())
				{
					if (UMovieSceneCustomBinding* CustomBinding = BindingReferences->GetCustomBinding(BindingId, BindingIndex))
					{
						if (UMovieSceneSpawnableBindingBase* SpawnableBinding = CustomBinding->AsSpawnable(SharedPlaybackState))
						{
							bCustomBinding = true;
							SpawnableBinding->PreDestroyObject(SpawnedObject, BindingId, BindingIndex, TemplateID);
							DestroySpawnedObject(*SpawnedObject, SpawnableBinding);
						}
					}
				}
			}
		}

		if (!bCustomBinding)
		{
			DestroySpawnedObject(*SpawnedObject, nullptr);
		}
	}

	Register.Remove(Key);

	if (FMovieSceneEvaluationState* State = SharedPlaybackState->FindCapability<FMovieSceneEvaluationState>())
	{
		State->Invalidate(BindingId, TemplateID);
	}

	return SpawnedObject != nullptr;
}

void FMovieSceneSpawnRegister::DestroyObjectsByPredicate(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, const TFunctionRef<bool(const FGuid&, ESpawnOwnership, FMovieSceneSequenceIDRef)>& Predicate)
{
	DestroyObjectsByPredicate(SharedPlaybackState, [&Predicate](const FGuid& Guid, ESpawnOwnership SpawnOwnership, FMovieSceneSequenceIDRef SequenceID, int32 BindingIndex) {
		return Predicate(Guid, SpawnOwnership, SequenceID);
		});
}

void FMovieSceneSpawnRegister::DestroyObjectsByPredicate(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, const TFunctionRef<bool(const FGuid&, ESpawnOwnership, FMovieSceneSequenceIDRef, int32)>& Predicate)
{
	for (auto It = Register.CreateIterator(); It; ++It)
	{
		if (Predicate(It.Value().Guid, It.Value().Ownership, It.Key().TemplateID, It.Key().BindingIndex))
		{
			UObject* SpawnedObject = It.Value().Object.Get();
			if (SpawnedObject)
			{
				bool bCustomBinding = false;
				// If we have a custom binding, it will handle object destruction
				if (FMovieSceneEvaluationState* State = SharedPlaybackState->FindCapability<FMovieSceneEvaluationState>())
				{
					if (UMovieSceneSequence* MovieSceneSequence = State->FindSequence(It.Key().TemplateID))
					{
						if (FMovieSceneBindingReferences* BindingReferences = MovieSceneSequence->GetBindingReferences())
						{
							if (UMovieSceneCustomBinding* CustomBinding = BindingReferences->GetCustomBinding(It.Key().BindingId, It.Key().BindingIndex))
							{
								if (UMovieSceneSpawnableBindingBase* SpawnableBinding = CustomBinding->AsSpawnable(SharedPlaybackState))
								{
									bCustomBinding = true;
									SpawnableBinding->PreDestroyObject(SpawnedObject, It.Key().BindingId, It.Key().BindingIndex, It.Key().TemplateID);
									PreDestroyObject(*SpawnedObject, It.Key().BindingId, 0, It.Key().TemplateID); 
									DestroySpawnedObject(*SpawnedObject, SpawnableBinding);
								}
							}
						}
					}
				}

				if (!bCustomBinding)
				{
					PreDestroyObject(*SpawnedObject, It.Key().BindingId, 0, It.Key().TemplateID);
					DestroySpawnedObject(*SpawnedObject, nullptr);
				}
			}

			It.RemoveCurrent();
		}
	}
}

void FMovieSceneSpawnRegister::ForgetExternallyOwnedSpawnedObjects(TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	FMovieSceneEvaluationState* State = SharedPlaybackState->FindCapability<FMovieSceneEvaluationState>();
	for (auto It = Register.CreateIterator(); It; ++It)
	{
		if (It.Value().Ownership == ESpawnOwnership::External)
		{
			if (State)
			{
				State->Invalidate(It.Key().BindingId, It.Key().TemplateID);
			}
			It.RemoveCurrent();
		}
	}
}

void FMovieSceneSpawnRegister::CleanUp(TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	TGuardValue<bool> CleaningUp(bCleaningUp, true);

	DestroyObjectsByPredicate(SharedPlaybackState, [&](const FGuid&, ESpawnOwnership, FMovieSceneSequenceIDRef){
		return true;
	});
}

void FMovieSceneSpawnRegister::CleanUpSequence(FMovieSceneSequenceIDRef TemplateID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	DestroyObjectsByPredicate(SharedPlaybackState, [&](const FGuid&, ESpawnOwnership, FMovieSceneSequenceIDRef ThisTemplateID){
		return ThisTemplateID == TemplateID;
	});
}

void FMovieSceneSpawnRegister::OnSequenceExpired(FMovieSceneSequenceIDRef TemplateID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	DestroyObjectsByPredicate(SharedPlaybackState, [&](const FGuid& ObjectId, ESpawnOwnership Ownership, FMovieSceneSequenceIDRef ThisTemplateID){
		return (Ownership == ESpawnOwnership::InnerSequence) && (TemplateID == ThisTemplateID);
	});
}

// Deprecated method redirects

UObject* FMovieSceneSpawnRegister::SpawnObject(const FGuid& BindingId, UMovieScene& MovieScene, FMovieSceneSequenceIDRef Template, IMovieScenePlayer& Player)
{
	return SpawnObject(BindingId, MovieScene, Template, Player.GetSharedPlaybackState());
}

bool FMovieSceneSpawnRegister::DestroySpawnedObject(const FGuid& BindingId, FMovieSceneSequenceIDRef TemplateID, IMovieScenePlayer& Player)
{
	return DestroySpawnedObject(BindingId, TemplateID, Player.GetSharedPlaybackState());
}

void FMovieSceneSpawnRegister::DestroyObjectsByPredicate(IMovieScenePlayer& Player, const TFunctionRef<bool(const FGuid&, ESpawnOwnership, FMovieSceneSequenceIDRef)>& Predicate)
{
	DestroyObjectsByPredicate(Player.GetSharedPlaybackState(), Predicate);
}

void FMovieSceneSpawnRegister::ForgetExternallyOwnedSpawnedObjects(FMovieSceneEvaluationState& State, IMovieScenePlayer& Player)
{
	ForgetExternallyOwnedSpawnedObjects(Player.GetSharedPlaybackState());
}

void FMovieSceneSpawnRegister::CleanUp(IMovieScenePlayer& Player)
{
	CleanUp(Player.GetSharedPlaybackState());
}

void FMovieSceneSpawnRegister::CleanUpSequence(FMovieSceneSequenceIDRef TemplateID, IMovieScenePlayer& Player)
{
	CleanUpSequence(TemplateID, Player.GetSharedPlaybackState());
}

void FMovieSceneSpawnRegister::OnSequenceExpired(FMovieSceneSequenceIDRef TemplateID, IMovieScenePlayer& Player)
{
	OnSequenceExpired(TemplateID, Player.GetSharedPlaybackState());
}

#if WITH_EDITOR

void FMovieSceneSpawnRegister::SaveDefaultSpawnableState(FMovieSceneSpawnable& Spawnable, FMovieSceneSequenceIDRef TemplateID, IMovieScenePlayer& Player)
{
	SaveDefaultSpawnableState(Spawnable.GetGuid(), TemplateID, Player.GetSharedPlaybackState());
}

void FMovieSceneSpawnRegister::SaveDefaultSpawnableState(FMovieSceneSpawnable& Spawnable, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	SaveDefaultSpawnableState(Spawnable.GetGuid(), TemplateID, SharedPlaybackState);
}

void FMovieSceneSpawnRegister::HandleConvertPossessableToSpawnable(UObject* OldObject, IMovieScenePlayer& Player, TOptional<FTransformData>& OutTransformData)
{
	HandleConvertPossessableToSpawnable(OldObject, Player.GetSharedPlaybackState(), OutTransformData);
}

UObject* FMovieSceneSpawnRegister::SpawnObject(FMovieSceneSpawnable& Spawnable, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	return SpawnObject(Spawnable.GetGuid(), *SharedPlaybackState->GetSequence(TemplateID)->GetMovieScene(), TemplateID, SharedPlaybackState);
}

UObject* FMovieSceneSpawnRegister::SpawnObject(FMovieSceneSpawnable& Spawnable, FMovieSceneSequenceIDRef TemplateID, IMovieScenePlayer& Player)
{
	return SpawnObject(Spawnable, TemplateID, Player.GetSharedPlaybackState());
}

bool FMovieSceneSpawnRegister::CanConvertToPossessable(const FGuid& Guid, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState, int32 BindingIndex) const
{
	if (UMovieSceneSequence* Sequence = SharedPlaybackState->GetSequence(TemplateID))
	{
		if (UMovieScene* MovieScene = Sequence->GetMovieScene())
		{
			if (FMovieSceneSpawnable* Spawnable = MovieScene->FindSpawnable(Guid))
			{
				return true;
			}
			else if (const FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences())
			{
				if (const UMovieSceneCustomBinding* CustomBinding = BindingReferences->GetCustomBinding(Guid, BindingIndex))
				{
					return CustomBinding->CanConvertToPossessable(Guid, TemplateID, SharedPlaybackState);
				}
			}
		}
	}
	return false;
}

#endif

