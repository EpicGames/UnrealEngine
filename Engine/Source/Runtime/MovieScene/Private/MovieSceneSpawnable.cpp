// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneSpawnable.h"
#include "UObject/UObjectAnnotation.h"
#include "MovieSceneSequence.h"
#include "MovieScene.h"
#include "IMovieScenePlayer.h"
#include "Misc/StringBuilder.h"
#include "GameFramework/Actor.h"
#include "Components/StaticMeshComponent.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSpawnable)

struct FIsSpawnable
{
	FIsSpawnable() : bIsSpawnable(false) {}
	explicit FIsSpawnable(bool bInIsSpawnable) : bIsSpawnable(bInIsSpawnable) {}

	bool IsDefault() const { return !bIsSpawnable; }

	bool bIsSpawnable;
};

static FUObjectAnnotationSparse<FIsSpawnable,true> SpawnablesAnnotation;

bool FMovieSceneSpawnable::IsSpawnableTemplate(const UObject& InObject)
{
	return !SpawnablesAnnotation.GetAnnotation(&InObject).IsDefault();
}

void FMovieSceneSpawnable::MarkSpawnableTemplate(const UObject& InObject)
{
	SpawnablesAnnotation.AddAnnotation(&InObject, FIsSpawnable(true));
}

void FMovieSceneSpawnable::CopyObjectTemplate(UObject& InSourceObject, UMovieSceneSequence& MovieSceneSequence)
{
	const FName ObjectName = ObjectTemplate ? ObjectTemplate->GetFName() : InSourceObject.GetFName();

	if (ObjectTemplate)
	{
		ObjectTemplate->Rename(*MakeUniqueObjectName(MovieSceneSequence.GetMovieScene(), ObjectTemplate->GetClass(), "ExpiredSpawnable").ToString());
		ObjectTemplate->MarkAsGarbage();
		ObjectTemplate = nullptr;
	}

	ObjectTemplate = MovieSceneSequence.MakeSpawnableTemplateFromInstance(InSourceObject, ObjectName);

	check(ObjectTemplate);

	MarkSpawnableTemplate(*ObjectTemplate);
	MovieSceneSequence.MarkPackageDirty();
}

FName FMovieSceneSpawnable::GetNetAddressableName(IMovieScenePlayer& Player, FMovieSceneSequenceID SequenceID) const
{
	return GetNetAddressableName(Player.GetSharedPlaybackState(), SequenceID);
}

FName FMovieSceneSpawnable::GetNetAddressableName(TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, FMovieSceneSequenceID SequenceID) const
{
	UObject* AddressingContext = nullptr;

	if (IMovieScenePlayer* Player = UE::MovieScene::FPlayerIndexPlaybackCapability::GetPlayer(SharedPlaybackState))
	{
		AddressingContext = Player->AsUObject();
	}
	if (!AddressingContext)
	{
		if (UObject* PlaybackContext = SharedPlaybackState->GetPlaybackContext())
		{
			AddressingContext = PlaybackContext;
		}
	}

	if (!AddressingContext)
	{
		return NAME_None;
	}

	TStringBuilder<128> AddressableName;

	// Spawnable name
	AddressableName.Append(*Name, Name.Len());

	// SequenceID
	AddressableName.Appendf(TEXT("_0x%08X"), SequenceID.GetInternalValue());

	// Spawnable GUID
	AddressableName.Appendf(TEXT("_%08X%08X%08X%08X"), Guid.A, Guid.B, Guid.C, Guid.D);

	// Actor / player Name
	if (AActor* OuterActor = AddressingContext->GetTypedOuter<AActor>())
	{
		AddressableName.AppendChar(TEXT('_'));
		OuterActor->GetFName().AppendString(AddressableName);
	}
	else
	{
		AddressableName.AppendChar(TEXT('_'));
		AddressingContext->GetFName().AppendString(AddressableName);
	}

	return FName(AddressableName.Len(), AddressableName.GetData());
}

void FMovieSceneSpawnable::AutoSetNetAddressableName()
{
	bNetAddressableName = false;

	AActor* Actor = Cast<AActor>(ObjectTemplate);
	if (Actor && Actor->FindComponentByClass<UStaticMeshComponent>() != nullptr)
	{
		bNetAddressableName = true;
	}
}
