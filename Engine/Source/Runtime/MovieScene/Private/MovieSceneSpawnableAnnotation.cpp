// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneSpawnableAnnotation.h"
#include "UObject/UObjectAnnotation.h"

static FUObjectAnnotationSparse<FMovieSceneSpawnableAnnotation,true> SpawnedObjectAnnotation;

void FMovieSceneSpawnableAnnotation::Add(UObject* SpawnedObject, const FGuid& ObjectBindingID, FMovieSceneSequenceID SequenceID, UMovieSceneSequence* InOriginatingSequence)
{
	if (SpawnedObject)
	{
		FMovieSceneSpawnableAnnotation Annotation;
		Annotation.ObjectBindingID = ObjectBindingID;
		Annotation.SequenceID = SequenceID;
		Annotation.OriginatingSequence = InOriginatingSequence;

		SpawnedObjectAnnotation.AddAnnotation(SpawnedObject, MoveTemp(Annotation));
	}
}

TOptional<FMovieSceneSpawnableAnnotation> FMovieSceneSpawnableAnnotation::Find(UObject* SpawnedObject)
{
	const FMovieSceneSpawnableAnnotation& Annotation = SpawnedObjectAnnotation.GetAnnotation(SpawnedObject);

	TOptional<FMovieSceneSpawnableAnnotation> ReturnValue;
	if (!Annotation.IsDefault())
	{
		ReturnValue = Annotation;
	}

	return ReturnValue;
}