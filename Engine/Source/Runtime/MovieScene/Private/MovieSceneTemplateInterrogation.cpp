// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compilation/MovieSceneTemplateInterrogation.h"
#include "Evaluation/Blending/MovieSceneBlendingAccumulator.h"

void FMovieSceneInterrogationData::Finalize(const FMovieSceneContext& Context, UObject* BindingOverride)
{
	if (Accumulator.IsValid())
	{
		Accumulator->Interrogate(Context, *this, BindingOverride);
	}
}

FMovieSceneBlendingAccumulator& FMovieSceneInterrogationData::GetAccumulator()
{
	if (!Accumulator.IsValid())
	{
		Accumulator = MakeShared<FMovieSceneBlendingAccumulator>();
	}

	return *Accumulator;
}

FMovieSceneInterrogationKey FMovieSceneInterrogationKey::GetTransformInterrogationKey()
{
	static FMovieSceneAnimTypeID TypeID = FMovieSceneAnimTypeID::Unique();
	return TypeID;
}
