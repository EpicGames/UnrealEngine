// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneTimeController.h"
#include "Engine/World.h"
#include "Engine/Engine.h"
#include "Engine/TimecodeProvider.h"
#include "AudioDevice.h"
#include "Misc/App.h"

static TAutoConsoleVariable<int32> CVarSequencerRelativeTimecodeSmoothing(
	TEXT("Sequencer.RelativeTimecodeSmoothing"),
	1,
	TEXT("If nonzero, accumulate with platform time since when the timecodes were equal."),
	ECVF_Default);

static TAutoConsoleVariable<float> CVarSequencerSecondsPerFrame(
	TEXT("Sequencer.SecondsPerFrame"),
	1.f,
	TEXT("Seconds per frame to wait when in play every frame mode."),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarSequencerAllowPausedClock(
	TEXT("Sequencer.AllowPausedClock"),
	true,
	TEXT("Allow game pause to pause the sequencer clock."),
	ECVF_Default);

void FMovieSceneTimeController::Tick(float DeltaSeconds, float InPlayRate)
{
	OnTick(DeltaSeconds, InPlayRate);
}

void FMovieSceneTimeController::Reset(const FQualifiedFrameTime& InStartTime)
{
	if (PlaybackStartTime.IsSet())
	{
		StopPlaying(InStartTime);
		StartPlaying(InStartTime);
	}
}

void FMovieSceneTimeController::PlayerStatusChanged(EMovieScenePlayerStatus::Type InStatus, const FQualifiedFrameTime& InCurrentTime)
{
	if (PlaybackStartTime.IsSet() && InStatus != EMovieScenePlayerStatus::Playing)
	{
		StopPlaying(InCurrentTime);
	}

	if (!PlaybackStartTime.IsSet() && InStatus == EMovieScenePlayerStatus::Playing)
	{
		StartPlaying(InCurrentTime);
	}
}

void FMovieSceneTimeController::PrepareToPlay(const FQualifiedFrameTime& InStartTime)
{
	UE_LOGF(LogMovieScene, VeryVerbose, "TimeController PrepareToPlay: Sequence started preparing: frame %d, subframe %f. Frame rate: %f fps.", InStartTime.Time.FrameNumber.Value, InStartTime.Time.GetSubFrame(), InStartTime.Rate.AsDecimal());

	OnPrepareToPlay(InStartTime);
}

bool FMovieSceneTimeController::IsReadyToPlay()
{
	return OnIsReadyToPlay();
}

void FMovieSceneTimeController::StartPlaying(const FQualifiedFrameTime& InStartTime)
{
	UE_LOGF(LogMovieScene, VeryVerbose, "TimeController Start: Sequence started: frame %d, subframe %f. Frame rate: %f fps.", InStartTime.Time.FrameNumber.Value, InStartTime.Time.GetSubFrame(), InStartTime.Rate.AsDecimal());

	PlaybackStartTime = InStartTime;
	OnStartPlaying(InStartTime);
}

void FMovieSceneTimeController::StopPlaying(const FQualifiedFrameTime& InStopTime)
{
	UE_LOGF(LogMovieScene, VeryVerbose, "TimeController Start: Sequence stopped.");

	OnStopPlaying(InStopTime);
	PlaybackStartTime.Reset();
}

FFrameTime FMovieSceneTimeController::RequestCurrentTime(const FQualifiedFrameTime& InCurrentTime, float InPlayRate, FFrameRate InDisplayRate)
{
	DisplayRate = InDisplayRate;
	return OnRequestCurrentTime(InCurrentTime, InPlayRate);
}

void FMovieSceneTimeController_ExternalClock::OnStartPlaying(const FQualifiedFrameTime& InStartTime)
{
	ClockStartTime = ClockLastUpdateTime = GetCurrentTime();
}

void FMovieSceneTimeController_ExternalClock::OnStopPlaying(const FQualifiedFrameTime& InStopTime)
{
	ClockLastUpdateTime.Reset();
	ClockStartTime.Reset();
	AccumulatedDilation = 0.0;
}

FFrameTime FMovieSceneTimeController_ExternalClock::OnRequestCurrentTime(const FQualifiedFrameTime& InCurrentTime, float InPlayRate)
{
	TOptional<FQualifiedFrameTime> StartTimeIfPlaying = GetPlaybackStartTime();
	if (!StartTimeIfPlaying.IsSet())
	{
		return InCurrentTime.Time;
	}

	double StartedTime = ClockStartTime.GetValue();
	double CurrentTime = GetCurrentTime();

	double UndilatedDeltaTime = (CurrentTime - StartedTime);

	AccumulatedDilation += (InPlayRate - 1.0) * (CurrentTime - ClockLastUpdateTime.GetValue());

	ClockLastUpdateTime = CurrentTime;

	double PausedTime = 0;
	if (CVarSequencerAllowPausedClock->GetBool())
	{
		const TIndirectArray<FWorldContext>& WorldContexts = GEngine->GetWorldContexts();
		for (const FWorldContext& Context : WorldContexts)
		{	
			if (((Context.WorldType == EWorldType::PIE) || (Context.WorldType == EWorldType::Game)) && (Context.World() != NULL))
			{
				PausedTime = Context.World()->RealTimeSeconds - Context.World()->AudioTimeSeconds;
				break;
			}
		}
	}

	double CurrentSequenceTimeSeconds = UndilatedDeltaTime + AccumulatedDilation - PausedTime;

	FFrameTime StartTime = StartTimeIfPlaying->ConvertTo(InCurrentTime.Rate);
	FFrameTime NewTime   = StartTime + CurrentSequenceTimeSeconds * InCurrentTime.Rate;

#if !NO_LOGGING
	UE_LOGF(LogMovieScene, VeryVerbose, "TimeController Clock Start Time: %f, Clock Now: %f, Dilation Offset: %f, Sequence Start Time: frame %d, subframe %f, Sequence Offset Seconds: %f, Sequence Now: frame %d, subframe %f",
		StartedTime, CurrentTime, AccumulatedDilation, StartTime.FrameNumber.Value, StartTime.GetSubFrame(), CurrentSequenceTimeSeconds, NewTime.FrameNumber.Value, NewTime.GetSubFrame());
#endif

	return NewTime;
}

double FMovieSceneTimeController_PlatformClock::GetCurrentTime() const
{
	return FPlatformTime::Seconds();
}

double FMovieSceneTimeController_AudioClock::GetCurrentTime() const
{
	FAudioDevice* AudioDevice = GEngine ? GEngine->GetMainAudioDevice().GetAudioDevice() : nullptr;
	return AudioDevice ? AudioDevice->GetInterpolatedAudioClock() : FPlatformTime::Seconds();
}

void FMovieSceneTimeController_RelativeTimecodeClock::OnStopPlaying(const FQualifiedFrameTime& InStopTime)
{
	FMovieSceneTimeController_ExternalClock::OnStopPlaying(InStopTime);

	AccumulatedFrameTime = 0;
	LastCurrentFrameTime.Reset();
	TimeSinceCurrentFrameTime.Reset();
}

double FMovieSceneTimeController_RelativeTimecodeClock::GetCurrentTime() const
{
	const TOptional<FQualifiedFrameTime> CurrentFrameTime = FApp::GetCurrentFrameTime();
	if (CurrentFrameTime.IsSet())
	{
		return CurrentFrameTime.GetValue().AsSeconds() + AccumulatedFrameTime;
	}
	else
	{
		return FPlatformTime::Seconds();
	}
}

FFrameTime FMovieSceneTimeController_RelativeTimecodeClock::OnRequestCurrentTime(const FQualifiedFrameTime& InCurrentTime, float InPlayRate)
{
	TOptional<FQualifiedFrameTime> CurrentFrameTime = FApp::GetCurrentFrameTime();

	// If the engine tick rate is faster than the timecode, there could be multiple ticks with the same timecode. 
	// That's not necessarily desirable, so accumulate with platform time since when the timecodes were equal.
	AccumulatedFrameTime = 0;
	if (CVarSequencerRelativeTimecodeSmoothing->GetInt() && CurrentFrameTime.IsSet())
	{
		if (LastCurrentFrameTime.IsSet() && LastCurrentFrameTime.GetValue() == CurrentFrameTime.GetValue().Time)
		{
			if (TimeSinceCurrentFrameTime.IsSet())
			{
				AccumulatedFrameTime = FPlatformTime::Seconds() - TimeSinceCurrentFrameTime.GetValue();
			}
			else
			{
				TimeSinceCurrentFrameTime = FPlatformTime::Seconds();
			}
		}
		else
		{
			TimeSinceCurrentFrameTime = FPlatformTime::Seconds();
		}
		
		LastCurrentFrameTime = CurrentFrameTime.GetValue().Time;
	}

	return FMovieSceneTimeController_ExternalClock::OnRequestCurrentTime(InCurrentTime, InPlayRate);
}

FFrameTime FMovieSceneTimeController_TimecodeClock::OnRequestCurrentTime(const FQualifiedFrameTime& InCurrentTime, float InPlayRate)
{
	if (GEngine && GEngine->GetTimecodeProvider() && GEngine->GetTimecodeProvider()->GetSynchronizationState() == ETimecodeProviderSynchronizationState::Synchronized)
	{
		FTimecode Timecode = FApp::GetTimecode();
		FFrameRate FrameRate = FApp::GetTimecodeFrameRate();

		// Convert timecode to raw number of frames at the timecode's framerate.
		FFrameNumber FrameNumber = Timecode.ToFrameNumber(FrameRate);
		return FFrameRate::TransformTime(FFrameTime(FrameNumber), FrameRate, InCurrentTime.Rate);
	}

	return FFrameTime(0);
}

void FMovieSceneTimeController_Tick::OnStartPlaying(const FQualifiedFrameTime& InStartTime)
{
	CurrentOffsetSeconds = 0.0;
}

void FMovieSceneTimeController_Tick::OnTick(float DeltaSeconds, float InPlayRate)
{
	CurrentOffsetSeconds += DeltaSeconds * InPlayRate;
}

FFrameTime FMovieSceneTimeController_Tick::OnRequestCurrentTime(const FQualifiedFrameTime& InCurrentTime, float InPlayRate)
{
	TOptional<FQualifiedFrameTime> StartTimeIfPlaying = GetPlaybackStartTime();
	if (!StartTimeIfPlaying.IsSet())
	{
		return InCurrentTime.Time;
	}
	else
	{
		FFrameTime StartTime = StartTimeIfPlaying->ConvertTo(InCurrentTime.Rate);
		return StartTime + CurrentOffsetSeconds * InCurrentTime.Rate;
	}
}

void FMovieSceneTimeController_PlayEveryFrame::OnStartPlaying(const FQualifiedFrameTime& InStartTime)
{
	PreviousPlatformTime = FPlatformTime::Seconds();
	CurrentTime = InStartTime.Time;
}

FFrameTime FMovieSceneTimeController_PlayEveryFrame::OnRequestCurrentTime(const FQualifiedFrameTime& InCurrentTime, float InPlayRate)
{
	double TimeDiff = FPlatformTime::Seconds() - PreviousPlatformTime;
	if (TimeDiff > CVarSequencerSecondsPerFrame->GetFloat())
	{
		// Update current time by moving onto the next frame
		FFrameRate CurrentDisplayRate = GetDisplayRate();
		FFrameTime OneDisplayFrame = CurrentDisplayRate.AsFrameTime(CurrentDisplayRate.AsInterval());
		CurrentTime = CurrentTime + ConvertFrameTime(OneDisplayFrame, CurrentDisplayRate, InCurrentTime.Rate);
		PreviousPlatformTime = FPlatformTime::Seconds();
	}
	return CurrentTime;
}