// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneTimeHelpers.h"
#include "CoreTypes.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "Misc/ScopedSlowTask.h"
#include "Sections/MovieSceneSubSection.h"
#include "Sections/MovieSceneSubSection.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "MovieScene.h"
#include "MovieSceneSection.h"
#include "MovieSceneTrack.h"
#include "MovieSceneSequence.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneChannel.h"
#include "IMovieSceneRetimingInterface.h"

namespace UE
{
namespace MovieScene
{

TRange<FFrameNumber> MigrateFrameRange(const TRange<FFrameNumber>& SourceRange, const IRetimingInterface& RetimingInterface)
{
	TRange<FFrameNumber> NewRange = TRange<FFrameNumber>::All();

	if (!SourceRange.GetLowerBound().IsOpen())
	{
		const FFrameNumber FrameNumber = RetimingInterface.RemapTime(SourceRange.GetLowerBoundValue());

		NewRange.SetLowerBound(
			SourceRange.GetLowerBound().IsExclusive()
			? TRangeBound<FFrameNumber>::Exclusive(FrameNumber)
			: TRangeBound<FFrameNumber>::Inclusive(FrameNumber)
		);
	}

	if (!SourceRange.GetUpperBound().IsOpen())
	{
		const FFrameNumber FrameNumber = RetimingInterface.RemapTime(SourceRange.GetUpperBoundValue());

		NewRange.SetUpperBound(
			SourceRange.GetUpperBound().IsExclusive()
			? TRangeBound<FFrameNumber>::Exclusive(FrameNumber)
			: TRangeBound<FFrameNumber>::Inclusive(FrameNumber)
		);
	}

	return NewRange;
}

void MigrateFrameTimes(const IRetimingInterface& Retimer, UMovieSceneSection* Section)
{
	Section->Modify();
	const bool bSectionWasLocked = Section->IsLocked();
	Section->SetIsLocked(false);

	TRangeBound<FFrameNumber> NewLowerBound, NewUpperBound;

	if (Section->HasStartFrame())
	{
		FFrameNumber NewLowerBoundFrame = Retimer.RemapTime(Section->GetInclusiveStartFrame());
		NewLowerBound = TRangeBound<FFrameNumber>::Inclusive(NewLowerBoundFrame);
	}

	if (Section->HasEndFrame())
	{
		FFrameNumber NewUpperBoundValue = Retimer.RemapTime(Section->GetExclusiveEndFrame());
		NewUpperBound = TRangeBound<FFrameNumber>::Exclusive(NewUpperBoundValue);
	}

	Section->SetRange(TRange<FFrameNumber>(NewLowerBound, NewUpperBound));

	if (Section->GetPreRollFrames() > 0)
	{
		FFrameNumber NewPreRollFrameCount = Retimer.RemapTime(FFrameNumber(Section->GetPreRollFrames()));
		Section->SetPreRollFrames(NewPreRollFrameCount.Value);
	}

	if (Section->GetPostRollFrames() > 0)
	{
		FFrameNumber NewPostRollFrameCount = Retimer.RemapTime(FFrameNumber(Section->GetPostRollFrames()));
		Section->SetPostRollFrames(NewPostRollFrameCount.Value);
	}

	Section->MigrateFrameTimes(Retimer);

	Section->Easing.AutoEaseInDuration    = Retimer.RemapTime(FFrameNumber(Section->Easing.AutoEaseInDuration)).Value;
	Section->Easing.AutoEaseOutDuration   = Retimer.RemapTime(FFrameNumber(Section->Easing.AutoEaseOutDuration)).Value;
	Section->Easing.ManualEaseInDuration  = Retimer.RemapTime(FFrameNumber(Section->Easing.ManualEaseInDuration)).Value;
	Section->Easing.ManualEaseOutDuration = Retimer.RemapTime(FFrameNumber(Section->Easing.ManualEaseOutDuration)).Value;

	for (const FMovieSceneChannelEntry& Entry : Section->GetChannelProxy().GetAllEntries())
	{
		for (FMovieSceneChannel* Channel : Entry.GetChannels())
		{
			Channel->RemapTimes(Retimer);
		}
	}

	Section->SetIsLocked(bSectionWasLocked);
}

void MigrateFrameTimes(const IRetimingInterface& Retimer, UMovieSceneTrack* Track)
{
	FScopedSlowTask SlowTask(Track->GetAllSections().Num());

	for (UMovieSceneSection* Section : Track->GetAllSections())
	{
		SlowTask.EnterProgressFrame();
		MigrateFrameTimes(Retimer, Section);
	}
}

void TimeHelpers::MigrateFrameTimes(FFrameRate SourceRate, FFrameRate DestinationRate, UMovieScene* MovieScene, bool bApplyRecursively)
{
	FFrameRateRetiming Retimer(SourceRate, DestinationRate);
	MigrateFrameTimes(Retimer, MovieScene, bApplyRecursively);
}

void TimeHelpers::MigrateFrameTimes(const IRetimingInterface& Retimer, UMovieScene* MovieScene, bool bApplyRecursively)
{
	MovieScene->Modify();
#if WITH_EDITOR
	const bool bMovieSceneReadOnly = MovieScene->IsReadOnly();
	MovieScene->SetReadOnly(false);
#endif

	Retimer.Begin(MovieScene);

	int32 TotalNumTracks = MovieScene->GetTracks().Num() + (MovieScene->GetCameraCutTrack() ? 1 : 0);
	for (const FMovieSceneBinding& Binding : ((const UMovieScene*)MovieScene)->GetBindings())
	{
		TotalNumTracks += Binding.GetTracks().Num();
	}

	FScopedSlowTask SlowTask(TotalNumTracks, NSLOCTEXT("MovieScene", "ChangingTickResolution", "Migrating sequence frame timing"));
	SlowTask.MakeDialogDelayed(0.25f, true);

	MovieScene->SetPlaybackRange(MigrateFrameRange(MovieScene->GetPlaybackRange(), Retimer));
#if WITH_EDITORONLY_DATA
	MovieScene->SetSelectionRange(MigrateFrameRange(MovieScene->GetSelectionRange(), Retimer));
#endif

	for (UMovieSceneTrack* Track : MovieScene->GetTracks())
	{
		SlowTask.EnterProgressFrame();
		UE::MovieScene::MigrateFrameTimes(Retimer, Track);

		// We iterate through recursively here (and not in MigrateFrameTimes) so that the movie scene is taken
		// into account for locking/modifying/etc.
		if (bApplyRecursively && Track->IsA<UMovieSceneSubTrack>())
		{
			for (UMovieSceneSection* Section : Track->GetAllSections())
			{
				UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section);
				if (SubSection)
				{
					if (UMovieSceneSequence* SubSequence = SubSection->GetSequence())
					{
						UMovieScene* ChildMovieScene = SubSequence->GetMovieScene();
						if (ChildMovieScene)
						{
							TUniquePtr<IRetimingInterface> ChildRetimer = Retimer.RecurseInto(ChildMovieScene);
							if (ChildRetimer)
							{
								TimeHelpers::MigrateFrameTimes(*ChildRetimer, ChildMovieScene, bApplyRecursively);
							}
						}
					}
				}
			}
		}
	}

	if (UMovieSceneTrack* Track = MovieScene->GetCameraCutTrack())
	{
		SlowTask.EnterProgressFrame();
		UE::MovieScene::MigrateFrameTimes(Retimer, Track);
	}

	for (const FMovieSceneBinding& Binding : ((const UMovieScene*)MovieScene)->GetBindings())
	{
		for (UMovieSceneTrack* Track : Binding.GetTracks())
		{
			SlowTask.EnterProgressFrame();
			UE::MovieScene::MigrateFrameTimes(Retimer, Track);
		}
	}

	{
		TArray<FMovieSceneMarkedFrame> MarkedFrames = MovieScene->GetMarkedFrames();

		// Clear the marked frames as the returned array is immutable
		MovieScene->DeleteMarkedFrames();

		for (FMovieSceneMarkedFrame& MarkedFrame : MarkedFrames)
		{
			MarkedFrame.FrameNumber = Retimer.RemapTime(MarkedFrame.FrameNumber);

			// Add it back in
			MovieScene->AddMarkedFrame(MarkedFrame);
		}

		// Ensure they're in order as they may not have been before.
		MovieScene->SortMarkedFrames();
	}

	Retimer.End(MovieScene);

#if WITH_EDITOR
	MovieScene->SetReadOnly(bMovieSceneReadOnly);
#endif
}

} // namespace MovieScene
} // namespace UE