// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneTrack.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "Containers/SortedMap.h"

#include "MovieSceneTimeHelpers.h"
#include "Evaluation/MovieSceneSegment.h"
#include "Compilation/MovieSceneSegmentCompiler.h"
#include "Compilation/MovieSceneCompilerRules.h"
#include "Compilation/MovieSceneEvaluationTreePopulationRules.h"
#include "Compilation/IMovieSceneTemplateGenerator.h"

#include "Decorations/IMovieSceneTrackDecoration.h"
#include "Decorations/IMovieSceneSectionDecoration.h"
#include "Decorations/IMovieSceneLifetimeDecoration.h"
#include "Decorations/MovieSceneTrackRowDecoration.h"

#include "Evaluation/MovieSceneEvaluationTrack.h"
#include "Evaluation/MovieSceneEvaluationTemplate.h"

#include "Evaluation/MovieSceneEvaluationCustomVersion.h"

#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneChannel.h"
#include "Compilation/IMovieSceneTrackTemplateProducer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneTrack)

int32 GMovieSceneRemoveMutedTracksOnCook = 0;
static FAutoConsoleVariableRef CVarMovieSceneRemoveMutedTracksOnCook(
	TEXT("MovieScene.RemoveMutedTracksOnCook"),
	GMovieSceneRemoveMutedTracksOnCook,
	TEXT("If 1 remove muted tracks on cook, otherwise leave as is."),
	ECVF_Default);

UMovieSceneTrack::UMovieSceneTrack(const FObjectInitializer& InInitializer)
	: Super(InInitializer)
{
#if WITH_EDITORONLY_DATA
	TrackTint = FColor(127, 127, 127, 0);
	SortingOrder = -1;
	bSupportsDefaultSections = true;
	bSupportsConditions = true;
#endif

	BuiltInTreePopulationMode = ETreePopulationMode::HighPassPerRow;
}

void UMovieSceneTrack::PostInitProperties()
{
	SetFlags(RF_Transactional);

	// Propagate sub object flags from our outer (movie scene) to ourselves. This is required for tracks that are stored on blueprints (archetypes) so that they can be referenced in worlds.
	if (GetOuter()->HasAnyFlags(RF_ClassDefaultObject|RF_ArchetypeObject))
	{
		SetFlags(GetOuter()->GetMaskedFlags(RF_PropagateToSubObjects));
	}

	Super::PostInitProperties();
}

void UMovieSceneTrack::PostLoad()
{
	Super::PostLoad();

	if (GetLinkerCustomVersion(FMovieSceneEvaluationCustomVersion::GUID) < FMovieSceneEvaluationCustomVersion::ChangeEvaluateNearestSectionDefault)
	{
		EvalOptions.bEvalNearestSection = EvalOptions.bEvaluateNearestSection_DEPRECATED;
	}

	// Remove any null sections
	for (int32 SectionIndex = 0; SectionIndex < GetAllSections().Num(); )
	{
		UMovieSceneSection* Section = GetAllSections()[SectionIndex];

		if (Section == nullptr)
		{
#if WITH_EDITOR
			UE_LOGF(LogMovieScene, Warning, "Removing null section from %ls:%ls", *GetPathName(), *GetDisplayName().ToString());
#endif
			RemoveSectionAt(SectionIndex);
		}
		else if (Section->GetRange().IsEmpty())
		{
#if WITH_EDITOR
			//UE_LOGF(LogMovieScene, Warning, "Removing section %ls:%ls with empty range", *GetPathName(), *GetDisplayName().ToString());
#endif
			RemoveSectionAt(SectionIndex);
		}
		else
		{
			++SectionIndex;
		}
	}
}

void UMovieSceneTrack::AddSection(FSectionParameter Section)
{
	CallAddSection(*Section.Section);
	Section.Section->OnAddedToTrack(this);
	OnSectionAddedImpl(Section.Section);
}

void UMovieSceneTrack::RemoveSection(FSectionParameter Section)
{
	CallRemoveSection(*Section.Section);
	Section.Section->OnRemovedFromTrack();
	OnSectionRemovedImpl(Section.Section);
}

void UMovieSceneTrack::RemoveSectionAt(FSectionIndexParameter SectionIndex)
{
	const TArray<UMovieSceneSection*>& Sections = GetAllSections();
	if (Sections.IsValidIndex(SectionIndex.SectionIndex))
	{
		UMovieSceneSection* Section = Sections[SectionIndex.SectionIndex];
		CallRemoveSectionAt(SectionIndex.SectionIndex);
		if (Section)
		{
			Section->OnRemovedFromTrack();
		}
	}
}

void UMovieSceneTrack::OnAddedToMovieScene(UMovieScene* MovieScene)
{
	OnAddedToMovieSceneImpl(MovieScene);

	// If this is being renamed into a movie scene structure, simulate re-addition of the
	//    decorations to ensure any external references within the MovieScene are updated.
	for (UObject* Decoration : GetDecorations())
	{
		if (IMovieSceneLifetimeDecoration* Lifetime = Cast<IMovieSceneLifetimeDecoration>(Decoration))
		{
			Lifetime->OnReconstruct(MovieScene);
		}
	}

	for (UMovieSceneSection* Section : GetAllSections())
	{
		for (UObject* Decoration : Section->GetDecorations())
		{
			if (IMovieSceneLifetimeDecoration* Lifetime = Cast<IMovieSceneLifetimeDecoration>(Decoration))
			{
				Lifetime->OnReconstruct(MovieScene);
			}
		}
	}
}

void UMovieSceneTrack::OnRemovedFromMovieScene()
{
	OnRemovedFromMovieSceneImpl();

	if (UMovieScene* MovieScene = GetTypedOuter<UMovieScene>())
	{
		for (UObject* Decoration : GetDecorations())
		{
			if (IMovieSceneLifetimeDecoration* Lifetime = Cast<IMovieSceneLifetimeDecoration>(Decoration))
			{
				Lifetime->OnDestroy(MovieScene);
			}
		}

		for (UMovieSceneSection* Section : GetAllSections())
		{
			for (UObject* Decoration : Section->GetDecorations())
			{
				if (IMovieSceneLifetimeDecoration* Lifetime = Cast<IMovieSceneLifetimeDecoration>(Decoration))
				{
					Lifetime->OnDestroy(MovieScene);
				}
			}
		}
	}
}

void UMovieSceneTrack::OnDecorationAdded(UObject* NewDecoration)
{
	if (IMovieSceneTrackDecoration* DecorationInterface = Cast<IMovieSceneTrackDecoration>(NewDecoration))
	{
		DecorationInterface->OnDecorationAdded(this);
	}
}

void UMovieSceneTrack::OnDecorationRemoved(UObject* Decoration)
{
	if (IMovieSceneTrackDecoration* DecorationInterface = Cast<IMovieSceneTrackDecoration>(Decoration))
	{
		DecorationInterface->OnDecorationRemoved();
	}
}

bool UMovieSceneTrack::IsPostLoadThreadSafe() const
{
	return true;
}

void UMovieSceneTrack::UpdateEasing()
{
	int32 MaxRows = GetMaxRowIndex();
	TArray<UMovieSceneSection*> RowSections;

	for (int32 RowIndex = 0; RowIndex <= MaxRows; ++RowIndex)
	{
		RowSections.Reset();

		for (UMovieSceneSection* Section : GetAllSections())
		{
			if (Section && Section->GetRowIndex() == RowIndex)
			{
				RowSections.Add(Section);
			}
		}

		for (int32 Index = 0; Index < RowSections.Num(); ++Index)
		{
			UMovieSceneSection* CurrentSection = RowSections[Index];

			FMovieSceneSupportsEasingParams SupportsEasingParams(CurrentSection);
			EMovieSceneTrackEasingSupportFlags EasingFlags = SupportsEasing(SupportsEasingParams);

			// Auto-deactivate manual easing if we lost the ability to use it.
			if (!EnumHasAllFlags(EasingFlags, EMovieSceneTrackEasingSupportFlags::ManualEaseIn))
			{
				CurrentSection->Easing.bManualEaseIn = false;
			}
			if (!EnumHasAllFlags(EasingFlags, EMovieSceneTrackEasingSupportFlags::ManualEaseOut))
			{
				CurrentSection->Easing.bManualEaseOut = false;
			}

			if (!EnumHasAllFlags(EasingFlags, EMovieSceneTrackEasingSupportFlags::AutomaticEasing))
			{
				if (CurrentSection->Easing.AutoEaseInDuration != 0 || CurrentSection->Easing.AutoEaseOutDuration != 0)
				{
					CurrentSection->Modify();

					CurrentSection->Easing.AutoEaseInDuration = 0;
					CurrentSection->Easing.AutoEaseOutDuration = 0;
				}
				continue;
			}

			// Check overlaps with exclusive ranges so that sections can butt up against each other
			UMovieSceneTrack* OuterTrack = CurrentSection->GetTypedOuter<UMovieSceneTrack>();
			int32 MaxEaseIn = 0;
			int32 MaxEaseOut = 0;
			bool bIsEntirelyUnderlapped = false;

			TRange<FFrameNumber> CurrentSectionRange = CurrentSection->GetRange();
			for (int32 OtherIndex = 0; OtherIndex < RowSections.Num(); ++OtherIndex)
			{
				if (OtherIndex == Index)
				{
					continue;
				}

				UMovieSceneSection* Other = RowSections[OtherIndex];
				TRange<FFrameNumber> OtherSectionRange = Other->GetRange();

				if (!OtherSectionRange.HasLowerBound() && !OtherSectionRange.HasUpperBound())
				{
					// If we're testing against an infinite range we want to use the PlayRange of the sequence
					// instead so that blends stop at the end of a clip instead of a quarter of the length.
					UMovieScene* OuterScene = OuterTrack->GetTypedOuter<UMovieScene>();
					OtherSectionRange = OuterScene->GetPlaybackRange();
				}

				bIsEntirelyUnderlapped = bIsEntirelyUnderlapped || OtherSectionRange.Contains(CurrentSectionRange);

				// Check the lower bound of the current section against the other section's upper bound
				const bool bSectionRangeContainsOtherUpperBound = !OtherSectionRange.GetUpperBound().IsOpen() && !CurrentSectionRange.GetLowerBound().IsOpen() && CurrentSectionRange.Contains(OtherSectionRange.GetUpperBoundValue());
				const bool bSectionRangeContainsOtherLowerBound = !OtherSectionRange.GetLowerBound().IsOpen() && !CurrentSectionRange.GetUpperBound().IsOpen() && CurrentSectionRange.Contains(OtherSectionRange.GetLowerBoundValue());
				if (bSectionRangeContainsOtherUpperBound && !bSectionRangeContainsOtherLowerBound)
				{
					const int32 Difference = UE::MovieScene::DiscreteSize(TRange<FFrameNumber>(CurrentSectionRange.GetLowerBound(), OtherSectionRange.GetUpperBound()));
					MaxEaseIn = FMath::Max(MaxEaseIn, Difference);
				}

				if (bSectionRangeContainsOtherLowerBound &&!bSectionRangeContainsOtherUpperBound)
				{
					const int32 Difference = UE::MovieScene::DiscreteSize(TRange<FFrameNumber>(OtherSectionRange.GetLowerBound(), CurrentSectionRange.GetUpperBound()));
					MaxEaseOut = FMath::Max(MaxEaseOut, Difference);
				}
			}

			const bool  bIsFinite = CurrentSectionRange.HasLowerBound() && CurrentSectionRange.HasUpperBound();
			const int32 MaxSize   = bIsFinite ? UE::MovieScene::DiscreteSize(CurrentSectionRange) : TNumericLimits<int32>::Max();

			if (MaxEaseOut == 0 && MaxEaseIn == 0 && bIsEntirelyUnderlapped)
			{
				MaxEaseOut = MaxEaseIn = MaxSize / 4;
			}

			// Only modify the section if the ease in or out times have actually changed
			MaxEaseIn  = FMath::Clamp(MaxEaseIn, 0, MaxSize);
			MaxEaseOut = FMath::Clamp(MaxEaseOut, 0, MaxSize);

			if (CurrentSection->Easing.AutoEaseInDuration != MaxEaseIn || CurrentSection->Easing.AutoEaseOutDuration != MaxEaseOut)
			{
				CurrentSection->Modify();

				CurrentSection->Easing.AutoEaseInDuration  = MaxEaseIn;
				CurrentSection->Easing.AutoEaseOutDuration = MaxEaseOut;
			}
		}
	}
}

FMovieSceneTrackRowSegmentBlenderPtr UMovieSceneTrack::GetRowSegmentBlender() const
{
	return FDefaultTrackRowSegmentBlender();
}

FMovieSceneTrackSegmentBlenderPtr UMovieSceneTrack::GetTrackSegmentBlender() const
{
	if (EvalOptions.bCanEvaluateNearestSection && EvalOptions.bEvalNearestSection)
	{
		return FEvaluateNearestSegmentBlender();
	}
	else
	{
		return FMovieSceneTrackSegmentBlenderPtr();
	}
}


int32 UMovieSceneTrack::GetMaxRowIndex() const
{
	int32 MaxRowIndex = 0;
	for (UMovieSceneSection* Section : GetAllSections())
	{
		MaxRowIndex = FMath::Max(MaxRowIndex, Section->GetRowIndex());
	}

	return MaxRowIndex;
}

bool UMovieSceneTrack::FixRowIndices()
{
	TMap<int32, int32> NewToOldRowIndices;

	bool bFixesMade = false;
	TArray<UMovieSceneSection*> Sections = GetAllSections();
	if (SupportsMultipleRows())
	{
		// remove any empty track rows by waterfalling down sections to be as compact as possible
		TArray<TArray<UMovieSceneSection*>> RowIndexToSectionsMap;
		RowIndexToSectionsMap.AddZeroed(GetMaxRowIndex() + 1);

		for (UMovieSceneSection* Section : Sections)
		{
			RowIndexToSectionsMap[Section->GetRowIndex()].Add(Section);
		}

		int32 NewIndex = 0;
		for (const TArray<UMovieSceneSection*>& SectionsForIndex : RowIndexToSectionsMap)
		{
			if (SectionsForIndex.Num() > 0)
			{
				for (UMovieSceneSection* SectionForIndex : SectionsForIndex)
				{
					if (SectionForIndex->GetRowIndex() != NewIndex)
					{
						int32 OldIndex = SectionForIndex->GetRowIndex();
						SectionForIndex->Modify();
						SectionForIndex->SetRowIndex(NewIndex);
						NewToOldRowIndices.FindOrAdd(NewIndex, OldIndex);
						bFixesMade = true;
					}
				}
				++NewIndex;
			}
		}

		// If there aren't multiple rows (ie. max row is 0), there shouldn't be any disabled rows either
		if (GetMaxRowIndex() == 0 && !RowsDisabled.IsEmpty())
		{
			Modify();
			RowsDisabled.Empty();
		}
	}
	else
	{
		for (int32 i = 0; i < Sections.Num(); ++i)
		{
			if (Sections[i]->GetRowIndex() != 0)
			{
				Sections[i]->Modify();
				Sections[i]->SetRowIndex(0);
				bFixesMade = true;
			}
		}
	}

	if (NewToOldRowIndices.Num())
	{
		OnRowIndicesChanged(NewToOldRowIndices);
	}
	return bFixesMade;
}

void UMovieSceneTrack::OnRowIndicesChanged(const TMap<int32, int32>& NewToOldRowIndices)
{
	// Patch track row metadata
	
	TMap<int32, FMovieSceneTrackRowMetadata> NewTrackRowMetadata;

	for (int32 NewRowIndex = 0; NewRowIndex <= GetMaxRowIndex(); ++NewRowIndex)
	{
		int32 IndexToCopy = NewRowIndex;
		if (const int32* OldRowIndex = NewToOldRowIndices.Find(NewRowIndex))
		{
			IndexToCopy = *OldRowIndex;
		}

		if (const FMovieSceneTrackRowMetadata* Metadata = TrackRowMetadata.Find(IndexToCopy))
		{
			NewTrackRowMetadata.Add(NewRowIndex, *Metadata);
		}
	}
	TrackRowMetadata = NewTrackRowMetadata;

	// Remap persistent row-disabled state
	{
		TArray<int32> OldRowsDisabled = MoveTemp(RowsDisabled);
		for (const TPair<int32, int32>& Pair : NewToOldRowIndices)
		{
			if (OldRowsDisabled.Contains(Pair.Value))
			{
				RowsDisabled.AddUnique(Pair.Key);
			}
		}
	}

#if WITH_EDITORONLY_DATA
	// Remap local (transient) row-disabled state
	{
		TArray<int32> OldLocalRowsDisabled = MoveTemp(LocalRowsDisabled);
		for (const TPair<int32, int32>& Pair : NewToOldRowIndices)
		{
			if (OldLocalRowsDisabled.Contains(Pair.Value))
			{
				LocalRowsDisabled.AddUnique(Pair.Key);
			}
		}
	}
#endif

	if (UMovieSceneTrackRowDecoration* TrackRowDecoration = FindDecoration<UMovieSceneTrackRowDecoration>())
	{
		TrackRowDecoration->OnRowIndicesChanged(NewToOldRowIndices);
	}
}

#if WITH_EDITOR

ECookOptimizationFlags UMovieSceneTrack::GetCookOptimizationFlags() const
{
	if (RemoveMutedTracksOnCook() && IsEvalDisabled())
	{
		return ECookOptimizationFlags::RemoveTrack;
	}
	return ECookOptimizationFlags::None; 
}

void UMovieSceneTrack::RemoveForCook()
{
	for (UMovieSceneSection* Section : GetAllSections())
	{
		if (Section)
		{
			Section->RemoveForCook();
		}
	}
	RemoveAllAnimationData();
}

bool UMovieSceneTrack::RemoveMutedTracksOnCook()
{
	return CVarMovieSceneRemoveMutedTracksOnCook->GetInt() != 0;
}

TArray<UMovieSceneCondition*> UMovieSceneTrack::GetAllConditions()
{
	TArray<UMovieSceneCondition*> Conditions;
	if (ConditionContainer.Condition)
	{
		Conditions.Add(ConditionContainer.Condition);
	}

	for (TPair<int32, FMovieSceneTrackRowMetadata>& TrackRowMetadataPair : TrackRowMetadata)
	{
		if (TrackRowMetadataPair.Value.ConditionContainer.Condition)
		{
			Conditions.Add(TrackRowMetadataPair.Value.ConditionContainer.Condition);
		}
	}

	for (UMovieSceneSection* Section : GetAllSections())
	{
		if (Section->ConditionContainer.Condition)
		{
			Conditions.Add(Section->ConditionContainer.Condition);
		}
	}
	return Conditions;
}

#endif

bool UMovieSceneTrack::IsEvalDisabled(const bool bInCheckLocal) const
{
	return bIsEvalDisabled
#if WITH_EDITORONLY_DATA
		|| (bInCheckLocal && bIsLocalEvalDisabled)
#endif
	;
}

bool UMovieSceneTrack::IsRowEvalDisabled(const int32 InRowIndex, const bool bInCheckLocal) const
{
	return RowsDisabled.Contains(InRowIndex)
#if WITH_EDITORONLY_DATA
		|| (bInCheckLocal && LocalRowsDisabled.Contains(InRowIndex))
#endif
		;
}

void UMovieSceneTrack::SetRowEvalDisabled(const bool bInEvalDisabled, const int32 InRowIndex)
{
	if (bInEvalDisabled)
	{
		RowsDisabled.AddUnique(InRowIndex);
	}
	else
	{
		RowsDisabled.Remove(InRowIndex);
	}
}

#if WITH_EDITOR

bool UMovieSceneTrack::IsLocalRowEvalDisabled(const int32 InRowIndex) const
{
	return LocalRowsDisabled.Contains(InRowIndex);
}

void UMovieSceneTrack::SetLocalRowEvalDisabled(const bool bInEvalDisabled, const int32 InRowIndex)
{
	if (bInEvalDisabled)
	{
		LocalRowsDisabled.AddUnique(InRowIndex);
	}
	else
	{
		LocalRowsDisabled.Remove(InRowIndex);
	}
}

#endif

const FMovieSceneTrackRowMetadata* UMovieSceneTrack::FindTrackRowMetadata(int32 RowIndex) const
{
	return TrackRowMetadata.Find(RowIndex);
}

FMovieSceneTrackRowMetadata* UMovieSceneTrack::FindTrackRowMetadata(int32 RowIndex) 
{
	return TrackRowMetadata.Find(RowIndex);
}

FMovieSceneTrackRowMetadata& UMovieSceneTrack::FindOrAddTrackRowMetadata(int32 RowIndex)
{
	return TrackRowMetadata.FindOrAdd(RowIndex);
}

FGuid UMovieSceneTrack::FindObjectBindingGuid() const
{
	const UMovieScene* MovieScene = GetTypedOuter<UMovieScene>();

	if (MovieScene)
	{
		for (const FMovieSceneBinding& Binding : MovieScene->GetBindings())
		{
			if (Binding.GetTracks().Contains(this))
			{
				return Binding.GetObjectGuid();
			}
		}
	}

	return FGuid();
}

void UMovieSceneTrack::AddSectionRangesToTree(TArrayView<UMovieSceneSection* const> Sections, TMovieSceneEvaluationTree<FMovieSceneTrackEvaluationData>& OutTree)
{
	if (PopulateEvaluationTree(OutTree))
	{
		return;
	}

	ETreePopulationMode ModeToUse = BuiltInTreePopulationMode;
	if (!ensureMsgf(ModeToUse != ETreePopulationMode::None, TEXT("No default tree population mode specified, and no PopulateEvaluationTree implemented - falling back to high-pass-per-row population.")))
	{
		ModeToUse = ETreePopulationMode::HighPassPerRow;
	}

	switch (ModeToUse)
	{
	case ETreePopulationMode::Blended:
		UE::MovieScene::FEvaluationTreePopulationRules::Blended(Sections, OutTree);
		break;

	case ETreePopulationMode::HighPass:
		UE::MovieScene::FEvaluationTreePopulationRules::HighPass(Sections, OutTree);
		break;

	case ETreePopulationMode::HighPassPerRow:
		UE::MovieScene::FEvaluationTreePopulationRules::HighPassPerRow(Sections, OutTree);
		break;
	}
}

void UMovieSceneTrack::AddSectionPrePostRollRangesToTree(TArrayView<UMovieSceneSection* const> Sections, TMovieSceneEvaluationTree<FMovieSceneTrackEvaluationData>& OutTree)
{
	// Always add pre and postroll ranges, regardless
	for (UMovieSceneSection* Section : Sections)
	{
		if (Section && Section->IsActive())
		{
			const TRange<FFrameNumber> SectionRange = Section->GetRange();
			if (!SectionRange.IsEmpty())
			{
				if (!SectionRange.GetLowerBound().IsOpen() && Section->GetPreRollFrames() > 0)
				{
					TRange<FFrameNumber> PreRollRange = UE::MovieScene::MakeDiscreteRangeFromUpper(TRangeBound<FFrameNumber>::FlipInclusion(SectionRange.GetLowerBoundValue()), Section->GetPreRollFrames());
					OutTree.Add(PreRollRange, FMovieSceneTrackEvaluationData::FromSection(Section).SetFlags(ESectionEvaluationFlags::PreRoll));
				}

				if (!SectionRange.GetUpperBound().IsOpen() && Section->GetPostRollFrames() > 0)
				{
					TRange<FFrameNumber> PostRollRange = UE::MovieScene::MakeDiscreteRangeFromLower(TRangeBound<FFrameNumber>(SectionRange.GetUpperBoundValue()), Section->GetPostRollFrames());
					OutTree.Add(PostRollRange, FMovieSceneTrackEvaluationData::FromSection(Section).SetFlags(ESectionEvaluationFlags::PostRoll));
				}
			}
		}
	}
}

void UMovieSceneTrack::PreCompile(FMovieSceneTrackPreCompileResult& OutPreCompileResult)
{
	PreCompileImpl(OutPreCompileResult);
}

const FMovieSceneTrackEvaluationField& UMovieSceneTrack::GetEvaluationField()
{
	if (EvaluationFieldGuid != GetSignature() 
#if WITH_EDITORONLY_DATA
			|| EvaluationFieldVersion != GetEvaluationFieldVersion()
#endif
			)
	{
		UpdateEvaluationTree();
	}

	return EvaluationField;
}

void UMovieSceneTrack::ForceUpdateEvaluationTree()
{
	UpdateEvaluationTree();
}

void UMovieSceneTrack::UpdateEvaluationTree()
{
	TMovieSceneEvaluationTree<FMovieSceneTrackEvaluationData> EvaluationTree;

	TArray<UMovieSceneSection*> Sections = GetAllSections();

	AddSectionRangesToTree(Sections, EvaluationTree);

	if (EvalOptions.bCanEvaluateNearestSection && EvalOptions.bEvalNearestSection)
	{
		UE::MovieScene::FEvaluationTreePopulationRules::PopulateNearestSection(Sections, EvaluationTree);
	}

	AddSectionPrePostRollRangesToTree(Sections, EvaluationTree);


	EvaluationField.Reset();

	TMap<UMovieSceneSection*, TArray<FMovieSceneTrackEvaluationFieldEntry>> SectionToEntry;
	for (FMovieSceneEvaluationTreeRangeIterator It(EvaluationTree); It; ++It)
	{
		TRange<FFrameNumber> Range = It.Range();

		TMovieSceneEvaluationTreeDataIterator<FMovieSceneTrackEvaluationData> TrackDataIt = EvaluationTree.GetAllData(It.Node());
		if (TrackDataIt)
		{
			for (const FMovieSceneTrackEvaluationData& TrackData : TrackDataIt)
			{
				UMovieSceneSection* Section = TrackData.Section.Get();
				SectionToEntry.FindOrAdd(Section).Add(FMovieSceneTrackEvaluationFieldEntry{decltype(FMovieSceneTrackEvaluationFieldEntry::Section)(Section), Range, TrackData.ForcedTime, TrackData.Flags, TrackData.SortOrder });
			}
		}
		else
		{
			// Add an eplicit entry for null, signifying the track itself, even though there are no sections at this time
			//SectionToEntry.FindOrAdd(nullptr).Add(FMovieSceneTrackEvaluationFieldEntry{ Range, ESectionEvaluationFlags::None, 0 });
		}
	}

	for (TTuple<UMovieSceneSection*, TArray<FMovieSceneTrackEvaluationFieldEntry>>& Pair : SectionToEntry)
	{
		int32 NumEntries = Pair.Value.Num();
		for (int32 Index = 0; Index < NumEntries; ++Index)
		{
			FMovieSceneTrackEvaluationFieldEntry* PredicateEntry = &Pair.Value[Index];

			int32 StartIndex = Index;

			while (Index < NumEntries-1)
			{
				const FMovieSceneTrackEvaluationFieldEntry& SubsequentEntry = Pair.Value[Index+1];
				if (SubsequentEntry.Range.Adjoins(PredicateEntry->Range) && SubsequentEntry.Flags == PredicateEntry->Flags && SubsequentEntry.ForcedTime == PredicateEntry->ForcedTime)
				{
					PredicateEntry->Range.SetUpperBound(SubsequentEntry.Range.GetUpperBound());
					++Index;
					continue;
				}

				break;
			}

			int32 NumToConsolidate = Index - StartIndex;
			if (NumToConsolidate > 0)
			{
				Pair.Value.RemoveAt(StartIndex + 1, NumToConsolidate, EAllowShrinking::No);
				NumEntries -= NumToConsolidate;
			}
		}

		// @todo: Do we need to handle with empty track segments?
		if (Pair.Value.Num() > 0)
		{
			EvaluationField.Entries.Append(Pair.Value);
		}
	}

	EvaluationFieldGuid = GetSignature();
#if WITH_EDITORONLY_DATA
	EvaluationFieldVersion = GetEvaluationFieldVersion();
#endif
}

