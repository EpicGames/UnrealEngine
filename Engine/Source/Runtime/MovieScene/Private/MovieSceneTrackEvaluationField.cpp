// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneTrackEvaluationField.h"
#include "MovieSceneTrack.h"
#include "MovieSceneSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneTrackEvaluationField)


FMovieSceneTrackEvaluationData FMovieSceneTrackEvaluationData::FromSection(UMovieSceneSection* InSection)
{
	FMovieSceneTrackEvaluationData NewData;
	NewData.Section = InSection;
	return NewData;
}

FMovieSceneTrackEvaluationData FMovieSceneTrackEvaluationData::FromTrack(UMovieSceneTrack* InTrack)
{
	FMovieSceneTrackEvaluationData NewData;
	NewData.Track = InTrack;
	return NewData;
}
