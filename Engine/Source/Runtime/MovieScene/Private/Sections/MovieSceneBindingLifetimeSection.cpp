// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneBindingLifetimeSection.h"
#include "Tracks/MovieSceneBindingLifetimeTrack.h"

#include "EntitySystem/MovieSceneEntityBuilder.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "Evaluation/MovieSceneEvaluationField.h"
#include "MovieScene.h"
#include "EntitySystem/MovieSceneInstanceRegistry.h"
#include "MovieSceneSequence.h"
#include "MovieSceneCommonHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneBindingLifetimeSection)

UMovieSceneBindingLifetimeSection::UMovieSceneBindingLifetimeSection(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	bSupportsInfiniteRange = true;
}

void UMovieSceneBindingLifetimeSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponentTypes = FBuiltInComponentTypes::Get();

	// Conditionally add this as a spawnable binding if our binding says so.
	// Also conditionally add the binding lifetime component to manage lifetime events.

	UMovieSceneSequence* Sequence = GetTypedOuter<UMovieSceneSequence>();
	ensure(Sequence);

	const FSequenceInstance& RootInstance = EntityLinker->GetInstanceRegistry()->GetInstance(Params.Sequence.RootInstanceHandle);
	TSharedRef<const FSharedPlaybackState> SharedPlaybackState = RootInstance.GetSharedPlaybackState();

	bool bSpawnable = MovieSceneHelpers::IsBoundToAnySpawnable(Sequence, Params.GetObjectBindingID(), SharedPlaybackState);

	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.Add(BuiltInComponentTypes->GenericObjectBinding, Params.GetObjectBindingID())
		.AddConditional(BuiltInComponentTypes->SpawnableBinding, Params.GetObjectBindingID(), Params.GetObjectBindingID().IsValid() && bSpawnable)
		.AddConditional(BuiltInComponentTypes->BindingLifetime, FMovieSceneBindingLifetimeComponentData{ EMovieSceneBindingLifetimeState::Active }, Params.GetObjectBindingID().IsValid())
	);


}

bool UMovieSceneBindingLifetimeSection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	// By default, Binding Lifetime sections do not populate any evaluation field entries
	// that is the job of its outer UMovieSceneTrack through a call to ExternalPopulateEvaluationField
	return true;
}

void UMovieSceneBindingLifetimeSection::ExternalPopulateEvaluationField(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	using namespace UE::MovieScene;

	const int32 MetaDataIndex = OutFieldBuilder->AddMetaData(InMetaData);

	const int32 EntityIndex = OutFieldBuilder->FindOrAddEntity(this, 1);
	OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex, MetaDataIndex);
}

