// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneBoolSection.h"
#include "UObject/SequencerObjectVersion.h"
#include "Channels/MovieSceneChannelProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneBoolSection)

UMovieSceneBoolSection::UMovieSceneBoolSection( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
	, DefaultValue_DEPRECATED(false)
{
	bSupportsInfiniteRange = true;
#if WITH_EDITORONLY_DATA
	bIsInfinite_DEPRECATED = true;
	bIsExternallyInverted = false;
#endif

	SetRange(TRange<FFrameNumber>::All());

	EvalOptions.EnableAndSetCompletionMode
		(GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToRestoreState ? 
			EMovieSceneCompletionMode::KeepState : 
			GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToProjectDefault ? 
			EMovieSceneCompletionMode::RestoreState : 
			EMovieSceneCompletionMode::ProjectDefault);

	ReconstructChannelProxy();
}

#if WITH_EDITORONLY_DATA

void UMovieSceneBoolSection::SetIsExternallyInverted(bool bInIsExternallyInverted)
{
	bIsExternallyInverted = bInIsExternallyInverted;
	ReconstructChannelProxy();
}

void UMovieSceneBoolSection::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{
		ReconstructChannelProxy();
	}
}

#endif


void UMovieSceneBoolSection::ReconstructChannelProxy()
{
#if WITH_EDITOR

	// set up the external value to retrieve the inverted value if necessary. This is used by visibility tracks that
	// are bound to "Actor Hidden in Game" properties, but displayed as "Visibility"
	struct FGetInvertedBool
	{
		static TOptional<bool> GetValue(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
		{
			return Bindings ? !Bindings->GetCurrentValue<bool>(InObject) : TOptional<bool>();
		}
	};

	TMovieSceneExternalValue<bool> ExternalValue;
	ExternalValue.OnGetExternalValue = bIsExternallyInverted ? FGetInvertedBool::GetValue :TMovieSceneExternalValue<bool>::GetValue;

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(BoolCurve, FMovieSceneChannelMetaData(), ExternalValue);

#else

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(BoolCurve);

#endif
}

void UMovieSceneBoolSection::PostEditImport()
{
	Super::PostEditImport();

	ReconstructChannelProxy();
}

void UMovieSceneBoolSection::PostLoad()
{
	if (!BoolCurve.GetDefault().IsSet() && DefaultValue_DEPRECATED)
	{
		BoolCurve.SetDefault(DefaultValue_DEPRECATED);
	}
	Super::PostLoad();
}
