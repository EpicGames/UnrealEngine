// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneHookSection.h"
#include "EntitySystem/MovieSceneEvaluationHookSystem.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneHookSection)

UMovieSceneHookSection::UMovieSceneHookSection(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	bRequiresRangedHook = false;
	bRequiresTriggerHooks = false;
}

void UMovieSceneHookSection::ImportRangedEntity(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();

	FMovieSceneEvaluationHookComponent Hook{ this, Params.GetObjectBindingID() };
	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.Add(Components->EvaluationHook, Hook)
		.AddMutualComponents()
	);
}

void UMovieSceneHookSection::ImportTriggerEntity(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	const int32 EventIndex = static_cast<int32>(Params.EntityID);

	TArrayView<const FFrameNumber> Times = GetTriggerTimes();
	if (!ensureMsgf(Times.IsValidIndex(EventIndex), TEXT("Attempting to import an event entity for an invalid index (Index: %d, Num: %d)"), EventIndex, Times.Num()))
	{
		return;
	}

	const FSequenceInstance&  ThisInstance   = EntityLinker->GetInstanceRegistry()->GetInstance(Params.Sequence.InstanceHandle);
	const FMovieSceneContext& Context        = ThisInstance.GetContext();

	TOptional<FFrameTime> RootTime = Context.GetSequenceToRootSequenceTransform().TryTransformTime(Times[EventIndex], Context.GetRootToSequenceWarpCounter());
	if (!RootTime)
	{
		return;
	}

	FMovieSceneEvaluationHookEvent NewEvent;
	NewEvent.Hook.Interface = this;
	NewEvent.Hook.ObjectBindingID = Params.GetObjectBindingID();

	NewEvent.RootTime = RootTime.GetValue();
	NewEvent.Type = EEvaluationHookEvent::Trigger;
	NewEvent.SequenceID = ThisInstance.GetSequenceID();
	NewEvent.TriggerIndex = EventIndex;

	UMovieSceneEvaluationHookSystem* System = EntityLinker->LinkSystem<UMovieSceneEvaluationHookSystem>();
	System->AddEvent(ThisInstance.GetRootInstanceHandle(), NewEvent);
}

void UMovieSceneHookSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	if (Params.EntityID == FMovieSceneEntityComponentFieldBuilder::InvalidEntityID)
	{
		ImportRangedEntity(EntityLinker, Params, OutImportedEntity);
	}
	else
	{
		ImportTriggerEntity(EntityLinker, Params, OutImportedEntity);
	}
}

bool UMovieSceneHookSection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	const int32 MetaDataIndex = OutFieldBuilder->AddMetaData(InMetaData);

	if (bRequiresTriggerHooks)
	{
		TArrayView<const FFrameNumber> Times = GetTriggerTimes();
		for (int32 Index = 0; Index < Times.Num(); ++Index)
		{
			if (EffectiveRange.Contains(Times[Index]))
			{
				TRange<FFrameNumber> Range(Times[Index]);
				OutFieldBuilder->AddOneShotEntity(Range, this, Index, MetaDataIndex);
			}
		}
	}

	if (bRequiresRangedHook)
	{
		const int32 EntityIndex = OutFieldBuilder->FindOrAddEntity(this, FMovieSceneEntityComponentFieldBuilder::InvalidEntityID);
		OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex, MetaDataIndex);
	}

	return true;
}
