// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneSectionTimingParameters.h"
#include "Evaluation/MovieSceneSequenceTransform.h"
#include "Misc/FrameRate.h"
#include "Variants/MovieSceneTimeWarpVariantPayloads.h"
#include "MovieScene.h"
#include "MovieSceneClock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSectionTimingParameters)

FMovieSceneSequenceTransform FMovieSceneSectionTimingParametersSeconds::MakeTransform(const FFrameRate& OuterFrameRate, const TRange<FFrameNumber>& OuterRange, double SourceDuration, double InnerPlayRate) const
{
	FMovieSceneSequenceTransform Result;

	check(OuterRange.HasLowerBound());

	if (SourceDuration <= 0)
	{
		// Zero source duration is handled by zero play rate (always evaluate time zero)
		Result.Add(0, FMovieSceneTimeWarpVariant(0.0));
		return Result;
	}

	// ----------------------------------------------------------------------------
	// First things first, subtract the section start bound
	AddPositionInOuterAsOffset(Result, OuterRange);

	// ----------------------------------------------------------------------------
	// Time warp
	AddPlayRate(Result);
	Result.Add(0, FMovieSceneTimeWarpVariant(InnerPlayRate));

	// ----------------------------------------------------------------------------
	// FrameRate conversion to seconds
	FMovieSceneTimeWarpVariant FrameRate;
	FrameRate.Set(FMovieSceneTimeWarpFrameRate{ OuterFrameRate });
	Result.Add(0, MoveTemp(FrameRate));

	const double StartTime   = InnerStartOffset;
	const double EndTime     = SourceDuration - InnerEndOffset;
	const double Duration    = EndTime - StartTime;
	double StartOffset = InnerStartOffset + FirstLoopStartOffset;

	// Accommodate negative play rates by playing from the end of the clip
	if (PlayRate.GetType() == EMovieSceneTimeWarpType::FixedPlayRate && (PlayRate.AsFixedPlayRate() * InnerPlayRate) < 0.0)
	{
		StartOffset += Duration;
	}

	// Start offset
	if (!FMath::IsNearlyZero(StartOffset))
	{
		AddInnerStartOffset(Result, FFrameTime::FromDecimal(StartOffset));
	}

	AddLoopingOrClampingAndReverse(Result, Duration, -StartTime);

	return Result;
}

void FMovieSceneSectionTimingParametersSeconds::AddOffset(FMovieSceneSequenceTransform& Transform, const FFrameTime& Offset) const
{
	Transform.Add(FMovieSceneTimeTransform(Offset));
}

void FMovieSceneSectionTimingParametersSeconds::AddPositionInOuterAsOffset(FMovieSceneSequenceTransform& Transform, const TRange<FFrameNumber>& OuterRange) const
{
	const FFrameNumber Offset = -OuterRange.GetLowerBoundValue();
	if (bClampToOuterRange)
	{
		FMovieSceneTimeWarpVariant Clamp;
		Clamp.Set(FMovieSceneTimeWarpClamp{ OuterRange.Size<FFrameNumber>() });
		Transform.Add(Offset, MoveTemp(Clamp));
	}

	Transform.Add(FMovieSceneTimeTransform(Offset));
}

void FMovieSceneSectionTimingParametersSeconds::AddPlayRate(FMovieSceneSequenceTransform& Transform) const
{
	Transform.Add(0, PlayRate.ShallowCopy());
}

void FMovieSceneSectionTimingParametersSeconds::AddFrameRateConversion(FMovieSceneSequenceTransform& Transform, const FFrameRate& OuterFrameRate, const FFrameRate& InnerFrameRate) const
{
	FMovieSceneTimeWarpVariant FrameRate;
	FrameRate.Set(FMovieSceneTimeWarpFrameRate{ OuterFrameRate / InnerFrameRate });
	Transform.Add(0, MoveTemp(FrameRate));
}

void FMovieSceneSectionTimingParametersSeconds::AddInnerStartOffset(FMovieSceneSequenceTransform& Transform, const FFrameTime& Offset) const
{
	Transform.Add(FMovieSceneTimeTransform(Offset));
}

void FMovieSceneSectionTimingParametersSeconds::AddLoopingOrClampingAndReverse(FMovieSceneSequenceTransform& Result, const double Duration, const double Offset) const
{
	// ----------------------------------------------------------------------------
	// Looping or clamping
	if (bLoop)
	{
		// Loop
		FMovieSceneTimeWarpVariant Loop;
		Loop.Set(FMovieSceneTimeWarpLoopFloat{ static_cast<float>(Duration) });
		Result.Add(FFrameTime::FromDecimal(Offset), MoveTemp(Loop));
	}
	else if (bClampToInnerRange)
	{
		// Clamp
		FMovieSceneTimeWarpVariant Clamp;
		Clamp.Set(FMovieSceneTimeWarpClampFloat{ static_cast<float>(Duration) });
		Result.Add(FFrameTime::FromDecimal(Offset), MoveTemp(Clamp));
	}

	// ----------------------------------------------------------------------------
	// Reverse
	if (bReverse)
	{
		Result.Add(FMovieSceneTimeTransform(FFrameTime::FromDecimal(Duration), -1.f));
	}
}

FMovieSceneSequenceTransform FMovieSceneSectionTimingParametersFrames::MakeTransform(const FFrameRate& OuterFrameRate, const TRange<FFrameNumber>& OuterRange, const FFrameRate& InnerFrameRate, const TRange<FFrameNumber>& InnerRange) const
{
	FMovieSceneSequenceTransform Result;

	check(OuterRange.HasLowerBound());
	check(InnerRange.HasLowerBound() && InnerRange.HasUpperBound());

	// ----------------------------------------------------------------------------
	// First things first, subtract the section start bound
	AddPositionInOuterAsOffset(Result, OuterRange);

	// ----------------------------------------------------------------------------
	// Time warp
	AddPlayRate(Result);

	// ----------------------------------------------------------------------------
	// FrameRate conversion
	if (InnerFrameRate != OuterFrameRate)
	{
		AddFrameRateConversion(Result, OuterFrameRate, InnerFrameRate);
	}

	FFrameNumber StartTime = InnerRange.GetLowerBoundValue() + InnerStartOffset;
	FFrameNumber EndTime   = InnerRange.GetUpperBoundValue() - InnerEndOffset;
	FFrameNumber Duration  = EndTime - StartTime;

	FFrameNumber LoopOffset(bLoop ? FirstLoopStartOffset.Value : 0);

	FFrameNumber NegativeRateOffset = 0;
	if (PlayRate.GetType() == EMovieSceneTimeWarpType::FixedPlayRate && PlayRate.AsFixedPlayRate() < 0.0)
	{
		NegativeRateOffset = Duration;
	}

	// Start offset
	AddInnerStartOffset(Result, StartTime + LoopOffset + NegativeRateOffset);

	// ----------------------------------------------------------------------------
	// Looping or clamping
	AddLoopingOrClampingAndReverse(Result, Duration, -StartTime);

	return Result;
}

void FMovieSceneSectionTimingParametersFrames::AddOffset(FMovieSceneSequenceTransform& Transform, const FFrameTime& Offset) const
{
	Transform.Add(FMovieSceneTimeTransform(Offset));
}

void FMovieSceneSectionTimingParametersFrames::AddPositionInOuterAsOffset(FMovieSceneSequenceTransform& Transform, const TRange<FFrameNumber>& OuterRange) const
{
	const FFrameNumber Offset = -OuterRange.GetLowerBoundValue();
	if (bClampToOuterRange)
	{
		FMovieSceneTimeWarpVariant Clamp;
		Clamp.Set(FMovieSceneTimeWarpClamp{ OuterRange.Size<FFrameNumber>() });
		Transform.Add(Offset, MoveTemp(Clamp));
	}

	Transform.Add(FMovieSceneTimeTransform(Offset));
}

void FMovieSceneSectionTimingParametersFrames::AddPlayRate(FMovieSceneSequenceTransform& Transform) const
{
	Transform.Add(0, PlayRate.ShallowCopy());
}

void FMovieSceneSectionTimingParametersFrames::AddFrameRateConversion(FMovieSceneSequenceTransform& Transform, const FFrameRate& OuterFrameRate, const FFrameRate& InnerFrameRate) const
{
	FMovieSceneTimeWarpVariant FrameRate;
	FrameRate.Set(FMovieSceneTimeWarpFrameRate{ OuterFrameRate / InnerFrameRate });
	Transform.Add(0, MoveTemp(FrameRate));
}

void FMovieSceneSectionTimingParametersFrames::AddInnerStartOffset(FMovieSceneSequenceTransform& Transform, const FFrameTime& Offset) const
{
	Transform.Add(FMovieSceneTimeTransform(Offset));
}

void FMovieSceneSectionTimingParametersFrames::AddLoopingOrClampingAndReverse(FMovieSceneSequenceTransform& Result, const FFrameNumber& Duration, const FFrameNumber& Offset) const
{
	if (bLoop)
	{
		// Loop
		FMovieSceneTimeWarpVariant Loop;
		Loop.Set(FMovieSceneTimeWarpLoop{ Duration });
		Result.Add(Offset, MoveTemp(Loop));
	}
	else if (bClampToInnerRange)
	{
		// Clamp
		FMovieSceneTimeWarpVariant Clamp;
		Clamp.Set(FMovieSceneTimeWarpClamp{ Duration });
		Result.Add(Offset, MoveTemp(Clamp));
	}

	if (bReverse)
	{
		// Reverse
		Result.Add(FMovieSceneTimeTransform(Duration, -1.f));
	}
}
