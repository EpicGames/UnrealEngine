// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneSpawnSection.h"
#include "UObject/SequencerObjectVersion.h"

#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "MovieSceneTimeHelpers.h"
#include "Evaluation/MovieSceneEvaluationField.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntityBuilder.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneSpawnablesSystem.h"
#include "MovieSceneBindingReferences.h"
#include "Bindings/MovieSceneSpawnableBinding.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSpawnSection)


UMovieSceneSpawnSection::UMovieSceneSpawnSection(const FObjectInitializer& Init)
	: Super(Init)
{
	EvalOptions.EnableAndSetCompletionMode
		(GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToProjectDefault ? 
			EMovieSceneCompletionMode::RestoreState : 
			EMovieSceneCompletionMode::ProjectDefault);

	BoolCurve.SetDefault(true);
}

void UMovieSceneSpawnSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.Add(FBuiltInComponentTypes::Get()->SpawnableBinding, Params.GetObjectBindingID())
		.AddTagConditional(FBuiltInComponentTypes::Get()->Tags.OldStyleSpawnable, GetTypedOuter<UMovieScene>()->FindSpawnable(Params.GetObjectBindingID()) != nullptr)
	);
}

bool UMovieSceneSpawnSection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	FGuid ObjectBindingID = OutFieldBuilder->GetSharedMetaData().ObjectBindingID;

	UMovieSceneSequence* ParentSequence = GetTypedOuter<UMovieSceneSequence>();
	UMovieScene* ParentMovieScene = GetTypedOuter<UMovieScene>();
	
	bool bSpawnable = false;
	
	if (ParentMovieScene && ParentSequence)
	{
		if (ParentMovieScene->FindSpawnable(ObjectBindingID))
		{
			bSpawnable = true;
		}
		else if (const FMovieSceneBindingReferences* BindingReferences = ParentSequence->GetBindingReferences())
		{
			if (BindingReferences)
			{
				if (Algo::AnyOf(BindingReferences->GetReferences(ObjectBindingID), [](const FMovieSceneBindingReference& BindingReference) { return BindingReference.CustomBinding && BindingReference.CustomBinding->IsA<UMovieSceneSpawnableBindingBase>(); }))
				{
					bSpawnable = true;
				}
			}
		}

	}

	if (!bSpawnable)
	{
		return true;
	}

	const int32 MetaDataIndex = OutFieldBuilder->AddMetaData(InMetaData);

	// Only add the valid section ranges to the tree
	TArrayView<const FFrameNumber> Times  = BoolCurve.GetTimes();
	TArrayView<const bool>         Values = BoolCurve.GetValues();

	if (Times.Num() == 0)
	{
		if (BoolCurve.GetDefault().Get(false))
		{
			// Add the whole section range
			OutFieldBuilder->AddPersistentEntity(EffectiveRange, this, 0, MetaDataIndex);
		}
		return true;
	}

	const int32 EntityIndex = OutFieldBuilder->FindOrAddEntity(this, 0);

	TRangeBound<FFrameNumber> StartBound = EffectiveRange.GetLowerBound();

	// Find the effective key
	int32 Index = FMath::Min(StartBound.IsOpen() ? 0 : Algo::UpperBound(Times, UE::MovieScene::DiscreteInclusiveLower(StartBound)), Times.Num()-1);

	bool bIsSpawned = Values[StartBound.IsOpen() ? 0 : FMath::Max(0, Algo::UpperBound(Times, UE::MovieScene::DiscreteInclusiveLower(StartBound))-1)];
	for ( ; Index < Times.Num(); ++Index)
	{
		if (!EffectiveRange.Contains(Times[Index]))
		{
			break;
		}

		if (bIsSpawned != Values[Index])
		{
			if (bIsSpawned)
			{
				// Add the last range to the tree
				TRange<FFrameNumber> Range(StartBound, TRangeBound<FFrameNumber>::Exclusive(Times[Index]));
				if (!Range.IsEmpty())
				{
					OutFieldBuilder->AddPersistentEntity(Range, EntityIndex, MetaDataIndex);
				}
			}

			bIsSpawned = Values[Index];

			if (bIsSpawned)
			{
				StartBound = TRangeBound<FFrameNumber>::Inclusive(Times[Index]);
			}
		}
	}

	TRange<FFrameNumber> TailRange(StartBound, EffectiveRange.GetUpperBound());
	if (!TailRange.IsEmpty() && bIsSpawned)
	{
		OutFieldBuilder->AddPersistentEntity(TailRange, EntityIndex, MetaDataIndex);
	}

	return true;
}
