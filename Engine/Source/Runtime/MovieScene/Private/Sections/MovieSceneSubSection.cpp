// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneSubSection.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "MovieScene.h"
#include "MovieSceneClock.h"
#include "MovieSceneTrack.h"
#include "MovieSceneSequence.h"
#include "MovieSceneTimeHelpers.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "Evaluation/MovieSceneEvaluationTemplate.h"
#include "Misc/AxisDisplayInfo.h"
#include "Misc/FrameRate.h"
#include "Logging/MessageLog.h"
#include "MovieSceneTransformTypes.h"
#include "Evaluation/MovieSceneRootOverridePath.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneInstanceRegistry.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"
#include "Tracks/MovieSceneTimeWarpTrack.h"
#include "Sections/MovieSceneSectionTimingParameters.h"
#include "Variants/MovieSceneTimeWarpGetter.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSubSection)

float DeprecatedMagicNumber = TNumericLimits<float>::Lowest();

#if WITH_EDITOR

namespace
{
FIntVector4 ReverseSwizzleFunc(const FIntVector4& InSwizzle)
{
	FIntVector4 ReverseSwizzle;
	for (int32 i = 0; i < 4; ++i)
	{
		int32 Index = 0;
		for (int32 j = 0; j < 4; ++j)
		{
			if (InSwizzle[j] == i)
			{
				Index = j;
				break;
			}
		}
		ReverseSwizzle[i] = Index;
	}

	return ReverseSwizzle;
};
} // empty namespace

struct FSubSectionEditorData
{
	FText LocationGroup = NSLOCTEXT("MovieSceneSubSection", "Origin Override Location", "Origin Override Location");
	FText RotationGroup = NSLOCTEXT("MovieSceneSubSection", "Origin Override Rotation", "Origin Override Rotation");
	
	FSubSectionEditorData(EMovieSceneTransformChannel Mask, UMovieSceneSubSection* SubSection)
	{
		static const TSet<FName> PropertyMetaDataKeys = { "UIMin", "UIMax", "SliderExponent", "LinearDeltaSensitivity", "Delta", "ClampMin", "ClampMax", "ForceUnits", "WheelStep" };

		const EAxisList::Type XAxis = EAxisList::Forward;
		const EAxisList::Type YAxis = EAxisList::Left;
		const EAxisList::Type ZAxis = EAxisList::Up;

		const FIntVector4 Swizzle = AxisDisplayInfo::GetTransformAxisSwizzle();
		const FIntVector4 ReverseSwizzle = ReverseSwizzleFunc(Swizzle);
		const int32 TranslationOrderOffset = 0;
		const int32 RotationOrderOffset = TranslationOrderOffset + 3;

		const FProperty* TranslationProperty = UMovieSceneSubSection::StaticClass()->FindPropertyByName(UMovieSceneSubSection::GetTranslationPropertyName());
		const FProperty* RotationProperty = UMovieSceneSubSection::StaticClass()->FindPropertyByName(UMovieSceneSubSection::GetRotationPropertyName());

		MetaData[0].SetIdentifiers("Override.Location.X", AxisDisplayInfo::GetAxisDisplayName(XAxis), LocationGroup);
		MetaData[0].SubPropertyPath = TEXT("Location.X");
		MetaData[0].SortOrder = TranslationOrderOffset + ReverseSwizzle[0];
		MetaData[0].bEnabled = EnumHasAllFlags(Mask, EMovieSceneTransformChannel::TranslationX);
		MetaData[0].Color = AxisDisplayInfo::GetAxisColor(XAxis);
		MetaData[0].bCanCollapseToTrack = false;
		for (const FName& PropertyMetaDataKey : PropertyMetaDataKeys)
		{
			MetaData[0].PropertyMetaData.Add(PropertyMetaDataKey, TranslationProperty->GetMetaData(PropertyMetaDataKey));
		}

		MetaData[1].SetIdentifiers("Override.Location.Y", AxisDisplayInfo::GetAxisDisplayName(YAxis), LocationGroup);
		MetaData[1].SubPropertyPath = TEXT("Location.Y");
		MetaData[1].SortOrder = TranslationOrderOffset + ReverseSwizzle[1];
		MetaData[1].bEnabled = EnumHasAllFlags(Mask, EMovieSceneTransformChannel::TranslationY);
		MetaData[1].Color = AxisDisplayInfo::GetAxisColor(YAxis);
		MetaData[1].bCanCollapseToTrack = false;
		MetaData[1].bInvertValue = AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward;
		for (const FName& PropertyMetaDataKey : PropertyMetaDataKeys)
		{
			MetaData[1].PropertyMetaData.Add(PropertyMetaDataKey, TranslationProperty->GetMetaData(PropertyMetaDataKey));
		}

		MetaData[2].SetIdentifiers("Override.Location.Z", AxisDisplayInfo::GetAxisDisplayName(ZAxis), LocationGroup);
		MetaData[2].SubPropertyPath = TEXT("Location.Z");
		MetaData[2].SortOrder = TranslationOrderOffset + ReverseSwizzle[2];
		MetaData[2].bEnabled = EnumHasAllFlags(Mask, EMovieSceneTransformChannel::TranslationZ);
		MetaData[2].Color = AxisDisplayInfo::GetAxisColor(ZAxis);
		MetaData[2].bCanCollapseToTrack = false;
		for (const FName& PropertyMetaDataKey : PropertyMetaDataKeys)
		{
			MetaData[2].PropertyMetaData.Add(PropertyMetaDataKey, TranslationProperty->GetMetaData(PropertyMetaDataKey));
		}

		MetaData[3].SetIdentifiers("Override.Rotation.X", NSLOCTEXT("MovieSceneSubSection", "RotationX", "Roll"), RotationGroup);
		MetaData[3].SubPropertyPath = TEXT("Rotation.X");
		MetaData[3].SortOrder = RotationOrderOffset + 0;
		MetaData[3].bEnabled = EnumHasAllFlags(Mask, EMovieSceneTransformChannel::RotationX);
		MetaData[3].Color = AxisDisplayInfo::GetAxisColor(EAxisList::X);
		MetaData[3].bCanCollapseToTrack = false;
		for (const FName& PropertyMetaDataKey : PropertyMetaDataKeys)
		{
			MetaData[3].PropertyMetaData.Add(PropertyMetaDataKey, RotationProperty->GetMetaData(PropertyMetaDataKey));
		}

		MetaData[4].SetIdentifiers("Override.Rotation.Y", NSLOCTEXT("MovieSceneSubSection", "RotationY", "Pitch"), RotationGroup);
		MetaData[4].SubPropertyPath = TEXT("Rotation.Y");
		MetaData[4].SortOrder = RotationOrderOffset + 1;
		MetaData[4].bEnabled = EnumHasAllFlags(Mask, EMovieSceneTransformChannel::RotationY);
		MetaData[4].Color = AxisDisplayInfo::GetAxisColor(EAxisList::Y);
		MetaData[4].bCanCollapseToTrack = false;
		for (const FName& PropertyMetaDataKey : PropertyMetaDataKeys)
		{
			MetaData[4].PropertyMetaData.Add(PropertyMetaDataKey, RotationProperty->GetMetaData(PropertyMetaDataKey));
		}

		MetaData[5].SetIdentifiers("Override.Rotation.Z", NSLOCTEXT("MovieSceneSubSection", "RotationZ", "Yaw"), RotationGroup);
		MetaData[5].SubPropertyPath = TEXT("Rotation.Z");
		MetaData[5].SortOrder = RotationOrderOffset + 2;
		MetaData[5].bEnabled = EnumHasAllFlags(Mask, EMovieSceneTransformChannel::RotationZ);
		MetaData[5].Color = AxisDisplayInfo::GetAxisColor(EAxisList::Z);
		MetaData[5].bCanCollapseToTrack = false;
		for (const FName& PropertyMetaDataKey : PropertyMetaDataKeys)
		{
			MetaData[5].PropertyMetaData.Add(PropertyMetaDataKey, RotationProperty->GetMetaData(PropertyMetaDataKey));
		}

		ExternalValues[0].OnGetExternalValue = [SubSection](UObject& InObject, FTrackInstancePropertyBindings* Bindings) { return GetValue(SubSection, 0); };
		ExternalValues[1].OnGetExternalValue = [SubSection](UObject& InObject, FTrackInstancePropertyBindings* Bindings) { return GetValue(SubSection, 1); };
		ExternalValues[2].OnGetExternalValue = [SubSection](UObject& InObject, FTrackInstancePropertyBindings* Bindings) { return GetValue(SubSection, 2); };
		ExternalValues[3].OnGetExternalValue = [SubSection](UObject& InObject, FTrackInstancePropertyBindings* Bindings) { return GetValue(SubSection, 3); };
		ExternalValues[4].OnGetExternalValue = [SubSection](UObject& InObject, FTrackInstancePropertyBindings* Bindings) { return GetValue(SubSection, 4); };
		ExternalValues[5].OnGetExternalValue = [SubSection](UObject& InObject, FTrackInstancePropertyBindings* Bindings) { return GetValue(SubSection, 5); };

	}

	static TOptional<double> GetValue(UMovieSceneSubSection* SubSection, int32 ChannelIndex)
	{
		if(!SubSection)
		{
			return TOptional<double>();
		}

		switch (ChannelIndex)
		{
		case 0:
		case 1:
		case 2:
			return SubSection->GetKeyPreviewPosition().IsSet() ? SubSection->GetKeyPreviewPosition().GetValue()[ChannelIndex] : TOptional<double>();
		case 3:
			return SubSection->GetKeyPreviewRotation().IsSet() ? SubSection->GetKeyPreviewRotation().GetValue().Roll: TOptional<double>();
		case 4:
			return SubSection->GetKeyPreviewRotation().IsSet() ? SubSection->GetKeyPreviewRotation().GetValue().Pitch: TOptional<double>();
		case 5:
			return SubSection->GetKeyPreviewRotation().IsSet() ? SubSection->GetKeyPreviewRotation().GetValue().Yaw: TOptional<double>();
		default:
			return TOptional<double>();
		}
	}

	FMovieSceneChannelMetaData MetaData[6];
	TMovieSceneExternalValue<double> ExternalValues[6];
};

#endif


/* UMovieSceneSubSection structors
 *****************************************************************************/

UMovieSceneSubSection::UMovieSceneSubSection(const FObjectInitializer& ObjInitializer)
	: Super(ObjInitializer)
	, StartOffset_DEPRECATED(DeprecatedMagicNumber)
	, TimeScale_DEPRECATED(DeprecatedMagicNumber)
	, PrerollTime_DEPRECATED(DeprecatedMagicNumber)
{
	NetworkMask = (uint8)(EMovieSceneServerClientMask::Server | EMovieSceneServerClientMask::Client);

	SetBlendType(EMovieSceneBlendType::Absolute);
	
	OriginOverrideMask = EMovieSceneTransformChannel::None;

#if WITH_EDITOR
	ResetKeyPreviewRotationAndLocation();
#endif
}

void UMovieSceneSubSection::DeleteChannels(TArrayView<const FName> ChannelNames)
{
	bool bDeletedAny = false;

	if (Parameters.TimeScale.GetType() == EMovieSceneTimeWarpType::Custom && TryModify())
	{
		if (UMovieSceneTimeWarpGetter* Getter = Parameters.TimeScale.AsCustom())
		{
			for (FName ChannelName : ChannelNames)
			{
				bDeletedAny |= Getter->DeleteChannel(Parameters.TimeScale, ChannelName);
			}
		}
	}

	if (bDeletedAny)
	{
		ChannelProxy = nullptr;
	}
}

EMovieSceneChannelProxyType UMovieSceneSubSection::CacheChannelProxy()
{
	FMovieSceneChannelProxyData Channels;

	if (Parameters.TimeScale.GetType() == EMovieSceneTimeWarpType::Custom)
	{
		if (UMovieSceneTimeWarpGetter* Curve = Parameters.TimeScale.AsCustom())
		{
			Curve->PopulateChannelProxy(Channels, UMovieSceneTimeWarpGetter::EAllowTopLevelChannels::No);
		}
	}

#if WITH_EDITOR	

	FSubSectionEditorData EditorData(OriginOverrideMask.GetChannels(), this);

	Channels.Add(Translation[0], EditorData.MetaData[0], EditorData.ExternalValues[0]);
	Channels.Add(Translation[1], EditorData.MetaData[1], EditorData.ExternalValues[1]);
	Channels.Add(Translation[2], EditorData.MetaData[2], EditorData.ExternalValues[2]);
	Channels.Add(Rotation[0], EditorData.MetaData[3], EditorData.ExternalValues[3]);
	Channels.Add(Rotation[1], EditorData.MetaData[4], EditorData.ExternalValues[4]);
	Channels.Add(Rotation[2], EditorData.MetaData[5], EditorData.ExternalValues[5]);

#else

	Channels.Add(Translation[0]);
	Channels.Add(Translation[1]);
	Channels.Add(Translation[2]);
	Channels.Add(Rotation[0]);
	Channels.Add(Rotation[1]);
	Channels.Add(Rotation[2]);
	
#endif
	

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(Channels));
	return EMovieSceneChannelProxyType::Dynamic;
}


FMovieSceneSequenceTransform UMovieSceneSubSection::OuterToInnerTransform_NoInnerTimeWarp() const
{
	UMovieSceneSequence* SequencePtr   = GetSequence();
	if (!SequencePtr)
	{
		return FMovieSceneSequenceTransform();
	}

	UMovieScene* MovieScenePtr = SequencePtr->GetMovieScene();

	TRange<FFrameNumber> SubRange = GetRange();
	if (!MovieScenePtr || SubRange.GetLowerBound().IsOpen())
	{
		return FMovieSceneSequenceTransform();
	}

	const FFrameRate   InnerFrameRate = MovieScenePtr->GetTickResolution();
	const FFrameRate   OuterFrameRate = GetTypedOuter<UMovieScene>()->GetTickResolution();

	const TRange<FFrameNumber> MovieScenePlaybackRange = GetValidatedInnerPlaybackRange(Parameters, *MovieScenePtr);

	FMovieSceneSectionTimingParametersFrames TimingParams = {
		Parameters.TimeScale.ShallowCopy(),
		Parameters.StartFrameOffset,
		Parameters.EndFrameOffset,
		Parameters.FirstLoopStartFrameOffset,
		Parameters.bCanLoop,
		false, // do not clamp sub-sections by default
		false
	};

	// determine if we need to generate a musical transform or a standard transform...
	FMovieSceneSequenceTransform ClockResult;
	if (MovieScenePtr->GetCustomClock() && MovieScenePtr->GetCustomClock()->MakeSubSequenceTransform(TimingParams, this, ClockResult))
	{
		return ClockResult;
	}

	return TimingParams.MakeTransform(OuterFrameRate, SubRange, InnerFrameRate, MovieScenePtr->GetPlaybackRange());
}

FMovieSceneSequenceTransform UMovieSceneSubSection::OuterToInnerTransform() const
{
	FMovieSceneSequenceTransform OuterToInner = OuterToInnerTransform_NoInnerTimeWarp();
	AppendInnerTimeWarpTransform(OuterToInner);
	return OuterToInner;
}

void UMovieSceneSubSection::AppendInnerTimeWarpTransform(FMovieSceneSequenceTransform& OutTransform) const
{
	UMovieSceneSequence* Sequence   = GetSequence();
	UMovieScene*         MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;

	if (!MovieScene)
	{
		return;
	}

	// Look for any time warp tracks inside the sub sequence
	for (UMovieSceneTrack* Track : MovieScene->GetTracks())
	{
		UMovieSceneTimeWarpTrack* TimeWarpTrack = Cast<UMovieSceneTimeWarpTrack>(Track);
		if (TimeWarpTrack && !TimeWarpTrack->IsEvalDisabled())
		{
			FMovieSceneNestedSequenceTransform TimeWarpTransform = TimeWarpTrack->GenerateTransform();

			if (!TimeWarpTransform.IsIdentity())
			{
				if (TimeWarpTransform.IsLinear() && OutTransform.IsLinear())
				{
					OutTransform = FMovieSceneSequenceTransform(OutTransform.AsLinear() * TimeWarpTransform.AsLinear());
				}
				else
				{
					OutTransform.NestedTransforms.Add(TimeWarpTransform);
				}
			}

			// Only 1 timewarp track supported
			return;
		}
	}
}

bool UMovieSceneSubSection::GetValidatedInnerPlaybackRange(TRange<FFrameNumber>& OutInnerPlaybackRange) const
{
	UMovieSceneSequence* SequencePtr = GetSequence();
	if (SequencePtr != nullptr)
	{
		UMovieScene* MovieScenePtr = SequencePtr->GetMovieScene();
		if (MovieScenePtr != nullptr)
		{
			OutInnerPlaybackRange = GetValidatedInnerPlaybackRange(Parameters, *MovieScenePtr);
			return true;
		}
	}
	return false;
}

FMovieSceneSubSectionOriginOverrideMask UMovieSceneSubSection::GetMask() const
{
	return OriginOverrideMask;
}

void UMovieSceneSubSection::SetMask(EMovieSceneTransformChannel NewMask)
{
	OriginOverrideMask = NewMask;

	ChannelProxy = nullptr;
}

#if WITH_EDITOR

void UMovieSceneSubSection::SetKeyPreviewPosition(TOptional<FVector> InPosition)
{
	if(InPosition.IsSet())
	{
		KeyPreviewPosition = InPosition.GetValue();
	}
	
}

void UMovieSceneSubSection::SetKeyPreviewRotation(TOptional<FRotator> InRotation)
{
	if(InRotation.IsSet())
	{
		KeyPreviewRotation = InRotation.GetValue();
	}
}

void UMovieSceneSubSection::ResetKeyPreviewRotationAndLocation()
{
	KeyPreviewPosition.Reset();
	KeyPreviewRotation.Reset();
}

#endif

TRange<FFrameNumber> UMovieSceneSubSection::GetValidatedInnerPlaybackRange(const FMovieSceneSectionParameters& SubSectionParameters, const UMovieScene& InnerMovieScene)
{
	const TRange<FFrameNumber> InnerPlaybackRange = InnerMovieScene.GetPlaybackRange();
	TRangeBound<FFrameNumber> ValidatedLowerBound = InnerPlaybackRange.GetLowerBound();
	TRangeBound<FFrameNumber> ValidatedUpperBound = InnerPlaybackRange.GetUpperBound();
	if (ValidatedLowerBound.IsClosed() && ValidatedUpperBound.IsClosed())
	{
		const FFrameRate TickResolution = InnerMovieScene.GetTickResolution();
		const FFrameRate DisplayRate = InnerMovieScene.GetDisplayRate();
		const FFrameNumber OneFrameInTicks = FFrameRate::TransformTime(FFrameTime(1), DisplayRate, TickResolution).FloorToFrame();

		ValidatedLowerBound.SetValue(ValidatedLowerBound.GetValue() + SubSectionParameters.StartFrameOffset);
		ValidatedUpperBound.SetValue(FMath::Max(ValidatedUpperBound.GetValue() - SubSectionParameters.EndFrameOffset, ValidatedLowerBound.GetValue() + OneFrameInTicks));
		return TRange<FFrameNumber>(ValidatedLowerBound, ValidatedUpperBound);
	}
	return InnerPlaybackRange;
}

FString UMovieSceneSubSection::GetPathNameInMovieScene() const
{
	UMovieScene* OuterMovieScene = GetTypedOuter<UMovieScene>();
	check(OuterMovieScene);
	return GetPathName(OuterMovieScene);
}

FMovieSceneSequenceID UMovieSceneSubSection::GetSequenceID() const
{
	FString FullPath = GetPathNameInMovieScene();
	if (SubSequence)
	{
		FullPath += TEXT(" / ");
		FullPath += SubSequence->GetPathName();
	}

	return FMovieSceneSequenceID(FCrc::Strihash_DEPRECATED(*FullPath));
}

void UMovieSceneSubSection::PostLoad()
{
	FFrameRate LegacyFrameRate = GetLegacyConversionFrameRate();

	TOptional<double> StartOffsetToUpgrade;
	if (StartOffset_DEPRECATED != DeprecatedMagicNumber)
	{
		StartOffsetToUpgrade = StartOffset_DEPRECATED;

		StartOffset_DEPRECATED = DeprecatedMagicNumber;
	}
	else if (Parameters.StartOffset_DEPRECATED != 0.f)
	{
		StartOffsetToUpgrade = Parameters.StartOffset_DEPRECATED;
	}

	if (StartOffsetToUpgrade.IsSet())
	{
		FFrameNumber StartFrame = UpgradeLegacyMovieSceneTime(this, LegacyFrameRate, StartOffsetToUpgrade.GetValue());
		Parameters.StartFrameOffset = StartFrame;
	}

	if (TimeScale_DEPRECATED != DeprecatedMagicNumber)
	{
		Parameters.TimeScale = TimeScale_DEPRECATED;

		TimeScale_DEPRECATED = DeprecatedMagicNumber;
	}

	if (PrerollTime_DEPRECATED != DeprecatedMagicNumber)
	{
		Parameters.PrerollTime_DEPRECATED = PrerollTime_DEPRECATED;

		PrerollTime_DEPRECATED = DeprecatedMagicNumber;
	}

	// Pre and post roll is now supported generically
	if (Parameters.PrerollTime_DEPRECATED > 0.f)
	{
		FFrameNumber ClampedPreRollFrames = UpgradeLegacyMovieSceneTime(this, LegacyFrameRate, Parameters.PrerollTime_DEPRECATED);
		SetPreRollFrames(ClampedPreRollFrames.Value);
	}

	if (Parameters.PostrollTime_DEPRECATED > 0.f)
	{
		FFrameNumber ClampedPostRollFrames = UpgradeLegacyMovieSceneTime(this, LegacyFrameRate, Parameters.PostrollTime_DEPRECATED);
		SetPreRollFrames(ClampedPostRollFrames.Value);
	}

	Super::PostLoad();
}

bool UMovieSceneSubSection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	if (SubSequence)
	{
		const int32 EntityIndex   = OutFieldBuilder->FindOrAddEntity(this, 0);
		const int32 MetaDataIndex = OutFieldBuilder->AddMetaData(InMetaData);
		OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex, MetaDataIndex);
	}

	return true;
}

void UMovieSceneSubSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	OutImportedEntity->AddBuilder(
		FEntityBuilder().AddTag(FBuiltInComponentTypes::Get()->Tags.Root)
	);

	BuildDefaultSubSectionComponents(EntityLinker, Params, OutImportedEntity);
}

void UMovieSceneSubSection::SetSequence(UMovieSceneSequence* Sequence)
{
	if (Sequence != nullptr)
	{
		// Confirm compatibility of the sequence as a subsequence with the parent sequence
		if (UMovieSceneSequence* CurrentSequence = GetTypedOuter<UMovieSceneSequence>();
			CurrentSequence != nullptr && !CurrentSequence->IsSubSequenceCompatible(*Sequence))
		{
			return;
		}
	}

	if (!TryModify())
	{
		return;
	}

	SubSequence = Sequence;

#if WITH_EDITOR
	OnSequenceChangedDelegate.ExecuteIfBound(SubSequence);
#endif
}

UMovieSceneSequence* UMovieSceneSubSection::GetSequence() const
{
	return SubSequence;
}

FMovieSceneTimeWarpVariant* UMovieSceneSubSection::GetTimeWarp()
{
	return &Parameters.TimeScale;
}

UObject* UMovieSceneSubSection::GetSourceObject() const
{
	return GetSequence();
}

#if WITH_EDITOR
void UMovieSceneSubSection::PreEditChange(FProperty* PropertyAboutToChange)
{
	if (PropertyAboutToChange && PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(UMovieSceneSubSection, SubSequence))
	{
		// Store the current subsequence in case it needs to be restored in PostEditChangeProperty because the new value would introduce a circular dependency
		PreviousSubSequence = SubSequence;
	}

	return Super::PreEditChange(PropertyAboutToChange);
}

void UMovieSceneSubSection::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UMovieSceneSubSection, SubSequence))
	{
		// Check whether the subsequence that was just set has tracks that contain the sequence that this subsection is in.
		const UMovieScene* SubSequenceMovieScene = SubSequence ? SubSequence->GetMovieScene() : nullptr;

		UMovieSceneSubTrack* TrackOuter = Cast<UMovieSceneSubTrack>(GetOuter());

		if (SubSequenceMovieScene && TrackOuter)
		{
			if (UMovieSceneSequence* CurrentSequence = TrackOuter->GetTypedOuter<UMovieSceneSequence>())
			{
				// Confirm that the newly set subsequence is compatible, otherwise we need to revert to previous
				if (CurrentSequence->IsSubSequenceCompatible(*SubSequence))
				{
					TArray<UMovieSceneSubTrack*> SubTracks;

					for (UMovieSceneTrack* Track : SubSequenceMovieScene->GetTracks())
					{
						if (UMovieSceneSubTrack* SubTrack = Cast<UMovieSceneSubTrack>(Track))
						{
							SubTracks.Add(SubTrack);
						}
					}

					for (const FMovieSceneBinding& Binding : SubSequenceMovieScene->GetBindings())
					{
						for (UMovieSceneTrack* Track : SubSequenceMovieScene->FindTracks(UMovieSceneSubTrack::StaticClass(), Binding.GetObjectGuid()))
						{
							if (UMovieSceneSubTrack* SubTrack = Cast<UMovieSceneSubTrack>(Track))
							{
								SubTracks.Add(SubTrack);
							}
						}
					}

					for (UMovieSceneSubTrack* SubTrack : SubTracks)
					{
						if (SubTrack->ContainsSequence(*CurrentSequence, true))
						{
							UE_LOGF(LogMovieScene, Error, "Invalid level sequence %ls. It is already contained by: %ls.", *SubSequence->GetDisplayName().ToString(), *CurrentSequence->GetDisplayName().ToString());

							// Restore to the previous sub sequence because there was a circular dependency
							SubSequence = PreviousSubSequence;
							break;
						}
					}
				}
				else
				{
					UE_LOGF(LogMovieScene, Error, "Level sequence %ls is not a compatible subsequence for %ls.", *SubSequence->GetDisplayName().ToString(), *CurrentSequence->GetDisplayName().ToString());
					SubSequence = PreviousSubSequence;
				}
			}
		}

		PreviousSubSequence = nullptr;
	}

	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FMovieSceneSectionParameters, TimeScale))
	{
		ChannelProxy = nullptr;
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);

	// recreate runtime instance when sequence is changed
	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UMovieSceneSubSection, SubSequence))
	{
		OnSequenceChangedDelegate.ExecuteIfBound(SubSequence);
	}
}
#endif

TOptional<TRange<FFrameNumber> > UMovieSceneSubSection::GetAutoSizeRange() const
{
	UMovieScene* MovieScene = SubSequence ? SubSequence->GetMovieScene() : nullptr;
	if (MovieScene)
	{
		// We probably want to just auto-size the section to the sub-sequence's scaled playback range... if this section
		// is looping, however, it's hard to know what we want to do. Let's just size it to one loop.
		const FMovieSceneInverseSequenceTransform InnerToOuter = OuterToInnerTransform().Inverse();
		const TRange<FFrameNumber> InnerPlaybackRange = UMovieSceneSubSection::GetValidatedInnerPlaybackRange(Parameters, *MovieScene);

		const FFrameTime IncAutoStartTime = InnerToOuter.TryTransformTime(UE::MovieScene::DiscreteInclusiveLower(InnerPlaybackRange)).Get(InnerPlaybackRange.GetLowerBoundValue());
		const FFrameTime ExcAutoEndTime   = InnerToOuter.TryTransformTime(UE::MovieScene::DiscreteExclusiveUpper(InnerPlaybackRange)).Get(InnerPlaybackRange.GetUpperBoundValue());

		return TRange<FFrameNumber>(GetInclusiveStartFrame(), GetInclusiveStartFrame() + (ExcAutoEndTime.RoundToFrame() - IncAutoStartTime.RoundToFrame()));
	}

	return Super::GetAutoSizeRange();
}

void UMovieSceneSubSection::TrimSection( FQualifiedFrameTime TrimTime, bool bTrimLeft, bool bDeleteKeys)
{
	TRange<FFrameNumber> InitialRange = GetRange();
	if ( !InitialRange.Contains( TrimTime.Time.GetFrame() ) )
	{
		return;
	}

	SetFlags(RF_Transactional);
	if (!TryModify())
	{
		return;
	}

	// If trimming off the left, set the offset of the shot
	if (bTrimLeft && InitialRange.GetLowerBound().IsClosed() && GetSequence())
	{
		// Sections need their offsets calculated in their local resolution. Different sequences can have different tick resolutions 
		// so we need to transform from the parent resolution to the local one before splitting them.
		UMovieScene* LocalMovieScene = GetSequence()->GetMovieScene();
		const FFrameRate LocalTickResolution = LocalMovieScene->GetTickResolution();
		const FFrameTime LocalTickResolutionTrimTime = FFrameRate::TransformTime(TrimTime.Time, TrimTime.Rate, LocalTickResolution);

		// The new first loop start offset is where the trim time fell inside the sub-sequence (this time is already
		// normalized in the case of looping sub-sequences).
		const FMovieSceneSequenceTransform OuterToInner(OuterToInnerTransform());
		const FFrameTime LocalTrimTime = OuterToInner.TransformTime(LocalTickResolutionTrimTime);
		// LocalTrimTime is now in the inner sequence timespace, but StartFrameOffset is an offset from the inner sequence's own
		// playback start time, so we need to account for that.
		TRange<FFrameNumber> LocalPlaybackRange = LocalMovieScene->GetPlaybackRange();
		const FFrameNumber LocalPlaybackStart = LocalPlaybackRange.HasLowerBound() ? LocalPlaybackRange.GetLowerBoundValue() : FFrameNumber(0);
		FFrameNumber NewStartOffset = LocalTrimTime.FrameNumber - LocalPlaybackStart;

		// Make sure we don't have negative offsets (this shouldn't happen, though).
		NewStartOffset = FMath::Max(FFrameNumber(0), NewStartOffset);
		
		const bool bCanLoop = Parameters.bCanLoop;
		if (!bCanLoop)
		{
			Parameters.StartFrameOffset = NewStartOffset;
		}
		else
		{
			Parameters.FirstLoopStartFrameOffset = NewStartOffset;
		}
	}

	// Actually trim the section range!
	UMovieSceneSection::TrimSection(TrimTime, bTrimLeft, bDeleteKeys);
}

void UMovieSceneSubSection::GetSnapTimes(TArray<FFrameNumber>& OutSnapTimes, bool bGetSectionBorders) const
{
	using namespace UE::MovieScene;

	Super::GetSnapTimes(OutSnapTimes, bGetSectionBorders);

	const FFrameNumber StartFrame = GetInclusiveStartFrame();
	const FFrameNumber EndFrame   = GetExclusiveEndFrame();

	UMovieSceneSequence* Sequence = GetSequence();
	UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;
	if (!MovieScene)
	{
		return;
	}

	auto VisitBoundary = [&OutSnapTimes](FFrameTime InTime)
	{
		OutSnapTimes.Add(InTime.RoundToFrame());
		return true;
	};

	FMovieSceneSequenceTransform OuterToInner = OuterToInnerTransform();

	if (!OuterToInner.ExtractBoundariesWithinRange(StartFrame, EndFrame, VisitBoundary))
	{
		FMovieSceneInverseSequenceTransform InnerToOuterTransform = OuterToInner.Inverse();

		TRange<FFrameNumber> PlaybackRange = MovieScene->GetPlaybackRange();

		TOptional<FFrameTime> SequenceStart = InnerToOuterTransform.TryTransformTime(PlaybackRange.GetLowerBoundValue());
		TOptional<FFrameTime> SequenceEnd   = InnerToOuterTransform.TryTransformTime(PlaybackRange.GetUpperBoundValue());

		if (SequenceStart && SequenceStart.GetValue() >= StartFrame && SequenceStart.GetValue() < EndFrame)
		{
			VisitBoundary(SequenceStart.GetValue());
		}

		if (SequenceEnd && SequenceEnd.GetValue() >= StartFrame && SequenceEnd.GetValue() < EndFrame)
		{
			VisitBoundary(SequenceEnd.GetValue());
		}
	}
}

void UMovieSceneSubSection::MigrateFrameTimes(FFrameRate SourceRate, FFrameRate DestinationRate)
{
	MigrateFrameTimes(UE::MovieScene::FFrameRateRetiming(SourceRate, DestinationRate));
}

void UMovieSceneSubSection::MigrateFrameTimes(const UE::MovieScene::IRetimingInterface& Retimer)
{
	if (Parameters.StartFrameOffset.Value > 0)
	{
		Parameters.StartFrameOffset = Retimer.RemapTime(Parameters.StartFrameOffset);
	}
	if (Parameters.EndFrameOffset.Value > 0)
	{
		Parameters.EndFrameOffset = Retimer.RemapTime(Parameters.EndFrameOffset);
	}
	if (Parameters.FirstLoopStartFrameOffset.Value > 0)
	{
		Parameters.FirstLoopStartFrameOffset = Retimer.RemapTime(Parameters.FirstLoopStartFrameOffset);
	}
}

FMovieSceneSubSequenceData UMovieSceneSubSection::GenerateSubSequenceData(const FSubSequenceInstanceDataParams& Params) const
{
	return FMovieSceneSubSequenceData(*this);
}

#if WITH_EDITOR
bool UMovieSceneSubSection::IsTransformOriginEditable() const
{
	const EMovieSceneTransformChannel SectionTransformChannels = OriginOverrideMask.GetChannels();

	const bool bChannelsActive = EnumHasAnyFlags(SectionTransformChannels, EMovieSceneTransformChannel::Translation) || EnumHasAnyFlags(SectionTransformChannels, EMovieSceneTransformChannel::Rotation);
	
	return IsActive() && !IsLocked() && bChannelsActive;
}
#endif

FFrameNumber UMovieSceneSubSection::MapTimeToSectionFrame(FFrameTime InPosition) const
{
	FFrameNumber LocalPosition = ((InPosition - Parameters.StartFrameOffset) * OuterToInnerTransform()).GetFrame();
	return LocalPosition;
}

bool UMovieSceneSubSection::HasAnyChannelData() const
{
	bool bHasAnyData = false;

	bHasAnyData |= Translation[0].HasAnyData();
	bHasAnyData |= Translation[1].HasAnyData();
	bHasAnyData |= Translation[2].HasAnyData();
	bHasAnyData |= Rotation[0].HasAnyData();
	bHasAnyData |= Rotation[1].HasAnyData();
	bHasAnyData |= Rotation[2].HasAnyData();

	return bHasAnyData;
}

void UMovieSceneSubSection::BuildDefaultSubSectionComponents(UMovieSceneEntitySystemLinker* EntityLinker, const UE::MovieScene::FEntityImportParams& Params, UE::MovieScene::FImportedEntity* OutImportedEntity) const
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();

	const bool bHasEasing = Easing.GetEaseInDuration() > 0 || Easing.GetEaseOutDuration() > 0;


	// When interrogating, the sequence ID is known, and set directly, so default to using this value.
	FMovieSceneSequenceID ResolvedSequenceID = MovieSceneSequenceID::Invalid;
	
	const TEntitySystemLinkerExtensionID<IInterrogationExtension> ID = IInterrogationExtension::GetExtensionID();
	
	const IInterrogationExtension* Interrogation = EntityLinker->FindExtension<IInterrogationExtension>(ID);

	if (Interrogation)
	{
		const FMovieSceneSequenceHierarchy* Hierarchy = Interrogation->GetHierarchy();
		const FSubSequencePath PathToRoot = FSubSequencePath(Params.Sequence.SequenceID, Hierarchy);
		ResolvedSequenceID = PathToRoot.ResolveChildSequenceID(this->GetSequenceID());
	}
	// During normal evaluation (i.e. not interrogating) the instance registry will have its instance populated, and the sequence ID can be resolved this way.
	else if (EntityLinker->GetInstanceRegistry()->IsHandleValid(Params.Sequence.InstanceHandle))
	{
		const FSubSequencePath PathToRoot = EntityLinker->GetInstanceRegistry()->GetInstance(Params.Sequence.InstanceHandle).GetSubSequencePath();
		ResolvedSequenceID = PathToRoot.ResolveChildSequenceID(this->GetSequenceID());
	}

	EMovieSceneTransformChannel Channels = OriginOverrideMask.GetChannels();

	const bool ActiveChannelsMask[] = {
		EnumHasAnyFlags(Channels, EMovieSceneTransformChannel::TranslationX) && Translation[0].HasAnyData(),
		EnumHasAnyFlags(Channels, EMovieSceneTransformChannel::TranslationY) && Translation[1].HasAnyData(),
		EnumHasAnyFlags(Channels, EMovieSceneTransformChannel::TranslationZ) && Translation[2].HasAnyData(),
		EnumHasAnyFlags(Channels, EMovieSceneTransformChannel::RotationX) && Rotation[0].HasAnyData(),
		EnumHasAnyFlags(Channels, EMovieSceneTransformChannel::RotationY) && Rotation[1].HasAnyData(),
		EnumHasAnyFlags(Channels, EMovieSceneTransformChannel::RotationZ) && Rotation[2].HasAnyData(),
	};

	bool bKeyPreviewPositionIsSet = false;
	bool bKeyPreviewRotationIsSet = false;

#if WITH_EDITOR
	bKeyPreviewPositionIsSet = KeyPreviewPosition.IsSet();
	bKeyPreviewRotationIsSet = KeyPreviewRotation.IsSet();
#endif
	
	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.Add(Components->SequenceID, ResolvedSequenceID)
		.AddTag(Components->Tags.SubInstance)
		.AddConditional(Components->HierarchicalEasingProvider, ResolvedSequenceID, bHasEasing)
		.AddConditional(Components->DoubleChannel[0], &Translation[0], ActiveChannelsMask[0] && !bKeyPreviewPositionIsSet)
		.AddConditional(Components->DoubleChannel[1], &Translation[1], ActiveChannelsMask[1] && !bKeyPreviewPositionIsSet)
		.AddConditional(Components->DoubleChannel[2], &Translation[2], ActiveChannelsMask[2] && !bKeyPreviewPositionIsSet)
		.AddConditional(Components->DoubleChannel[3], &Rotation[0], ActiveChannelsMask[3] && !bKeyPreviewRotationIsSet)
		.AddConditional(Components->DoubleChannel[4], &Rotation[1], ActiveChannelsMask[4] && !bKeyPreviewRotationIsSet)
		.AddConditional(Components->DoubleChannel[5], &Rotation[2], ActiveChannelsMask[5] && !bKeyPreviewRotationIsSet)
	);

	// Build Key preview entity data. Since the channel data is not written when we have preview data, this data will be used in the transform origin system.
#if WITH_EDITOR
	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.AddConditional(Components->DoubleResult[0], KeyPreviewPosition.IsSet() ? KeyPreviewPosition.GetValue().X : 0.0f, EnumHasAnyFlags(Channels, EMovieSceneTransformChannel::TranslationX) && KeyPreviewPosition.IsSet())
		.AddConditional(Components->DoubleResult[1], KeyPreviewPosition.IsSet() ? KeyPreviewPosition.GetValue().Y : 0.0f, EnumHasAnyFlags(Channels, EMovieSceneTransformChannel::TranslationY) && KeyPreviewPosition.IsSet())
		.AddConditional(Components->DoubleResult[2], KeyPreviewPosition.IsSet() ? KeyPreviewPosition.GetValue().Z : 0.0f, EnumHasAnyFlags(Channels, EMovieSceneTransformChannel::TranslationZ) && KeyPreviewPosition.IsSet())
		.AddConditional(Components->DoubleResult[3], KeyPreviewPosition.IsSet() ? KeyPreviewRotation.GetValue().Roll : 0.0f, EnumHasAnyFlags(Channels, EMovieSceneTransformChannel::RotationX) && KeyPreviewRotation.IsSet())
		.AddConditional(Components->DoubleResult[4], KeyPreviewPosition.IsSet() ? KeyPreviewRotation.GetValue().Pitch : 0.0f, EnumHasAnyFlags(Channels, EMovieSceneTransformChannel::RotationY) && KeyPreviewRotation.IsSet())
		.AddConditional(Components->DoubleResult[5], KeyPreviewPosition.IsSet() ? KeyPreviewRotation.GetValue().Yaw : 0.0f, EnumHasAnyFlags(Channels, EMovieSceneTransformChannel::RotationX) && KeyPreviewRotation.IsSet())
	);
#endif
	
}


