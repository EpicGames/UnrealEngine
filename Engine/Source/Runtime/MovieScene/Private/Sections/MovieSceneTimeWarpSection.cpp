// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneTimeWarpSection.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "MovieSceneSequence.h"
#include "Variants/MovieScenePlayRateCurve.h"
#include "Evaluation/MovieSceneSequenceTransform.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneTimeWarpSection)

UMovieSceneTimeWarpSection::UMovieSceneTimeWarpSection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSupportsInfiniteRange = true;
	SetRange(TRange<FFrameNumber>::All());
}

void UMovieSceneTimeWarpSection::PostLoad()
{
	Super::PostLoad();

	// Ensure any custom getter's outer is a public object so the getter can be imported cross-package.
	// When a parent sequence compiles its hierarchy, the sub-sequence's time warp getter ends up
	// stored in FMovieSceneSubSequenceData::OuterToInnerTransform. Serializing that compiled data
	// requires a cross-package import of the getter, which needs an importable (RF_Public) outer chain.
	if (TimeWarp.GetType() == EMovieSceneTimeWarpType::Custom)
	{
		if (UMovieSceneTimeWarpGetter* Getter = TimeWarp.AsCustom())
		{
			if (UObject* GetterOuter = Getter->GetOuter())
			{
				if (!GetterOuter->HasAnyFlags(RF_Public))
				{
					if (UMovieSceneSequence* Sequence = GetTypedOuter<UMovieSceneSequence>())
					{
						Getter->Rename(nullptr, Sequence, REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional | REN_AllowPackageLinkerMismatch);
					}
				}
			}
		}
	}
}

FMovieSceneNestedSequenceTransform UMovieSceneTimeWarpSection::GenerateTransform() const
{
	FFrameNumber Offset = HasStartFrame() ? GetInclusiveStartFrame() : 0;
	return FMovieSceneNestedSequenceTransform(Offset, TimeWarp.ShallowCopy());
}

EMovieSceneChannelProxyType UMovieSceneTimeWarpSection::CacheChannelProxy()
{
	FMovieSceneChannelProxyData Channels;

	if (TimeWarp.GetType() == EMovieSceneTimeWarpType::Custom)
	{
		if (UMovieSceneTimeWarpGetter* Custom = TimeWarp.AsCustom())
		{
			Custom->PopulateChannelProxy(Channels, UMovieSceneTimeWarpGetter::EAllowTopLevelChannels::Yes);
		}
	}

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(Channels));
	return EMovieSceneChannelProxyType::Dynamic;
}

FMovieSceneTimeWarpVariant* UMovieSceneTimeWarpSection::GetTimeWarp()
{
	return &TimeWarp;
}

#if WITH_EDITOR

bool UMovieSceneTimeWarpSection::Modify(bool bAlwaysMarkDirty)
{
	UMovieSceneTimeWarpGetter* Custom = TimeWarp.GetType() == EMovieSceneTimeWarpType::Custom ? TimeWarp.AsCustom() : nullptr;
	if (Custom)
	{
		Custom->Modify(bAlwaysMarkDirty);
	}
	return Super::Modify(bAlwaysMarkDirty);
}

void UMovieSceneTimeWarpSection::PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMovieSceneTimeWarpSection, TimeWarp))
	{
		ChannelProxy = nullptr;
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif // WITH_EDITOR
