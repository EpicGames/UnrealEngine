// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Misc/App.h"
#include "AutoRTFM.h"
#include "EntitySystem/MovieSceneEntityBuilder.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneTaskScheduler.h"
#include "EntitySystem/MovieSceneEntityFactoryTemplates.h"

#if WITH_DEV_AUTOMATION_TESTS

#define TEST_CHECK_TRUE(b) do \
{ \
	if (!(b)) \
	{ \
		FString String(FString::Printf(TEXT("FAILED: %s:%u"), TEXT(__FILE__), __LINE__)); \
		ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Info, String)); \
		return false; \
	} \
} while(false)

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutoRTFMMovieSceneTests, "AutoRTFM + Movie Scene", EAutomationTestFlags::EngineFilter | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ServerContext | EAutomationTestFlags::CommandletContext)

using namespace UE::MovieScene;

struct FMyTask final : ITaskContext
{
	void Run(FEntityAllocationWriteContext& WriteContext) const { WasHit += 1; }

	static uint32 WasHit;
};

uint32 FMyTask::WasHit = 0;

bool FAutoRTFMMovieSceneTests::RunTest(const FString & Parameters)
{
	if (!AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled())
	{
		ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Info, TEXT("SKIPPED 'FAutoRTFMMovieSceneTests' test. AutoRTFM disabled.")));
		return true;
	}
	
	{
		// Reset this as we'll test against it later.
		FMyTask::WasHit = 0;

		FComponentRegistry ComponentRegistry;
		FEntityManager EntityManager;
		FEntitySystemScheduler Scheduler(&EntityManager);

		TStatId StatId;

		Scheduler.BeginConstruction();

		// Create a chain of tasks that depend on each other (A -> B -> C ... etc) to ensures
		// that we hit the codepath in FEntitySystemScheduler::PrerequisiteCompleted
		FTaskID LastTask;
		for (int32 TaskIndex = 0; TaskIndex < 10; ++TaskIndex)
		{
			FTaskID ThisTask = Scheduler.AddTask<FMyTask>(FTaskParams(StatId));
			if (LastTask)
			{
				Scheduler.AddPrerequisite(LastTask, ThisTask);
			}
			LastTask = ThisTask;
		}

		Scheduler.EndConstruction();

		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				Scheduler.ExecuteTasks();
				AutoRTFM::AbortTransaction();
			});

		TEST_CHECK_TRUE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		TEST_CHECK_TRUE(0 == FMyTask::WasHit);//-V547

		Result = AutoRTFM::Transact([&]
			{
				Scheduler.ExecuteTasks();
			});

		TEST_CHECK_TRUE(AutoRTFM::ETransactionResult::Committed == Result);
		TEST_CHECK_TRUE(10 == FMyTask::WasHit);//-V547
	}

	{
		// Reset this as we'll test against it later.
		FMyTask::WasHit = 0;

		FComponentRegistry ComponentRegistry;
		FEntityManager EntityManager;
		FEntitySystemScheduler Scheduler(&EntityManager);

		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				TStatId StatId;
				Scheduler.BeginConstruction();
				Scheduler.AddTask<FMyTask>(FTaskParams(StatId));
				Scheduler.EndConstruction();
				Scheduler.ExecuteTasks();
				AutoRTFM::AbortTransaction();
			});

		TEST_CHECK_TRUE(AutoRTFM::ETransactionResult::AbortedByRequest == Result);
		TEST_CHECK_TRUE(0 == FMyTask::WasHit);

		Result = AutoRTFM::Transact([&]
			{
				TStatId StatId;
				Scheduler.BeginConstruction();
				Scheduler.AddTask<FMyTask>(FTaskParams(StatId));
				Scheduler.EndConstruction();
				Scheduler.ExecuteTasks();
			});

		TEST_CHECK_TRUE(AutoRTFM::ETransactionResult::Committed == Result);
		TEST_CHECK_TRUE(1 == FMyTask::WasHit);
	}

	{
		FComponentHeader Header;
		FEntityManager EntityManager;
		FEntityAllocationWriteContext Context(EntityManager);

		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				TComponentLock<UE::MovieScene::FWriteErased> Lock(&Header, EComponentHeaderLockMode::Mutex, Context);
			});

		TEST_CHECK_TRUE(AutoRTFM::ETransactionResult::Committed == Result);
	}

	// Test we can construct and destruct an entity system scheduler.
	{
		FEntityManager EntityManager;

		AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]
			{
				FEntitySystemScheduler Scheduler(&EntityManager);
			});

		TEST_CHECK_TRUE(AutoRTFM::ETransactionResult::Committed == Result);
	}

	return true;
}

#undef TEST_CHECK_TRUE

#endif //WITH_DEV_AUTOMATION_TESTS
