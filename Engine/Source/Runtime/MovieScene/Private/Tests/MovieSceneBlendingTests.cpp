// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "Evaluation/Blending/MovieSceneBlendingAccumulator.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Evaluation/Blending/MovieSceneMultiChannelBlending.h"

#define LOCTEXT_NAMESPACE "MovieSceneBlendingTests"

namespace UE
{
namespace MovieScene
{
namespace Test
{

static const int32 GBlendingStartingValue = 0xefefefef;
static int32 GBlendingTestValue = GBlendingStartingValue;

} // namespace Test
} // namespace MovieScene
} // namespace UE

struct FInt32Actuator : TMovieSceneBlendingActuator<int32>
{
	FInt32Actuator() : TMovieSceneBlendingActuator<int32>(GetID()) {}

	static FMovieSceneBlendingActuatorID GetID()
	{
		static FMovieSceneAnimTypeID TypeID = FMovieSceneAnimTypeID::Unique();
		return FMovieSceneBlendingActuatorID(TypeID);
	}

	virtual void Actuate(UObject* InObject, TCallTraits<int32>::ParamType InValue, const TBlendableTokenStack<int32>& OriginalStack, const FMovieSceneContext& Context, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player)
	{
		using namespace UE::MovieScene::Test;
		
		ensure(!InObject);
		GBlendingTestValue = InValue;
	}

	virtual int32 RetrieveCurrentValue(UObject* InObject, IMovieScenePlayer* Player) const
	{
		using namespace UE::MovieScene::Test;

		return GBlendingTestValue;
	}
};

struct FNullPlayer : IMovieScenePlayer
{
	FMovieSceneRootEvaluationTemplateInstance RootInstance;
	FMovieSceneRootEvaluationTemplateInstance& GetEvaluationTemplate()
	{
		return RootInstance;
	}

	virtual void SetViewportSettings(const TMap<FViewportClient*, EMovieSceneViewportParams>& ViewportParamsMap) {}
	virtual void GetViewportSettings(TMap<FViewportClient*, EMovieSceneViewportParams>& ViewportParamsMap) const {}
	virtual EMovieScenePlayerStatus::Type GetPlaybackStatus() const { return EMovieScenePlayerStatus::Stopped; }
	virtual void SetPlaybackStatus(EMovieScenePlayerStatus::Type InPlaybackStatus) {}
};

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieSceneBlendingTest, "System.Engine.Sequencer.Blending.Basic", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieSceneBlendingTest::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene::Test;

	const FMovieSceneBlendingActuatorID ID = FInt32Actuator::GetID();
	const FMovieSceneEvaluationScope Scope(FMovieSceneEvaluationKey(), EMovieSceneCompletionMode::KeepState);
	const FMovieSceneContext Context(FMovieSceneEvaluationRange(0, FFrameRate()));

	FNullPlayer Player;
	FPersistentEvaluationData PersistentDataProxy(Player);

	FMovieSceneBlendingAccumulator Accumulator;
	Accumulator.DefineActuator(ID, MakeShared<FInt32Actuator>());

	{
		// Result should be (1*1) + (1*1) + (10*0.5)
		Accumulator.BlendToken<int32>(ID, Scope, Context, 1, EMovieSceneBlendType::Absolute, 1.f);
		Accumulator.BlendToken<int32>(ID, Scope, Context, 1, EMovieSceneBlendType::Additive, 1.f);
		Accumulator.BlendToken<int32>(ID, Scope, Context, 10, EMovieSceneBlendType::Additive, .5f);

		Accumulator.Apply(Context, PersistentDataProxy, Player);

		int32 Expected = 7;
		if (GBlendingTestValue != Expected)
		{
			AddError(FString::Printf(TEXT("Expected result 1 to be %d, actual %d."), GBlendingTestValue, Expected));
		}
	}

	GBlendingTestValue = GBlendingStartingValue;

	{
		// Result should be GBlendingStartingValue + 500 + 10
		Accumulator.BlendToken<int32>(ID, Scope, Context, 10, EMovieSceneBlendType::Additive, 1.f);
		Accumulator.BlendToken<int32>(ID, Scope, Context, 500, EMovieSceneBlendType::Relative, 1.f);

		Accumulator.Apply(Context, PersistentDataProxy, Player);

		int32 Expected = GBlendingStartingValue + 510;
		if (GBlendingTestValue != Expected) //-V547
		{
			AddError(FString::Printf(TEXT("Expected result 2 to be %d, actual %d."), GBlendingTestValue, Expected));
		}
	}

	GBlendingTestValue = GBlendingStartingValue;

	{
		// Result should be 85
		Accumulator.BlendToken<int32>(ID, Scope, Context, 7, EMovieSceneBlendType::Absolute, 1.f);
		Accumulator.BlendToken<int32>(ID, Scope, Context, 18, EMovieSceneBlendType::Absolute, 1.f);
		Accumulator.BlendToken<int32>(ID, Scope, Context, 31, EMovieSceneBlendType::Absolute, 1.f);
		Accumulator.BlendToken<int32>(ID, Scope, Context, 29, EMovieSceneBlendType::Absolute, 1.f);

		Accumulator.Apply(Context, PersistentDataProxy, Player);

		int32 Expected = 85 / 4;
		if (GBlendingTestValue != Expected) //-V547
		{
			AddError(FString::Printf(TEXT("Expected result 3 to be %d, actual %d."), GBlendingTestValue, Expected));
		}
	}

	GBlendingTestValue = GBlendingStartingValue;

	
	{
		// (7483647 + (217 * 0.5) + (97483647 * 0.1)) / 1.6 = 10770075.125
		Accumulator.BlendToken<int32>(ID, Scope, Context, 7483647, EMovieSceneBlendType::Absolute, 1.f);
		Accumulator.BlendToken<int32>(ID, Scope, Context, 217, EMovieSceneBlendType::Absolute, .5f);
		Accumulator.BlendToken<int32>(ID, Scope, Context, 97483647, EMovieSceneBlendType::Absolute, .1f);

		Accumulator.Apply(Context, PersistentDataProxy, Player);

		int32 Expected = 10770075;
		if (GBlendingTestValue != Expected) //-V547
		{
			AddError(FString::Printf(TEXT("Expected result 4 to be %d, actual %d."), GBlendingTestValue, Expected));
		}
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieSceneBlendingStressTest, "System.Engine.Sequencer.Blending.StressTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::Disabled)
bool FMovieSceneBlendingStressTest::RunTest(const FString& Parameters)
{
	TArray<FMovieSceneBlendingActuatorID> ActuatorIDs;
	int32 NumActuatorTypes = 100;
	for (int32 Index = 0; Index < NumActuatorTypes; ++Index)
	{
		ActuatorIDs.Add(FMovieSceneBlendingActuatorID(FMovieSceneAnimTypeID::Unique()));
	}

	const FMovieSceneBlendingActuatorID ID = FInt32Actuator::GetID();
	const FMovieSceneEvaluationScope Scope(FMovieSceneEvaluationKey(), EMovieSceneCompletionMode::KeepState);
	const FMovieSceneContext Context(FMovieSceneEvaluationRange(0, FFrameRate()));
	
	FNullPlayer Player;
	FPersistentEvaluationData PersistentDataProxy(Player);

	FMovieSceneBlendingAccumulator Accumulator;

	uint32 NumIterations = 1000000;
	uint32 NumTokens = 100;

	for (uint32 Iteration = 0; Iteration < NumIterations; ++Iteration)
	{
		for (uint32 Token = 0; Token < NumTokens; ++Token)
		{
			FMovieSceneBlendingActuatorID ThisID = ActuatorIDs[Token % NumActuatorTypes];
			if (!Accumulator.FindActuator<int32>(ThisID))
			{
				Accumulator.DefineActuator(ThisID, MakeShared<FInt32Actuator>());
			}

			Accumulator.BlendToken(ThisID, Scope, Context, 1, EMovieSceneBlendType::Absolute, 1.f);
		}

		Accumulator.Apply(Context, PersistentDataProxy, Player);
	}

	return true;
}


#undef LOCTEXT_NAMESPACE
