// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneTestObjects.h"
#include "Misc/AutomationTest.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "Compilation/MovieSceneSegmentCompiler.h"
#include "Compilation/MovieSceneCompilerRules.h"
#include "Evaluation/MovieSceneEvaluationTrack.h"
#include "Evaluation/MovieSceneEvaluationField.h"
#include "Algo/Find.h"
#include "UObject/Package.h"
#include "MovieSceneTimeHelpers.h"


// ~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
// TODO: Reimplement compiler automation tests
// ~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
UE_MOVIESCENE_TODO(Reimplement compiler automation tests)

#if 0
#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieSceneCompilerPerfTest, "System.Engine.Sequencer.Compiler.Perf", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::Disabled)
bool FMovieSceneCompilerPerfTest::RunTest(const FString& Parameters)
{
	static bool  bInvalidateEveryIteration = false;
	static int32 NumIterations             = 1000000;

	FFrameRate TickResolution(1000, 1);

	UTestMovieSceneSequence* Sequence = NewObject<UTestMovieSceneSequence>(GetTransientPackage());
	Sequence->MovieScene->SetTickResolutionDirectly(TickResolution);

	for (int32 i = 0; i < 100; ++i)
	{
		UTestMovieSceneTrack* Track = Sequence->MovieScene->AddTrack<UTestMovieSceneTrack>();

		int32 NumSections = FMath::Rand() % 10;
		for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
		{
			UTestMovieSceneSection* Section = NewObject<UTestMovieSceneSection>(Track);

			double StartSeconds    = FMath::FRand() * 60.f;
			double DurationSeconds = FMath::FRand() * 60.f;
			Section->SetRange(TRange<FFrameNumber>::Inclusive( (StartSeconds * TickResolution).RoundToFrame(), ((StartSeconds + DurationSeconds) * TickResolution).RoundToFrame() ));
			Track->SectionArray.Add(Section);
		}
	}

	struct FTestMovieScenePlayer : IMovieScenePlayer
	{
		FMovieSceneRootEvaluationTemplateInstance RootInstance;
		virtual FMovieSceneRootEvaluationTemplateInstance& GetEvaluationTemplate() override { return RootInstance; }
		virtual void SetViewportSettings(const TMap<FViewportClient*, EMovieSceneViewportParams>& ViewportParamsMap) override {}
		virtual void GetViewportSettings(TMap<FViewportClient*, EMovieSceneViewportParams>& ViewportParamsMap) const override {}
		virtual EMovieScenePlayerStatus::Type GetPlaybackStatus() const override { return EMovieScenePlayerStatus::Playing; }
		virtual void SetPlaybackStatus(EMovieScenePlayerStatus::Type InPlaybackStatus) override {}
	} TestPlayer;

	UMovieSceneCompiledDataManager* CompiledDataManager = UMovieSceneCompiledDataManager::GetPrecompiledData();

	FMovieSceneCompiledDataID DataID = CompiledDataManager->Compile(Sequence);
	TestPlayer.RootInstance.Initialize(*Sequence, TestPlayer);

	for (int32 i = 0; i < NumIterations; ++i)
	{
		if (bInvalidateEveryIteration)
		{
			CompiledDataManager->RemoveTrackTemplateField(DataID);
		}

		double StartSeconds    = FMath::FRand() * 60.f;
		double DurationSeconds = FMath::FRand() * 1.f;

		FMovieSceneEvaluationRange EvaluatedRange(TRange<FFrameTime>(StartSeconds * TickResolution, (StartSeconds + DurationSeconds) * TickResolution), TickResolution, EPlayDirection::Forwards);
		TestPlayer.RootInstance.Evaluate(EvaluatedRange, TestPlayer);
	}

	return true;
}

TRange<FFrameNumber> MakeRange(FFrameNumber LowerBound, FFrameNumber UpperBound, ERangeBoundTypes::Type LowerType, ERangeBoundTypes::Type UpperType)
{
	return LowerType == ERangeBoundTypes::Inclusive
		? UpperType == ERangeBoundTypes::Inclusive
			? TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(LowerBound), TRangeBound<FFrameNumber>::Inclusive(UpperBound))
			: TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(LowerBound), TRangeBound<FFrameNumber>::Exclusive(UpperBound))
		: UpperType == ERangeBoundTypes::Inclusive
			? TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Exclusive(LowerBound), TRangeBound<FFrameNumber>::Inclusive(UpperBound))
			: TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Exclusive(LowerBound), TRangeBound<FFrameNumber>::Exclusive(UpperBound));
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieSceneCompilerEvaluationRangeTest, "System.Engine.Sequencer.Compiler.EvaluationRange", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieSceneCompilerEvaluationRangeTest::RunTest(const FString& Parameters)
{
	double StartSecondTimes   [] = { 400.0f, 400.0f, 400.0f, 400.0f, 400.1f, 400.5f, 400.8f };
	double DurationSecondTimes[] = { 400.0f, 400.1f, 400.5f, 400.8f, 400.0f, 400.0f, 400.0f };

	FFrameRate TickResolution(1, 1);

	for (int32 Index = 0; Index < 7; ++Index)
	{
		double StartSeconds = StartSecondTimes[Index];
		double DurationSeconds = DurationSecondTimes[Index];

		FFrameTime LowerBound = StartSeconds * TickResolution;
		FFrameTime UpperBound = (StartSeconds + DurationSeconds) * TickResolution;

		FMovieSceneEvaluationRange EvaluatedRange(TRange<FFrameTime>(LowerBound, UpperBound), TickResolution, EPlayDirection::Forwards);

		TRange<FFrameTime> TimeRange = EvaluatedRange.GetRange();
		UTEST_EQUAL("Range - Lower Bound", TimeRange.GetLowerBoundValue(), LowerBound);
		UTEST_EQUAL("Range - Upper Bound", TimeRange.GetUpperBoundValue(), UpperBound);

		TRange<FFrameNumber> FrameNumberRange = EvaluatedRange.GetFrameNumberRange();

		// Subframe evaluation range tests
		// For example, a range of [400.5, 800.5[ will return a lower bound frame of 401 since the frame is already beyond 400
		UTEST_EQUAL("FrameNumberRange - Lower Bound Frame", FrameNumberRange.GetLowerBoundValue().Value, FMath::CeilToInt(StartSeconds));

		// For example, a range of [400, 800.5[ will return an upper bound frame of 801 so that a key at 800 will be evaluated
		UTEST_EQUAL("FrameNumberRange - Upper Bound Frame", FrameNumberRange.GetUpperBoundValue().Value, FMath::CeilToInt(StartSeconds + DurationSeconds));
	}
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieSceneCompilerRangeTest, "System.Engine.Sequencer.Compiler.Ranges", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieSceneCompilerRangeTest::RunTest(const FString& Parameters)
{
	FFrameNumber CompileAtTimes[] = {
		-3, -2, -1, 0, 1, 2, 3, 9, 10, 11, 12, 13, 14, 15, 16, 17, 18, 19, 20, 21, 22, 23, 24,
	};
	// Test each combination of inc/excl boundary conditions for adjacent and adjoining ranges
	TRange<FFrameNumber> Ranges[] = {
		MakeRange(-2, -1, ERangeBoundTypes::Inclusive, ERangeBoundTypes::Inclusive),
		MakeRange(-2, -1, ERangeBoundTypes::Inclusive, ERangeBoundTypes::Exclusive),
		MakeRange(-2, -1, ERangeBoundTypes::Exclusive, ERangeBoundTypes::Inclusive),
		MakeRange(-2, -1, ERangeBoundTypes::Exclusive, ERangeBoundTypes::Exclusive),

		MakeRange(-1, -1, ERangeBoundTypes::Inclusive, ERangeBoundTypes::Inclusive),
		MakeRange(-1, -1, ERangeBoundTypes::Inclusive, ERangeBoundTypes::Exclusive),
		MakeRange(-1, -1, ERangeBoundTypes::Exclusive, ERangeBoundTypes::Inclusive),
		MakeRange(-1, -1, ERangeBoundTypes::Exclusive, ERangeBoundTypes::Exclusive),

		MakeRange(-1,  0, ERangeBoundTypes::Inclusive, ERangeBoundTypes::Inclusive),
		MakeRange(-1,  0, ERangeBoundTypes::Inclusive, ERangeBoundTypes::Exclusive),
		MakeRange(-1,  0, ERangeBoundTypes::Exclusive, ERangeBoundTypes::Inclusive),
		MakeRange(-1,  0, ERangeBoundTypes::Exclusive, ERangeBoundTypes::Exclusive),

		MakeRange( 0,  0, ERangeBoundTypes::Inclusive, ERangeBoundTypes::Inclusive),
		MakeRange( 0,  0, ERangeBoundTypes::Inclusive, ERangeBoundTypes::Exclusive),
		MakeRange( 0,  0, ERangeBoundTypes::Exclusive, ERangeBoundTypes::Inclusive),
		MakeRange( 0,  0, ERangeBoundTypes::Exclusive, ERangeBoundTypes::Exclusive),

		MakeRange( 0,  1, ERangeBoundTypes::Inclusive, ERangeBoundTypes::Inclusive),
		MakeRange( 0,  1, ERangeBoundTypes::Inclusive, ERangeBoundTypes::Exclusive),
		MakeRange( 0,  1, ERangeBoundTypes::Exclusive, ERangeBoundTypes::Inclusive),
		MakeRange( 0,  1, ERangeBoundTypes::Exclusive, ERangeBoundTypes::Exclusive),

		MakeRange( 1,  1, ERangeBoundTypes::Inclusive, ERangeBoundTypes::Inclusive),
		MakeRange( 1,  1, ERangeBoundTypes::Inclusive, ERangeBoundTypes::Exclusive),
		MakeRange( 1,  1, ERangeBoundTypes::Exclusive, ERangeBoundTypes::Inclusive),
		MakeRange( 1,  1, ERangeBoundTypes::Exclusive, ERangeBoundTypes::Exclusive),

		MakeRange( 0,  2, ERangeBoundTypes::Inclusive, ERangeBoundTypes::Inclusive),
		MakeRange( 0,  2, ERangeBoundTypes::Inclusive, ERangeBoundTypes::Exclusive),
		MakeRange( 0,  2, ERangeBoundTypes::Exclusive, ERangeBoundTypes::Inclusive),
		MakeRange( 0,  2, ERangeBoundTypes::Exclusive, ERangeBoundTypes::Exclusive),

		MakeRange( 10, 15, ERangeBoundTypes::Inclusive, ERangeBoundTypes::Inclusive),
		MakeRange( 9,  15, ERangeBoundTypes::Exclusive, ERangeBoundTypes::Inclusive),
		MakeRange( 10, 15, ERangeBoundTypes::Exclusive, ERangeBoundTypes::Inclusive),
		MakeRange( 11, 15, ERangeBoundTypes::Exclusive, ERangeBoundTypes::Inclusive),
		
		MakeRange( 13, 17, ERangeBoundTypes::Inclusive, ERangeBoundTypes::Inclusive),
		MakeRange( 13, 18, ERangeBoundTypes::Inclusive, ERangeBoundTypes::Exclusive),
		MakeRange( 13, 19, ERangeBoundTypes::Inclusive, ERangeBoundTypes::Inclusive),
		MakeRange( 13, 18, ERangeBoundTypes::Inclusive, ERangeBoundTypes::Inclusive),

		// Explicitly test two adjacent ranges that would produce effectively empty space in between them when iterating
		MakeRange( 21, 22, ERangeBoundTypes::Inclusive, ERangeBoundTypes::Inclusive),
		MakeRange( 23, 24, ERangeBoundTypes::Inclusive, ERangeBoundTypes::Inclusive),
	};

	UTestMovieSceneSequence* Sequence = NewObject<UTestMovieSceneSequence>(GetTransientPackage());
	for (TRange<FFrameNumber> Range : Ranges)
	{
		UTestMovieSceneTrack*   Track   = Sequence->MovieScene->AddTrack<UTestMovieSceneTrack>();
		UTestMovieSceneSection* Section = NewObject<UTestMovieSceneSection>(Track);

		Section->SetRange(Range);
		Track->SectionArray.Add(Section);
	}

	struct FTemplateStore : IMovieSceneSequenceTemplateStore
	{
		FMovieSceneEvaluationTemplate& AccessTemplate(UMovieSceneSequence&) override
		{
			return Template;
		}

		FMovieSceneEvaluationTemplate Template;
	} Store;

	// Compile individual times
	for (FFrameNumber Time : CompileAtTimes)
	{
		FMovieSceneCompiler::CompileRange(TRange<FFrameNumber>::Inclusive(Time, Time), *Sequence, Store);
	}

	// Compile a whole range
	Store.Template = FMovieSceneEvaluationTemplate();
	FMovieSceneCompiler::CompileRange(TRange<FFrameNumber>::All(), *Sequence, Store);

	// Compile the whole sequence
	Store.Template = FMovieSceneEvaluationTemplate();
	FMovieSceneCompiler::Compile(*Sequence, Store);

	return true;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieSceneCompilerEmptySpaceOnTheFlyTest, "System.Engine.Sequencer.Compiler.Empty Space On The Fly", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieSceneCompilerEmptySpaceOnTheFlyTest::RunTest(const FString& Parameters)
{
	// Tests that compiling ranges that contain empty space works correctly by verifying that the result evaluation field entries are either populated or empty as expected

	struct FResult
	{
		TRange<FFrameNumber> FieldRange;
		bool                 bExpectEmpty;
	};

	struct FTest
	{
		TArray<TRange<FFrameNumber>> CompileRanges;
		TArray<FResult>              ExpectedResults;
	};

	TRange<FFrameNumber> SectionRanges[] = {
		TRange<FFrameNumber>(0,  10),
		TRange<FFrameNumber>(20, 30),
		TRange<FFrameNumber>(40, 50),
		TRange<FFrameNumber>(60, 70),
	};

	FResult ExpectedResults[] = {
		{ TRange<FFrameNumber>(0,  10), false },
		{ TRange<FFrameNumber>(10, 20), true  },
		{ TRange<FFrameNumber>(20, 30), false },
		{ TRange<FFrameNumber>(30, 40), true  },
		{ TRange<FFrameNumber>(40, 50), false },
		{ TRange<FFrameNumber>(50, 60), true  },
		{ TRange<FFrameNumber>(60, 70), false },
	};

	TArray<FTest> Tests;

	// Test 0: Test that compiling a range that only overlaps a section results in only that section's time being compiled
	{
		Tests.Emplace();
		Tests.Last().CompileRanges   = { TRange<FFrameNumber>(5, 6) };
		Tests.Last().ExpectedResults = { ExpectedResults[0] };
	}

	// Test 1: Test that compiling a range that overlaps both a section and empty space results in an entry for the section and the empty space
	{
		Tests.Emplace();
		Tests.Last().CompileRanges   = { TRange<FFrameNumber>(6, 15) };
		Tests.Last().ExpectedResults = { ExpectedResults[0], ExpectedResults[1] };
	}

	// Test 2: Test that compiling a range that only overlaps empty space works as expected
	{
		Tests.Emplace();
		Tests.Last().CompileRanges   = { TRange<FFrameNumber>(14, 15) };
		Tests.Last().ExpectedResults = { ExpectedResults[1] };
	}

	// Test 3: Test that compiling a section range followed by a range that overlaps both that section, and subsequent empty space compiles the empty space correctly
	{
		Tests.Emplace();
		Tests.Last().CompileRanges   = { TRange<FFrameNumber>(5, 6), TRange<FFrameNumber>(6, 15) };
		Tests.Last().ExpectedResults = { ExpectedResults[0], ExpectedResults[1] };
	}

	// Test 4: Test that compiling section range followed by a range that overlaps empty space preceeding that section and the section itself, compiles correctly (reverse of Test3)
	{
		Tests.Emplace();
		Tests.Last().CompileRanges   = { TRange<FFrameNumber>(24, 25), TRange<FFrameNumber>(15, 24), TRange<FFrameNumber>(5, 6) };
		Tests.Last().ExpectedResults = { ExpectedResults[0], ExpectedResults[1], ExpectedResults[2] };
	}

	// Test 5: Test that compiling a range encomassing the entire track results in the correct field ranges
	{
		Tests.Emplace();
		Tests.Last().CompileRanges   = { TRange<FFrameNumber>(0, 70) };
		Tests.Last().ExpectedResults = TArray<FResult>(&ExpectedResults[0], UE_ARRAY_COUNT(ExpectedResults));
	}

	UTestMovieSceneSequence* Sequence = NewObject<UTestMovieSceneSequence>(GetTransientPackage());
	UTestMovieSceneTrack*    Track    = Sequence->MovieScene->AddTrack<UTestMovieSceneTrack>();

	for (TRange<FFrameNumber> Range : SectionRanges)
	{
		UTestMovieSceneSection* Section = NewObject<UTestMovieSceneSection>(Track);
		Section->SetRange(Range);
		Track->SectionArray.Add(Section);
	}

	UMovieSceneCompiledDataManager* CompileDataManager = NewObject<UMovieSceneCompiledDataManager>(GetTransientPackage());

	struct FTemplateStore : IMovieSceneSequenceTemplateStore
	{
		FMovieSceneEvaluationTemplate& AccessTemplate(UMovieSceneSequence&) override
		{
			return Template;
		}

		FMovieSceneEvaluationTemplate Template;
	} Store;


	for (int32 Index = 0; Index < Tests.Num(); ++Index)
	{
		const FTest& Test = Tests[Index];

		// Wipe the evaluation template before each test
		Store.Template = FMovieSceneEvaluationTemplate();

		// Compile all the ranges that the test demands
		for (TRange<FFrameNumber> CompileRange : Test.CompileRanges)
		{
			FMovieSceneCompiler::CompileRange(CompileRange, *Sequence, Store);
		}

		// Verify that the resulting evaluation field is what we expect
		TArrayView<const FMovieSceneFrameRange> FieldRanges = Store.Template.EvaluationField.GetRanges();
		if (!FieldRanges.Num())
		{
			AddError(FString::Printf(TEXT("Test index %02d: No evaluation field entries were compiled."), Index));
			continue;
		}

		for (const FResult& Result : Test.ExpectedResults)
		{
			// Find the field entry that exactly matches our expected result
			const FMovieSceneFrameRange* FieldRange = Algo::FindBy(FieldRanges, Result.FieldRange, &FMovieSceneFrameRange::Value);
			if (!FieldRange)
			{
				AddError(FString::Printf(TEXT("Test index %02d: Expected to find an evaluation field entry for range %s but did not."), Index, *LexToString(Result.FieldRange)));
			}
			else
			{
				// Verify that the field entry is either empty or populated as the test expects
				const int32 FieldIndex       = FieldRange - FieldRanges.GetData();
				const bool  FieldIsEmptyHere = (Store.Template.EvaluationField.GetGroup(FieldIndex).TrackLUT.Num() == 0);

				if (Result.bExpectEmpty != FieldIsEmptyHere)
				{
					const TCHAR* ExpectedString = Result.bExpectEmpty ? TEXT("empty")     : TEXT("populated");
					const TCHAR* ActualString   = FieldIsEmptyHere    ? TEXT("populated") : TEXT("empty");
					AddError(FString::Printf(TEXT("Test index %02d: Expected evaluation field entry range %s to be %s but it was %s."), Index, ExpectedString, *LexToString(Result.FieldRange), ActualString));
				}
			}
		}
	}

	return true;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieSceneCompilerSubSequencesTest, "System.Engine.Sequencer.Compiler.SubSequences", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieSceneCompilerSubSequencesTest::RunTest(const FString& Parameters)
{
	struct FTest
	{
		TArray<TRange<FFrameNumber>> CompileRanges;
	};

	struct FTemplateStore : public IMovieSceneSequenceTemplateStore
	{
		FTemplateStore(const TArray<UTestMovieSceneSequence*>& InExpectedSequences)
			: ExpectedSequences(InExpectedSequences)
		{
			Templates.SetNum(InExpectedSequences.Num());
		}

		FMovieSceneEvaluationTemplate& AccessTemplate(UMovieSceneSequence& InSequence) override
		{
			for (int32 i = 0; i < ExpectedSequences.Num(); ++i)
			{
				if (ExpectedSequences[i] == &InSequence)
				{
					return Templates[i];
				}
			}
			check(false);
			return Templates[0];
		}

	private:
		TArray<UTestMovieSceneSequence*> ExpectedSequences;
		TArray<FMovieSceneEvaluationTemplate> Templates;
	};

	UTestMovieSceneSequence* RootSequence = NewObject<UTestMovieSceneSequence>(GetTransientPackage());
	UTestMovieSceneSubTrack* RootSubTrack = RootSequence->MovieScene->AddTrack<UTestMovieSceneSubTrack>();
	
	UTestMovieSceneSequence* Shot1Sequence = NewObject<UTestMovieSceneSequence>(GetTransientPackage());
	Shot1Sequence->GetMovieScene()->SetPlaybackRange(0, 100);
	
	UTestMovieSceneSubSection* Shot1SubSection = NewObject<UTestMovieSceneSubSection>(RootSubTrack);
	Shot1SubSection->SetRange(TRange<FFrameNumber>(0, 100));
	Shot1SubSection->SetSequence(Shot1Sequence);
	RootSubTrack->SectionArray.Add(Shot1SubSection);
	
	UTestMovieSceneTrack* Shot1Track = Shot1Sequence->MovieScene->AddTrack<UTestMovieSceneTrack>();
	UTestMovieSceneSection* Shot1Section = NewObject<UTestMovieSceneSection>(Shot1Track);
	Shot1Section->SetRange(TRange<FFrameNumber>(0, 60));
	Shot1Track->SectionArray.Add(Shot1Section);
	
	FMovieSceneSequenceID Shot1SequenceID = Shot1SubSection->GetSequenceID();

	{
		FTemplateStore Store({ RootSequence, Shot1Sequence });
		FMovieSceneCompiler::Compile(*RootSequence, Store);

		const FMovieSceneEvaluationTemplate& RootTemplate = Store.AccessTemplate(*RootSequence);
		UTEST_EQUAL("Ranges count", RootTemplate.EvaluationField.GetRanges().Num(), 3);
		UTEST_EQUAL("First range", RootTemplate.EvaluationField.GetRange(1), TRange<FFrameNumber>(0, 60));
		const FMovieSceneEvaluationGroup& EvalGroup = RootTemplate.EvaluationField.GetGroup(1);
		UTEST_EQUAL("Sequence ID", EvalGroup.SegmentPtrLUT[0].SequenceID, Shot1SequenceID);
	}

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS
#endif // #if 0
