// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "EntitySystem/MovieSceneEntityBuilder.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneTaskScheduler.h"
#include "EntitySystem/MovieSceneEntityFactoryTemplates.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

#define LOCTEXT_NAMESPACE "MovieSceneCustomSchedulerTests"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieSceneCustomSchedulerTest, 
		"System.Engine.Sequencer.EntitySystem.Scheduler", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieSceneCustomSchedulerTest::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene;

	FComponentRegistry ComponentRegistry;
	FEntityManager EntityManager;
	FEntitySystemScheduler Scheduler(&EntityManager);

	EntityManager.SetComponentRegistry(&ComponentRegistry);

	TComponentTypeID<int> IntComponent = ComponentRegistry.NewComponentType<int>(TEXT("Integer"));
	TComponentTypeID<float> FloatComponent = ComponentRegistry.NewComponentType<float>(TEXT("Float"));
	TComponentTypeID<double> DoubleComponent = ComponentRegistry.NewComponentType<double>(TEXT("Double"));
	TComponentTypeID<FString> StringComponent = ComponentRegistry.NewComponentType<FString>(TEXT("String"));

	TSortedMap<FMovieSceneEntityID, int> IntEntities;
	TSortedMap<FMovieSceneEntityID, int> IntAndFloatEntities;
	TSortedMap<FMovieSceneEntityID, int> IntAndDoubleEntities;

	// Populate the entity manager
	for (int32 Index = 0; Index < 256; ++Index)
	{
		FMovieSceneEntityID Entity = FEntityBuilder()
		.Add(IntComponent, 0)
		.Add(FloatComponent, static_cast<float>(Index))
		.Add(StringComponent, FString())
		.CreateEntity(&EntityManager);

		IntEntities.Add(Entity, Index);
		IntAndFloatEntities.Add(Entity, Index);
	}
	for (int32 Index = 0; Index < 256; ++Index)
	{
		FMovieSceneEntityID Entity = FEntityBuilder()
		.Add(IntComponent, 0)
		.Add(DoubleComponent, static_cast<double>(Index))
		.Add(StringComponent, FString())
		.CreateEntity(&EntityManager);

		IntEntities.Add(Entity, Index);
		IntAndDoubleEntities.Add(Entity, Index);
	}

	static float Factor = 1.f;

	// Construct tasks
	struct FIntTask
	{
		const TSortedMap<FMovieSceneEntityID, int>* IntEntitiesPtr;

		FIntTask(const TSortedMap<FMovieSceneEntityID, int>* InIntEntities)
			: IntEntitiesPtr(InIntEntities)
		{}

		void ForEachEntity(FMovieSceneEntityID EntityID, int& OutIndex) const
		{
			OutIndex = IntEntitiesPtr->FindChecked(EntityID);
		}
	};

	struct FFloatTask
	{
		static void ForEachEntity(int InIndex, float& OutFloat)
		{
			OutFloat = static_cast<float>(InIndex) * Factor;
		}
	};
	struct FDoubleTask
	{
		static void ForEachEntity(int InIndex, double& OutDouble)
		{
			OutDouble = static_cast<double>(InIndex) * Factor;
		}
	};

	struct FFloatStringTask
	{
		static void ForEachEntity(float InFloat, FString& OutString)
		{
			OutString = LexToString(InFloat);
		}
	};
	struct FDoubleStringTask
	{
		static void ForEachEntity(double InDouble, FString& OutString)
		{
			OutString = LexToString(InDouble);
		}
	};

	Scheduler.BeginConstruction();

	Scheduler.BeginSystem(0);
	{
		FEntityTaskBuilder()
		.ReadEntityIDs()
		.Write(IntComponent)
		.Fork_PerEntity<FIntTask>(&EntityManager, &Scheduler, &IntEntities);

		// 1 and 2 depend on this
		Scheduler.PropagatePrerequisite(1);
		Scheduler.PropagatePrerequisite(2);
	}
	Scheduler.EndSystem(0);

	Scheduler.BeginSystem(1);
	{
		FEntityTaskBuilder()
		.Read(IntComponent)
		.Write(FloatComponent)
		.Fork_PerEntity<FFloatTask>(&EntityManager, &Scheduler);

		Scheduler.PropagatePrerequisite(3);
	}
	Scheduler.EndSystem(1);

	Scheduler.BeginSystem(2);
	{
		FEntityTaskBuilder()
		.Read(IntComponent)
		.Write(DoubleComponent)
		.Fork_PerEntity<FDoubleTask>(&EntityManager, &Scheduler);

		Scheduler.PropagatePrerequisite(4);
	}
	Scheduler.EndSystem(2);

	Scheduler.BeginSystem(3);
	{
		FEntityTaskBuilder()
		.Read(FloatComponent)
		.Write(StringComponent)
		.Fork_PerEntity<FFloatStringTask>(&EntityManager, &Scheduler);
	}
	Scheduler.EndSystem(3);

	Scheduler.BeginSystem(4);
	{
		FEntityTaskBuilder()
		.Read(DoubleComponent)
		.Write(StringComponent)
		.Fork_PerEntity<FDoubleStringTask>(&EntityManager, &Scheduler);
	}
	Scheduler.EndSystem(4);

	Scheduler.EndConstruction();

	auto TestResults = [&]
	{
		TStringBuilder<256> What;
		for (TPair<FMovieSceneEntityID, int> Pair : IntAndFloatEntities)
		{
			What.Reset();
			What.Appendf(TEXT("Test Factor %i, entity ID %i"), Factor, Pair.Key.AsIndex());

			const float ExpectedResult = static_cast<float>(Pair.Value) * Factor;
			const float Result         = EntityManager.ReadComponentChecked(Pair.Key, FloatComponent);
			this->TestEqual(*What, Result, ExpectedResult);

			FString StringResult = EntityManager.ReadComponentChecked(Pair.Key, StringComponent);
			this->TestEqual(*What, *StringResult, *LexToString(ExpectedResult));
		}

		for (TPair<FMovieSceneEntityID, int> Pair :  IntAndDoubleEntities)
		{
			What.Reset();
			What.Appendf(TEXT("Test Factor %i, entity ID %i"), Factor, Pair.Key.AsIndex());

			const double ExpectedResult = static_cast<double>(Pair.Value) * Factor;
			const double Result         = EntityManager.ReadComponentChecked(Pair.Key, DoubleComponent);
			this->TestEqual(*What, Result, ExpectedResult);

			FString StringResult = EntityManager.ReadComponentChecked(Pair.Key, StringComponent);
			this->TestEqual(*What, *StringResult, *LexToString(ExpectedResult));
		}
	};

	Factor = 1.f;
	Scheduler.ShuffleTasks();
	Scheduler.ExecuteTasks();
	TestResults();

	Factor = 10.f;
	Scheduler.ShuffleTasks();
	Scheduler.ExecuteTasks();
	TestResults();

	Factor = 20.f;
	Scheduler.ShuffleTasks();
	Scheduler.ExecuteTasks();
	TestResults();

	return true;
}

#undef LOCTEXT_NAMESPACE

#endif // WITH_DEV_AUTOMATION_TESTS

