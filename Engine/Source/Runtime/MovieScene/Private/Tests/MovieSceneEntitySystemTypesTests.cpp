// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "EntitySystem/MovieSceneEntityBuilder.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/MovieSceneEntityFactoryTemplates.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/ArrayView.h"
#include "Misc/AutomationTest.h"
#include "Templates/Greater.h"

#if WITH_DEV_AUTOMATION_TESTS

#define LOCTEXT_NAMESPACE "MovieSceneEntitySystemTypesTests"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieSceneEntityComponentFilterTest, 
		"System.Engine.Sequencer.EntitySystem.EntityComponentFilter", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieSceneEntityComponentFilterTest::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene;

	FComponentTypeID ComponentTypes[] = {
		FComponentTypeID::FromBitIndex(0),
		FComponentTypeID::FromBitIndex(1),
		FComponentTypeID::FromBitIndex(2),
		FComponentTypeID::FromBitIndex(3),
		FComponentTypeID::FromBitIndex(4),
		FComponentTypeID::FromBitIndex(5),
		FComponentTypeID::FromBitIndex(6),
		FComponentTypeID::FromBitIndex(7),
	};

	FEntityComponentFilter Filters[4];

	Filters[0].Reset();
	Filters[1].All(FComponentMask({ ComponentTypes[0], ComponentTypes[2] }));
	Filters[2].None(FComponentMask({ ComponentTypes[0], ComponentTypes[2] }));
	Filters[3].Any(FComponentMask({ ComponentTypes[0], ComponentTypes[2] }));
	
	{
		UTEST_TRUE("Filter 1.1", Filters[1].Match(FComponentMask({ ComponentTypes[0], ComponentTypes[2] })));
		UTEST_TRUE("Filter 1.2", Filters[1].Match(FComponentMask({ ComponentTypes[0], ComponentTypes[2], ComponentTypes[3] })));
		UTEST_FALSE("Filter 1.3", Filters[1].Match(FComponentMask({ ComponentTypes[0] })));
		UTEST_FALSE("Filter 1.4", Filters[1].Match(FComponentMask()));
	}

	{
		UTEST_TRUE("Filter 2.1", Filters[2].Match(FComponentMask()));
		UTEST_TRUE("Filter 2.2", Filters[2].Match(FComponentMask({ ComponentTypes[1] })));
		UTEST_TRUE("Filter 2.3", Filters[2].Match(FComponentMask({ ComponentTypes[1], ComponentTypes[3] })));
		UTEST_FALSE("Filter 2.4", Filters[2].Match(FComponentMask({ ComponentTypes[0] })));
		UTEST_FALSE("Filter 2.5", Filters[2].Match(FComponentMask({ ComponentTypes[2], ComponentTypes[3] })));
	}

	{
		UTEST_FALSE("Filter 3.1", Filters[3].Match(FComponentMask()));
		UTEST_FALSE("Filter 3.2", Filters[3].Match(FComponentMask({ ComponentTypes[1] })));
		UTEST_FALSE("Filter 3.3", Filters[3].Match(FComponentMask({ ComponentTypes[1], ComponentTypes[3] })));
		UTEST_TRUE("Filter 3.4", Filters[3].Match(FComponentMask({ ComponentTypes[0] })));
		UTEST_TRUE("Filter 3.5", Filters[3].Match(FComponentMask({ ComponentTypes[2], ComponentTypes[3] })));
	}

	return true;
}
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieSceneMutuallyInclusiveComponentsTest, 
		"System.Engine.Sequencer.EntitySystem.MutuallyInclusiveComponents", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieSceneMutuallyInclusiveComponentsTest::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene;

	FComponentTypeID ComponentTypes[] = {
		FComponentTypeID::FromBitIndex(0),
		FComponentTypeID::FromBitIndex(1),
		FComponentTypeID::FromBitIndex(2),
		FComponentTypeID::FromBitIndex(3),
		FComponentTypeID::FromBitIndex(4),
		FComponentTypeID::FromBitIndex(5),
		FComponentTypeID::FromBitIndex(6),
		FComponentTypeID::FromBitIndex(7),
		FComponentTypeID::FromBitIndex(8),
	};

	FMutualInclusivityGraph InclusivityGraph;

	// The inclusivity rules for this test are as follows:
	//       0 requires 1 (Rule A)
	//       3 requires 4 (Rule B)
	//       4 requires 5 and 6 (Rule C)
	//       if 2 and 6 are present, 5 and 7 must be present (Rule D)
	//       if 1 or 5 are present, 8 must be present (Rule E)
	// We define theese rules out-of-order to try and ensure order doesn't impact the algorithm

	InclusivityGraph.DefineMutualInclusionRule(ComponentTypes[4], { ComponentTypes[5], ComponentTypes[6] });
	InclusivityGraph.DefineMutualInclusionRule(ComponentTypes[3], { ComponentTypes[4] });
	InclusivityGraph.DefineMutualInclusionRule(ComponentTypes[0], { ComponentTypes[1] });

	InclusivityGraph.DefineComplexInclusionRule(FComplexInclusivityFilter({ ComponentTypes[6], ComponentTypes[2] }, EComplexInclusivityFilterMode::AllOf), { ComponentTypes[7], ComponentTypes[5] });
	InclusivityGraph.DefineComplexInclusionRule(FComplexInclusivityFilter({ ComponentTypes[5], ComponentTypes[1] }, EComplexInclusivityFilterMode::AnyOf), { ComponentTypes[8] });

	struct FTest
	{
		FComponentMask Input;
		FComponentMask ExpectedOutput;
	};
	FTest Tests[] = {
		{ { ComponentTypes[0] }, { ComponentTypes[0], ComponentTypes[1], ComponentTypes[8]} }, // Test Rule A feeding E
		{ { ComponentTypes[3] }, { ComponentTypes[3], ComponentTypes[4], ComponentTypes[5], ComponentTypes[6], ComponentTypes[8] } }, // Test Rule B feeding C feeding E
		{ { ComponentTypes[4] }, { ComponentTypes[4], ComponentTypes[5], ComponentTypes[6], ComponentTypes[8] } }, // Test Rule C and E
		{ { ComponentTypes[2], ComponentTypes[6] }, { ComponentTypes[2], ComponentTypes[5], ComponentTypes[6], ComponentTypes[7], ComponentTypes[8] } }, // Test Rule D which feeds E
		{ { ComponentTypes[2], ComponentTypes[4] }, { ComponentTypes[2], ComponentTypes[4], ComponentTypes[5], ComponentTypes[6], ComponentTypes[7], ComponentTypes[8] } }, // Test Rule C feeding rule D whcih feeds rule E
		{ { ComponentTypes[0], ComponentTypes[2], ComponentTypes[6] }, { ComponentTypes[0], ComponentTypes[1], ComponentTypes[2], ComponentTypes[5], ComponentTypes[6], ComponentTypes[7], ComponentTypes[8] } }, // Test Rule A, C, D and E (A feeds E, C feeds D)
	};


	for (int32 Index = 0; Index < UE_ARRAY_COUNT(Tests); ++Index)
	{
		const FTest& Test = Tests[Index];

		FComponentMask Result;
		FMutualComponentInitializers MutualInitializers;
		InclusivityGraph.ComputeMutuallyInclusiveComponents(EMutuallyInclusiveComponentType::All, Test.Input, Result, MutualInitializers);

		// Combine the result with the input since ComputeMutuallyInclusiveComponents only adds to Result
		// if they did not exist in Input
		Result.CombineWithBitwiseOR(Test.Input, EBitwiseOperatorFlags::MaxSize);

		if (!Result.CompareSetBits(Test.ExpectedOutput) || Result.NumComponents() != Test.ExpectedOutput.NumComponents())
		{
			auto ToString = [](const FComponentMask& Mask)
			{

				TStringBuilder<32> String;
				for (FComponentMaskIterator It(Mask.Iterate()); It; ++It)
				{
					if (String.Len() != 0)
					{
						String += TEXT("|");
					}
					String += LexToString(It.GetIndex());
				}
				return FString(String.ToString());
			};

			AddError(FString::Printf(TEXT("Test %d failed: result from input %s was %s: expected %s."),
				Index,
				*ToString(Test.Input),
				*ToString(Result),
				*ToString(Test.ExpectedOutput)
				));
		}
	}

	return true;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieSceneEntityMigrationTest, 
		"System.Engine.Sequencer.EntitySystem.Migration", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieSceneEntityMigrationTest::RunTest(const FString& Parameters)
{
	/*~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
	This test is designed to guarantee the correctness of FEntityManager::MigrateEntity
	with respect to relocation of entity-component data. It works by allocating a number
	of entities with a 'random' (it's a seeded random stream for determinism) combination
	of components, then randomly mutating them. After each mutation, the entity-component
	data is verified against the expected component data, and the test will fail if any
	of the component data, or the entity's ID differs from the expected result.
	~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~*/

	using namespace UE::MovieScene;

	FComponentRegistry ComponentRegistry;
	FEntityManager EntityManager;

	TGuardValue<FEntityManager*> EntityManagerForDebugging(GEntityManagerForDebuggingVisualizers, &EntityManager);

	EntityManager.SetComponentRegistry(&ComponentRegistry);

	TComponentTypeID<int32> IntComponentTypes[2];
	TComponentTypeID<FString> StringComponentTypes[2];

	// We allocate entities with any combination of the following 4 components (including none)
	// We keep the component count low in order to ensure that there are multiple entities in
	// each allocation (without which, this test would be useless)
	ComponentRegistry.NewComponentType(&IntComponentTypes[0], TEXT("Int 0"));
	ComponentRegistry.NewComponentType(&IntComponentTypes[1], TEXT("Int 1"));
	ComponentRegistry.NewComponentType(&StringComponentTypes[0], TEXT("String 0"));
	ComponentRegistry.NewComponentType(&StringComponentTypes[1], TEXT("String 1"));

	struct FEntityValue
	{
		FMovieSceneEntityID ID;

		FString StringValues[2];

		int32 IntValues[2];

		uint8 EnabledMask;
	};

	constexpr int32 Seed = 0xfeedbeef;
	FRandomStream RandomStream(Seed);

	TArray<FEntityValue> AllEntities;

	auto TestEntities = [this, &AllEntities, &EntityManager, &IntComponentTypes, &StringComponentTypes]
	{
		for (const FEntityValue& EntityValue : AllEntities)
		{
			FEntityInfo EntityInfo = EntityManager.GetEntity(EntityValue.ID);

			if (EntityValue.EnabledMask == 0)
			{
				UTEST_TRUE(TEXT("Entity Is Empty"), EntityInfo.Data.Allocation == nullptr);
			}
			else if (EntityInfo.Data.Allocation)
			{
				UTEST_EQUAL(TEXT("Entity ID"), EntityValue.ID, EntityInfo.Data.Allocation->GetEntityIDs()[EntityInfo.Data.ComponentOffset]);
			}
			else
			{
				AddError(TEXT("Entity ID has no data but should"));
				return false;
			}

			TComponentLock<TReadOptional<int32>> IntValue0 = EntityManager.ReadComponent(EntityValue.ID, IntComponentTypes[0]);
			TComponentLock<TReadOptional<int32>> IntValue1 = EntityManager.ReadComponent(EntityValue.ID, IntComponentTypes[1]);

			bool ShouldHaveIntComponents[] = {
				static_cast<bool>(EntityValue.EnabledMask & (1 << 0)),
				static_cast<bool>(EntityValue.EnabledMask & (1 << 1)),
			};

			UTEST_EQUAL(TEXT("Has Int Value 0"), IntValue0.IsValid(), ShouldHaveIntComponents[0]);
			UTEST_EQUAL(TEXT("Has Int Value 1"), IntValue1.IsValid(), ShouldHaveIntComponents[1]);

			if (ShouldHaveIntComponents[0] && IntValue0)
			{
				UTEST_EQUAL(TEXT("Int Value 0"), *IntValue0.AsPtr(), EntityValue.IntValues[0])
			}
			if (ShouldHaveIntComponents[1] && IntValue1)
			{
				UTEST_EQUAL(TEXT("Int Value 1"), *IntValue1.AsPtr(), EntityValue.IntValues[1])
			}

			TComponentLock<TReadOptional<FString>> StringValue0 = EntityManager.ReadComponent(EntityValue.ID, StringComponentTypes[0]);
			TComponentLock<TReadOptional<FString>> StringValue1 = EntityManager.ReadComponent(EntityValue.ID, StringComponentTypes[1]);

			bool ShouldHaveStringComponents[] = {
				static_cast<bool>(EntityValue.EnabledMask & (1 << 2)),
				static_cast<bool>(EntityValue.EnabledMask & (1 << 3)),
			};

			UTEST_EQUAL(TEXT("Has String Value 0"), StringValue0.IsValid(), ShouldHaveStringComponents[0]);
			UTEST_EQUAL(TEXT("Has String Value 1"), StringValue1.IsValid(), ShouldHaveStringComponents[1]);

			if (ShouldHaveStringComponents[0] && StringValue0)
			{
				UTEST_EQUAL(TEXT("String Value 0"), *StringValue0.AsPtr(), EntityValue.StringValues[0])
			}
			if (ShouldHaveStringComponents[1] && StringValue1)
			{
				UTEST_EQUAL(TEXT("String Value 1"), *StringValue1.AsPtr(), EntityValue.StringValues[1])
			}
		}
		return true;
	};

	constexpr int32 NumTypes = 16;
	constexpr int32 NumPerType = 10;
	for (int32 TypeIndex = 0; TypeIndex < NumTypes; ++TypeIndex)
	{
		const uint8 ComponentMask = TypeIndex;

		// Allocate NumPerType entities to ensure we have many entities in the allocation
		for (int32 EntityIndex = 0; EntityIndex < NumPerType; ++EntityIndex)
		{
			FEntityValue NewValue;
			NewValue.IntValues[0] = FMath::RandRange(0, 100);
			NewValue.IntValues[1] = FMath::RandRange(0, 100);
			NewValue.StringValues[0] = FString::Printf(TEXT("String Value %i"), FMath::RandRange(0, 100));
			NewValue.StringValues[1] = FString::Printf(TEXT("String Value %i"), FMath::RandRange(0, 100));
			NewValue.EnabledMask = ComponentMask;

			NewValue.ID = FEntityBuilder()
			.AddConditional(IntComponentTypes[0], NewValue.IntValues[0], (ComponentMask & 1 << 0))
			.AddConditional(IntComponentTypes[1], NewValue.IntValues[1], (ComponentMask & 1 << 1))
			.AddConditional(StringComponentTypes[0], NewValue.StringValues[0], (ComponentMask & 1 << 2))
			.AddConditional(StringComponentTypes[1], NewValue.StringValues[1], (ComponentMask & 1 << 3))
			.CreateEntity(&EntityManager);

			AllEntities.Emplace(MoveTemp(NewValue));
		}
	}

	// Shuffle the array of entities so we mutate them in a random order
	for (int32 i = 0; i < AllEntities.Num()-2; i++)
	{
		const int32 j = RandomStream.RandRange(i+1, AllEntities.Num() - 1);
		AllEntities.Swap(i, j);
	}

	// 'randomly' mutate entities
	for (int32 Index = 0; Index < AllEntities.Num(); ++Index)
	{
		const uint8 NewType = RandomStream.RandRange(0, 0b1111);

		AddInfo(FString::Printf(TEXT("Mutating entity index %i from %i to %i"), Index, (int32)AllEntities[Index].EnabledMask, (int32)NewType));

		AllEntities[Index].EnabledMask = NewType;

		for (int32 IntIndex = 0; IntIndex < 2; ++IntIndex)
		{
			if (NewType & (1 << IntIndex))
			{
				EntityManager.AddComponent(AllEntities[Index].ID, IntComponentTypes[IntIndex], AllEntities[Index].IntValues[IntIndex]);
			}
			else
			{
				EntityManager.RemoveComponent(AllEntities[Index].ID, IntComponentTypes[IntIndex]);
			}
		}

		for (int32 StringIndex = 0; StringIndex < 2; ++StringIndex)
		{
			if (NewType & (0x4 << StringIndex))
			{
				EntityManager.AddComponent(AllEntities[Index].ID, StringComponentTypes[StringIndex], AllEntities[Index].StringValues[StringIndex]);
			}
			else
			{
				EntityManager.RemoveComponent(AllEntities[Index].ID, StringComponentTypes[StringIndex]);
			}
		}

		if (!TestEntities())
		{
			return false;
		}
	}

	return true;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieSceneBlendTargetComponentTest, 
		"System.Engine.Sequencer.EntitySystem.BlendTargetComponent",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieSceneBlendTargetComponentTest::RunTest(const FString& Parameters)
{
	/*~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
	This test checks the behavior of FHierarchicalBlendTarget w.r.t.
	move operations and memory management
	~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~*/

	using namespace UE::MovieScene;

	auto CompareBlendTarget = [this](const FHierarchicalBlendTarget& BlendTarget, TArrayView<const int16> Expected, const TCHAR* InTestName)
	{
		TArrayView<const int16> Result = BlendTarget.AsArray();
		if (Result.Num() != Expected.Num() || !CompareItems(Result.GetData(), Expected.GetData(), Result.Num()))
		{
			TStringBuilder<128> ErrorString;
			ErrorString.Appendf(TEXT("Test %s: Blend Target does not match the expected result ("), InTestName);
			ErrorString.Join(Result, TEXT(","));
			ErrorString.Append(TEXT(" != "));
			ErrorString.Join(Expected, TEXT(","));
			ErrorString.Append(TEXT(")"));
			this->AddError(ErrorString.ToString());
		}
	};

	// Simple test - use inline allocation
	{
		FHierarchicalBlendTarget BlendTarget;
		BlendTarget.Add(0);
		BlendTarget.Add(100);
		BlendTarget.Add(-100);
		BlendTarget.Add(-10);
		BlendTarget.Add(10);
		BlendTarget.Add(500);
		BlendTarget.Add(1000);

		this->TestEqual(TEXT("Inline Basic Capacity"), BlendTarget.GetCapacity(), 7);
		CompareBlendTarget(BlendTarget, { 1000, 500, 100, 10, 0, -10, -100 }, TEXT("Inline Basic"));
		// Verify that calling GetTypeHash does not segfault on the manual memory management
		(void)GetTypeHash(BlendTarget);
	}

	// Test adding duplicates
	{
		FHierarchicalBlendTarget BlendTarget;
		BlendTarget.Add(0);
		BlendTarget.Add(100);
		BlendTarget.Add(-100);
		BlendTarget.Add(-10);
		BlendTarget.Add(10);
		BlendTarget.Add(500);
		BlendTarget.Add(1000);
		BlendTarget.Add(0);
		BlendTarget.Add(100);
		BlendTarget.Add(-100);
		BlendTarget.Add(-10);
		BlendTarget.Add(10);
		BlendTarget.Add(500);
		BlendTarget.Add(1000);

		this->TestEqual(TEXT("Inline Duplicate Capacity"), BlendTarget.GetCapacity(), 7);
		CompareBlendTarget(BlendTarget, { 1000, 500, 100, 10, 0, -10, -100 }, TEXT("Inline Duplicate"));
		// Verify that calling GetTypeHash does not segfault on the manual memory management
		(void)GetTypeHash(BlendTarget);
	}

	// Simple test - use inline allocation
	{
		FHierarchicalBlendTarget BlendTarget;
		BlendTarget.Add(0);
		BlendTarget.Add(100);
		BlendTarget.Add(-100);
		BlendTarget.Add(-10);
		BlendTarget.Add(10);
		BlendTarget.Add(500);
		BlendTarget.Add(1000);

		this->TestEqual(TEXT("Inline Basic Capacity"), BlendTarget.GetCapacity(), 7);
		CompareBlendTarget(BlendTarget, { 1000, 500, 100, 10, 0, -10, -100 }, TEXT("Inline Basic"));
		// Verify that calling GetTypeHash does not segfault on the manual memory management
		(void)GetTypeHash(BlendTarget);
	}

	// Test growing the allocation
	{
		FRandomStream RandomStream;
		RandomStream.GenerateNewSeed();

		FHierarchicalBlendTarget BlendTarget;
		TSet<int16> ExpectedSet;
		// Add 
		for (int16 Index = 0; Index < 25; ++Index)
		{
			const int16 Value = static_cast<int16>(FMath::RandRange(-10000, 10000));
			BlendTarget.Add(Value);
			ExpectedSet.Add(Value);
		}

		TArray<int16> Expected = ExpectedSet.Array();
		Algo::Sort(Expected, TGreater<>());

		this->TestEqual(TEXT("Basic Heap Capacity"), BlendTarget.GetCapacity(), 32);
		CompareBlendTarget(BlendTarget, Expected, TEXT("Basic Heap"));
		// Verify that calling GetTypeHash does not segfault on the manual memory management
		(void)GetTypeHash(BlendTarget);
	}

	// Test growing the allocation with duplicates
	{
		FRandomStream RandomStream;
		RandomStream.GenerateNewSeed();

		FHierarchicalBlendTarget BlendTarget;
		TSet<int16> ExpectedSet;
		// Add 
		for (int16 Index = 0; Index < 25; ++Index)
		{
			const int16 Value = static_cast<int16>(FMath::RandRange(-10000, 10000));
			BlendTarget.Add(Value);
			ExpectedSet.Add(Value);
		}

		for (int16 Expected : ExpectedSet)
		{
			BlendTarget.Add(Expected);
		}

		TArray<int16> Expected = ExpectedSet.Array();
		Algo::Sort(Expected, TGreater<>());

		this->TestEqual(TEXT("Duplicate Heap Capacity"), BlendTarget.GetCapacity(), 32);
		CompareBlendTarget(BlendTarget, Expected, TEXT("Duplicate Heap"));
		// Verify that calling GetTypeHash does not segfault on the manual memory management
		(void)GetTypeHash(BlendTarget);
	}

	// Test move and copy operators for heap allocations
	{
		FRandomStream RandomStream;
		RandomStream.GenerateNewSeed();

		FHierarchicalBlendTarget BlendTarget[2];
		TSet<int16> ExpectedSet[2];
		// Add 
		for (int16 Index = 0; Index < 25; ++Index)
		{
			const int16 Value1 = static_cast<int16>(FMath::RandRange(-10000, 10000));
			const int16 Value2 = static_cast<int16>(FMath::RandRange(-10000, 10000));
			BlendTarget[0].Add(Value1);
			ExpectedSet[0].Add(Value1);
			BlendTarget[1].Add(Value2);
			ExpectedSet[1].Add(Value2);
		}

		TArray<int16> Expected[2] = { ExpectedSet[0].Array(), ExpectedSet[1].Array() };
		Algo::Sort(Expected[0], TGreater<>());
		Algo::Sort(Expected[1], TGreater<>());

		// Make some copies
		{
			// Copy Construction
			FHierarchicalBlendTarget BlendTargetCopy(BlendTarget[0]);
			CompareBlendTarget(BlendTargetCopy, Expected[0], TEXT("Copy 1"));
			CompareBlendTarget(BlendTarget[0], Expected[0], TEXT("Copied From 2"));
			// Verify that calling GetTypeHash does not segfault on the manual memory management
			(void)GetTypeHash(BlendTargetCopy);
			(void)GetTypeHash(BlendTarget[0]);

			// Copy Assignment
			BlendTargetCopy = BlendTarget[1];
			CompareBlendTarget(BlendTargetCopy, Expected[1], TEXT("Copy 2"));
			CompareBlendTarget(BlendTarget[1], Expected[1], TEXT("Copied From 2"));
			// Verify that calling GetTypeHash does not segfault on the manual memory management
			(void)GetTypeHash(BlendTargetCopy);
			(void)GetTypeHash(BlendTarget[1]);
		}

		// Make some moves
		{
			FHierarchicalBlendTarget BlendTargetCopy1(BlendTarget[0]);
			FHierarchicalBlendTarget BlendTargetCopy2(BlendTarget[1]);

			// Move Construction
			FHierarchicalBlendTarget BlendTargetMoved(MoveTemp(BlendTargetCopy1));
			CompareBlendTarget(BlendTargetMoved, Expected[0], TEXT("Move 1"));
			CompareBlendTarget(BlendTargetCopy1, FHierarchicalBlendTarget().AsArray(), TEXT("Moved From 1"));
			// Verify that calling GetTypeHash does not segfault on the manual memory management
			(void)GetTypeHash(BlendTargetMoved);
			(void)GetTypeHash(BlendTargetCopy1);

			// Move Assignment
			BlendTargetMoved = MoveTemp(BlendTargetCopy2);
			CompareBlendTarget(BlendTargetMoved, Expected[1], TEXT("Move 2"));
			CompareBlendTarget(BlendTargetCopy2, FHierarchicalBlendTarget().AsArray(), TEXT("Moved From 2"));
			// Verify that calling GetTypeHash does not segfault on the manual memory management
			(void)GetTypeHash(BlendTargetMoved);
			(void)GetTypeHash(BlendTargetCopy2);
		}
	}

	/** Now perform some operator tests on various allocation types */
	FRandomStream Random(0xfeedbeef);

	auto MakeBlendTarget = [&Random](int32 Size)
	{
		FHierarchicalBlendTarget Target;
		while (--Size >= 0)
		{
			Target.Add(Random.RandHelper(MAX_int16));
		}
		return Target;
	};

	auto CopyConstruct = [&MakeBlendTarget](int32 Size1)
	{
		FHierarchicalBlendTarget A = MakeBlendTarget(Size1);
		FHierarchicalBlendTarget B(A);

		check(A.AsArray().GetData() != B.AsArray().GetData());
	};
	auto CopyAssign = [&MakeBlendTarget](int32 Size1, int32 Size2)
	{
		FHierarchicalBlendTarget A = MakeBlendTarget(Size1);
		FHierarchicalBlendTarget B = MakeBlendTarget(Size2);
		A = B;

		check(A.AsArray().GetData() != B.AsArray().GetData());
	};
	auto MoveConstruct = [&MakeBlendTarget](int32 Size1)
	{
		FHierarchicalBlendTarget A = MakeBlendTarget(Size1);
		FHierarchicalBlendTarget B(MoveTemp(A));

		check(A.AsArray().GetData() != B.AsArray().GetData());
	};
	auto MoveAssign = [&MakeBlendTarget](int32 Size1, int32 Size2)
	{
		FHierarchicalBlendTarget A = MakeBlendTarget(Size1);
		FHierarchicalBlendTarget B = MakeBlendTarget(Size2);
		A = MoveTemp(B);

		check(A.AsArray().GetData() != B.AsArray().GetData());
	};

	// Test Inline to Inline
	CopyConstruct(5);
	CopyAssign(5, 7);
	MoveConstruct(5);
	MoveAssign(5, 7);

	// Test Heap to Inline
	CopyConstruct(5);
	CopyAssign(5, 15);
	MoveConstruct(5);
	MoveAssign(5, 15);

	// Test Inline to Heap
	CopyConstruct(15);
	CopyAssign(15, 5);
	MoveConstruct(15);
	MoveAssign(15, 5);

	// Test Copying Heap to Heap (Same Capacity)
	CopyConstruct(9);
	CopyAssign(9, 15);
	MoveConstruct(9);
	MoveAssign(9, 15);

	// Test Copying Heap to Heap (Different Capacity)
	CopyConstruct(35);
	CopyAssign(35, 15);
	MoveConstruct(35);
	MoveAssign(35, 15);

	// Test Copying Heap to Heap (Different Capacity)
	CopyConstruct(15);
	CopyAssign(15, 35);
	MoveConstruct(15);
	MoveAssign(15, 35);

	return true;
}

#undef LOCTEXT_NAMESPACE

#endif // WITH_DEV_AUTOMATION_TESTS

