// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Misc/CoreMiscDefines.h"
#include "MovieSceneFwd.h"
#include "Misc/AutomationTest.h"
#include "Evaluation/MovieScenePlaybackCapabilities.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::MovieScene::Tests
{

// A couple of simple test capabilities.
struct FTestSimpleCapability
{
	static int32 TimesDestroyed;

	int32 IntValue;
	FString StringValue;

	FTestSimpleCapability() 
		: IntValue(0)
	{}
	FTestSimpleCapability(int32 InIntValue, const FString& InStringValue) 
		: IntValue(InIntValue)
		, StringValue(InStringValue)
	{}
	~FTestSimpleCapability()
	{
		++TimesDestroyed;
	}

	UE_DECLARE_MOVIESCENE_PLAYBACK_CAPABILITY(FTestSimpleCapability)
};

UE_DEFINE_MOVIESCENE_PLAYBACK_CAPABILITY(FTestSimpleCapability)

int32 FTestSimpleCapability::TimesDestroyed = 0;

struct FTestOtherSimpleCapability
{
	float FloatValue = 0.f;

	UE_DECLARE_MOVIESCENE_PLAYBACK_CAPABILITY(FTestOtherSimpleCapability)
};

UE_DEFINE_MOVIESCENE_PLAYBACK_CAPABILITY(FTestOtherSimpleCapability)

// Interface capability and sample implementation.
struct FTestCapabilityBase
{
	static int32 TimesDestroyedBase;

	float FloatValue = 0.f;

	virtual ~FTestCapabilityBase() { ++TimesDestroyedBase; }

	virtual int32 GetIntValue() { return 0; }
	virtual FString GetStringValue() { return FString(); }
	virtual void DoSomething() = 0;

	UE_DECLARE_MOVIESCENE_PLAYBACK_CAPABILITY(FTestCapabilityBase)
};

UE_DEFINE_MOVIESCENE_PLAYBACK_CAPABILITY(FTestCapabilityBase)

int32 FTestCapabilityBase::TimesDestroyedBase = 0;

struct FTestCapabilityDerived : FTestCapabilityBase
{
	static int32 TimesDestroyedDerived;

	int32 IntValue;
	FString StringValue;

	FTestCapabilityDerived(int32 InIntValue, const FString& InStringValue)
		: IntValue(InIntValue)
		, StringValue(InStringValue)
	{}
	virtual ~FTestCapabilityDerived() { ++TimesDestroyedDerived; }

	virtual int32 GetIntValue() override { return IntValue; }
	virtual FString GetStringValue() override { return StringValue; }
	virtual void DoSomething() override { ++IntValue; }
};

int32 FTestCapabilityDerived::TimesDestroyedDerived = 0;

// Base capability that implements the IPlaybackCapability interface.
struct FTestInvalidatableCapability : IPlaybackCapability
{
	int32 TimesInvalidated = 0;

	virtual void InvalidateCachedData(UMovieSceneEntitySystemLinker* Linker) override
	{
		++TimesInvalidated;
	}

	UE_DECLARE_MOVIESCENE_PLAYBACK_CAPABILITY(FTestInvalidatableCapability)
};

UE_DEFINE_MOVIESCENE_PLAYBACK_CAPABILITY(FTestInvalidatableCapability)

// Derived version of the simple test capability, but with specific destructor counter
struct FTestSimpleCapabilityDerived : FTestSimpleCapability
{
	static int32 TimesDestroyedDerived;

	FTestSimpleCapabilityDerived()
	{}
	FTestSimpleCapabilityDerived(int32 InIntValue, const FString& InStringValue)
		: FTestSimpleCapability(InIntValue, InStringValue)
	{}
	~FTestSimpleCapabilityDerived()
	{
		++TimesDestroyedDerived;
	}
};

int32 FTestSimpleCapabilityDerived::TimesDestroyedDerived = 0;

// Derived version of the simple test capability, but with interface
struct FTestSimpleCapabilityDerivedWithInterface : FTestSimpleCapability, IPlaybackCapability
{
	int32 TimesInvalidated = 0;

	virtual void InvalidateCachedData(UMovieSceneEntitySystemLinker* Linker) override
	{
		++TimesInvalidated;
	}
};

} // namespace UE::MovieScene::Tests

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieScenePlaybackCapabilitiesSimpleTest, 
		"System.Engine.Sequencer.Capabilities.Simple", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieScenePlaybackCapabilitiesSimpleTest::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Tests;

	FTestSimpleCapability::TimesDestroyed = 0;
	{
		FPlaybackCapabilities Caps;
		UTEST_FALSE("No capability", Caps.HasCapability<FTestSimpleCapability>());

		FTestSimpleCapability& SimpleCap = Caps.AddCapability<FTestSimpleCapability>(42, TEXT("Just a test"));
		UTEST_EQUAL("Int value", SimpleCap.IntValue, 42);
		UTEST_EQUAL("String value", SimpleCap.StringValue, TEXT("Just a test"));

		UTEST_TRUE("Found capability", Caps.HasCapability<FTestSimpleCapability>());

		FTestSimpleCapability* SimpleCap2 = Caps.FindCapability<FTestSimpleCapability>();
		UTEST_NOT_NULL("Retrived test capacity", SimpleCap2);
		UTEST_EQUAL("Same capacity?", SimpleCap2, &SimpleCap);
	}
	UTEST_EQUAL("Destroyed", FTestSimpleCapability::TimesDestroyed, 1);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieScenePlaybackCapabilitiesInlineInterfaceTest, 
		"System.Engine.Sequencer.Capabilities.InlineInterface", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieScenePlaybackCapabilitiesInlineInterfaceTest::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Tests;

	FTestCapabilityBase::TimesDestroyedBase = 0;
	FTestCapabilityDerived::TimesDestroyedDerived = 0;
	{
		FPlaybackCapabilities Caps;
		UTEST_FALSE("No capability", Caps.HasCapability<FTestCapabilityBase>());

		FTestCapabilityBase& InterfaceCap = Caps.AddCapability<FTestCapabilityDerived>(42, TEXT("Just a test"));
		UTEST_EQUAL("Int value", InterfaceCap.GetIntValue(), 42);
		UTEST_EQUAL("String value", InterfaceCap.GetStringValue(), TEXT("Just a test"));

		UTEST_TRUE("Found capability", Caps.HasCapability<FTestCapabilityBase>());

		FTestCapabilityBase* InterfaceCap2 = Caps.FindCapability<FTestCapabilityBase>();
		UTEST_NOT_NULL("Retrived interface capacity", InterfaceCap2);
		UTEST_EQUAL("Same capacity?", InterfaceCap2, &InterfaceCap);
	}
	UTEST_EQUAL("Times destroyed", FTestCapabilityBase::TimesDestroyedBase, 1);
	UTEST_EQUAL("Times destroyed", FTestCapabilityDerived::TimesDestroyedDerived, 1);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieScenePlaybackCapabilitiesRawPointerTest, 
		"System.Engine.Sequencer.Capabilities.RawPointer", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieScenePlaybackCapabilitiesRawPointerTest::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Tests;

	FTestSimpleCapability::TimesDestroyed = 0;
	{
		FTestSimpleCapability ActualCap;
		{
			FPlaybackCapabilities Caps;
			UTEST_FALSE("No capability", Caps.HasCapability<FTestSimpleCapability>());

			FTestSimpleCapability& PtrCap = Caps.AddCapabilityRaw<FTestSimpleCapability>(&ActualCap);
			UTEST_TRUE("Found capability", Caps.HasCapability<FTestSimpleCapability>());

			FTestSimpleCapability* PtrCap2 = Caps.FindCapability<FTestSimpleCapability>();
			UTEST_NOT_NULL("Retrived test capacity", PtrCap2);
			UTEST_EQUAL("Same capacity?", PtrCap2, &PtrCap);
			UTEST_EQUAL("Same capacity?", PtrCap2, &ActualCap);
		}
		UTEST_EQUAL("Not destroyed", FTestSimpleCapability::TimesDestroyed, 0);
	}
	UTEST_EQUAL("Destroyed", FTestSimpleCapability::TimesDestroyed, 1);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieScenePlaybackCapabilitiesSharedTest, 
		"System.Engine.Sequencer.Capabilities.Shared", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieScenePlaybackCapabilitiesSharedTest::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Tests;

	FTestSimpleCapability::TimesDestroyed = 0;
	{
		TSharedRef<FTestSimpleCapability> ActualCap = MakeShared<FTestSimpleCapability>();
		UTEST_EQUAL("Ref count", ActualCap.GetSharedReferenceCount(), 1);
		{
			FPlaybackCapabilities Caps;
			UTEST_FALSE("No capability", Caps.HasCapability<FTestSimpleCapability>());

			FTestSimpleCapability& PtrCap = Caps.AddCapabilityShared<FTestSimpleCapability>(ActualCap);
			UTEST_TRUE("Found capability", Caps.HasCapability<FTestSimpleCapability>());
			UTEST_EQUAL("Ref count", ActualCap.GetSharedReferenceCount(), 2);

			FTestSimpleCapability* PtrCap2 = Caps.FindCapability<FTestSimpleCapability>();
			UTEST_NOT_NULL("Retrived test capacity", PtrCap2);
			UTEST_EQUAL("Same capacity?", PtrCap2, &PtrCap);
			UTEST_EQUAL("Same capacity?", PtrCap2, ActualCap.ToSharedPtr().Get());
		}
		UTEST_EQUAL("Not destroyed", FTestSimpleCapability::TimesDestroyed, 0);
		UTEST_EQUAL("Ref count", ActualCap.GetSharedReferenceCount(), 1);
	}
	UTEST_EQUAL("Destroyed", FTestSimpleCapability::TimesDestroyed, 1);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieScenePlaybackCapabilitiesInvalidateTest, 
		"System.Engine.Sequencer.Capabilities.Invalidate", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieScenePlaybackCapabilitiesInvalidateTest::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Tests;

	// Test simple inline
	{
		FPlaybackCapabilities Caps;
		FTestInvalidatableCapability& Cap = Caps.AddCapability<FTestInvalidatableCapability>();
		UTEST_EQUAL("Not invalidated", Cap.TimesInvalidated, 0);
		Caps.InvalidateCachedData(nullptr);
		UTEST_EQUAL("Invalidated", Cap.TimesInvalidated, 1);
	}

	// Test inline subclass implementing interface when base class doesn't
	{
		FPlaybackCapabilities Caps;
		FTestSimpleCapability& Cap = Caps.AddCapability<FTestSimpleCapabilityDerivedWithInterface>();
		FTestSimpleCapabilityDerivedWithInterface* ActualCap = static_cast<FTestSimpleCapabilityDerivedWithInterface*>(&Cap);
		UTEST_EQUAL("Not invalidated", ActualCap->TimesInvalidated, 0);
		Caps.InvalidateCachedData(nullptr);
		UTEST_EQUAL("Invalidated", ActualCap->TimesInvalidated, 1);
	}

	// Test raw pointer
	{
		FTestInvalidatableCapability ActualCap;

		FPlaybackCapabilities Caps;
		Caps.AddCapabilityRaw<FTestInvalidatableCapability>(&ActualCap);
		UTEST_EQUAL("Not invalidated", ActualCap.TimesInvalidated, 0);
		Caps.InvalidateCachedData(nullptr);
		UTEST_EQUAL("Invalidated", ActualCap.TimesInvalidated, 1);
	}

	// Test shared pointer
	{
		TSharedRef<FTestInvalidatableCapability> ActualCap = MakeShared<FTestInvalidatableCapability>();

		FPlaybackCapabilities Caps;
		Caps.AddCapabilityShared<FTestInvalidatableCapability>(ActualCap);
		UTEST_EQUAL("Not invalidated", ActualCap->TimesInvalidated, 0);
		Caps.InvalidateCachedData(nullptr);
		UTEST_EQUAL("Invalidated", ActualCap->TimesInvalidated, 1);
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieScenePlaybackCapabilitiesMultipleTest, 
		"System.Engine.Sequencer.Capabilities.Multiple", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieScenePlaybackCapabilitiesMultipleTest::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Tests;

	FTestOtherSimpleCapability RawPtrCap;
	TSharedRef<FTestInvalidatableCapability> SharedCap = MakeShared<FTestInvalidatableCapability>();
	{
		FPlaybackCapabilities Caps;

		Caps.AddCapabilityRaw<FTestOtherSimpleCapability>(&RawPtrCap);
		Caps.AddCapabilityShared<FTestInvalidatableCapability>(SharedCap);

		UTEST_FALSE("Found capability", Caps.HasCapability<FTestSimpleCapability>());
		UTEST_FALSE("Found capability", Caps.HasCapability<FTestCapabilityBase>());
		UTEST_TRUE("Found capability", Caps.HasCapability<FTestOtherSimpleCapability>());
		UTEST_TRUE("Found capability", Caps.HasCapability<FTestInvalidatableCapability>());

		Caps.AddCapability<FTestSimpleCapability>(8, TEXT("Other simple test"));
		Caps.AddCapability<FTestCapabilityDerived>(12, TEXT("Another implementation test"));

		UTEST_TRUE("Found capability", Caps.HasCapability<FTestSimpleCapability>());
		UTEST_TRUE("Found capability", Caps.HasCapability<FTestCapabilityBase>());
		UTEST_TRUE("Found capability", Caps.HasCapability<FTestOtherSimpleCapability>());
		UTEST_TRUE("Found capability", Caps.HasCapability<FTestInvalidatableCapability>());

		UTEST_EQUAL("Before invalidation", SharedCap->TimesInvalidated, 0);
		Caps.InvalidateCachedData(nullptr);
		UTEST_EQUAL("After invalidation", SharedCap->TimesInvalidated, 1);
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieScenePlaybackCapabilitiesInheritanceTest, 
		"System.Engine.Sequencer.Capabilities.Inheritance", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieScenePlaybackCapabilitiesInheritanceTest::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Tests;

	struct ICapabilityAccessor
	{
		virtual ~ICapabilityAccessor() {}
		virtual FTestCapabilityBase& GetCapability() = 0;
	};

	struct FCapabilityAccessor : ICapabilityAccessor
	{
		FTestCapabilityDerived Derived;

		FCapabilityAccessor() : Derived(42, TEXT("Life, the universe, everything")) {}

		virtual FTestCapabilityBase& GetCapability() override { return Derived; }
	};

	FCapabilityAccessor Accessor;
	{
		FPlaybackCapabilities Caps;

		Caps.AddCapabilityRaw<FTestCapabilityBase>(&Accessor.GetCapability());

		UTEST_TRUE("Found capability", Caps.HasCapability<FTestCapabilityBase>());
		FTestCapabilityBase* Base = Caps.FindCapability<FTestCapabilityBase>();
		UTEST_NOT_NULL("Got capability pointer", Base);
		UTEST_EQUAL("Same capability pointer", Base, static_cast<FTestCapabilityBase*>(&Accessor.Derived));
		UTEST_EQUAL("Checking IntValue", Base->GetIntValue(), 42);
		Base->DoSomething();
		UTEST_EQUAL("Checking IntValue", Base->GetIntValue(), 43);
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieScenePlaybackCapabilitiesOverwriteTest, 
		"System.Engine.Sequencer.Capabilities.Overwrite", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieScenePlaybackCapabilitiesOverwriteTest::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Tests;

	FTestSimpleCapability Cap1(1, "First");
	FTestSimpleCapability Cap2(2, "Second");
	{
		FPlaybackCapabilities Caps;

		Caps.AddCapabilityRaw<FTestSimpleCapability>(&Cap1);

		FTestSimpleCapability* ActualCap = Caps.FindCapability<FTestSimpleCapability>();
		UTEST_TRUE("Found capability", Caps.HasCapability<FTestSimpleCapability>());
		UTEST_EQUAL("Same capability pointer", ActualCap, &Cap1);
		UTEST_EQUAL("Checking IntValue", ActualCap->IntValue, 1);
		UTEST_EQUAL("Checking StringValue", ActualCap->StringValue, TEXT("First"));

		Caps.OverwriteCapabilityRaw<FTestSimpleCapability>(&Cap2);
		
		ActualCap = Caps.FindCapability<FTestSimpleCapability>();
		UTEST_TRUE("Found capability", Caps.HasCapability<FTestSimpleCapability>());
		UTEST_EQUAL("Same capability pointer", ActualCap, &Cap2);
		UTEST_EQUAL("Checking IntValue", ActualCap->IntValue, 2);
		UTEST_EQUAL("Checking StringValue", ActualCap->StringValue, TEXT("Second"));
	}

	FTestSimpleCapability::TimesDestroyed = 0;
	FTestSimpleCapabilityDerived::TimesDestroyedDerived = 0;
	{
		FPlaybackCapabilities Caps;

		// Add the base, overwrite with the derived
		// (this is only allowed because they both have the same size)
		Caps.AddCapability<FTestSimpleCapability>(1, TEXT("Base"));

		FTestSimpleCapability* BaseCap = Caps.FindCapability<FTestSimpleCapability>();
		UTEST_NOT_NULL("Found capabality", BaseCap);
		UTEST_EQUAL("Checking IntValue", BaseCap->IntValue, 1);
		UTEST_EQUAL("Checking StringValue", BaseCap->StringValue, TEXT("Base"));

		Caps.OverwriteCapability<FTestSimpleCapabilityDerived>(2, TEXT("Derived"));

		UTEST_EQUAL("Times destroyed", FTestSimpleCapability::TimesDestroyed, 1);
		UTEST_EQUAL("Times destroyed", FTestSimpleCapabilityDerived::TimesDestroyedDerived, 0);

		FTestSimpleCapability* DerivedCap = Caps.FindCapability<FTestSimpleCapability>();
		UTEST_NOT_NULL("Found capabality", DerivedCap);
		UTEST_EQUAL("Checking IntValue", DerivedCap->IntValue, 2);
		UTEST_EQUAL("Checking StringValue", DerivedCap->StringValue, TEXT("Derived"));
	}
	UTEST_EQUAL("Times destroyed", FTestSimpleCapability::TimesDestroyed, 2);
	UTEST_EQUAL("Times destroyed", FTestSimpleCapabilityDerived::TimesDestroyedDerived, 1);

	FTestSimpleCapability::TimesDestroyed = 0;
	FTestSimpleCapabilityDerived::TimesDestroyedDerived = 0;
	{
		FPlaybackCapabilities Caps;

		// Add the derived, overwrite with the base
		// (this is only allowed because they both have the same size)
		Caps.AddCapability<FTestSimpleCapabilityDerived>(2, TEXT("Derived"));

		FTestSimpleCapability* DerivedCap = Caps.FindCapability<FTestSimpleCapability>();
		UTEST_NOT_NULL("Found capabality", DerivedCap);
		UTEST_EQUAL("Checking IntValue", DerivedCap->IntValue, 2);
		UTEST_EQUAL("Checking StringValue", DerivedCap->StringValue, TEXT("Derived"));

		Caps.OverwriteCapability<FTestSimpleCapability>(1, TEXT("Base"));

		UTEST_EQUAL("Times destroyed", FTestSimpleCapability::TimesDestroyed, 1);
		UTEST_EQUAL("Times destroyed", FTestSimpleCapabilityDerived::TimesDestroyedDerived, 1);

		FTestSimpleCapability* BaseCap = Caps.FindCapability<FTestSimpleCapability>();
		UTEST_NOT_NULL("Found capabality", BaseCap);
		UTEST_EQUAL("Checking IntValue", BaseCap->IntValue, 1);
		UTEST_EQUAL("Checking StringValue", BaseCap->StringValue, TEXT("Base"));
	}
	UTEST_EQUAL("Times destroyed", FTestSimpleCapability::TimesDestroyed, 2);
	UTEST_EQUAL("Times destroyed", FTestSimpleCapabilityDerived::TimesDestroyedDerived, 1);

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS
