// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Evaluation/MovieScenePlaybackManager.h"
#include "Misc/AutomationTest.h"
#include "MovieSceneFwd.h"
#include "MovieSceneTimeHelpers.h"
#include "Tests/MovieSceneTestObjects.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

#define LOCTEXT_NAMESPACE "MovieScenePlaybackManagerTests"

namespace UE::MovieScene::Test
{

/**
 * Default test sequence is 10 seconds long, at 30fps, with 60000 ticks/second.
 */
struct FMakeTestSequenceParams
{
	int32 StartTick = 0;
	int32 DurationTicks = 60000 * 10;

	int32 TickResolution = 60000;
	int32 DisplayRate = 30;

	FMakeTestSequenceParams()
	{}
	FMakeTestSequenceParams(int32 InDurationTicks) 
		: DurationTicks(InDurationTicks)
	{}
	FMakeTestSequenceParams(int32 InStartTick, int32 InDurationTicks) 
		: StartTick(InStartTick)
		, DurationTicks(InDurationTicks)
	{}
};

UMovieSceneSequence* MakeTestSequence(const FMakeTestSequenceParams& Params)
{
	UTestMovieSceneSequence* Sequence = NewObject<UTestMovieSceneSequence>(GetTransientPackage());
	
	Sequence->MovieScene->SetDisplayRate(FFrameRate(Params.DisplayRate, 1));
	Sequence->MovieScene->SetTickResolutionDirectly(FFrameRate(Params.TickResolution, 1));
	Sequence->MovieScene->SetPlaybackRange(FFrameNumber(Params.StartTick), Params.DurationTicks);
	
	return Sequence;
}

TRange<FFrameTime> MakeDiscreteTimeRange(FFrameTime MinInclusive, FFrameTime MaxExclusive)
{
	return TRange<FFrameTime>(
			TRangeBound<FFrameTime>::Inclusive(MinInclusive),
			TRangeBound<FFrameTime>::Exclusive(MaxExclusive));
}

TRange<FFrameTime> MakeContinuedTimeRange(FFrameTime MinExclusive, FFrameTime MaxInclusive)
{
	return TRange<FFrameTime>(
			TRangeBound<FFrameTime>::Exclusive(MinExclusive),
			TRangeBound<FFrameTime>::Inclusive(MaxInclusive));
}

TRange<FFrameTime> MakeHullTimeRange(FFrameTime MinInclusive, FFrameTime MaxInclusive)
{
	return TRange<FFrameTime>(
			TRangeBound<FFrameTime>::Inclusive(MinInclusive),
			TRangeBound<FFrameTime>::Inclusive(MaxInclusive));
}

}  // namespace UE::MovieScene::Test

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMovieScenePlaybackManagerTestStartEndTimes, 
		"System.Engine.Sequencer.PlaybackManager.StartEndTimes", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieScenePlaybackManagerTestStartEndTimes::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Test;

	FMakeTestSequenceParams MakeParams;
	UMovieSceneSequence* Sequence = MakeTestSequence(MakeParams);

	FMovieScenePlaybackManager Manager(Sequence);

	UTEST_EQUAL("PlaybackRange", 
			Manager.GetEffectivePlaybackRange(),
			MakeDiscreteTimeRange(0, 30 * 10));

	Manager.SetStartOffset(30 * 1);

	UTEST_EQUAL("PlaybackRange", 
			Manager.GetEffectivePlaybackRange(),
			MakeDiscreteTimeRange(30 * 1, 30 * 10));

	Manager.SetEndOffset(FFrameTime(30 * 2, 0.5));

	UTEST_EQUAL("PlaybackRange", 
			Manager.GetEffectivePlaybackRange(),
			MakeDiscreteTimeRange(FFrameTime(30 * 1), FFrameTime(30 * 7 + 29, 0.5)));

	Manager.SetEndOffsetAsTime(30 * 8);

	UTEST_EQUAL("PlaybackRange", 
			Manager.GetEffectivePlaybackRange(),
			MakeDiscreteTimeRange(30 * 1, 30 * 8));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMovieScenePlaybackManagerTestLoopingNoDissection, 
		"System.Engine.Sequencer.PlaybackManager.Looping.NoDissection", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieScenePlaybackManagerTestLoopingNoDissection::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Test;

	FMakeTestSequenceParams MakeParams;
	UMovieSceneSequence* Sequence = MakeTestSequence(MakeParams);

	FMovieScenePlaybackManager Manager(Sequence);
	Manager.SetDissectLooping(EMovieSceneLoopDissection::None);
	Manager.SetNumLoopsToPlay(4);
	Manager.SetPlaybackStatus(EMovieScenePlayerStatus::Playing);

	const int32 TR = 60000;
	const FFrameTime LastValidTick(10 * TR - 1);

	FMovieScenePlaybackManager::FContexts Contexts;

	{
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 0);

		Contexts.Reset();
		Manager.UpdateTo(30 * 5, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 1);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeHullTimeRange(0, TR * 5));
		UTEST_FALSE("Context0_Jumped", Contexts[0].HasJumped());
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 0);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Playing);

		Contexts.Reset();
		Manager.UpdateTo(30 * 12, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 1);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeHullTimeRange(0, TR * 2));
		UTEST_TRUE("Context0_Jumped", Contexts[0].HasJumped());
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 1);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Playing);

		Contexts.Reset();
		Manager.UpdateTo(30 * 23, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 1);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeHullTimeRange(0, TR * 3));
		UTEST_TRUE("Context0_Jumped", Contexts[0].HasJumped());
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 3);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Playing);

		Contexts.Reset();
		Manager.UpdateTo(30 * 11, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 1);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeContinuedTimeRange(TR * 3, LastValidTick));
		UTEST_FALSE("Context0_Jumped", Contexts[0].HasJumped());
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 4);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Stopped);
	}

	Manager.ResetNumLoopsCompleted();
	Manager.SetPlayDirection(EPlayDirection::Backwards);
	Manager.SetPlaybackStatus(EMovieScenePlayerStatus::Playing);

	{
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 0);

		Manager.SetCurrentTime(30 * 10);

		Contexts.Reset();
		Manager.UpdateTo(30 * 5, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 1);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeHullTimeRange(TR * 5, LastValidTick));
		UTEST_FALSE("Context0_Jumped", Contexts[0].HasJumped());
		UTEST_EQUAL("Context0_Backwards", Contexts[0].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 0);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Playing);

		Contexts.Reset();
		Manager.UpdateTo(30 * -2, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 1);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeHullTimeRange(TR * 8, LastValidTick));
		UTEST_TRUE ("Context0_Jumped", Contexts[0].HasJumped());
		UTEST_EQUAL("Context0_Backwards", Contexts[0].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 1);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Playing);

		Contexts.Reset();
		Manager.UpdateTo(30 * -13, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 1);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeHullTimeRange(TR * 7, LastValidTick));
		UTEST_TRUE ("Context0_Jumped", Contexts[0].HasJumped());
		UTEST_EQUAL("Context0_Backwards", Contexts[0].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 3);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Playing);

		Contexts.Reset();
		Manager.UpdateTo(30 * -1, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 1);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeDiscreteTimeRange(0, TR * 7));
		UTEST_FALSE("Context0_Jumped", Contexts[0].HasJumped());
		UTEST_EQUAL("Context0_Backwards", Contexts[0].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 4);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Stopped);
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMovieScenePlaybackManagerTestLoopingDissectOne, 
		"System.Engine.Sequencer.PlaybackManager.Looping.DissectOne", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieScenePlaybackManagerTestLoopingDissectOne::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Test;

	FMakeTestSequenceParams MakeParams;
	UMovieSceneSequence* Sequence = MakeTestSequence(MakeParams);

	FMovieScenePlaybackManager Manager(Sequence);
	Manager.SetDissectLooping(EMovieSceneLoopDissection::DissectOne);
	Manager.SetNumLoopsToPlay(4);
	Manager.SetPlaybackStatus(EMovieScenePlayerStatus::Playing);

	const int32 TR = 60000;
	const FFrameTime LastValidTick(10 * TR - 1);

	FMovieScenePlaybackManager::FContexts Contexts;

	{

		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 0);

		Contexts.Reset();
		Manager.UpdateTo(30 * 5, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 1);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeHullTimeRange(0, TR * 5));
		UTEST_FALSE("Context0_Jumped", Contexts[0].HasJumped());
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 0);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Playing);

		Contexts.Reset();
		Manager.UpdateTo(30 * 12, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 2);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeContinuedTimeRange(TR * 5, LastValidTick));
		UTEST_FALSE("Context0_Jumped", Contexts[0].HasJumped());
		UTEST_EQUAL("Context1", Contexts[1].GetRange(), MakeHullTimeRange(0, TR * 2));
		UTEST_TRUE ("Context1_Jumped", Contexts[1].HasJumped());
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 1);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Playing);

		Contexts.Reset();
		Manager.UpdateTo(30 * 23, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 2);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeContinuedTimeRange(TR * 2, LastValidTick));
		UTEST_FALSE("Context0_Jumped", Contexts[0].HasJumped());
		UTEST_EQUAL("Context1", Contexts[1].GetRange(), MakeHullTimeRange(0, TR * 3));
		UTEST_TRUE ("Context1_Jumped", Contexts[1].HasJumped());
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 3);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Playing);

		Contexts.Reset();
		Manager.UpdateTo(30 * 11, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 1);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeContinuedTimeRange(TR * 3, LastValidTick));
		UTEST_FALSE("Context0_Jumped", Contexts[0].HasJumped());
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 4);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Stopped);

	}

	Manager.ResetNumLoopsCompleted();
	Manager.SetPlayDirection(EPlayDirection::Backwards);
	Manager.SetPlaybackStatus(EMovieScenePlayerStatus::Playing);

	{
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 0);

		Manager.SetCurrentTime(30 * 10);

		Contexts.Reset();
		Manager.UpdateTo(30 * 5, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 1);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeHullTimeRange(TR * 5, LastValidTick));
		UTEST_FALSE("Context0_Jumped", Contexts[0].HasJumped());
		UTEST_EQUAL("Context0_Backwards", Contexts[0].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 0);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Playing);

		Contexts.Reset();
		Manager.UpdateTo(30 * -2, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 2);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeDiscreteTimeRange(0, TR * 5));
		UTEST_FALSE("Context0_Jumped", Contexts[0].HasJumped());
		UTEST_EQUAL("Context0_Backwards", Contexts[0].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("Context1", Contexts[1].GetRange(), MakeHullTimeRange(TR * 8, LastValidTick));
		UTEST_TRUE ("Context1_Jumped", Contexts[1].HasJumped());
		UTEST_EQUAL("Context1_Backwards", Contexts[1].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 1);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Playing);

		Contexts.Reset();
		Manager.UpdateTo(30 * -13, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 2);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeDiscreteTimeRange(0, TR * 8));
		UTEST_FALSE("Context0_Jumped", Contexts[0].HasJumped());
		UTEST_EQUAL("Context0_Backwards", Contexts[0].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("Context1", Contexts[1].GetRange(), MakeHullTimeRange(TR * 7, LastValidTick));
		UTEST_TRUE ("Context1_Jumped", Contexts[1].HasJumped());
		UTEST_EQUAL("Context1_Backwards", Contexts[1].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 3);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Playing);

		Contexts.Reset();
		Manager.UpdateTo(30 * -1, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 1);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeDiscreteTimeRange(0, TR * 7));
		UTEST_FALSE("Context0_Jumped", Contexts[0].HasJumped());
		UTEST_EQUAL("Context0_Backwards", Contexts[0].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 4);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Stopped);
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMovieScenePlaybackManagerTestLoopingDissectAll, 
		"System.Engine.Sequencer.PlaybackManager.Looping.DissectAll", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieScenePlaybackManagerTestLoopingDissectAll::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Test;

	FMakeTestSequenceParams MakeParams;
	UMovieSceneSequence* Sequence = MakeTestSequence(MakeParams);

	FMovieScenePlaybackManager Manager(Sequence);
	Manager.SetDissectLooping(EMovieSceneLoopDissection::DissectAll);
	Manager.SetNumLoopsToPlay(4);
	Manager.SetPlaybackStatus(EMovieScenePlayerStatus::Playing);

	const int32 TR = 60000;
	const FFrameTime LastValidTick(10 * TR - 1);

	FMovieScenePlaybackManager::FContexts Contexts;

	{

		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 0);

		Contexts.Reset();
		Manager.UpdateTo(30 * 5, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 1);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeHullTimeRange(0, TR * 5));
		UTEST_FALSE("Context0_Jumped", Contexts[0].HasJumped());
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 0);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Playing);

		Contexts.Reset();
		Manager.UpdateTo(30 * 12, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 2);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeContinuedTimeRange(TR * 5, LastValidTick));
		UTEST_FALSE("Context0_Jumped", Contexts[0].HasJumped());
		UTEST_EQUAL("Context1", Contexts[1].GetRange(), MakeHullTimeRange(0, TR * 2));
		UTEST_TRUE ("Context1_Jumped", Contexts[1].HasJumped());
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 1);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Playing);

		Contexts.Reset();
		Manager.UpdateTo(30 * 23, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 3);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeContinuedTimeRange(TR * 2, LastValidTick));
		UTEST_FALSE("Context0_Jumped", Contexts[0].HasJumped());
		UTEST_EQUAL("Context1", Contexts[1].GetRange(), MakeHullTimeRange(0, LastValidTick));
		UTEST_TRUE ("Context1_Jumped", Contexts[1].HasJumped());
		UTEST_EQUAL("Context2", Contexts[2].GetRange(), MakeHullTimeRange(0, TR * 3));
		UTEST_TRUE ("Context2_Jumped", Contexts[2].HasJumped());
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 3);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Playing);

		Contexts.Reset();
		Manager.UpdateTo(30 * 11, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 1);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeContinuedTimeRange(TR * 3, LastValidTick));
		UTEST_FALSE("Context0_Jumped", Contexts[0].HasJumped());
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 4);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Stopped);

	}

	Manager.ResetNumLoopsCompleted();
	Manager.SetPlayDirection(EPlayDirection::Backwards);
	Manager.SetPlaybackStatus(EMovieScenePlayerStatus::Playing);

	{
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 0);

		Manager.SetCurrentTime(30 * 10);

		Contexts.Reset();
		Manager.UpdateTo(30 * 5, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 1);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeHullTimeRange(TR * 5, LastValidTick));
		UTEST_FALSE("Context0_Jumped", Contexts[0].HasJumped());
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 0);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Playing);

		Contexts.Reset();
		Manager.UpdateTo(30 * -2, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 2);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeDiscreteTimeRange(0, TR * 5));
		UTEST_FALSE("Context0_Jumped", Contexts[0].HasJumped());
		UTEST_EQUAL("Context0_Backwards", Contexts[0].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("Context1", Contexts[1].GetRange(), MakeHullTimeRange(TR * 8, LastValidTick));
		UTEST_TRUE ("Context1_Jumped", Contexts[1].HasJumped());
		UTEST_EQUAL("Context1_Backwards", Contexts[1].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 1);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Playing);

		Contexts.Reset();
		Manager.UpdateTo(30 * -13, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 3);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeDiscreteTimeRange(0, TR * 8));
		UTEST_FALSE("Context0_Jumped", Contexts[0].HasJumped());
		UTEST_EQUAL("Context0_Backwards", Contexts[0].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("Context1", Contexts[1].GetRange(), MakeHullTimeRange(0, LastValidTick));
		UTEST_TRUE ("Context1_Jumped", Contexts[1].HasJumped());
		UTEST_EQUAL("Context1_Backwards", Contexts[1].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("Context2", Contexts[2].GetRange(), MakeHullTimeRange(TR * 7, LastValidTick));
		UTEST_TRUE ("Context2_Jumped", Contexts[2].HasJumped());
		UTEST_EQUAL("Context2_Backwards", Contexts[2].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 3);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Playing);

		Contexts.Reset();
		Manager.UpdateTo(30 * -1, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 1);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeDiscreteTimeRange(0, TR * 7));
		UTEST_FALSE("Context0_Jumped", Contexts[0].HasJumped());
		UTEST_EQUAL("Context0_Backwards", Contexts[0].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 4);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Stopped);
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMovieScenePlaybackManagerTestUpdateToEnd, 
		"System.Engine.Sequencer.PlaybackManager.UpdateToEnd", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieScenePlaybackManagerTestUpdateToEnd::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Test;

	FMakeTestSequenceParams MakeParams;
	UMovieSceneSequence* Sequence = MakeTestSequence(MakeParams);

	FMovieScenePlaybackManager Manager(Sequence);
	Manager.SetNumLoopsToPlay(1);

	const int32 TR = 60000;
	const FFrameTime LastValidTick(10 * TR - 1);
	const FFrameTime LastValidFrame = ConvertFrameTime(LastValidTick, FFrameRate(60000, 1), FFrameRate(30, 1));

	FMovieScenePlaybackManager::FContexts Contexts;

	TArray<EMovieSceneLoopDissection> Dissections{
		EMovieSceneLoopDissection::None,
		EMovieSceneLoopDissection::DissectOne,
		EMovieSceneLoopDissection::DissectAll
	};

	for (EMovieSceneLoopDissection Dissection : Dissections)
	{
		Manager.SetDissectLooping(Dissection);

		Manager.ResetNumLoopsCompleted();
		Manager.SetPlayDirection(EPlayDirection::Forwards);
		Manager.SetPlaybackStatus(EMovieScenePlayerStatus::Playing);
		Manager.SetCurrentTime(0);

		{
			UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 0);

			Contexts.Reset();
			Manager.UpdateTo(LastValidFrame, Contexts);
			UTEST_EQUAL("NumContexts", Contexts.Num(), 1);
			UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeHullTimeRange(0, LastValidTick));
			UTEST_FALSE("Context0_Jumped", Contexts[0].HasJumped());
			UTEST_EQUAL("Context0_Forwards", Contexts[0].GetDirection(), EPlayDirection::Forwards);
			UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 1);
			UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Stopped);
		}

		Manager.ResetNumLoopsCompleted();
		Manager.SetPlayDirection(EPlayDirection::Backwards);
		Manager.SetPlaybackStatus(EMovieScenePlayerStatus::Playing);
		Manager.SetCurrentTime(LastValidFrame);

		{
			UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 0);

			Contexts.Reset();
			Manager.UpdateTo(0, Contexts);
			UTEST_EQUAL("NumContexts", Contexts.Num(), 1);
			UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeHullTimeRange(0, LastValidTick));
			UTEST_FALSE("Context0_Jumped", Contexts[0].HasJumped());
			UTEST_EQUAL("Context0_Backwards", Contexts[0].GetDirection(), EPlayDirection::Backwards);
			UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 1);
			UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Stopped);
		}
	}

	return true;
}
IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMovieScenePlaybackManagerTestCustomEndTime, 
		"System.Engine.Sequencer.PlaybackManager.CustomEndTime", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieScenePlaybackManagerTestCustomEndTime::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Test;

	FMakeTestSequenceParams MakeParams;
	UMovieSceneSequence* Sequence = MakeTestSequence(MakeParams);

	const int32 FPS = 30;
	const int32 TR = 60000;
	const FFrameTime LastValidTick(10 * TR - 1);

	FMovieScenePlaybackManager Manager(Sequence);
	Manager.SetPlaybackEndTime(6 * FPS); // End with the frame at 6 seconds.

	FMovieScenePlaybackManager::FContexts Contexts;

	Manager.SetPlayDirection(EPlayDirection::Forwards);
	Manager.SetDissectLooping(EMovieSceneLoopDissection::DissectOne);

	Manager.SetCurrentTime(0);
	Manager.SetPlaybackStatus(EMovieScenePlayerStatus::Playing);

	{
		Manager.SetNumLoopsToPlay(1);
		Contexts.Reset();
		Manager.UpdateTo(8 * FPS, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 1);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeHullTimeRange(0, TR * 6));
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 1);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Stopped);
	}

	Manager.ResetNumLoopsCompleted();
	Manager.SetCurrentTime(0);
	Manager.SetPlaybackStatus(EMovieScenePlayerStatus::Playing);

	{
		Manager.SetNumLoopsToPlay(2);
		Contexts.Reset();
		Manager.UpdateTo(8 * FPS, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 1);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeHullTimeRange(0, TR * 8));
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 0);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Playing);

		Contexts.Reset();
		Manager.UpdateTo(12 * FPS, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 2);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeContinuedTimeRange(TR * 8, LastValidTick));
		UTEST_EQUAL("Context1", Contexts[1].GetRange(), MakeHullTimeRange(0, TR * 2));
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 1);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Playing);

		Contexts.Reset();
		Manager.UpdateTo(8 * FPS, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 1);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeContinuedTimeRange(TR * 2, TR * 6));
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 2);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Stopped);
	}

	Manager.ResetNumLoopsCompleted();
	Manager.SetCurrentTime(0);
	Manager.SetPlaybackStatus(EMovieScenePlayerStatus::Playing);

	{
		Manager.SetNumLoopsToPlay(3);
		Contexts.Reset();
		Manager.UpdateTo(40 * FPS, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 2);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeHullTimeRange(0, LastValidTick));
		UTEST_EQUAL("Context1", Contexts[1].GetRange(), MakeHullTimeRange(0, TR * 6));
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 3);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Stopped);
	}

	Manager.ResetNumLoopsCompleted();
	Manager.SetCurrentTime(0);
	Manager.SetPlaybackStatus(EMovieScenePlayerStatus::Playing);

	{
		Manager.SetNumLoopsToPlay(4);
		Contexts.Reset();
		Manager.UpdateTo(12 * FPS, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 2);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeHullTimeRange(0, LastValidTick));
		UTEST_EQUAL("Context1", Contexts[1].GetRange(), MakeHullTimeRange(0, TR * 2));
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 1);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Playing);

		Contexts.Reset();
		Manager.UpdateTo(29 * FPS, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 2);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeContinuedTimeRange(TR * 2, LastValidTick));
		UTEST_EQUAL("Context1", Contexts[1].GetRange(), MakeHullTimeRange(0, TR * 6));
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 4);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Stopped);
	}

	Manager.SetPlayDirection(EPlayDirection::Forwards);
	Manager.SetDissectLooping(EMovieSceneLoopDissection::DissectAll);

	Manager.ResetNumLoopsCompleted();
	Manager.SetCurrentTime(0);
	Manager.SetPlaybackStatus(EMovieScenePlayerStatus::Playing);

	{
		Manager.SetNumLoopsToPlay(1);
		Contexts.Reset();
		Manager.UpdateTo(8 * FPS, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 1);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeHullTimeRange(0, TR * 6));
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 1);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Stopped);
	}

	Manager.ResetNumLoopsCompleted();
	Manager.SetCurrentTime(0);
	Manager.SetPlaybackStatus(EMovieScenePlayerStatus::Playing);

	{
		Manager.SetNumLoopsToPlay(2);
		Contexts.Reset();
		Manager.UpdateTo(8 * FPS, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 1);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeHullTimeRange(0, TR * 8));
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 0);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Playing);

		Contexts.Reset();
		Manager.UpdateTo(12 * FPS, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 2);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeContinuedTimeRange(TR * 8, LastValidTick));
		UTEST_EQUAL("Context1", Contexts[1].GetRange(), MakeHullTimeRange(0, TR * 2));
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 1);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Playing);

		Contexts.Reset();
		Manager.UpdateTo(8 * FPS, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 1);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeContinuedTimeRange(TR * 2, TR * 6));
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 2);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Stopped);
	}

	Manager.ResetNumLoopsCompleted();
	Manager.SetCurrentTime(0);
	Manager.SetPlaybackStatus(EMovieScenePlayerStatus::Playing);

	{
		Manager.SetNumLoopsToPlay(3);
		Contexts.Reset();
		Manager.UpdateTo(40 * FPS, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 3);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeHullTimeRange(0, LastValidTick));
		UTEST_EQUAL("Context1", Contexts[1].GetRange(), MakeHullTimeRange(0, LastValidTick));
		UTEST_EQUAL("Context2", Contexts[2].GetRange(), MakeHullTimeRange(0, TR * 6));
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 3);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Stopped);
	}

	Manager.ResetNumLoopsCompleted();
	Manager.SetCurrentTime(0);
	Manager.SetPlaybackStatus(EMovieScenePlayerStatus::Playing);

	{
		Manager.SetNumLoopsToPlay(4);
		Contexts.Reset();
		Manager.UpdateTo(12 * FPS, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 2);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeHullTimeRange(0, LastValidTick));
		UTEST_EQUAL("Context1", Contexts[1].GetRange(), MakeHullTimeRange(0, TR * 2));
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 1);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Playing);

		Contexts.Reset();
		Manager.UpdateTo(29 * FPS, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 3);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeContinuedTimeRange(TR * 2, LastValidTick));
		UTEST_EQUAL("Context1", Contexts[1].GetRange(), MakeHullTimeRange(0, LastValidTick));
		UTEST_EQUAL("Context2", Contexts[2].GetRange(), MakeHullTimeRange(0, TR * 6));
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 4);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Stopped);
	}

	Manager.SetPlayDirection(EPlayDirection::Backwards);
	Manager.SetDissectLooping(EMovieSceneLoopDissection::DissectOne);

	Manager.ResetNumLoopsCompleted();
	Manager.SetCurrentTime(10 * FPS);
	Manager.SetPlaybackStatus(EMovieScenePlayerStatus::Playing);

	{
		Manager.SetNumLoopsToPlay(1);
		Contexts.Reset();
		Manager.UpdateTo(2 * FPS, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 1);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeHullTimeRange(TR * 6, LastValidTick));
		UTEST_EQUAL("Context0_Direction", Contexts[0].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 1);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Stopped);
	}

	Manager.ResetNumLoopsCompleted();
	Manager.SetCurrentTime(10 * FPS);
	Manager.SetPlaybackStatus(EMovieScenePlayerStatus::Playing);

	{
		Manager.SetNumLoopsToPlay(2);
		Contexts.Reset();
		Manager.UpdateTo(2 * FPS, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 1);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeHullTimeRange(TR * 2, LastValidTick));
		UTEST_EQUAL("Context0_Direction", Contexts[0].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 0);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Playing);

		Contexts.Reset();
		Manager.UpdateTo(-3 * FPS, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 2);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeDiscreteTimeRange(0, TR * 2));
		UTEST_EQUAL("Context0_Direction", Contexts[0].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("Context1", Contexts[1].GetRange(), MakeHullTimeRange(TR * 7, LastValidTick));
		UTEST_EQUAL("Context1_Direction", Contexts[1].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 1);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Playing);

		Contexts.Reset();
		Manager.UpdateTo(4 * FPS, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 1);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeDiscreteTimeRange(TR * 6, TR * 7));
		UTEST_EQUAL("Context0_Direction", Contexts[0].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 2);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Stopped);
	}

	Manager.ResetNumLoopsCompleted();
	Manager.SetCurrentTime(10 * FPS);
	Manager.SetPlaybackStatus(EMovieScenePlayerStatus::Playing);

	{
		Manager.SetNumLoopsToPlay(3);
		Contexts.Reset();
		Manager.UpdateTo(-30 * FPS, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 2);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeHullTimeRange(0, LastValidTick));
		UTEST_EQUAL("Context0_Direction", Contexts[0].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("Context1", Contexts[1].GetRange(), MakeHullTimeRange(TR * 6, LastValidTick));
		UTEST_EQUAL("Context1_Direction", Contexts[1].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 3);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Stopped);
	}

	Manager.ResetNumLoopsCompleted();
	Manager.SetCurrentTime(10 * FPS);
	Manager.SetPlaybackStatus(EMovieScenePlayerStatus::Playing);

	{
		Manager.SetNumLoopsToPlay(4);
		Contexts.Reset();
		Manager.UpdateTo(-2 * FPS, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 2);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeHullTimeRange(0, LastValidTick));
		UTEST_EQUAL("Context0_Direction", Contexts[0].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("Context1", Contexts[1].GetRange(), MakeHullTimeRange(TR * 8, LastValidTick));
		UTEST_EQUAL("Context1_Direction", Contexts[1].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 1);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Playing);

		Contexts.Reset();
		Manager.UpdateTo(-29 * FPS, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 2);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeDiscreteTimeRange(0, TR * 8));
		UTEST_EQUAL("Context0_Direction", Contexts[0].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("Context1", Contexts[1].GetRange(), MakeHullTimeRange(TR * 6, LastValidTick));
		UTEST_EQUAL("Context1_Direction", Contexts[1].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 4);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Stopped);
	}

	Manager.SetDissectLooping(EMovieSceneLoopDissection::DissectOne);
	Manager.SetPingPongPlayback(true);

	Manager.SetPlayDirection(EPlayDirection::Forwards);
	Manager.ResetNumLoopsCompleted();
	Manager.SetCurrentTime(0);
	Manager.SetPlaybackStatus(EMovieScenePlayerStatus::Playing);

	{
		Manager.SetNumLoopsToPlay(1);
		Contexts.Reset();
		Manager.UpdateTo(8 * FPS, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 1);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeHullTimeRange(0, TR * 6));
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 1);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Stopped);
	}

	Manager.SetPlayDirection(EPlayDirection::Forwards);
	Manager.ResetNumLoopsCompleted();
	Manager.SetCurrentTime(0);
	Manager.SetPlaybackStatus(EMovieScenePlayerStatus::Playing);

	{
		Manager.SetNumLoopsToPlay(2);
		Contexts.Reset();
		Manager.UpdateTo(8 * FPS, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 1);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeHullTimeRange(0, TR * 8));
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 0);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Playing);

		Contexts.Reset();
		Manager.UpdateTo(12 * FPS, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 2);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeContinuedTimeRange(TR * 8, LastValidTick));
		UTEST_EQUAL("Context0_Direction", Contexts[0].GetDirection(), EPlayDirection::Forwards);
		UTEST_EQUAL("Context1", Contexts[1].GetRange(), MakeHullTimeRange(TR * 8 - 1, LastValidTick));
		UTEST_EQUAL("Context1_Direction", Contexts[1].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 1);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Playing);

		Contexts.Reset();
		Manager.UpdateTo(2 * FPS, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 1);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeDiscreteTimeRange(TR * 6, TR * 8 - 1));
		UTEST_EQUAL("Context0_Direction", Contexts[0].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 2);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Stopped);
	}

	Manager.SetPlayDirection(EPlayDirection::Forwards);
	Manager.ResetNumLoopsCompleted();
	Manager.SetCurrentTime(0);
	Manager.SetPlaybackStatus(EMovieScenePlayerStatus::Playing);

	{
		Manager.SetNumLoopsToPlay(3);
		Contexts.Reset();
		Manager.UpdateTo(40 * FPS, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 2);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeHullTimeRange(0, LastValidTick));
		UTEST_EQUAL("Context0_Direction", Contexts[0].GetDirection(), EPlayDirection::Forwards);
		UTEST_EQUAL("Context1", Contexts[1].GetRange(), MakeHullTimeRange(0, TR * 6));
		UTEST_EQUAL("Context1_Direction", Contexts[1].GetDirection(), EPlayDirection::Forwards);
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 3);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Stopped);
	}

	Manager.SetPlayDirection(EPlayDirection::Forwards);
	Manager.ResetNumLoopsCompleted();
	Manager.SetCurrentTime(0);
	Manager.SetPlaybackStatus(EMovieScenePlayerStatus::Playing);

	{
		Manager.SetNumLoopsToPlay(4);
		Contexts.Reset();
		Manager.UpdateTo(12 * FPS, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 2);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeHullTimeRange(0, LastValidTick));
		UTEST_EQUAL("Context0_Direction", Contexts[0].GetDirection(), EPlayDirection::Forwards);
		UTEST_EQUAL("Context1", Contexts[1].GetRange(), MakeHullTimeRange(TR * 8 - 1, LastValidTick));
		UTEST_EQUAL("Context1_Direction", Contexts[1].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 1);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Playing);

		Contexts.Reset();
		Manager.UpdateTo(-30 * FPS, Contexts);
		UTEST_EQUAL("NumContexts", Contexts.Num(), 2);
		UTEST_EQUAL("Context0", Contexts[0].GetRange(), MakeDiscreteTimeRange(0, TR * 8 - 1));
		UTEST_EQUAL("Context0_Direction", Contexts[0].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("Context1", Contexts[1].GetRange(), MakeHullTimeRange(TR * 6, LastValidTick));
		UTEST_EQUAL("Context1_Direction", Contexts[1].GetDirection(), EPlayDirection::Backwards);
		UTEST_EQUAL("NumLoops", Manager.GetNumLoopsCompleted(), 4);
		UTEST_EQUAL("Status", Manager.GetPlaybackStatus(), EMovieScenePlayerStatus::Stopped);
	}

	return true;
}

#undef LOCTEXT_NAMESPACE

