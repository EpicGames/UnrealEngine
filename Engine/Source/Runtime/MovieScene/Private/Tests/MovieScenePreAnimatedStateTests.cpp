// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "MovieSceneExecutionToken.h"
#include "IMovieScenePlayer.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "Evaluation/MovieScenePreAnimatedState.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedCaptureSources.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedObjectGroupManager.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateExtension.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateStorage.h"
#include "MovieSceneTestObjects.h"
#include "UObject/ObjectKey.h"
#include "UObject/Package.h"

#define LOCTEXT_NAMESPACE "MovieScenePreAnimatedStateTests"

namespace Impl
{
	struct FPreAnimatedToken : IMovieScenePreAnimatedGlobalToken
	{
		int32* Ptr;
		int32 Value;

		FPreAnimatedToken(int32* InPtr, int32 InValue) : Ptr(InPtr), Value(InValue) {}

		virtual void RestoreState(const UE::MovieScene::FRestoreStateParams& Params) override
		{
			*Ptr = Value;
		}
	};

	struct FPreAnimatedTokenProducer : IMovieScenePreAnimatedGlobalTokenProducer
	{
		int32* Ptr;
		mutable int32 InitializeCount;

		FPreAnimatedTokenProducer(int32* InPtr) : Ptr(InPtr), InitializeCount(0) {}

		virtual void InitializeForAnimation() const override
		{
			++InitializeCount;
			*Ptr = 0;
		}

		virtual IMovieScenePreAnimatedGlobalTokenPtr CacheExistingState() const override
		{
			return FPreAnimatedToken(Ptr, *Ptr);
		}
	};

	struct FTestMovieScenePlayer : IMovieScenePlayer
	{
		FMovieSceneRootEvaluationTemplateInstance Template;
		virtual FMovieSceneRootEvaluationTemplateInstance& GetEvaluationTemplate() override { return Template; }
		virtual void SetViewportSettings(const TMap<FViewportClient*, EMovieSceneViewportParams>& ViewportParamsMap) override {}
		virtual void GetViewportSettings(TMap<FViewportClient*, EMovieSceneViewportParams>& ViewportParamsMap) const override {}
		virtual EMovieScenePlayerStatus::Type GetPlaybackStatus() const override { return EMovieScenePlayerStatus::Playing; }
		virtual void SetPlaybackStatus(EMovieScenePlayerStatus::Type InPlaybackStatus) override {}
	};

	static const int32 TestMagicNumber = 0xdeadbeef;

	int32 TestValue1 = TestMagicNumber;
	int32 TestValue2 = TestMagicNumber;

	FMovieSceneTrackIdentifier TrackID = FMovieSceneTrackIdentifier::Invalid();

	FMovieSceneEvaluationKey TrackKey1(MovieSceneSequenceID::Root, ++TrackID);
	FMovieSceneEvaluationKey SectionKey1(MovieSceneSequenceID::Root, TrackID, 0);

	FMovieSceneEvaluationKey TrackKey2(MovieSceneSequenceID::Root, ++TrackID);
	FMovieSceneEvaluationKey SectionKey2(MovieSceneSequenceID::Root, TrackID, 0);

	FMovieSceneAnimTypeID AnimType1 = FMovieSceneAnimTypeID::Unique();
	FMovieSceneAnimTypeID AnimType2 = FMovieSceneAnimTypeID::Unique();

	TStrongObjectPtr<UMovieSceneEntitySystemLinker> GEntitySystemLinker;

	UMovieSceneEntitySystemLinker* GetTestLinker()
	{
		if (!GEntitySystemLinker)
		{
			GEntitySystemLinker.Reset(NewObject<UMovieSceneEntitySystemLinker>(GetTransientPackage()));
		}
		return GEntitySystemLinker.Get();
	}

	void Assert(FAutomationTestBase* Test, int32 Actual, int32 Expected, const TCHAR* Message)
	{
		if (Actual != Expected)
		{
			Test->AddError(FString::Printf(TEXT("%s. Expected %d, actual %d."), Message, Expected, Actual));
		}
	}

	void OnFinishedEvaluating(UMovieSceneEntitySystemLinker* InLinker, const FMovieSceneEvaluationKey& InKey, const UE::MovieScene::FRootInstanceHandle RootInstanceHandle)
	{
		using namespace UE::MovieScene;

		if (FPreAnimatedTemplateCaptureSources* TemplateMetaData = InLinker->PreAnimatedState.GetTemplateMetaData())
		{
			TemplateMetaData->StopTrackingCaptureSource(InKey, RootInstanceHandle);
		}
	}

	void ResetValues()
	{
		TestValue1 = TestMagicNumber;
		TestValue2 = TestMagicNumber;
	}

	// Trivial pre-animated value for the harness below; RestoreState is a no-op as these tests exercise entry
	// lifetime rather than restored values.
	struct FTestStorageValue
	{
		int32 Value = 0;
		void RestoreState(const FObjectKey& Key, const UE::MovieScene::FRestoreStateParams& Params) {}
	};

	using FTestPreAnimatedStorage = UE::MovieScene::TSimplePreAnimatedStateStorage<FObjectKey, FTestStorageValue>;

	// Backs a fresh linker with a real sparse-array storage and an object group manager, so tests can drive
	// grouped and ungrouped meta-data and storage-entry lifetime through FPreAnimatedStateExtension directly.
	struct FPreAnimatedStorageHarness
	{
		TStrongObjectPtr<UMovieSceneEntitySystemLinker> Linker;
		TSharedPtr<FTestPreAnimatedStorage> Storage;
		UE::MovieScene::TPreAnimatedStorageID<FTestPreAnimatedStorage> StorageID;
		TSharedPtr<UE::MovieScene::FPreAnimatedObjectGroupManager> GroupManager;

		FPreAnimatedStorageHarness()
		{
			using namespace UE::MovieScene;

			Linker.Reset(NewObject<UMovieSceneEntitySystemLinker>(GetTransientPackage()));

			StorageID = FPreAnimatedStateExtension::RegisterStorage<FTestPreAnimatedStorage>();
			Storage = MakeShared<FTestPreAnimatedStorage>();
			Storage->Initialize(StorageID, &Extension());
			Extension().AddStorage(StorageID, Storage);

			GroupManager = Extension().GetOrCreateGroupManager<FPreAnimatedObjectGroupManager>();
		}

		UE::MovieScene::FPreAnimatedStateExtension& Extension()
		{
			return Linker->PreAnimatedState;
		}

		void SetCapturingGlobalState(bool bEnabled)
		{
			Extension().NumRequestsForGlobalState = bEnabled ? 1 : 0;
		}

		// Allocate a real storage entry with a cached value, and build a grouped or ungrouped entry that references it.
		UE::MovieScene::FPreAnimatedStateEntry MakeEntry(UObject* Object, bool bGrouped)
		{
			using namespace UE::MovieScene;

			FObjectKey Key(Object);

			FPreAnimatedStorageIndex StorageIndex = Storage->GetOrCreateStorageIndex(Key);

			FTestStorageValue CachedValue;
			CachedValue.Value = 42;
			Storage->AssignPreAnimatedValue(StorageIndex, EPreAnimatedStorageRequirement::Transient, MoveTemp(CachedValue));

			FPreAnimatedStateEntry Entry;
			Entry.GroupHandle = bGrouped ? GroupManager->MakeGroupForKey(Key) : FPreAnimatedStorageGroupHandle();
			Entry.ValueHandle.TypeID = StorageID;
			Entry.ValueHandle.StorageIndex = StorageIndex;
			return Entry;
		}

		UE::MovieScene::FPreAnimatedStateMetaData MakeContributor(const UE::MovieScene::FPreAnimatedStateEntry& Entry, bool bWantsRestoreState)
		{
			UE::MovieScene::FPreAnimatedStateMetaData MetaData;
			MetaData.Entry = Entry;
			MetaData.bWantsRestoreState = bWantsRestoreState;
			return MetaData;
		}
	};
}

void UTestMovieSceneEvalHookSection::Begin(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const
{
	using namespace Impl;

	FScopedPreAnimatedCaptureSource CaptureSource(SharedPlaybackState, this, Params.SequenceID, EvalOptions.CompletionMode == EMovieSceneCompletionMode::RestoreState);
	UMovieSceneEntitySystemLinker* Linker = SharedPlaybackState->GetLinker();
	Linker->PreAnimatedState.SavePreAnimatedState(AnimType1, FPreAnimatedTokenProducer(&TestValue1));

	TestValue1 = StartValue;
}
void UTestMovieSceneEvalHookSection::Update(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const
{
	using namespace Impl;

	FScopedPreAnimatedCaptureSource CaptureSource(SharedPlaybackState, this, Params.SequenceID, EvalOptions.CompletionMode == EMovieSceneCompletionMode::RestoreState);
	UMovieSceneEntitySystemLinker* Linker = SharedPlaybackState->GetLinker();
	Linker->PreAnimatedState.SavePreAnimatedState(AnimType1, FPreAnimatedTokenProducer(&TestValue1));

	++TestValue1;
}
void UTestMovieSceneEvalHookSection::End(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const
{
	using namespace Impl;

	FScopedPreAnimatedCaptureSource CaptureSource(SharedPlaybackState, this, Params.SequenceID, EvalOptions.CompletionMode == EMovieSceneCompletionMode::RestoreState);
	UMovieSceneEntitySystemLinker* Linker = SharedPlaybackState->GetLinker();
	Linker->PreAnimatedState.SavePreAnimatedState(AnimType1, FPreAnimatedTokenProducer(&TestValue1));

	TestValue1 = EndValue;
}
void UTestMovieSceneEvalHookSection::Trigger(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const
{
	using namespace Impl;

	FScopedPreAnimatedCaptureSource CaptureSource(SharedPlaybackState, this, Params.SequenceID, EvalOptions.CompletionMode == EMovieSceneCompletionMode::RestoreState);
	UMovieSceneEntitySystemLinker* Linker = SharedPlaybackState->GetLinker();
	Linker->PreAnimatedState.SavePreAnimatedState(AnimType1, FPreAnimatedTokenProducer(&TestValue1));

	--TestValue1;
}

/** Tests that multiple calls to SavePreAnimated state works correctly */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieScenePreAnimatedStateGlobalTest, "System.Engine.Sequencer.Pre-Animated State.Global", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieScenePreAnimatedStateGlobalTest::RunTest(const FString& Parameters)
{
	using namespace Impl;
	using namespace UE::MovieScene;

	UMovieSceneEntitySystemLinker* Linker = GetTestLinker();

	ResetValues();

	TSharedRef<FSharedPlaybackState> PlaybackState = MakeShared<FSharedPlaybackState>(Linker);
	FMovieSceneInstancePreAnimatedState& State(PlaybackState->GetPreAnimatedState());
	State.EnableGlobalPreAnimatedStateCapture();

	FPreAnimatedTokenProducer Producer(&TestValue1);

	// Save the global state of TestValue1.
	State.SavePreAnimatedState(AnimType1, Producer);
	State.SavePreAnimatedState(AnimType1, Producer);
	State.SavePreAnimatedState(AnimType1, Producer);

	Assert(this, Producer.InitializeCount, 1, TEXT("Should have called FPreAnimatedTokenProducer::InitializeForAnimation exactly once."));
	Assert(this, TestValue1, 0, TEXT("TestValue1 did not initialize correctly."));

	TestValue1 = 50;

	State.RestorePreAnimatedState();

	Assert(this, TestValue1, TestMagicNumber, TEXT("TestValue1 did not restore correctly."));

	return true;
}

/** This specifically tests that a single call to SavePreAnimatedState while capturing for an entity, and globally, produces the correct results when restoring the entity */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieScenePreAnimatedStateEntityTest, "System.Engine.Sequencer.Pre-Animated State.Entity", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieScenePreAnimatedStateEntityTest::RunTest(const FString& Parameters)
{
	using namespace Impl;
	using namespace UE::MovieScene;

	ResetValues();

	UMovieSceneEntitySystemLinker* Linker = GetTestLinker();

	TSharedRef<FSharedPlaybackState> PlaybackState = MakeShared<FSharedPlaybackState>(Linker);
	FMovieSceneInstancePreAnimatedState& State(PlaybackState->GetPreAnimatedState());
	State.EnableGlobalPreAnimatedStateCapture();

	FPreAnimatedTokenProducer Producer(&TestValue1);

	{
		FScopedPreAnimatedCaptureSource CaptureSource(PlaybackState, SectionKey1, true);
		State.SavePreAnimatedState(AnimType1, Producer);
	}

	Assert(this, Producer.InitializeCount, 1, TEXT("Should have called FPreAnimatedTokenProducer::InitializeForAnimation exactly once."));
	Assert(this, TestValue1, 0, TEXT("TestValue1 did not initialize correctly."));

	TestValue1 = 50;

	OnFinishedEvaluating(Linker, SectionKey1, FRootInstanceHandle());
	Assert(this, TestValue1, TestMagicNumber, TEXT("Section did not restore correctly."));

	TestValue1 = 100;
	State.RestorePreAnimatedState();
	Assert(this, TestValue1, 100, TEXT("Global state should not still exist (it should have been cleared with the entity)."));
	return true;
}

/** Tests that overlapping entities that restore in different orders than they saved correctly restores to the original state */ 
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieScenePreAnimatedStateOverlappingEntitiesTest, "System.Engine.Sequencer.Pre-Animated State.Overlapping Entities", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieScenePreAnimatedStateOverlappingEntitiesTest::RunTest(const FString& Parameters)
{
	using namespace Impl;
	using namespace UE::MovieScene;


	ResetValues();

	UMovieSceneEntitySystemLinker* Linker = GetTestLinker();

	TSharedRef<FSharedPlaybackState> PlaybackState = MakeShared<FSharedPlaybackState>(Linker);
	FMovieSceneInstancePreAnimatedState& State(PlaybackState->GetPreAnimatedState());
	State.EnableGlobalPreAnimatedStateCapture();

	FPreAnimatedTokenProducer Producer(&TestValue1);

	// 1. Save a global token
	{
		State.SavePreAnimatedState(AnimType1, Producer);

		Assert(this, Producer.InitializeCount, 1, TEXT("Should have called FPreAnimatedTokenProducer::InitializeForAnimation exactly once."));
		Assert(this, TestValue1, 0, TEXT("TestValue1 did not initialize correctly."));
	}

	// 2. Save a token for the track's evaluation
	{
		FScopedPreAnimatedCaptureSource CaptureSource(PlaybackState, TrackKey1, true);
		State.SavePreAnimatedState(AnimType1, Producer);

		TestValue1 = 50;
		Assert(this, Producer.InitializeCount, 1, TEXT("Should not have called Initialize when capturing for the track."));
	}

	// 3. Save a token for the section's evaluation
	{
		FScopedPreAnimatedCaptureSource CaptureSource(PlaybackState, SectionKey1, true);
		State.SavePreAnimatedState(AnimType1, Producer);

		TestValue1 = 100;
		Assert(this, Producer.InitializeCount, 1, TEXT("Should not have called Initialize when capturing for the section."));
	}

	// 4. Save a token for another section's evaluation
	{
		FScopedPreAnimatedCaptureSource CaptureSource(PlaybackState, SectionKey2, true);
		State.SavePreAnimatedState(AnimType1, Producer);

		TestValue1 = 150;
		Assert(this, Producer.InitializeCount, 1, TEXT("Should not have called Initialize when capturing for the section."));
	}

	// Restore the section first - ensure it does not restore the value (because the track is still animating it)
	OnFinishedEvaluating(Linker, SectionKey1, FRootInstanceHandle());
	Assert(this, TestValue1, 150, TEXT("Section 1 should not have restored."));

	// Restore the track - it should not restore either, because section 2 is still active
	OnFinishedEvaluating(Linker, TrackKey1, FRootInstanceHandle());
	Assert(this, TestValue1, 150, TEXT("Track should not have restored."));

	// Restore the section - since it's the last entity animating the object with 'RestoreState' it should restore to the orignal value
	OnFinishedEvaluating(Linker, SectionKey2, FRootInstanceHandle());
	Assert(this, TestValue1, 0, TEXT("Section 2 did not restore correctly."));

	// Restore globally - ensure that test value goes back to the original value
	State.RestorePreAnimatedState();
	Assert(this, TestValue1, TestMagicNumber, TEXT("Global state did not restore correctly."));

	return true;
}

/** Tests an edge case where one section keeps state, while another subsequent section restores state. the second section should restore to its starting value, not the original state before the first section. */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieScenePreAnimatedStateKeepThenRestoreEntityTest, "System.Engine.Sequencer.Pre-Animated State.Keep Then Restore Entity", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieScenePreAnimatedStateKeepThenRestoreEntityTest::RunTest(const FString& Parameters)
{
	using namespace Impl;
	using namespace UE::MovieScene;


	ResetValues();

	UMovieSceneEntitySystemLinker* Linker = GetTestLinker();

	TSharedRef<FSharedPlaybackState> PlaybackState = MakeShared<FSharedPlaybackState>(Linker);
	FMovieSceneInstancePreAnimatedState& State(PlaybackState->GetPreAnimatedState());
	State.EnableGlobalPreAnimatedStateCapture();

	FPreAnimatedTokenProducer Producer(&TestValue1);

	// Indicate that the entity should not capture state
	{
		FScopedPreAnimatedCaptureSource CaptureSource(PlaybackState, SectionKey1, false);
		// Save state - this will only save globally
		State.SavePreAnimatedState(AnimType1, Producer);
	}

	Assert(this, Producer.InitializeCount, 1, TEXT("Should have called FPreAnimatedTokenProducer::InitializeForAnimation exactly once."));
	Assert(this, TestValue1, 0, TEXT("TestValue1 did not initialize correctly."));

	TestValue1 = 50;

	// Restore state for the entity only - this should not do anything since we specified KeepState above
	OnFinishedEvaluating(Linker, SectionKey1, FRootInstanceHandle());
	Assert(this, TestValue1, 50, TEXT("Section should not have restored state."));

	{
		// Indicate that SectionKey2 is now animating, and wants to restore state
		FScopedPreAnimatedCaptureSource CaptureSource(PlaybackState, SectionKey2, true);
		State.SavePreAnimatedState(AnimType1, Producer);
	}

	Assert(this, Producer.InitializeCount, 1, TEXT("Should not have called FPreAnimatedTokenProducer::InitializeForAnimation a second time."));

	TestValue1 = 100;

	// Restoring section key 2 here should result in the test value being the same value that was set while section 1 was evaluating (50)
	OnFinishedEvaluating(Linker, SectionKey2, FRootInstanceHandle());
	Assert(this, TestValue1, 50, TEXT("Section 2 did not restore to the correct value. It should restore back to the value that was set in section 1 (it doesn't restore state)."));

	// We should still have the global state of the object cached which will restore it to the original state
	State.RestorePreAnimatedState();
	Assert(this, TestValue1, TestMagicNumber, TEXT("Global state did not restore correctly."));

	return true;
}

/** Tests that templates, evaluation hooks and track instances all save/restore state correctly */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieScenePreAnimatedStateTrackTypesTest, "System.Engine.Sequencer.Pre-Animated State.Track Types", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieScenePreAnimatedStateTrackTypesTest::RunTest(const FString& Parameters)
{
	using namespace Impl;
	using namespace UE::MovieScene;

	ResetValues();

	static const int32 StartValue    = 1000;
	static const int32 EndValue      = 2000;
	static const int32 SectionLength = 1000;

	FFrameRate TickResolution(1000, 1);

	UTestMovieSceneSequence* Sequence = NewObject<UTestMovieSceneSequence>(GetTransientPackage());
	Sequence->MovieScene->SetTickResolutionDirectly(TickResolution);

	// Make a keep-state eval hook section
	{
		UTestMovieSceneEvalHookTrack*   Track = Sequence->MovieScene->AddTrack<UTestMovieSceneEvalHookTrack>();
		UTestMovieSceneEvalHookSection* Section = NewObject<UTestMovieSceneEvalHookSection>(Track);

		Section->EvalOptions.CompletionMode = EMovieSceneCompletionMode::KeepState;
		Section->StartValue   = StartValue;
		Section->EndValue     = EndValue;

		Section->SetRange(TRange<FFrameNumber>(0, SectionLength));
		Track->SectionArray.Add(Section);
	}
	// Make a restore-state eval hook section
	{
		UTestMovieSceneEvalHookTrack*   Track = Sequence->MovieScene->AddTrack<UTestMovieSceneEvalHookTrack>();
		UTestMovieSceneEvalHookSection* Section = NewObject<UTestMovieSceneEvalHookSection>(Track);

		Section->EvalOptions.CompletionMode = EMovieSceneCompletionMode::RestoreState;
		Section->StartValue   = StartValue;
		Section->EndValue     = EndValue;

		Section->SetRange(TRange<FFrameNumber>(2000, 2000+SectionLength));
		Track->SectionArray.Add(Section);
	}

	FTestMovieScenePlayer TestPlayer;

	UMovieSceneCompiledDataManager* CompiledDataManager = UMovieSceneCompiledDataManager::GetPrecompiledData();

	FMovieSceneCompiledDataID DataID = CompiledDataManager->Compile(Sequence);
	TestPlayer.Template.Initialize(*Sequence, TestPlayer, CompiledDataManager);
	TestPlayer.Template.EnableGlobalPreAnimatedStateCapture();

	// Test the keep state section
	{
		static const int32 NumEvaluations = 100;
		for (int32 i = 0; i < NumEvaluations; ++i)
		{
			FMovieSceneEvaluationRange EvaluatedRange(TRange<FFrameTime>(i*(SectionLength/NumEvaluations), (i+1)*(SectionLength/NumEvaluations)), TickResolution, EPlayDirection::Forwards);
			TestPlayer.Template.EvaluateSynchronousBlocking(EvaluatedRange);

			Assert(this, TestValue1, StartValue + i, TEXT("Keep-State EvaluationHook did not Begin or Update correctly."));
		}

		FMovieSceneEvaluationRange EvaluatedRange(TRange<FFrameTime>(SectionLength, SectionLength+100), TickResolution, EPlayDirection::Forwards);
		TestPlayer.Template.EvaluateSynchronousBlocking(EvaluatedRange);
		Assert(this, TestValue1, EndValue, TEXT("Keep-State EvaluationHook did not End correctly."));

		TestPlayer.RestorePreAnimatedState();
		Assert(this, TestValue1, TestMagicNumber, TEXT("Keep-State Global pre-animated state did not restore correctly."));
	}

	// Test the Restore state section
	{
		static const int32 NumEvaluations = 100;
		for (int32 i = 0; i < NumEvaluations; ++i)
		{
			FMovieSceneEvaluationRange EvaluatedRange(TRange<FFrameTime>(2000 + i*(SectionLength/NumEvaluations), 2000 + (i+1)*(SectionLength/NumEvaluations)), TickResolution, EPlayDirection::Forwards);
			TestPlayer.Template.EvaluateSynchronousBlocking(EvaluatedRange);

			Assert(this, TestValue1, StartValue + i, TEXT("Restore-State EvaluationHook did not Begin or Update correctly."));
		}

		FMovieSceneEvaluationRange FinalRange(TRange<FFrameTime>(2000 + SectionLength, 2000 + SectionLength + 100), TickResolution, EPlayDirection::Forwards);
		TestPlayer.Template.EvaluateSynchronousBlocking(FinalRange);
		Assert(this, TestValue1, TestMagicNumber, TEXT("Restore-State EvaluationHook did not End correctly."));

		TestPlayer.RestorePreAnimatedState();
		Assert(this, TestValue1, TestMagicNumber, TEXT("Global pre-animated state did not restore correctly after Restore-State Hook."));
	}

	return true;
}

/** Simulates the flow of FSequencer transitioning into/out of PIE */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieScenePreAnimatedStateContextChangedTest, "System.Engine.Sequencer.Pre-Animated State.Context Changed", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieScenePreAnimatedStateContextChangedTest::RunTest(const FString& Parameters)
{
	using namespace Impl;
	using namespace UE::MovieScene;

	ResetValues();

	static const int32 StartValue    = 1000;
	static const int32 EndValue      = 2000;
	static const int32 SectionLength = 1000;

	FFrameRate TickResolution(1000, 1);

	UTestMovieSceneSequence* Sequence = NewObject<UTestMovieSceneSequence>(GetTransientPackage());
	Sequence->MovieScene->SetTickResolutionDirectly(TickResolution);

	// Make a keep-state eval hook section
	{
		UTestMovieSceneEvalHookTrack*   Track = Sequence->MovieScene->AddTrack<UTestMovieSceneEvalHookTrack>();
		UTestMovieSceneEvalHookSection* Section = NewObject<UTestMovieSceneEvalHookSection>(Track);

		Section->EvalOptions.CompletionMode = EMovieSceneCompletionMode::KeepState;
		Section->StartValue   = StartValue;
		Section->EndValue     = EndValue;

		Section->SetRange(TRange<FFrameNumber>(0, SectionLength));
		Track->SectionArray.Add(Section);
	}

	FTestMovieScenePlayer TestPlayer;

	UMovieSceneCompiledDataManager* CompiledDataManager = UMovieSceneCompiledDataManager::GetPrecompiledData();

	FMovieSceneCompiledDataID DataID = CompiledDataManager->Compile(Sequence);
	TestPlayer.Template.Initialize(*Sequence, TestPlayer, CompiledDataManager);
	TestPlayer.Template.EnableGlobalPreAnimatedStateCapture();

	// ---------------------------------------------
	// Simulate in-editor
	{
		static const int32 NumEvaluations = 5;
		for (int32 i = 0; i < NumEvaluations; ++i)
		{
			FMovieSceneEvaluationRange EvaluatedRange(TRange<FFrameTime>(i*(SectionLength/NumEvaluations), (i+1)*(SectionLength/NumEvaluations)), TickResolution, EPlayDirection::Forwards);
			TestPlayer.Template.EvaluateSynchronousBlocking(EvaluatedRange);

			Assert(this, TestValue1, StartValue + i, TEXT("In-Editor 1: EvaluationHook did not Begin or Update correctly."));
		}

		TestPlayer.Template.PlaybackContextChanged(TestPlayer);
		TestEqual(TEXT("Pre-Animated State still exists for In-Editor 1."), TestPlayer.PreAnimatedState.ContainsAnyStateForSequence(), false);
		Assert(this, TestValue1, TestMagicNumber, TEXT("In-Editor 1: Global pre-animated state did not restore correctly."));
	}

	// ---------------------------------------------
	// Simulate PIE 
	{
		static const int32 NumEvaluations = 5;
		for (int32 i = 0; i < NumEvaluations; ++i)
		{
			FMovieSceneEvaluationRange EvaluatedRange(TRange<FFrameTime>(i*(SectionLength/NumEvaluations), (i+1)*(SectionLength/NumEvaluations)), TickResolution, EPlayDirection::Forwards);
			TestPlayer.Template.EvaluateSynchronousBlocking(EvaluatedRange);

			Assert(this, TestValue1, StartValue + i, TEXT("PIE: EvaluationHook did not Begin or Update correctly."));
		}

		TestPlayer.Template.PlaybackContextChanged(TestPlayer);
		TestEqual(TEXT("Pre-Animated State still exists for PIE."), TestPlayer.PreAnimatedState.ContainsAnyStateForSequence(), false);
		Assert(this, TestValue1, TestMagicNumber, TEXT("PIE: Global pre-animated state did not restore correctly."));
	}

	Assert(this, TestValue1, TestMagicNumber, TEXT("Post PIE: Global pre-animated state did not restore correctly."));
	return true;
}

/** Tests an edge case where one section keeps state, while another subsequent section restores state. the second section should restore to its starting value, not the original state before the first section. */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieScenePreAnimatedStatePerformanceTest, "System.Engine.Sequencer.Pre-Animated State.Performance", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::Disabled)
bool FMovieScenePreAnimatedStatePerformanceTest::RunTest(const FString& Parameters)
{
	using namespace Impl;
	using namespace UE::MovieScene;


	ResetValues();

	UMovieSceneEntitySystemLinker* Linker = GetTestLinker();

	TSharedRef<FSharedPlaybackState> PlaybackState = MakeShared<FSharedPlaybackState>(Linker);
	FMovieSceneInstancePreAnimatedState& State(PlaybackState->GetPreAnimatedState());
	State.EnableGlobalPreAnimatedStateCapture();

	FPreAnimatedTokenProducer Producer(&TestValue1);

	// Indicate that the entity should not capture state
	FScopedPreAnimatedCaptureSource CaptureSource(PlaybackState, SectionKey1, false);

	for (int32 Iteration = 0; Iteration < 1000000; ++Iteration)
	{
		State.SavePreAnimatedState(AnimType1, Producer);
	}
	return true;
}

/** A grouped value with overlapping Restore State and Keep State contributors and no global capture: removing the Restore State contributor keeps the storage entry for the Keep State contributor, and the group teardown releases it once. */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieScenePreAnimatedStateGroupedKeepOutlivesRestoreNoGlobalTest, "System.Engine.Sequencer.Pre-Animated State.Grouped Keep Outlives Restore No Global", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieScenePreAnimatedStateGroupedKeepOutlivesRestoreNoGlobalTest::RunTest(const FString& Parameters)
{
	using namespace Impl;
	using namespace UE::MovieScene;

	FPreAnimatedStorageHarness Harness;
	Harness.SetCapturingGlobalState(false);

	FPreAnimatedStateEntry Entry = Harness.MakeEntry(Harness.Linker.Get(), /*bGrouped*/ true);

	FPreAnimatedStateMetaData Restore = Harness.MakeContributor(Entry, /*bWantsRestoreState*/ true);
	FPreAnimatedStateMetaData Keep    = Harness.MakeContributor(Entry, /*bWantsRestoreState*/ false);

	Harness.Extension().AddMetaData(Restore);
	Harness.Extension().AddMetaData(Keep);
	TestTrue(TEXT("Entry should exist while both contributors are present."), Harness.Extension().MetaDataExists(Entry));

	Harness.Extension().RemoveMetaData(Restore);
	TestTrue(TEXT("Entry should still exist while the Keep State contributor remains."), Harness.Extension().MetaDataExists(Entry));
	TestTrue(TEXT("Storage entry should remain allocated while the Keep State contributor remains."), Harness.Storage->IsStorageInitialized(Entry.ValueHandle.StorageIndex));

	Harness.Extension().DiscardStateForGroup(Entry.GroupHandle);

	return true;
}

/** Overlapping Restore State and Keep State contributors on a grouped value while capturing global state: the value is promoted to persistent and the storage entry kept until the group is torn down. */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieScenePreAnimatedStateGroupedKeepOutlivesRestoreWithGlobalTest, "System.Engine.Sequencer.Pre-Animated State.Grouped Keep Outlives Restore With Global", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieScenePreAnimatedStateGroupedKeepOutlivesRestoreWithGlobalTest::RunTest(const FString& Parameters)
{
	using namespace Impl;
	using namespace UE::MovieScene;

	FPreAnimatedStorageHarness Harness;
	Harness.SetCapturingGlobalState(true);

	FPreAnimatedStateEntry Entry = Harness.MakeEntry(Harness.Linker.Get(), /*bGrouped*/ true);

	Harness.Extension().AddMetaData(Harness.MakeContributor(Entry, /*bWantsRestoreState*/ true));
	Harness.Extension().AddMetaData(Harness.MakeContributor(Entry, /*bWantsRestoreState*/ false));

	Harness.Extension().RemoveMetaData(Harness.MakeContributor(Entry, /*bWantsRestoreState*/ true));
	TestTrue(TEXT("Entry should still exist while the Keep State contributor remains."), Harness.Extension().MetaDataExists(Entry));
	TestTrue(TEXT("Cached value should be kept (promoted to persistent) while capturing global state."), Harness.Storage->IsStorageInitialized(Entry.ValueHandle.StorageIndex));

	Harness.Extension().DiscardStateForGroup(Entry.GroupHandle);

	return true;
}

/** Removing both contributors (Restore then Keep) leaves the entry for a later sweep; the group teardown then releases it once. */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieScenePreAnimatedStateGroupedDrainThenDiscardTest, "System.Engine.Sequencer.Pre-Animated State.Grouped Drain Then Discard Group", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieScenePreAnimatedStateGroupedDrainThenDiscardTest::RunTest(const FString& Parameters)
{
	using namespace Impl;
	using namespace UE::MovieScene;

	FPreAnimatedStorageHarness Harness;
	Harness.SetCapturingGlobalState(false);

	FPreAnimatedStateEntry Entry = Harness.MakeEntry(Harness.Linker.Get(), /*bGrouped*/ true);

	FPreAnimatedStateMetaData Restore = Harness.MakeContributor(Entry, /*bWantsRestoreState*/ true);
	FPreAnimatedStateMetaData Keep    = Harness.MakeContributor(Entry, /*bWantsRestoreState*/ false);

	Harness.Extension().AddMetaData(Restore);
	Harness.Extension().AddMetaData(Keep);

	Harness.Extension().RemoveMetaData(Restore);
	Harness.Extension().RemoveMetaData(Keep);
	TestTrue(TEXT("Entry should linger after the last contributor is removed, until the group is swept."), Harness.Extension().MetaDataExists(Entry));

	Harness.Extension().DiscardStateForGroup(Entry.GroupHandle);

	return true;
}

/** A new Restore State contributor arriving after an earlier one was removed finds the kept storage entry still valid and requiring transient storage. */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieScenePreAnimatedStateGroupedReAddRestoreTest, "System.Engine.Sequencer.Pre-Animated State.Grouped Re-Add Restore", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieScenePreAnimatedStateGroupedReAddRestoreTest::RunTest(const FString& Parameters)
{
	using namespace Impl;
	using namespace UE::MovieScene;

	FPreAnimatedStorageHarness Harness;
	Harness.SetCapturingGlobalState(false);

	FPreAnimatedStateEntry Entry = Harness.MakeEntry(Harness.Linker.Get(), /*bGrouped*/ true);

	Harness.Extension().AddMetaData(Harness.MakeContributor(Entry, /*bWantsRestoreState*/ true));
	Harness.Extension().AddMetaData(Harness.MakeContributor(Entry, /*bWantsRestoreState*/ false));
	Harness.Extension().RemoveMetaData(Harness.MakeContributor(Entry, /*bWantsRestoreState*/ true));

	Harness.Extension().AddMetaData(Harness.MakeContributor(Entry, /*bWantsRestoreState*/ true));

	TestEqual(TEXT("A new Restore State contributor should require transient storage."),
		(int32)Harness.Extension().GetStorageRequirement(Entry), (int32)EPreAnimatedStorageRequirement::Transient);
	TestTrue(TEXT("The kept storage entry should still be valid for the new contributor."),
		Harness.Storage->IsStorageInitialized(Entry.ValueHandle.StorageIndex));

	Harness.Extension().DiscardStateForGroup(Entry.GroupHandle);

	return true;
}

/** An ungrouped value with overlapping Restore State and Keep State contributors and no global capture: removing the Restore State contributor keeps the storage entry, and a later storage sweep releases it once. */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieScenePreAnimatedStateUngroupedKeepOutlivesRestoreNoGlobalTest, "System.Engine.Sequencer.Pre-Animated State.Ungrouped Keep Outlives Restore No Global", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieScenePreAnimatedStateUngroupedKeepOutlivesRestoreNoGlobalTest::RunTest(const FString& Parameters)
{
	using namespace Impl;
	using namespace UE::MovieScene;

	FPreAnimatedStorageHarness Harness;
	Harness.SetCapturingGlobalState(false);

	FPreAnimatedStateEntry Entry = Harness.MakeEntry(Harness.Linker.Get(), /*bGrouped*/ false);

	FPreAnimatedStateMetaData Restore = Harness.MakeContributor(Entry, /*bWantsRestoreState*/ true);
	FPreAnimatedStateMetaData Keep    = Harness.MakeContributor(Entry, /*bWantsRestoreState*/ false);

	Harness.Extension().AddMetaData(Restore);
	Harness.Extension().AddMetaData(Keep);

	Harness.Extension().RemoveMetaData(Restore);
	TestTrue(TEXT("Ungrouped entry should still exist while the Keep State contributor remains."), Harness.Extension().MetaDataExists(Entry));
	TestTrue(TEXT("Storage entry should remain allocated while the Keep State contributor remains."), Harness.Storage->IsStorageInitialized(Entry.ValueHandle.StorageIndex));

	Harness.Extension().RemoveMetaData(Keep);
	Harness.Extension().DiscardStateForStorage(Entry.ValueHandle.TypeID, Entry.ValueHandle.StorageIndex);

	return true;
}

#undef LOCTEXT_NAMESPACE
