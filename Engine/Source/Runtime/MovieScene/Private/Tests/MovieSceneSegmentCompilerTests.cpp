// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneTestObjects.h"
#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "Compilation/MovieSceneSegmentCompiler.h"
#include "Compilation/MovieSceneCompilerRules.h"
#include "Evaluation/MovieSceneEvaluationTrack.h"
#include "Evaluation/MovieSceneEvaluationField.h"
#include "UObject/Package.h"
#include "MovieSceneTimeHelpers.h"

// ~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
// TODO: Reimplement or remove segment blender automation tests
// ~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
UE_MOVIESCENE_TODO(Reimplement or remove segment blender automation tests)

#if 0

#if WITH_DEV_AUTOMATION_TESTS

namespace Impl
{
	static const TRangeBound<FFrameNumber> Inf = TRangeBound<FFrameNumber>::Open();

	typedef TTuple<TRange<FFrameNumber>, TArray<int32>> FEvaluationTreeIteratorResult;

	/** Compiler rules to sort by priority */
	struct FSortByPrioritySegmentBlender : FMovieSceneTrackSegmentBlender
	{
		FSortByPrioritySegmentBlender(bool bInAllowEmptySegments)
		{
			bAllowEmptySegments = bInAllowEmptySegments;
		}

		virtual TOptional<FMovieSceneSegment> InsertEmptySpace(const TRange<FFrameNumber>& Range, const FMovieSceneSegment* PreviousSegment, const FMovieSceneSegment* NextSegment) const override
		{
			return bAllowEmptySegments ? FMovieSceneSegment(Range) : TOptional<FMovieSceneSegment>();
		}

		virtual void Blend(FSegmentBlendData& BlendData) const
		{
			BlendData.Sort(
				[](const FMovieSceneSectionData& A, const FMovieSceneSectionData& B)
				{
					// Sort by template index and flags when we don't have any sections
					if (!A.Section || !B.Section)
					{
						if (A.Flags == ESectionEvaluationFlags::None && B.Flags == ESectionEvaluationFlags::None)
						{
							return A.TemplateIndex < B.TemplateIndex;
						}

						return A.Flags != ESectionEvaluationFlags::None;
					}
					return A.Section->GetOverlapPriority() > B.Section->GetOverlapPriority();
				}
			);
		}
	};

	FString Join(TArrayView<const FSectionEvaluationData> Stuff)
	{
		FString Result;
		for (const FSectionEvaluationData& Thing : Stuff)
		{
			if (Result.Len())
			{
				Result += TEXT(", ");
			}

			Result += FString::Printf(TEXT("(Impl: %d, ForcedTime: %i, Flags: %u)"), Thing.ImplIndex, Thing.ForcedTime.Value, (uint8)Thing.Flags);
		}

		return Result;
	}

	FORCEINLINE_DEBUGGABLE void AssertSegmentImpls(FAutomationTestBase* Test, TArrayView<const FSectionEvaluationData> ExpectedImpls, TArrayView<const FSectionEvaluationData> ActualImpls, const TCHAR* AdditionalText = TEXT(""))
	{
		FString ActualImplsString = Join(ActualImpls);
		FString ExpectedImplsString = Join(ExpectedImpls);

		if (ActualImplsString != ExpectedImplsString)
		{
			Test->AddError(FString::Printf(TEXT("Compiled data does not match for segment %s.\nExpected: %s\nActual:   %s."), AdditionalText, *ExpectedImplsString, *ActualImplsString));
		}
	}
	
	FORCEINLINE_DEBUGGABLE void AssertSegmentValues(FAutomationTestBase* Test, TArrayView<const FMovieSceneSegment> Expected, TArrayView<const FMovieSceneSegment> Actual)
	{
		for (const FMovieSceneSegment& Segment : Actual)
		{
			Test->AddInfo(FString::Printf(TEXT("Actual   %s, Impls: %s"), *LexToString(Segment.Range), *Join(Segment.Impls)));
		}
		for (const FMovieSceneSegment& Segment : Expected)
		{
			Test->AddInfo(FString::Printf(TEXT("Expected %s, Impls: %s"), *LexToString(Segment.Range), *Join(Segment.Impls)));
		}

		if (Actual.Num() != Expected.Num())
		{
			Test->AddError(FString::Printf(TEXT("Wrong number of compiled segments. Expected %d, actual %d."), Expected.Num(), Actual.Num()));
		}
		else for (int32 Index = 0; Index < Expected.Num(); ++Index)
		{
			const FMovieSceneSegment& ExpectedSegment = Expected[Index];
			const FMovieSceneSegment& ActualSegment = Actual[Index];

			if (ExpectedSegment.Range != ActualSegment.Range)
			{
				Test->AddError(FString::Printf(TEXT("Incorrect compiled segment range at segment index %d. Expected:\n%s\nActual:\n%s"), Index, *LexToString(ExpectedSegment.Range), *LexToString(ActualSegment.Range)));
			}

			AssertSegmentImpls(Test, ExpectedSegment.Impls, ActualSegment.Impls, *FString::Printf(TEXT("index %d"), Index));
		}
	}

	FORCEINLINE_DEBUGGABLE void AssertSegmentAtTime(FAutomationTestBase* Test, FMovieSceneEvaluationTrack& InTrack, FFrameNumber InTime, TArrayView<const FSectionEvaluationData> ExpectedImpls)
	{
		FMovieSceneSegmentIdentifier ID = InTrack.GetSegmentFromTime(InTime);
		if (!ID.IsValid())
		{
			Test->AddError(TEXT("No segment compiled for frame %i"), InTime.Value);
		}
		else
		{
			AssertSegmentImpls(Test, ExpectedImpls, InTrack.GetSegment(ID).Impls, *FString::Printf(TEXT("frame %i"), InTime.Value));
		}
	}

	FORCEINLINE void AssertSegmentAtTime(FAutomationTestBase* Test, FMovieSceneEvaluationTrack& InTrack, FFrameNumber InTime, std::initializer_list<FSectionEvaluationData> ExpectedImpls)
	{
		AssertSegmentAtTime(Test, InTrack, InTime, MakeArrayView(ExpectedImpls));
	}

	FORCEINLINE_DEBUGGABLE void AssertEvaluationTree(FAutomationTestBase* Test, const TMovieSceneEvaluationTree<int32>& Tree, TArrayView<const FEvaluationTreeIteratorResult> Expected)
	{
		int32 Index = 0;
		for (FMovieSceneEvaluationTreeRangeIterator It(Tree); It; ++It, ++Index)
		{
			if (Index >= Expected.Num())
			{
				Test->AddError(FString::Printf(TEXT("Too many ranges iterated (expected %d, iterated >= %d)"), Expected.Num(), Index + 1));
				break;
			}

			if (It.Range() != Expected[Index].Get<0>())
			{
				Test->AddError(FString::Printf(TEXT("Incorrect range iterated at index %d. %s != %s"), Index, *LexToString(It.Range()), *LexToString(Expected[Index].Get<0>())));
				continue;
			}

			int32 Count = 0;
			const TArray<int32>& ExpectedData = Expected[Index].Get<1>();
			for (auto AllDataIt = Tree.GetAllData(It.Node()); AllDataIt; ++AllDataIt, ++Count)
			{
				if (!ExpectedData.Contains(*AllDataIt))
				{
					Test->AddError(FString::Printf(TEXT("Incorrect data encountered at index %d. Count not find %d in expected data."), Index, *AllDataIt));
					continue;
				}
			}

			if (Count != ExpectedData.Num())
			{
				Test->AddError(FString::Printf(TEXT("Incorrect number of data entries iterated at index %d. Found %d entries, expected %d"), Index, Count, ExpectedData.Num()));
			}
		}
	}
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieSceneCompilerBasicTest, "System.Engine.Sequencer.Compiler.Basic", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieSceneCompilerBasicTest::RunTest(const FString& Parameters)
{
	using namespace Impl;

	// Specify descending priorities on the segments so we always sort the compiled segments in the order they're defined within the data
	//	This is our test layout
	//	Time				-inf				10							20				25					30							inf
	//											[============= 0 ===========]
	//																		[============== 1 ==================]
	//											[========================== 2 ==================================]
	//						[================== 3 ==========================]
	//																						[================== 4 ==========================]
	//----------------------------------------------------------------------------------------------------------------------------------------
	//	Expected Impls		[ 		3			|			0,2,3			|		1,2		|		1,2,4		|			4				]

	// Override the blender (so it doesn't try and get it from the null track ptr)
	FScopedOverrideTrackSegmentBlender ScopedBlenderOverride(FSortByPrioritySegmentBlender(false));

	FMovieSceneEvaluationTrack Track;

	Track.AddTreeData(TRange<FFrameNumber>(10,	20),										FSectionEvaluationData(0));
	Track.AddTreeData(TRange<FFrameNumber>(20,	30),										FSectionEvaluationData(1));
	Track.AddTreeData(TRange<FFrameNumber>(10,	30),										FSectionEvaluationData(2));
	Track.AddTreeData(TRange<FFrameNumber>(Inf, TRangeBound<FFrameNumber>::Exclusive(20)),	FSectionEvaluationData(3));
	Track.AddTreeData(TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(25), Inf),	FSectionEvaluationData(4));

	FMovieSceneSegment Expected[] = {
		FMovieSceneSegment(TRange<FFrameNumber>(Inf, TRangeBound<FFrameNumber>::Exclusive(10)),	{ FSectionEvaluationData(3) }),
		FMovieSceneSegment(TRange<FFrameNumber>(10,	20),										{ FSectionEvaluationData(0), FSectionEvaluationData(2), FSectionEvaluationData(3) }),
		FMovieSceneSegment(TRange<FFrameNumber>(20,	25),										{ FSectionEvaluationData(1), FSectionEvaluationData(2) }),
		FMovieSceneSegment(TRange<FFrameNumber>(25,	30),										{ FSectionEvaluationData(1), FSectionEvaluationData(2), FSectionEvaluationData(4) }),
		FMovieSceneSegment(TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(30),Inf),	{ FSectionEvaluationData(4) })
	};

	// Compile all the segments
	Track.GetSegmentsInRange(TRange<FFrameNumber>::All());

	AssertSegmentValues(this, Expected, Track.GetSortedSegments());
	return true;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieSceneCompilerEmptySpaceTest, "System.Engine.Sequencer.Compiler.Empty Space", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieSceneCompilerEmptySpaceTest::RunTest(const FString& Parameters)
{
	using namespace Impl;

	// Specify descending priorities on the segments so we always sort the compiled segments in the order they're defined within the data
	//	This is our test layout
	//	Time			-Inf			10				20				30				40				Inf
	//									[====== 0 ======]				[====== 1 ======]
	//----------------------------------------------------------------------------------------------------------------------------------------
	//	Expected Impls	[	Empty		|		0		|	Empty		|		1		|	Empty		]

	FMovieSceneEvaluationTrack Track;

	Track.AddTreeData(TRange<FFrameNumber>(10,	20), FSectionEvaluationData(0));
	Track.AddTreeData(TRange<FFrameNumber>(30,	40), FSectionEvaluationData(1));

	// Override the blender (so it doesn't try and get it from the null track ptr)
	FScopedOverrideTrackSegmentBlender ScopedBlenderOverride(FSortByPrioritySegmentBlender(true));

	// Compile all the segments
	Track.GetSegmentsInRange(TRange<FFrameNumber>::All());

	FMovieSceneSegment Expected[] = {
		FMovieSceneSegment(TRange<FFrameNumber>(Inf, TRangeBound<FFrameNumber>::Exclusive(10)), 	{ }),
		FMovieSceneSegment(TRange<FFrameNumber>(10,	20), 											{ FSectionEvaluationData(0) }),
		FMovieSceneSegment(TRange<FFrameNumber>(20,	30), 											{  }),
		FMovieSceneSegment(TRange<FFrameNumber>(30,	40), 											{ FSectionEvaluationData(1) }),
		FMovieSceneSegment(TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(40),	Inf), 	{  })
	};
	AssertSegmentValues(this, Expected, Track.GetSortedSegments());
	return true;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieSceneCustomCompilerTest, "System.Engine.Sequencer.Compiler.Custom", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieSceneCustomCompilerTest::RunTest(const FString& Parameters)
{
	using namespace Impl;

	//	Test that we don't get duplicate entries in the resulting segments except for differing flag sets

	//	This is our test layout
	//	Time			-Inf			10			15			20			25			30						40				Inf
	//									[===== 0 (preroll) =====]
	//												[========== 0 ==========]
	//															[========== 0 ==========][========== 0 =========]

	FMovieSceneEvaluationTrack Track;

	Track.AddUniqueTreeData(TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(10),	TRangeBound<FFrameNumber>::Exclusive(20)), FSectionEvaluationData(0, ESectionEvaluationFlags::PreRoll));
	Track.AddUniqueTreeData(TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(15),	TRangeBound<FFrameNumber>::Exclusive(25)), FSectionEvaluationData(0));
	Track.AddUniqueTreeData(TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(20),	TRangeBound<FFrameNumber>::Exclusive(30)), FSectionEvaluationData(0));
	Track.AddUniqueTreeData(TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(30),	TRangeBound<FFrameNumber>::Exclusive(40)), FSectionEvaluationData(0));

	// Override the blender (so it doesn't try and get it from the null track ptr)
	FScopedOverrideTrackSegmentBlender ScopedBlenderOverride(FSortByPrioritySegmentBlender(false));

	AssertSegmentAtTime(this, Track, 12,		{ FSectionEvaluationData(0, ESectionEvaluationFlags::PreRoll) });
	AssertSegmentAtTime(this, Track, 17,		{ FSectionEvaluationData(0, ESectionEvaluationFlags::PreRoll), FSectionEvaluationData(0) });
	AssertSegmentAtTime(this, Track, 22,		{ FSectionEvaluationData(0) });
	AssertSegmentAtTime(this, Track, 27,		{ FSectionEvaluationData(0) });
	AssertSegmentAtTime(this, Track, 30,		{ FSectionEvaluationData(0) });

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieSceneTrackCompilerTest, "System.Engine.Sequencer.Compiler.Tracks", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieSceneTrackCompilerTest::RunTest(const FString& Parameters)
{
	using namespace Impl;

	//	Track 0 test layout:
	//	Time								-inf				10							20				25					30							inf
	//	------------------------------------------------------------------------------------------------------------------------------------------------------
	//	Track 0:												[====================== 0 ==================]
	//																						[=============== 1 =================]
	//	------------------------------------------------------------------------------------------------------------------------------------------------------
	//	Additive Camera Rules Expected		[ 					|			   0			|		(0,1)	|		 1			|							]
	//	Nearest Section Expected			[ 		0 (10)		|			   0			|		(0,1)	|		 1			|			1 (30)			]
	//	No Nearest Section Expected			[ 					|			   0			|		(0,1)	|		 1			|							]
	//	High-pass Filter Expected			[ 					|						0					|		 1			|							]

	{
		UTestMovieSceneTrack* Track = NewObject<UTestMovieSceneTrack>(GetTransientPackage());
		Track->EvalOptions.bCanEvaluateNearestSection = true;

		UTestMovieSceneSection* Section0 = NewObject<UTestMovieSceneSection>(Track);
		Section0->SetRange(TRange<FFrameNumber>(
			TRangeBound<FFrameNumber>::Inclusive(10),
			TRangeBound<FFrameNumber>::Exclusive(25)
		));
		Section0->SetRowIndex(0);

		UTestMovieSceneSection* Section1 = NewObject<UTestMovieSceneSection>(Track);
		Section1->SetRange(TRange<FFrameNumber>(
			TRangeBound<FFrameNumber>::Inclusive(20),
			TRangeBound<FFrameNumber>::Exclusive(30)
		));
		Section1->SetRowIndex(1);
		
		Track->SectionArray.Add(Section0);
		Track->SectionArray.Add(Section1);

		// Test compiling the track with the additive camera rules
		{
			FScopedOverrideTrackSegmentBlender ScopedBlenderOverride{FMovieSceneAdditiveCameraTrackBlender()};

			FMovieSceneEvaluationTrack EvalTrack = Track->GenerateTrackTemplate();
			EvalTrack.GetSegmentsInRange(TRange<FFrameNumber>::All());

			FMovieSceneSegment Expected[] = {
				FMovieSceneSegment(TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(10),	TRangeBound<FFrameNumber>::Exclusive(20)), { FSectionEvaluationData(0) }),
				FMovieSceneSegment(TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(20),	TRangeBound<FFrameNumber>::Exclusive(25)), { FSectionEvaluationData(0), FSectionEvaluationData(1) }),
				FMovieSceneSegment(TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(25),	TRangeBound<FFrameNumber>::Exclusive(30)), { FSectionEvaluationData(1) }),
			};
			AssertSegmentValues(this, Expected, EvalTrack.GetSortedSegments());
		}

		// Test compiling with 'evaluate nearest section' enabled
		{
			Track->EvalOptions.bEvalNearestSection = true;

			FMovieSceneEvaluationTrack EvalTrack = Track->GenerateTrackTemplate();
			EvalTrack.GetSegmentsInRange(TRange<FFrameNumber>::All());

			FMovieSceneSegment Expected[] = {
				FMovieSceneSegment(TRange<FFrameNumber>(Inf,										TRangeBound<FFrameNumber>::Exclusive(10)),	{ FSectionEvaluationData(0, 10) }),
				FMovieSceneSegment(TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(10),	TRangeBound<FFrameNumber>::Exclusive(20)),	{ FSectionEvaluationData(0) }),
				FMovieSceneSegment(TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(20),	TRangeBound<FFrameNumber>::Exclusive(25)),	{ FSectionEvaluationData(0), FSectionEvaluationData(1) }),
				FMovieSceneSegment(TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(25),	TRangeBound<FFrameNumber>::Exclusive(30)),	{ FSectionEvaluationData(1) }),
				FMovieSceneSegment(TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(30),	Inf), 										{ FSectionEvaluationData(1, 30) }),
			};
			AssertSegmentValues(this, Expected, EvalTrack.GetSortedSegments());
		}

		// Test compiling without 'evaluate nearest section' enabled
		{
			Track->EvalOptions.bEvalNearestSection = false;

			FMovieSceneEvaluationTrack EvalTrack = Track->GenerateTrackTemplate();
			EvalTrack.GetSegmentsInRange(TRange<FFrameNumber>::All());

			FMovieSceneSegment Expected[] = {
				FMovieSceneSegment(TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(10),	TRangeBound<FFrameNumber>::Exclusive(20)), { FSectionEvaluationData(0) }),
				FMovieSceneSegment(TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(20),	TRangeBound<FFrameNumber>::Exclusive(25)), { FSectionEvaluationData(0), FSectionEvaluationData(1) }),
				FMovieSceneSegment(TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(25),	TRangeBound<FFrameNumber>::Exclusive(30)), { FSectionEvaluationData(1) }),
			};
			AssertSegmentValues(this, Expected, EvalTrack.GetSortedSegments());
		}

		// Test high-pass filter
		{
			Track->EvalOptions.bEvalNearestSection = false;
			Track->bHighPassFilter = true;

			FMovieSceneEvaluationTrack EvalTrack = Track->GenerateTrackTemplate();
			EvalTrack.GetSegmentsInRange(TRange<FFrameNumber>::All());

			FMovieSceneSegment Expected[] = {
				FMovieSceneSegment(TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(10),	TRangeBound<FFrameNumber>::Exclusive(25)), { FSectionEvaluationData(0) }),
				FMovieSceneSegment(TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(25),	TRangeBound<FFrameNumber>::Exclusive(30)), { FSectionEvaluationData(1) }),
			};
			AssertSegmentValues(this, Expected, EvalTrack.GetSortedSegments());
		}
	}

	// Track 1 test layout:
	//	Time								-inf				10			15			20				25					30							inf
	//	-----------------------------------------------------------------------------------------------------------------------------------------------------
	//	Track 1:															[==== 3 ====(==== 3,2 ======)======= 2 =========]
	//															[============================ 0 ============================]
	//										[================================================ 1 ========================================================]
	//	-----------------------------------------------------------------------------------------------------------------------------------------------------
	//	Additive Camera Rules Expected		[ 		1			|	(1,0)	|			(1,0,3)			|		(1,0,2)		|			1				]
	//	Nearest Section Expected			[ 		1			|	(0,1)	|			(3,0,1)			|		(2,0,1)		|			1				]
	//	No Nearest Section Expected			[ 		1			|	(0,1)	|			(3,0,1)			|		(2,0,1)		|			1				]
	//	High-Pass Filter Expected			[ 		1			|	0		|			3				|		2			|			1				]
	{
		UTestMovieSceneTrack* Track = NewObject<UTestMovieSceneTrack>(GetTransientPackage());

		UTestMovieSceneSection* Section0 = NewObject<UTestMovieSceneSection>(Track);
		Section0->SetRange(TRange<FFrameNumber>(10, 30));
		Section0->SetRowIndex(1);

		UTestMovieSceneSection* Section1 = NewObject<UTestMovieSceneSection>(Track);
		Section1->SetRange(TRange<FFrameNumber>::All());
		Section1->SetRowIndex(2);
		
		UTestMovieSceneSection* Section2 = NewObject<UTestMovieSceneSection>(Track);
		Section2->SetRange(TRange<FFrameNumber>(20, 30));
		Section2->SetRowIndex(0);

		UTestMovieSceneSection* Section3 = NewObject<UTestMovieSceneSection>(Track);
		Section3->SetRange(TRange<FFrameNumber>(15, 25));
		Section3->SetRowIndex(0);
		Section3->SetOverlapPriority(100.f);

		Track->SectionArray.Add(Section0);
		Track->SectionArray.Add(Section1);
		Track->SectionArray.Add(Section2);
		Track->SectionArray.Add(Section3);

		// Additive camera rules prescribe that they are evaluated in order of start time
		FMovieSceneSegment Expected[] = {
			FMovieSceneSegment(TRange<FFrameNumber>(Inf,											TRangeBound<FFrameNumber>::Exclusive(10)),	{ }),
			FMovieSceneSegment(TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(10),		TRangeBound<FFrameNumber>::Exclusive(15)),	{ }),
			FMovieSceneSegment(TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(15),		TRangeBound<FFrameNumber>::Exclusive(25)),	{ }),
			FMovieSceneSegment(TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(25),		TRangeBound<FFrameNumber>::Exclusive(30)),	{ }),
			FMovieSceneSegment(TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(30),		Inf),										{ }),
		};

		// Test compiling the track with the additive camera rules
		{
			FScopedOverrideTrackSegmentBlender ScopedBlenderOverride{FMovieSceneAdditiveCameraTrackBlender()};

			FMovieSceneEvaluationTrack EvalTrack = Track->GenerateTrackTemplate();
			EvalTrack.GetSegmentsInRange(TRange<FFrameNumber>::All());

			// Additive camera rules prescribe that they are evaluated in order of start time
			Expected[0].Impls = { FSectionEvaluationData(1) };
			Expected[1].Impls = { FSectionEvaluationData(1), FSectionEvaluationData(0) };
			Expected[2].Impls = { FSectionEvaluationData(1), FSectionEvaluationData(0), FSectionEvaluationData(3) };
			Expected[3].Impls = { FSectionEvaluationData(1), FSectionEvaluationData(0), FSectionEvaluationData(2) };
			Expected[4].Impls = { FSectionEvaluationData(1) };

			AssertSegmentValues(this, Expected, EvalTrack.GetSortedSegments());
		}

		// Test compiling with 'evaluate nearest section' enabled
		{
			Track->EvalOptions.bEvalNearestSection = true;

			FMovieSceneEvaluationTrack EvalTrack = Track->GenerateTrackTemplate();
			EvalTrack.GetSegmentsInRange(TRange<FFrameNumber>::All());

			Expected[0].Impls = { FSectionEvaluationData(1) };
			Expected[1].Impls = { FSectionEvaluationData(0), FSectionEvaluationData(1) };
			Expected[2].Impls = { FSectionEvaluationData(3), FSectionEvaluationData(0), FSectionEvaluationData(1) };
			Expected[3].Impls = { FSectionEvaluationData(2), FSectionEvaluationData(0), FSectionEvaluationData(1) };
			Expected[4].Impls = { FSectionEvaluationData(1) };

			AssertSegmentValues(this, Expected, EvalTrack.GetSortedSegments());
		}

		// Test compiling without 'evaluate nearest section' enabled
		{
			Track->EvalOptions.bEvalNearestSection = false;

			FMovieSceneEvaluationTrack EvalTrack = Track->GenerateTrackTemplate();
			EvalTrack.GetSegmentsInRange(TRange<FFrameNumber>::All());

			Expected[0].Impls = { FSectionEvaluationData(1) };
			Expected[1].Impls = { FSectionEvaluationData(0), FSectionEvaluationData(1) };
			Expected[2].Impls = { FSectionEvaluationData(3), FSectionEvaluationData(0), FSectionEvaluationData(1) };
			Expected[3].Impls = { FSectionEvaluationData(2), FSectionEvaluationData(0), FSectionEvaluationData(1) };
			Expected[4].Impls = { FSectionEvaluationData(1) };

			AssertSegmentValues(this, Expected, EvalTrack.GetSortedSegments());
		}

		// Test compiling with high pass filter
		{
			Track->EvalOptions.bEvalNearestSection = false;
			Track->bHighPassFilter = true;

			FMovieSceneEvaluationTrack EvalTrack = Track->GenerateTrackTemplate();
			EvalTrack.GetSegmentsInRange(TRange<FFrameNumber>::All());

			Expected[0].Impls = { FSectionEvaluationData(1) };
			Expected[1].Impls = { FSectionEvaluationData(0) };
			Expected[2].Impls = { FSectionEvaluationData(3) };
			Expected[3].Impls = { FSectionEvaluationData(2) };
			Expected[4].Impls = { FSectionEvaluationData(1) };

			AssertSegmentValues(this, Expected, EvalTrack.GetSortedSegments());
		}
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieSceneCompilerTreeBasicTest, "System.Engine.Sequencer.Compiler.Tree Basic", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieSceneCompilerTreeBasicTest::RunTest(const FString& Parameters)
{
	using namespace Impl;

	// Specify descending priorities on the segments so we always sort the compiled segments in the order they're defined within the data
	//	This is our test layout
	//	Time				-inf				10							20				25					30							inf
	//											[============= 0 ===========]
	//																		[============== 1 ==================]
	//											[========================== 2 ==================================]
	//						[================== 3 ==========================]
	//																						[================== 4 ==========================]
	//----------------------------------------------------------------------------------------------------------------------------------------
	//	Expected Impls		[ 		3			|			0,2,3			|		1,2		|		1,2,4		|			4				]

	TMovieSceneEvaluationTree<FSectionEvaluationData> EvalTree;

	EvalTree.Add(TRange<FFrameNumber>(10,	20), 										FSectionEvaluationData(0));
	EvalTree.Add(TRange<FFrameNumber>(20,	30), 										FSectionEvaluationData(1));
	EvalTree.Add(TRange<FFrameNumber>(10,	30), 										FSectionEvaluationData(2));
	EvalTree.Add(TRange<FFrameNumber>(Inf,	TRangeBound<FFrameNumber>::Exclusive(20)), 	FSectionEvaluationData(3));
	EvalTree.Add(TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(25),	Inf), 	FSectionEvaluationData(4));

	EvalTree.Compact();

	TArray<FMovieSceneSegment> Segments;

	for (FMovieSceneEvaluationTreeRangeIterator It(EvalTree); It; ++It)
	{
		FMovieSceneSegment NewSegment(It.Range());

		for (FSectionEvaluationData EvalData : EvalTree.GetAllData(It.Node()))
		{
			NewSegment.Impls.Add(EvalData);
		}

		Algo::SortBy(NewSegment.Impls, &FSectionEvaluationData::ImplIndex);
		Segments.Add(NewSegment);
	}

	FMovieSceneSegment Expected[] = {
		FMovieSceneSegment(TRange<FFrameNumber>(Inf, 	TRangeBound<FFrameNumber>::Exclusive(10)), 	{ FSectionEvaluationData(3) }),
		FMovieSceneSegment(TRange<FFrameNumber>(10,	20), 											{ FSectionEvaluationData(0), FSectionEvaluationData(2), FSectionEvaluationData(3) }),
		FMovieSceneSegment(TRange<FFrameNumber>(20,	25), 											{ FSectionEvaluationData(1), FSectionEvaluationData(2) }),
		FMovieSceneSegment(TRange<FFrameNumber>(25,	30), 											{ FSectionEvaluationData(1), FSectionEvaluationData(2), FSectionEvaluationData(4) }),
		FMovieSceneSegment(TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(30),	Inf), 	{ FSectionEvaluationData(4) })
	};
	AssertSegmentValues(this, Expected, Segments);
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieSceneCompilerTreeIteratorTest, "System.Engine.Sequencer.Compiler.Tree Iterator", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieSceneCompilerTreeIteratorTest::RunTest(const FString& Parameters)
{
	using namespace Impl;

	//	Time				-inf				10							20				25					30							inf
	//											[============= 0 ===========]
	//																		[============== 1 ==================]
	//											[========================== 2 ==================================]
	//						[================== 3 ==========================]
	//																						[================== 4 ==========================]
	//----------------------------------------------------------------------------------------------------------------------------------------
	//	Expected Impls		[ 		3			|			0,2,3			|		1,2		|		1,2,4		|			4				]

	TMovieSceneEvaluationTree<int32> Tree;

	Tree.Add(TRange<FFrameNumber>(10,	20), 										0);
	Tree.Add(TRange<FFrameNumber>(20,	30), 										1);
	Tree.Add(TRange<FFrameNumber>(10,	30), 										2);
	Tree.Add(TRange<FFrameNumber>(Inf,	TRangeBound<FFrameNumber>::Exclusive(20)), 	3);
	Tree.Add(TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(25),	Inf), 	4);

	FEvaluationTreeIteratorResult Expected[] = {
		MakeTuple( TRange<FFrameNumber>(Inf, 	TRangeBound<FFrameNumber>::Exclusive(10)),   TArray<int32>({ 3 })         ),
		MakeTuple( TRange<FFrameNumber>(10,		20),                                         TArray<int32>({ 0, 2, 3 })   ),
		MakeTuple( TRange<FFrameNumber>(20,		25),                                         TArray<int32>({ 1, 2 })      ),
		MakeTuple( TRange<FFrameNumber>(25,		30),                                         TArray<int32>({ 1, 2, 4 })   ),
		MakeTuple( TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(30), Inf),      TArray<int32>({ 4 })         ),
	};

	AssertEvaluationTree(this, Tree, Expected);
	return true;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieSceneCompilerTreeIteratorBoundsTest, "System.Engine.Sequencer.Compiler.Tree Iterator Bounds", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieSceneCompilerTreeIteratorBoundsTest::RunTest(const FString& Parameters)
{
	using namespace Impl;

	//	Time				-inf				10							20				25					30							inf
	//						[======== 0 ========][========= 1 ==============][============= 2 ==================][=========== 3 =============]

	TMovieSceneEvaluationTree<int32> Tree;

	Tree.Add(TRange<FFrameNumber>(Inf,												TRangeBound<FFrameNumber>::Exclusive(10)), 	0);
	Tree.Add(TRange<FFrameNumber>(FFrameNumber(10),									TRangeBound<FFrameNumber>::Exclusive(20)), 	1);
	Tree.Add(TRange<FFrameNumber>(FFrameNumber(20),									TRangeBound<FFrameNumber>::Exclusive(30)), 	2);
	Tree.Add(TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(30),			Inf), 										3);


	FEvaluationTreeIteratorResult Expected[] = {
		MakeTuple( TRange<FFrameNumber>(Inf,										TRangeBound<FFrameNumber>::Exclusive(10)), 	TArray<int32>({ 0 }) ),
		MakeTuple( TRange<FFrameNumber>(FFrameNumber(10),							TRangeBound<FFrameNumber>::Exclusive(20)), 	TArray<int32>({ 1 }) ),
		MakeTuple( TRange<FFrameNumber>(FFrameNumber(20),							TRangeBound<FFrameNumber>::Exclusive(30)), 	TArray<int32>({ 2 }) ),
		MakeTuple( TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(30),	Inf), 										TArray<int32>({ 3 }) ),
	};

	AssertEvaluationTree(this, Tree, Expected);

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS
#endif // #if 0
