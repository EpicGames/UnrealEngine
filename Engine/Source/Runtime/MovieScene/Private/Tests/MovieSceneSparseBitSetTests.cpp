// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "MovieSceneFwd.h"
#include "Misc/AutomationTest.h"
#include "Containers/SparseBitSet.h"
#include "Math/RandomStream.h"
#include "Misc/GeneratedTypeName.h"

#if WITH_DEV_AUTOMATION_TESTS
	
	template<typename T>
	struct TBitIt
	{
		TBitIt()
			: RemainingBits(0)
		{}

		TBitIt(T InRemainingBits)
			: RemainingBits(InRemainingBits)
		{}

		FORCEINLINE friend bool operator==(const TBitIt& A, const TBitIt& B)
		{
			return A.RemainingBits == B.RemainingBits;
		}

		explicit operator bool() const
		{
			return RemainingBits != 0;
		}

		uint32 operator*()
		{
			return UE::MovieScene::Private::CountTrailingZeros(RemainingBits);
		}

		TBitIt& operator++()
		{
			// Clear the lowest 1 bit
			RemainingBits = RemainingBits & (RemainingBits - 1);
			return *this;
		}
	private:
		T RemainingBits;
	};



namespace UE::MovieScene::Tests
{

	template<typename SparseBitSet>
	void TestBitSet(FAutomationTestBase* Test, SparseBitSet& InBitSet)
	{
		// Set random bits in the bitset
		const uint32 MaxBitIndex = InBitSet.GetMaxNumBits();

		// Static int for re-running tests using the same seed as a previous run
		static int32 SeedOverride = 0;
		const int32 InitialSeed = SeedOverride == 0 ? FMath::Rand() : SeedOverride;
		FRandomStream Random(InitialSeed);

		UE_LOGF(LogMovieScene, Display, "Running tests for %ls with a random seed %i...", GetGeneratedTypeName<SparseBitSet>(), InitialSeed);

		TSet<uint32> SetIndices;
		const int32 Num = Random.RandHelper(static_cast<int32>(FMath::Min(InBitSet.GetMaxNumBits(), 128u)));

		for (int32 Index = 0; Index < Num; ++Index)
		{
			const uint32 Bit = Random.GetUnsignedInt() % MaxBitIndex;
			InBitSet.SetBit(Bit);
			SetIndices.Add(Bit);
		}

		Test->TestEqual(TEXT("Num Set Bits"), InBitSet.CountSetBits(), static_cast<uint32>(SetIndices.Num()));

		// Test all bits are the correct state
		for (uint32 BitIndex = 0; BitIndex < FMath::Min(MaxBitIndex, 64u*64u); ++BitIndex)
		{
			Test->TestEqual(FString::Printf(TEXT("Bit index %d"), BitIndex), InBitSet.IsBitSet(BitIndex), SetIndices.Contains(BitIndex));
		}

		// Test the iterator
		int32 NumIterated = 0;
		for (int32 BitIndex : InBitSet)
		{
			++NumIterated;
			if (!SetIndices.Contains(BitIndex))
			{
				Test->AddError(FString::Printf(TEXT("Bit %i was iterated but it shouldn't be set!"), BitIndex));
			}
		}

		Test->TestEqual(TEXT("Number of iterated bits"), NumIterated, SetIndices.Num());
	}

} // namespace UE::MovieScene::Tests

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieSceneSparseBitSetTest, 
		"System.Engine.Sequencer.SparseBitSet", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieSceneSparseBitSetTest::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Tests;

	{
		// 8 bit buckets
		TFixedSparseBitSet<uint8, TDynamicSparseBitSetBucketStorage<uint8, 4>> BitSet8_8;
		TestEqual(TEXT("TFixedSparseBitSet<uint8, ...<uint8>>::GetMaxNumBits"), BitSet8_8.GetMaxNumBits(), 8u*8u);
		TestBitSet(this, BitSet8_8);

		TFixedSparseBitSet<uint16, TDynamicSparseBitSetBucketStorage<uint8, 4>> BitSet16_8;
		TestEqual(TEXT("TFixedSparseBitSet<uint16, ...<uint8>>::GetMaxNumBits"), BitSet16_8.GetMaxNumBits(), 16u*8u);
		TestBitSet(this, BitSet16_8);

		TFixedSparseBitSet<uint32, TDynamicSparseBitSetBucketStorage<uint8, 4>> BitSet32_8;
		TestEqual(TEXT("TFixedSparseBitSet<uint32, ...<uint8>>::GetMaxNumBits"), BitSet32_8.GetMaxNumBits(), 32u*8u);
		TestBitSet(this, BitSet32_8);

		TFixedSparseBitSet<uint64, TDynamicSparseBitSetBucketStorage<uint8, 4>> BitSet64_8;
		TestEqual(TEXT("TFixedSparseBitSet<uint64, ...<uint8>>::GetMaxNumBits"), BitSet64_8.GetMaxNumBits(), 64u*8u);
		TestBitSet(this, BitSet64_8);

		// 16 bit buckets
		TFixedSparseBitSet<uint8, TDynamicSparseBitSetBucketStorage<uint16, 4>> BitSet8_16;
		TestEqual(TEXT("TFixedSparseBitSet<uint8, ...<uint16>>::GetMaxNumBits"), BitSet8_16.GetMaxNumBits(), 8u*16u);
		TestBitSet(this, BitSet8_16);

		TFixedSparseBitSet<uint16, TDynamicSparseBitSetBucketStorage<uint16, 4>> BitSet16_16;
		TestEqual(TEXT("TFixedSparseBitSet<uint16, ...<uint16>>::GetMaxNumBits"), BitSet16_16.GetMaxNumBits(), 16u*16u);
		TestBitSet(this, BitSet16_16);

		TFixedSparseBitSet<uint32, TDynamicSparseBitSetBucketStorage<uint16, 4>> BitSet32_16;
		TestEqual(TEXT("TFixedSparseBitSet<uint32, ...<uint16>>::GetMaxNumBits"), BitSet32_16.GetMaxNumBits(), 32u*16u);
		TestBitSet(this, BitSet32_16);

		TFixedSparseBitSet<uint64, TDynamicSparseBitSetBucketStorage<uint16, 4>> BitSet64_16;
		TestEqual(TEXT("TFixedSparseBitSet<uint64, ...<uint16>>::GetMaxNumBits"), BitSet64_16.GetMaxNumBits(), 64u*16u);
		TestBitSet(this, BitSet64_16);

		// 32 bit buckets
		TFixedSparseBitSet<uint8, TDynamicSparseBitSetBucketStorage<uint32, 4>> BitSet8_32;
		TestEqual(TEXT("TFixedSparseBitSet<uint8, ...<uint32>>::GetMaxNumBits"), BitSet8_32.GetMaxNumBits(), 8u*32u);
		TestBitSet(this, BitSet8_32);

		TFixedSparseBitSet<uint16, TDynamicSparseBitSetBucketStorage<uint32, 4>> BitSet16_32;
		TestEqual(TEXT("TFixedSparseBitSet<uint16, ...<uint32>>::GetMaxNumBits"), BitSet16_32.GetMaxNumBits(), 16u*32u);
		TestBitSet(this, BitSet16_32);

		TFixedSparseBitSet<uint32, TDynamicSparseBitSetBucketStorage<uint32, 4>> BitSet32_32;
		TestEqual(TEXT("TFixedSparseBitSet<uint32, ...<uint32>>::GetMaxNumBits"), BitSet32_32.GetMaxNumBits(), 32u*32u);
		TestBitSet(this, BitSet32_32);

		TFixedSparseBitSet<uint64, TDynamicSparseBitSetBucketStorage<uint32, 4>> BitSet64_32;
		TestEqual(TEXT("TFixedSparseBitSet<uint64, ...<uint32>>::GetMaxNumBits"), BitSet64_32.GetMaxNumBits(), 64u*32u);
		TestBitSet(this, BitSet64_32);

		// 64 bit buckets
		TFixedSparseBitSet<uint8, TDynamicSparseBitSetBucketStorage<uint64, 4>> BitSet8_64;
		TestEqual(TEXT("TFixedSparseBitSet<uint8, ...<uint64>>::GetMaxNumBits"), BitSet8_64.GetMaxNumBits(), 8u*64u);
		TestBitSet(this, BitSet8_64);

		TFixedSparseBitSet<uint16, TDynamicSparseBitSetBucketStorage<uint64, 4>> BitSet16_64;
		TestEqual(TEXT("TFixedSparseBitSet<uint16, ...<uint64>>::GetMaxNumBits"), BitSet16_64.GetMaxNumBits(), 16u*64u);
		TestBitSet(this, BitSet16_64);

		TFixedSparseBitSet<uint32, TDynamicSparseBitSetBucketStorage<uint64, 4>> BitSet32_64;
		TestEqual(TEXT("TFixedSparseBitSet<uint32, ...<uint64>>::GetMaxNumBits"), BitSet32_64.GetMaxNumBits(), 32u*64u);
		TestBitSet(this, BitSet32_64);

		TFixedSparseBitSet<uint64, TDynamicSparseBitSetBucketStorage<uint64, 4>> BitSet64_64;
		TestEqual(TEXT("TFixedSparseBitSet<uint64, ...<uint64>>::GetMaxNumBits"), BitSet64_64.GetMaxNumBits(), 64u*64u);
		TestBitSet(this, BitSet64_64);
	}

	{
		// 8 bit buckets
		TDynamicSparseBitSet<uint8, TDynamicSparseBitSetBucketStorage<uint8, 4>> BitSet8_8;
		TestBitSet(this, BitSet8_8);

		TDynamicSparseBitSet<uint16, TDynamicSparseBitSetBucketStorage<uint8, 4>> BitSet16_8;
		TestBitSet(this, BitSet16_8);

		TDynamicSparseBitSet<uint32, TDynamicSparseBitSetBucketStorage<uint8, 4>> BitSet32_8;
		TestBitSet(this, BitSet32_8);

		TDynamicSparseBitSet<uint64, TDynamicSparseBitSetBucketStorage<uint8, 4>> BitSet64_8;
		TestBitSet(this, BitSet64_8);

		// 16 bit buckets
		TDynamicSparseBitSet<uint8, TDynamicSparseBitSetBucketStorage<uint16, 4>> BitSet8_16;
		TestBitSet(this, BitSet8_16);

		TDynamicSparseBitSet<uint16, TDynamicSparseBitSetBucketStorage<uint16, 4>> BitSet16_16;
		TestBitSet(this, BitSet16_16);

		TDynamicSparseBitSet<uint32, TDynamicSparseBitSetBucketStorage<uint16, 4>> BitSet32_16;
		TestBitSet(this, BitSet32_16);

		TDynamicSparseBitSet<uint64, TDynamicSparseBitSetBucketStorage<uint16, 4>> BitSet64_16;
		TestBitSet(this, BitSet64_16);

		// 32 bit buckets
		TDynamicSparseBitSet<uint8, TDynamicSparseBitSetBucketStorage<uint32, 4>> BitSet8_32;
		TestBitSet(this, BitSet8_32);

		TDynamicSparseBitSet<uint16, TDynamicSparseBitSetBucketStorage<uint32, 4>> BitSet16_32;
		TestBitSet(this, BitSet16_32);

		TDynamicSparseBitSet<uint32, TDynamicSparseBitSetBucketStorage<uint32, 4>> BitSet32_32;
		TestBitSet(this, BitSet32_32);

		TDynamicSparseBitSet<uint64, TDynamicSparseBitSetBucketStorage<uint32, 4>> BitSet64_32;
		TestBitSet(this, BitSet64_32);

		// 64 bit buckets
		TDynamicSparseBitSet<uint8, TDynamicSparseBitSetBucketStorage<uint64, 4>> BitSet8_64;
		TestBitSet(this, BitSet8_64);

		TDynamicSparseBitSet<uint16, TDynamicSparseBitSetBucketStorage<uint64, 4>> BitSet16_64;
		TestBitSet(this, BitSet16_64);

		TDynamicSparseBitSet<uint32, TDynamicSparseBitSetBucketStorage<uint64, 4>> BitSet32_64;
		TestBitSet(this, BitSet32_64);

		TDynamicSparseBitSet<uint64, TDynamicSparseBitSetBucketStorage<uint64, 4>> BitSet64_64;
		TestBitSet(this, BitSet64_64);
	}
	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS
