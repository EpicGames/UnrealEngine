// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneTestObjects.h"
#include "Compilation/MovieSceneCompilerRules.h"
#include "Compilation/MovieSceneSegmentCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneTestObjects)

FMovieSceneEvalTemplatePtr UTestMovieSceneTrack::CreateTemplateForSection(const UMovieSceneSection& InSection) const
{
	return FTestMovieSceneEvalTemplate();
}

void UTestMovieSceneTrack::AddSection(UMovieSceneSection& Section)
{
	if (UTestMovieSceneSection* TestSection = Cast<UTestMovieSceneSection>(&Section))
	{
		SectionArray.Add(TestSection);
	}
}

bool UTestMovieSceneTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UTestMovieSceneSection::StaticClass();
}

UMovieSceneSection* UTestMovieSceneTrack::CreateNewSection()
{
	return NewObject<UTestMovieSceneSection>(this, NAME_None, RF_Transactional);
}


bool UTestMovieSceneTrack::HasSection(const UMovieSceneSection& Section) const
{
	return SectionArray.Contains(&Section);
}


bool UTestMovieSceneTrack::IsEmpty() const
{
	return SectionArray.Num() == 0;
}

void UTestMovieSceneTrack::RemoveSection(UMovieSceneSection& Section)
{
	SectionArray.Remove(&Section);
}

void UTestMovieSceneTrack::RemoveSectionAt(int32 SectionIndex)
{
	SectionArray.RemoveAt(SectionIndex);
}



void UTestMovieSceneEvalHookTrack::AddSection(UMovieSceneSection& Section)
{
	if (UTestMovieSceneEvalHookSection* TestSection = Cast<UTestMovieSceneEvalHookSection>(&Section))
	{
		SectionArray.Add(TestSection);
	}
}

bool UTestMovieSceneEvalHookTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UTestMovieSceneEvalHookSection::StaticClass();
}

UMovieSceneSection* UTestMovieSceneEvalHookTrack::CreateNewSection()
{
	return NewObject<UTestMovieSceneEvalHookSection>(this, NAME_None, RF_Transactional);
}


bool UTestMovieSceneEvalHookTrack::HasSection(const UMovieSceneSection& Section) const
{
	return SectionArray.Contains(&Section);
}


bool UTestMovieSceneEvalHookTrack::IsEmpty() const
{
	return SectionArray.Num() == 0;
}

void UTestMovieSceneEvalHookTrack::RemoveSection(UMovieSceneSection& Section)
{
	SectionArray.Remove(&Section);
}

void UTestMovieSceneEvalHookTrack::RemoveSectionAt(int32 SectionIndex)
{
	SectionArray.RemoveAt(SectionIndex);
}
