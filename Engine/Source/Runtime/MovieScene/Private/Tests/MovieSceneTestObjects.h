// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compilation/IMovieSceneTrackTemplateProducer.h"
#include "Evaluation/MovieSceneEvalTemplate.h"
#include "GameFramework/Actor.h"
#include "MovieScene.h"
#include "MovieSceneSection.h"
#include "MovieSceneSequence.h"
#include "MovieSceneTrack.h"
#include "Sections/MovieSceneHookSection.h"
#include "Sections/MovieSceneSubSection.h"
#include "StructUtils/InstancedStruct.h"
#include "Tracks/MovieSceneSubTrack.h"

#include "MovieSceneTestObjects.generated.h"

USTRUCT()
struct FTestMovieSceneEvalTemplate : public FMovieSceneEvalTemplate
{
	GENERATED_BODY()

	virtual UScriptStruct& GetScriptStructImpl() const { return *StaticStruct(); }
};

UCLASS(MinimalAPI)
class UTestMovieSceneTrack : public UMovieSceneTrack, public IMovieSceneTrackTemplateProducer
{
public:

	GENERATED_BODY()

	virtual const TArray<UMovieSceneSection*>& GetAllSections() const override { return SectionArray; }
	virtual FMovieSceneEvalTemplatePtr CreateTemplateForSection(const UMovieSceneSection& InSection) const override;

	virtual bool IsEmpty() const override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual void AddSection(UMovieSceneSection& Section) override;
	virtual class UMovieSceneSection* CreateNewSection() override;
	virtual bool HasSection(const UMovieSceneSection& Section) const override;
	virtual void RemoveSection(UMovieSceneSection& Section) override;
	virtual void RemoveSectionAt(int32 SectionIndex) override;

#if WITH_EDITORONLY_DATA
	virtual FText GetDisplayName() const override { return FText::FromString(TEXT("Unnamed Track")); }
#endif

	UPROPERTY()
	bool bHighPassFilter;

	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> SectionArray;
};

UCLASS(MinimalAPI)
class UTestMovieSceneSection : public UMovieSceneSection
{
public:
	GENERATED_BODY()
};

UCLASS(MinimalAPI)
class UTestMovieSceneSequence : public UMovieSceneSequence
{
public:
	GENERATED_BODY()

	UTestMovieSceneSequence(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
	{
		if (!HasAnyFlags(RF_ClassDefaultObject))
		{
			MovieScene = ObjInit.CreateDefaultSubobject<UMovieScene>(this, "MovieScene");
		}
	}

	//~ UMovieSceneSequence interface
	virtual void BindPossessableObject(const FGuid& ObjectId, UObject& PossessedObject, UObject* Context) override {}
	virtual bool CanPossessObject(UObject& Object, UObject* InPlaybackContext) const override { return false; }
	virtual void LocateBoundObjects(const FGuid& ObjectId, UObject* Context, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const override {}
	virtual UObject* GetParentObject(UObject* Object) const override { return nullptr; }
	virtual void UnbindPossessableObjects(const FGuid& ObjectId) override {}
	virtual void UnbindObjects(const FGuid& ObjectId, const TArray<UObject*>& InObjects, UObject* Context) override {}
	virtual void UnbindInvalidObjects(const FGuid& ObjectId, UObject* Context) override {}
	virtual UMovieScene* GetMovieScene() const override { return MovieScene; }

	UPROPERTY()
	TObjectPtr<UMovieScene> MovieScene;
};

UCLASS(MinimalAPI)
class UTestMovieSceneSubTrack : public UMovieSceneSubTrack
{
public:
	GENERATED_BODY()

	virtual const TArray<UMovieSceneSection*>& GetAllSections() const override { return SectionArray; }

	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> SectionArray;
};

UCLASS(MinimalAPI)
class UTestMovieSceneSubSection : public UMovieSceneSubSection
{
	GENERATED_BODY()
};

UCLASS(MinimalAPI)
class UTestMovieSceneEvalHookTrack : public UMovieSceneTrack
{
public:
	GENERATED_BODY()

	virtual const TArray<UMovieSceneSection*>& GetAllSections() const override { return SectionArray; }

	virtual bool IsEmpty() const override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual void AddSection(UMovieSceneSection& Section) override;
	virtual UMovieSceneSection* CreateNewSection() override;
	virtual bool HasSection(const UMovieSceneSection& Section) const override;
	virtual void RemoveSection(UMovieSceneSection& Section) override;
	virtual void RemoveSectionAt(int32 SectionIndex) override;
#if WITH_EDITORONLY_DATA
	virtual FText GetDisplayName() const override { return FText(); }
#endif

	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> SectionArray;
};

UCLASS(MinimalAPI)
class UTestMovieSceneEvalHookSection : public UMovieSceneHookSection
{
public:
	GENERATED_BODY()

	UTestMovieSceneEvalHookSection(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
	{
		bRequiresRangedHook = true;
		bRequiresTriggerHooks = true;
		EvalOptions.CompletionMode = EMovieSceneCompletionMode::RestoreState;
	}

	int32 StartValue = 1000;
	int32 EndValue   = 2000;

	TArray<FFrameNumber> TriggerTimes;

	virtual void Begin(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const override;
	virtual void Update(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const override;
	virtual void End(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const override;
	virtual void Trigger(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const override;

	virtual TArrayView<const FFrameNumber> GetTriggerTimes() const
	{
		return TriggerTimes;
	}
};

UENUM(BlueprintType)
enum class ETestMovieSceneEnum : uint8
{
	One,
	Two,
	Three
};

USTRUCT(BlueprintType)
struct FTestMovieSceneStruct
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category="Test")
	float First = 0.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category="Test")
	float Second = 0.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category="Test")
	ETestMovieSceneEnum Enum = ETestMovieSceneEnum::One;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category="Test")
	FVector Vector = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category="Test")
	TArray<int32> MultipleIntegers;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category="Test")
	TArray<FVector> MultipleVectors;
};

USTRUCT(BlueprintType)
struct FTestMovieSceneStruct2
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category="Test")
	float First = 0.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category="Test")
	float Second = 0.f;
};

UCLASS(MinimalAPI, BlueprintType)
class UTestMovieSceneObject : public UObject
{
	GENERATED_BODY()
};

UCLASS(MinimalAPI, BlueprintType)
class ATestMovieSceneArrayPropertiesActor : public AActor
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category="Test")
	bool TestBool = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category="Test")
	ETestMovieSceneEnum TestEnum = ETestMovieSceneEnum::One;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category="Test")
	int32 TestInt32 = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category="Test")
	TObjectPtr<UTestMovieSceneObject> TestObject;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category="Test")
	FVector TestVector = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category="Test")
	TArray<float> MultipleFloats;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category="Test")
	FTestMovieSceneStruct SingleStruct;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category="Test")
	TArray<FTestMovieSceneStruct> MultipleStructs;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category="Test")
	FInstancedStruct SingleInstancedStruct;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category="Test")
	TArray<FInstancedStruct> MultipleInstancedStructs;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter="SetTestSetterFloat", Interp, Category="Test")
	float TestSetterFloat = 0;

public:

	int32 NumSetTestSetterFloatCalls = 0;

public:

	ATestMovieSceneArrayPropertiesActor(const FObjectInitializer& ObjInit)
		: AActor(ObjInit)
	{}

	UFUNCTION(BlueprintCallable, Category="Test")
	void SetTestSetterFloat(float InValue)
	{
		TestSetterFloat = InValue;
		++NumSetTestSetterFloatCalls;
	}
};

