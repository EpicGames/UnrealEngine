// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Evaluation/MovieSceneSequenceTransform.h"
#include "Evaluation/MovieSceneSectionParameters.h"
#include "Variants/MovieSceneTimeWarpVariantPayloads.h"
#include "Containers/ArrayView.h"
#include "Misc/AutomationTest.h"
#include "MovieSceneTimeHelpers.h"
#include "UObject/Package.h"

#define LOCTEXT_NAMESPACE "MovieSceneTransformTests"

// Range equality.
bool IsEqual(TRangeBound<FFrameNumber> A, TRangeBound<FFrameNumber> B)
{
	if (A.IsOpen() || B.IsOpen())
	{
		return A.IsOpen() == B.IsOpen();
	}
	else if (A.IsInclusive() != B.IsInclusive())
	{
		return false;
	}
	
	return A.GetValue() == B.GetValue();
}

// Range equality.
bool IsEqual(TRange<FFrameNumber> A, TRange<FFrameNumber> B)
{
	return IsEqual(A.GetLowerBound(), B.GetLowerBound()) && IsEqual(A.GetUpperBound(), B.GetUpperBound());
}

// Frame number equality.
bool IsEqual(FFrameNumber A, FFrameNumber B)
{
	return A.Value == B.Value;
}

// Frame time equality.
bool IsEqual(FFrameTime A, FFrameTime B)
{
	return IsEqual(A.FrameNumber, B.FrameNumber) && FMath::IsNearlyEqual(A.GetSubFrame(), B.GetSubFrame());
}

// Most time transformations are not "round" so they return a frame time that must be rounded down to a frame number,
// except for time warping which doesn't stretch anything and returns a frame number.
template<typename TTransform>
FFrameNumber TransformToFrameNumber(TTransform Transform, FFrameNumber Value)
{
	return (Value * Transform).FloorToFrame();
}

// Generic method for testing the transform of frames and times.
template<typename TTransform>
bool TestTransform(FAutomationTestBase& Test, TTransform Transform, TArrayView<FFrameNumber> InSource, TArrayView<FFrameNumber> InExpected, const TCHAR* TestName)
{
	check(InSource.Num() == InExpected.Num());

	bool bSuccess = true;
	for (int32 Index = 0; Index < InSource.Num(); ++Index)
	{
		FFrameNumber Result = TransformToFrameNumber(Transform, InSource[Index]);
		if (!IsEqual(Result, InExpected[Index]))
		{
			Test.AddError(FString::Printf(TEXT("Test '%s' failed (Index %d). Transform %s did not apply correctly (%s != %s)"),
				TestName,
				Index,
				*LexToString(Transform),
				*LexToString(Result),
				*LexToString(InExpected[Index])));

			bSuccess = false;
		}
	}

	return bSuccess;
}

// A variant of the above method for testing the transform of ranges.
template<typename TTransform>
bool TestTransform(FAutomationTestBase& Test, TTransform Transform, TArrayView<TRange<FFrameNumber>> InSource, TArrayView<TRange<FFrameNumber>> InExpected, const TCHAR* TestName)
{
	check(InSource.Num() == InExpected.Num());

	bool bSuccess = true;
	for (int32 Index = 0; Index < InSource.Num(); ++Index)
	{
		TRange<FFrameNumber> Result = InSource[Index] * Transform;
		if (!IsEqual(Result, InExpected[Index]))
		{
			Test.AddError(FString::Printf(TEXT("Test '%s' failed (Index %d). Transform %s did not apply correctly (%s != %s)"),
				TestName,
				Index,
				*LexToString(Transform),
				*LexToString(Result),
				*LexToString(InExpected[Index])));

			bSuccess = false;
		}
	}

	return bSuccess;
}

// Calculate the transform that transforms from range A to range B
FMovieSceneSequenceTransform TransformRange(FFrameNumber StartA, FFrameNumber EndA, FFrameNumber StartB, FFrameNumber EndB)
{
	float Scale = double( (EndB - StartB).Value ) / (EndA - StartA).Value;
	return FMovieSceneSequenceTransform(StartB, Scale) * FMovieSceneSequenceTransform(-StartA);
}

// Linear transform tests
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieSceneSubSectionCoreLinearTransformsTest, 
		"System.Engine.Sequencer.Core.LinearTransforms", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieSceneSubSectionCoreLinearTransformsTest::RunTest(const FString& Parameters)
{
	FFrameNumber SourceTimes[] = {
		FFrameNumber(500),
		FFrameNumber(525)
	};

	bool bSuccess = true;

	{
		FFrameNumber ExpectedTimes[] = {
			FFrameNumber(500),
			FFrameNumber(525)
		};
		FMovieSceneTimeTransform Transform(0);
		bSuccess = TestTransform(*this, Transform, SourceTimes, ExpectedTimes, TEXT("IdentityTransform")) && bSuccess;

		Transform = Transform.Inverse();
		bSuccess = TestTransform(*this, Transform, ExpectedTimes, SourceTimes, TEXT("IdentityTransformInverse")) && bSuccess;
	}

	{
		FFrameNumber ExpectedTimes[] = {
			FFrameNumber(1000),
			FFrameNumber(1050)
		};
		FMovieSceneTimeTransform Transform(0, 2.f);
		bSuccess = TestTransform(*this, Transform, SourceTimes, ExpectedTimes, TEXT("OffsetTransform")) && bSuccess;

		Transform = Transform.Inverse();
		bSuccess = TestTransform(*this, Transform, ExpectedTimes, SourceTimes, TEXT("OffsetTransformInverse")) && bSuccess;
	}

	{
		FFrameNumber ExpectedTimes[] = {
			FFrameNumber(0),
			FFrameNumber(50)
		};
		FMovieSceneTimeTransform Transform(-1000, 2.f);
		bSuccess = TestTransform(*this, Transform, SourceTimes, ExpectedTimes, TEXT("OffsetAndScaleTransform")) && bSuccess;

		Transform = Transform.Inverse();
		bSuccess = TestTransform(*this, Transform, ExpectedTimes, SourceTimes, TEXT("OffsetAndScaleTransformInverse")) && bSuccess;
	}

	{
		FFrameNumber ExpectedTimes[] = {
			FFrameNumber(0),
			FFrameNumber(50)
		};
		FMovieSceneTimeTransform Transform = FMovieSceneTimeTransform(0, 2.f) * FMovieSceneTimeTransform(-500);
		bSuccess = TestTransform(*this, Transform, SourceTimes, ExpectedTimes, TEXT("OffsetAndScaleTransformObtainedFromMultiplication")) && bSuccess;
	}

	return bSuccess;
}

// Sequence transform tests 
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieSceneSubSectionCoreSequenceTransformsTest,
		"System.Engine.Sequencer.Core.SequenceTransforms",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieSceneSubSectionCoreSequenceTransformsTest::RunTest(const FString& Parameters)
{
	// We test using ranges since that implicitly tests frame number transformation as well
	static const TRangeBound<FFrameNumber> OpenBound;

	TRange<FFrameNumber> InfiniteRange(OpenBound, OpenBound);
	TRange<FFrameNumber> OpenLowerRange(OpenBound, FFrameNumber(200));
	TRange<FFrameNumber> OpenUpperRange(FFrameNumber(100), OpenBound);
	TRange<FFrameNumber> ClosedRange(FFrameNumber(100), FFrameNumber(200));

	TRange<FFrameNumber> SourceRanges[] = {
		InfiniteRange, OpenLowerRange, OpenUpperRange, ClosedRange
	};

	bool bSuccess = true;

	{
		// Test Multiplication with an identity transform
		FMovieSceneSequenceTransform IdentityTransform;

		TRange<FFrameNumber> Expected[] = {
			InfiniteRange, OpenLowerRange, OpenUpperRange, ClosedRange
		};
		
		bSuccess = TestTransform(*this, IdentityTransform.LinearTransform, SourceRanges, Expected, TEXT("IdentityTransform")) && bSuccess;
	}

	{
		// Test a simple translation
		FMovieSceneSequenceTransform Transform(100, 1);

		TRange<FFrameNumber> Expected[] = {
			InfiniteRange, TRange<FFrameNumber>(OpenBound, FFrameNumber(300)), TRange<FFrameNumber>(FFrameNumber(200), OpenBound), TRange<FFrameNumber>(200, 300)
		};

		bSuccess = TestTransform(*this, Transform.LinearTransform, SourceRanges, Expected, TEXT("Simple Translation")) && bSuccess;
	}

	{
		// Test a simple translation + time scale

		// Transform 100 - 200 to -200 - 1000
		FMovieSceneSequenceTransform Transform = TransformRange(100, 200, -200, 1000);

		TRange<FFrameNumber> Expected[] = {
			InfiniteRange, TRange<FFrameNumber>(OpenBound, FFrameNumber(1000)), TRange<FFrameNumber>(FFrameNumber(-200), OpenBound), TRange<FFrameNumber>(-200, 1000)
		};

		bSuccess = TestTransform(*this, Transform.LinearTransform, SourceRanges, Expected, TEXT("Simple Translation + half speed")) && bSuccess;
	}

	{
		// Test that transforming a frame number by the same transform multiple times, does the same as the equivalent accumulated transform

		// scales by 2, then offsets by 100
		FMovieSceneSequenceTransform SeedTransform = FMovieSceneSequenceTransform(100, 0.5f);
		FMovieSceneSequenceTransform AccumulatedTransform;

		FFrameTime SeedValue = 10;
		for (int32 i = 0; i < 5; ++i)
		{
			AccumulatedTransform = SeedTransform * AccumulatedTransform;

			SeedValue = SeedValue * SeedTransform;
		}

		FFrameTime AccumValue = FFrameTime(10) * AccumulatedTransform;
		if (AccumValue != SeedValue)
		{
			AddError(FString::Printf(TEXT("Accumulated transform does not have the same effect as separate transformations (%i+%.5f != %i+%.5f)"), AccumValue.FrameNumber.Value, AccumValue.GetSubFrame(), SeedValue.FrameNumber.Value, SeedValue.GetSubFrame()));
		}

		FMovieSceneInverseSequenceTransform InverseTransform = AccumulatedTransform.Inverse();

		TOptional<FFrameTime> InverseValue = InverseTransform.TryTransformTime(AccumValue);
		if (!InverseValue.IsSet())
		{
			AddError(FString::Printf(TEXT("Inverse accumulated transform did not return a valid time")));
		}
		else if (InverseValue.GetValue() != 10)
		{
			AddError(FString::Printf(TEXT("Inverse accumulated transform does not return value back to its original value (%i+%.5f != 10)"), InverseValue->FrameNumber.Value, InverseValue->GetSubFrame()));
		}
	}

	return true;
}

// Miscellaneous warping and scaling tests
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieSceneSubSectionCoreWarpingAndScalingTransformsTest,
		"System.Engine.Sequencer.Core.WarpingAndScalingTransforms",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieSceneSubSectionCoreWarpingAndScalingTransformsTest::RunTest(const FString& Parameters)
{
	{
		// Sub-sequence at 0, playing at x2
		FMovieSceneSequenceTransform Transform;
		Transform.LinearTransform = FMovieSceneTimeTransform(0, 2.f);
		Transform.AddLoop(0, 30);
		TestEqual("Transform time 1", FFrameNumber(10) * Transform, FFrameTime(20));


		FMovieSceneTransformBreadcrumbs Breadcrumbs;
		FMovieSceneInverseSequenceTransform Inv = Transform.Inverse();

		Breadcrumbs.AddBreadcrumb(15);
		TestEqual("Inverse time 3", Inv.TryTransformTime(FFrameNumber(20), Breadcrumbs), TOptional<FFrameTime>(10));

		Breadcrumbs.Reset();
		Breadcrumbs.AddBreadcrumb(45);
		TestEqual("Inverse time 4", Inv.TryTransformTime(FFrameNumber(20), Breadcrumbs), TOptional<FFrameTime>(FFrameTime(25)));
	}

	{
		// Sub-sequence at 0, playing at x2, with start offset 20
		FMovieSceneSequenceTransform Transform;
		Transform.LinearTransform = FMovieSceneTimeTransform(20, 2.f);
		Transform.AddLoop(20, 50);
		TestEqual("Transform time 5", FFrameNumber(10) * Transform, FFrameTime(40));
		TestEqual("Transform time 6", FFrameNumber(18) * Transform, FFrameTime(26));

		FMovieSceneTransformBreadcrumbs Breadcrumbs;
		FMovieSceneInverseSequenceTransform Inv = Transform.Inverse();

		Breadcrumbs.AddBreadcrumb(35);	// 35 should be in the middle of the first loop
		TestEqual("Inverse time 7", Inv.TryTransformTime(FFrameNumber(40), Breadcrumbs), TOptional<FFrameTime>(FFrameTime(10)));

		Breadcrumbs.Reset();
		Breadcrumbs.AddBreadcrumb(65);	// 65 should be in the middle of the second loop
		TestEqual("Inverse time 8", Inv.TryTransformTime(FFrameNumber(40), Breadcrumbs), TOptional<FFrameTime>(FFrameTime(25)));
	}

	{
		// Sub-sequence at 3, playing at x2
		FMovieSceneSequenceTransform Transform;
		Transform.LinearTransform = FMovieSceneTimeTransform(-6, 2.f);
		Transform.AddLoop(0, 30);
		TestEqual("Transform time 9", FFrameNumber(13) * Transform, FFrameTime(20));
		TestEqual("Transform time 10", FFrameNumber(21) * Transform, FFrameTime(6));

		FMovieSceneTransformBreadcrumbs Breadcrumbs;
		FMovieSceneInverseSequenceTransform Inv = Transform.Inverse();

		Breadcrumbs.AddBreadcrumb(15);	// 15 should be in the middle of the first loop
		TestEqual("Inverse time 11", Inv.TryTransformTime(FFrameNumber(20), Breadcrumbs), TOptional<FFrameTime>(FFrameTime(13)));

		Breadcrumbs.Reset();
		Breadcrumbs.AddBreadcrumb(45);	// 45 should be in the middle of the second loop
		TestEqual("Inverse time 12", Inv.TryTransformTime(FFrameNumber(20), Breadcrumbs), TOptional<FFrameTime>(FFrameTime(28)));
	}

	{
		// Sub-sequence at 3, playing at x2, with start offset 20
		FMovieSceneSequenceTransform Transform;
		Transform.LinearTransform = FMovieSceneTimeTransform(-6 + 20, 2.f);
		Transform.AddLoop(20, 50);
		TestEqual("Transform time 13", FFrameNumber(13) * Transform, FFrameTime(40));
		TestEqual("Transform time 14", FFrameNumber(21) * Transform, FFrameTime(26));

		FMovieSceneTransformBreadcrumbs Breadcrumbs;
		FMovieSceneInverseSequenceTransform Inv = Transform.Inverse();

		Breadcrumbs.AddBreadcrumb(35);	// 35 should be in the middle of the first loop
		TestEqual("Inverse time 15", Inv.TryTransformTime(FFrameNumber(40), Breadcrumbs), TOptional<FFrameTime>(FFrameTime(13)));

		Breadcrumbs.Reset();
		Breadcrumbs.AddBreadcrumb(65);	// 65 should be in the middle of the second loop
		TestEqual("Inverse time 16", Inv.TryTransformTime(FFrameNumber(40), Breadcrumbs), TOptional<FFrameTime>(FFrameTime(28)));
	}

	{
		// Two levels of sub-sequences: one placed at 10 and warping, the second placed at 6 with x2 scaling
		FMovieSceneSequenceTransform Transform;
		Transform.LinearTransform = FMovieSceneTimeTransform(-10, 1.f);
		Transform.AddLoop(0, 30);
		Transform.NestedTransforms.Emplace(FMovieSceneTimeTransform(-12, 2.f));
		TestEqual("Transform time 17", FFrameNumber(18) * Transform, FFrameTime(4));
		TestEqual("Transform time 18", FFrameNumber(55) * Transform, FFrameTime(18));

		FMovieSceneTransformBreadcrumbs Breadcrumbs;
		FMovieSceneInverseSequenceTransform Inv = Transform.Inverse();

		Breadcrumbs.AddBreadcrumb(15);
		TestEqual("Inverse time 17", Inv.TryTransformTime(FFrameNumber(4), Breadcrumbs), TOptional<FFrameTime>(FFrameTime(18)));

		Breadcrumbs.Reset();
		Breadcrumbs.AddBreadcrumb(45);
		TestEqual("Inverse time 18", Inv.TryTransformTime(FFrameNumber(18), Breadcrumbs), TOptional<FFrameTime>(FFrameTime(55)));
	}

	{
		// Two levels of sub-sequences: one placed at 10, the second placed at 6 with x2 scaling and warping
		FMovieSceneSequenceTransform Transform;
		Transform.LinearTransform = FMovieSceneTimeTransform(-10, 1.f);
		Transform.NestedTransforms.Emplace(FMovieSceneTimeTransform(-12, 2.f));
		Transform.AddLoop(0, 14);
		TestEqual("Transform time 17", FFrameNumber(19) * Transform, FFrameTime(6));
		TestEqual("Transform time 18", FFrameNumber(32) * Transform, FFrameTime(4));

		FMovieSceneTransformBreadcrumbs Breadcrumbs;
		FMovieSceneInverseSequenceTransform Inv = Transform.Inverse();

		Breadcrumbs.AddBreadcrumb(7); // half way through loop 0
		TestEqual("Inverse time 17", Inv.TryTransformTime(FFrameNumber(6), Breadcrumbs), TOptional<FFrameTime>(FFrameTime(19)));

		Breadcrumbs.Reset();
		Breadcrumbs.AddBreadcrumb(28); // half way through loop 2
		TestEqual("Inverse time 18", Inv.TryTransformTime(FFrameNumber(4), Breadcrumbs), TOptional<FFrameTime>(FFrameTime(32)));
	}

	{
		// Sub-sequence at 3, playing at x2, with start offset 20, but all contained inside a higher offset of 100.
		FMovieSceneSequenceTransform Transform;
		Transform.LinearTransform.Offset = FFrameTime(-100);
		Transform.NestedTransforms.Emplace(FMovieSceneTimeTransform(-6 + 20, 2.f));
		Transform.AddLoop(20, 50);

		TestEqual("Transform time 19", FFrameNumber(113) * Transform, FFrameTime(40));
		TestEqual("Transform time 20", FFrameNumber(121) * Transform, FFrameTime(26));

		FMovieSceneTransformBreadcrumbs Breadcrumbs;
		FMovieSceneInverseSequenceTransform Inv = Transform.Inverse();

		Breadcrumbs.AddBreadcrumb(35); // Loop 0
		TestEqual("Inverse time 21", Inv.TryTransformTime(FFrameNumber(40), Breadcrumbs), TOptional<FFrameTime>(FFrameTime(113)));

		Breadcrumbs.Reset();
		Breadcrumbs.AddBreadcrumb(65); // Loop 1
		TestEqual("Inverse time 22", Inv.TryTransformTime(FFrameNumber(40), Breadcrumbs), TOptional<FFrameTime>(FFrameTime(128)));
	}

	{
		// Zero-timescale transform on a sub-sequence. Any frame numbers transformed in should be equal to the frame offset
		FMovieSceneSequenceTransform Transform;
		Transform.NestedTransforms.Add(FMovieSceneTimeTransform(0)); // no outer offset
		Transform.NestedTransforms.Add(FMovieSceneTimeWarpVariant(0.0)); // 0 timescale
		Transform.NestedTransforms.Add(FMovieSceneTimeTransform(30)); // 30 inner frame offset

		TestEqual("Outer time 40 through 0 timescale with 30 offset", FFrameNumber(40) * Transform, FFrameTime(30));
		TestEqual("Outer time 0 through 0 timescale with 30 offset", FFrameNumber(0) * Transform, FFrameTime(30));
		TestEqual("Outer time 173 through 0 timescale with 30 offset", FFrameNumber(173) * Transform, FFrameTime(30));
	}

	{
		// Zero-timescale transform on a sub-sequence. Same as previous, but we also will invert this transform and ensure timescale is correctly infinite
		// and any transforms by that infinite transform. Anything transformed out should just be equal to the outer offset
		FMovieSceneSequenceTransform Transform;
		Transform.NestedTransforms.Add(FMovieSceneTimeTransform(-10)); // 10 outer offset
		Transform.NestedTransforms.Add(FMovieSceneTimeWarpVariant(0.0)); // 0 timescale
		Transform.NestedTransforms.Add(FMovieSceneTimeTransform(30)); // 30 inner frame offset

		FMovieSceneInverseSequenceTransform Inv = Transform.Inverse();
		if (Inv.IsLinear())
		{
			AddError(FString::Printf(TEXT("Inverse of a transform with zero timescale is not correctly warping")));
		}

		TestEqual("Inner time 40 through inf timescale with 10 outer offset",  Inv.TryTransformTime(FFrameNumber(40)),  TOptional<FFrameTime>());
		TestEqual("Inner time 0 through inf timescale with 10 outer offset",   Inv.TryTransformTime(FFrameNumber(0)),   TOptional<FFrameTime>());
		TestEqual("Inner time 173 through inf timescale with 10 outer offset", Inv.TryTransformTime(FFrameNumber(173)), TOptional<FFrameTime>());
		TestEqual("Inner time 30 through inf timescale with 10 outer offset",  Inv.TryTransformTime(FFrameNumber(30)),  TOptional<FFrameTime>(FFrameTime(10)));

		// Re-invert the inverse transform. This should be equivalent to the original transform and we shouldn't have lost anything.

		// @todo: is this necessary? We haven't needed an Inverse for an Inverse anywhere else in the codebase
		// FMovieSceneSequenceTransform InvInv = Inv.Inverse();
		// TestEqual("Doubly inverted zero-timescale transform should be equal to original", InvInv, Transform);
	}

	{
		// Multiple levels of sub sequences with zero-timescale thrown in
		FMovieSceneSequenceTransform OuterTransform;
		OuterTransform.NestedTransforms.Add(FMovieSceneTimeTransform(-10)); // 10 outer offset
		OuterTransform.NestedTransforms.Add(FMovieSceneTimeWarpVariant(0.0)); // 0 timescale
		OuterTransform.NestedTransforms.Add(FMovieSceneTimeTransform(30)); // 30 inner frame offset

		FMovieSceneSequenceTransform InnerTransform;
		InnerTransform.LinearTransform.Offset = FFrameNumber(5); // An inner frame offset of 5

		FMovieSceneSequenceTransform CompleteTransform = InnerTransform * OuterTransform;

		TestEqual("Subsequence frame through zero timescale transform", FFrameNumber(40)* CompleteTransform, FFrameTime(35));
		TestEqual("Subsequence frame through zero timescale transform", FFrameNumber(0)* CompleteTransform, FFrameTime(35));
		TestEqual("Subsequence frame through zero timescale transform", FFrameNumber(173)* CompleteTransform, FFrameTime(35));

		FMovieSceneInverseSequenceTransform InvCompleteTransform = CompleteTransform.Inverse();

		TestEqual("Inner time 40 through inf timescale with 10 outer offset",  InvCompleteTransform.TryTransformTime(FFrameNumber(40)),  TOptional<FFrameTime>());
		TestEqual("Inner time 0 through inf timescale with 10 outer offset",   InvCompleteTransform.TryTransformTime(FFrameNumber(0)),   TOptional<FFrameTime>());
		TestEqual("Inner time 173 through inf timescale with 10 outer offset", InvCompleteTransform.TryTransformTime(FFrameNumber(173)), TOptional<FFrameTime>());
		TestEqual("Inner time 35 through inf timescale with 10 outer offset",  InvCompleteTransform.TryTransformTime(FFrameNumber(35)),  TOptional<FFrameTime>(FFrameTime(10)));

		// Re-invert the inverse transform. This should be equivalent to the original transform and we shouldn't have lost anything.
		// @todo: is this necessary? We haven't needed an Inverse for an Inverse anywhere else in the codebase
		//FMovieSceneSequenceTransform InvInv = InvCompleteTransform.InverseNoLooping();
		//TestEqual("Doubly inverted zero-timescale transform should be equal to original", InvInv, CompleteTransform);
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
