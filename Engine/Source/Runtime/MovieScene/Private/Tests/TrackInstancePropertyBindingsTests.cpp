// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "MovieSceneFwd.h"
#include "Misc/AutomationTest.h"
#include "MovieSceneTestObjects.h"
#include "TrackInstancePropertyBindings.h"

#if WITH_DEV_AUTOMATION_TESTS

#define UTEST_CACHED_BINDING(Message, Bindings, Object)\
	{\
		FTrackInstancePropertyBindings::FResolvedPropertyAndFunction PropAndFunction = Bindings.FindOrAdd(Object);\
		UTEST_EQUAL(Message, PropAndFunction.ResolvedProperty.IsType<FTrackInstancePropertyBindings::FCachedProperty>(), true);\
	}

#define UTEST_VOLATILE_BINDING(Message, Bindings, Object)\
	{\
		FTrackInstancePropertyBindings::FResolvedPropertyAndFunction PropAndFunction = Bindings.FindOrAdd(Object);\
		UTEST_EQUAL(Message, PropAndFunction.ResolvedProperty.IsType<FTrackInstancePropertyBindings::FVolatileProperty>(), true);\
	}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTrackInstancePropertyBindingsTests, 
		"System.Engine.Sequencer.TrackInstancePropertyBindings", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FTrackInstancePropertyBindingsTests::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene;

	// TestBool property
	{
		ATestMovieSceneArrayPropertiesActor* TestActor = NewObject<ATestMovieSceneArrayPropertiesActor>();

		FTrackInstancePropertyBindings Bindings(TEXT("TestBool"), TEXT("TestBool"));
		UTEST_EQUAL("", TestActor->TestBool, false);
		Bindings.CallFunction<bool>(*TestActor, true);
		UTEST_EQUAL("", TestActor->TestBool, true);
		UTEST_CACHED_BINDING("", Bindings, *TestActor);
	}

	// TestEnum property
	{
		ATestMovieSceneArrayPropertiesActor* TestActor = NewObject<ATestMovieSceneArrayPropertiesActor>();

		FTrackInstancePropertyBindings Bindings(TEXT("TestEnum"), TEXT("TestEnum"));
		UTEST_EQUAL("", TestActor->TestEnum, ETestMovieSceneEnum::One);
		Bindings.CallFunctionForEnum(*TestActor, (int64)ETestMovieSceneEnum::Two);
		UTEST_EQUAL("", TestActor->TestEnum, ETestMovieSceneEnum::Two);
		UTEST_CACHED_BINDING("", Bindings, *TestActor);
	}

	// TestInt32 property
	{
		ATestMovieSceneArrayPropertiesActor* TestActor = NewObject<ATestMovieSceneArrayPropertiesActor>();

		FTrackInstancePropertyBindings Bindings(TEXT("TestInt32"), TEXT("TestInt32"));
		UTEST_EQUAL("", TestActor->TestInt32, 0);
		Bindings.CallFunction<int32>(*TestActor, 42);
		UTEST_EQUAL("", TestActor->TestInt32, 42);
		UTEST_CACHED_BINDING("", Bindings, *TestActor);
	}

	// TestObject property
	{
		ATestMovieSceneArrayPropertiesActor* TestActor = NewObject<ATestMovieSceneArrayPropertiesActor>();

		UTestMovieSceneObject* TestObject = NewObject<UTestMovieSceneObject>();

		FTrackInstancePropertyBindings Bindings(TEXT("TestObject"), TEXT("TestObject"));
		UTEST_EQUAL("", TestActor->TestObject.Get(), (UTestMovieSceneObject*)nullptr);
		Bindings.CallFunction<UObject*>(*TestActor, (UObject*)TestObject);
		UTEST_EQUAL("", TestActor->TestObject.Get(), TestObject);
		UTEST_CACHED_BINDING("", Bindings, *TestActor);
	}

	// TestVector.X/Y property
	{
		ATestMovieSceneArrayPropertiesActor* TestActor = NewObject<ATestMovieSceneArrayPropertiesActor>();

		FTrackInstancePropertyBindings BindingsX(TEXT("X"), TEXT("TestVector.X"));
		UTEST_EQUAL("", TestActor->TestVector, FVector(0, 0, 0));
		BindingsX.CallFunction<double>(*TestActor, 1.0);
		UTEST_EQUAL("", TestActor->TestVector, FVector(1, 0, 0));
		UTEST_CACHED_BINDING("", BindingsX, *TestActor);

		FTrackInstancePropertyBindings BindingsY(TEXT("Y"), TEXT("TestVector.Y"));
		UTEST_EQUAL("", TestActor->TestVector, FVector(1, 0, 0));
		BindingsY.CallFunction<double>(*TestActor, 2.0);
		UTEST_EQUAL("", TestActor->TestVector, FVector(1, 2, 0));
		UTEST_CACHED_BINDING("", BindingsY, *TestActor);
	}

	// MultipleFloats property
	{
		ATestMovieSceneArrayPropertiesActor* TestActor = NewObject<ATestMovieSceneArrayPropertiesActor>();

		TestActor->MultipleFloats.Add(0);
		TestActor->MultipleFloats.Add(1);

		FTrackInstancePropertyBindings Bindings(TEXT("MultipleFloats[1]"), TEXT("MultipleFloats[1]"));
		UTEST_EQUAL("", TestActor->MultipleFloats[1], 1.f);
		Bindings.CallFunction<float>(*TestActor, 11.f);
		UTEST_EQUAL("", TestActor->MultipleFloats[1], 11.f);
		UTEST_VOLATILE_BINDING("", Bindings, *TestActor);
	}

	// SingleStruct.Second property
	{
		ATestMovieSceneArrayPropertiesActor* TestActor = NewObject<ATestMovieSceneArrayPropertiesActor>();

		FTrackInstancePropertyBindings Bindings(TEXT("Second"), TEXT("SingleStruct.Second"));
		UTEST_EQUAL("", TestActor->SingleStruct.Second, 0.f);
		Bindings.CallFunction<float>(*TestActor, 1.f);
		UTEST_EQUAL("", TestActor->SingleStruct.Second, 1.f);
		UTEST_CACHED_BINDING("", Bindings, *TestActor);
	}

	// SingleStruct.MultipleIntegers property
	{
		ATestMovieSceneArrayPropertiesActor* TestActor = NewObject<ATestMovieSceneArrayPropertiesActor>();

		TestActor->SingleStruct.MultipleIntegers.Add(2);
		TestActor->SingleStruct.MultipleIntegers.Add(3);

		FTrackInstancePropertyBindings Bindings(TEXT("MultipleIntegers[1]"), TEXT("SingleStruct.MultipleIntegers[1]"));
		UTEST_EQUAL("", TestActor->SingleStruct.MultipleIntegers[1], 3);
		Bindings.CallFunction<int32>(*TestActor, 4);
		UTEST_EQUAL("", TestActor->SingleStruct.MultipleIntegers[1], 4);
		UTEST_VOLATILE_BINDING("", Bindings, *TestActor);
	}

	// SingleStruct.MultipleVectors.X property
	{
		ATestMovieSceneArrayPropertiesActor* TestActor = NewObject<ATestMovieSceneArrayPropertiesActor>();

		TestActor->SingleStruct.MultipleVectors.Add(FVector(2, 3, 4));
		TestActor->SingleStruct.MultipleVectors.Add(FVector(5, 6, 7));

		FTrackInstancePropertyBindings Bindings(TEXT("X"), TEXT("SingleStruct.MultipleVectors[1].X"));
		UTEST_EQUAL("", TestActor->SingleStruct.MultipleVectors[1].X, 5.0);
		Bindings.CallFunction<double>(*TestActor, 10.0);
		UTEST_EQUAL("", TestActor->SingleStruct.MultipleVectors[1].X, 10.0);
		UTEST_VOLATILE_BINDING("", Bindings, *TestActor);
	}

	// MultipleStruct.Second property
	{
		ATestMovieSceneArrayPropertiesActor* TestActor = NewObject<ATestMovieSceneArrayPropertiesActor>();

		TestActor->MultipleStructs.Reserve(2);
		TestActor->MultipleStructs.Add(FTestMovieSceneStruct());
		TestActor->MultipleStructs.Add(FTestMovieSceneStruct{ 3.f, 5.f });

		FTrackInstancePropertyBindings Bindings(TEXT("Second"), TEXT("MultipleStructs[1].Second"));
		UTEST_EQUAL("", TestActor->MultipleStructs[1].Second, 5.f);
		Bindings.CallFunction<float>(*TestActor, 10.f);
		UTEST_EQUAL("", TestActor->MultipleStructs[1].Second, 10.f);
		UTEST_VOLATILE_BINDING("", Bindings, *TestActor);

		const FTrackInstancePropertyBindings::FResolvedPropertyAndFunction& PropAndFunction = Bindings.FindOrAdd(*TestActor);
		void* OldContainerAddress = PropAndFunction.GetContainerAddress();
		UTEST_EQUAL("", OldContainerAddress, (void*)&TestActor->MultipleStructs[1]);

		// Force re-allocating the array and check that we haven't cached the old addresses.
		TestActor->MultipleStructs.Reserve(4);
		UTEST_NOT_EQUAL("", OldContainerAddress, (void*)&TestActor->MultipleStructs[1]); 
		void* NewContainerAddress = PropAndFunction.GetContainerAddress();
		UTEST_EQUAL("", NewContainerAddress, (void*)&TestActor->MultipleStructs[1]);
	}

	// MultipleStruct.Enum property
	{
		ATestMovieSceneArrayPropertiesActor* TestActor = NewObject<ATestMovieSceneArrayPropertiesActor>();

		TestActor->MultipleStructs.Reserve(2);
		TestActor->MultipleStructs.Add(FTestMovieSceneStruct());
		TestActor->MultipleStructs.Add(FTestMovieSceneStruct());

		FTrackInstancePropertyBindings Bindings(TEXT("Enum"), TEXT("MultipleStructs[1].Enum"));
		UTEST_EQUAL("", TestActor->MultipleStructs[1].Enum, ETestMovieSceneEnum::One);
		Bindings.CallFunctionForEnum(*TestActor, (int64)ETestMovieSceneEnum::Two);
		UTEST_EQUAL("", TestActor->MultipleStructs[1].Enum, ETestMovieSceneEnum::Two);
		UTEST_VOLATILE_BINDING("", Bindings, *TestActor);

		const FTrackInstancePropertyBindings::FResolvedPropertyAndFunction& PropAndFunction = Bindings.FindOrAdd(*TestActor);
		void* OldContainerAddress = PropAndFunction.GetContainerAddress();
		UTEST_EQUAL("", OldContainerAddress, (void*)&TestActor->MultipleStructs[1]);

		// Force re-allocating the array and check that we haven't cached the old addresses.
		TestActor->MultipleStructs.Reserve(4);
		UTEST_NOT_EQUAL("", OldContainerAddress, (void*)&TestActor->MultipleStructs[1]); 
		void* NewContainerAddress = PropAndFunction.GetContainerAddress();
		UTEST_EQUAL("", NewContainerAddress, (void*)&TestActor->MultipleStructs[1]);
	}

	// MultipleStruct.MultipleIntegers property
	{
		ATestMovieSceneArrayPropertiesActor* TestActor = NewObject<ATestMovieSceneArrayPropertiesActor>();

		TestActor->MultipleStructs.Reserve(2);
		TestActor->MultipleStructs.Add(FTestMovieSceneStruct());
		TestActor->MultipleStructs.Add(FTestMovieSceneStruct());
		TestActor->MultipleStructs[1].MultipleIntegers.Add(7);
		TestActor->MultipleStructs[1].MultipleIntegers.Add(8);

		FTrackInstancePropertyBindings Bindings(TEXT("MultipleIntegers[1]"), TEXT("MultipleStructs[1].MultipleIntegers[1]"));
		UTEST_EQUAL("", TestActor->MultipleStructs[1].MultipleIntegers[1], 8);
		Bindings.CallFunction<int32>(*TestActor, 10);
		UTEST_EQUAL("", TestActor->MultipleStructs[1].MultipleIntegers[1], 10);
		UTEST_VOLATILE_BINDING("", Bindings, *TestActor);
	}

	// MultipleStruct.MultipleVectors.X property
	{
		ATestMovieSceneArrayPropertiesActor* TestActor = NewObject<ATestMovieSceneArrayPropertiesActor>();

		TestActor->MultipleStructs.Reserve(2);
		TestActor->MultipleStructs.Add(FTestMovieSceneStruct());
		TestActor->MultipleStructs.Add(FTestMovieSceneStruct{ 3.f, 5.f });
		TestActor->MultipleStructs[1].MultipleVectors.Add(FVector(11, 12, 13));
		TestActor->MultipleStructs[1].MultipleVectors.Add(FVector(14, 15, 16));

		FTrackInstancePropertyBindings Bindings(TEXT("X"), TEXT("MultipleStructs[1].MultipleVectors[1].X"));
		UTEST_EQUAL("", TestActor->MultipleStructs[1].MultipleVectors[1].X, 14.0);
		Bindings.CallFunction<double>(*TestActor, 20.0);
		UTEST_EQUAL("", TestActor->MultipleStructs[1].MultipleVectors[1].X, 20.0);
		UTEST_VOLATILE_BINDING("", Bindings, *TestActor);
	}

	// SingleInstancedStruct.Second property
	{
		ATestMovieSceneArrayPropertiesActor* TestActor = NewObject<ATestMovieSceneArrayPropertiesActor>();

		FTestMovieSceneStruct& TestStruct = TestActor->SingleInstancedStruct.InitializeAs<FTestMovieSceneStruct>();
		TestStruct.First = 2.f;
		TestStruct.Second = 4.f;

		FTrackInstancePropertyBindings Bindings(TEXT("Second"), TEXT("SingleInstancedStruct.Second"));
		UTEST_EQUAL("", TestStruct.Second, 4.f);
		Bindings.CallFunction<float>(*TestActor, 8.f);
		UTEST_EQUAL("", TestStruct.Second, 8.f);
		UTEST_VOLATILE_BINDING("", Bindings, *TestActor);

		FMovieSceneEasingSettings& Easing = TestActor->SingleInstancedStruct.InitializeAs<FMovieSceneEasingSettings>();
		Easing.AutoEaseInDuration = 5000;
		Easing.AutoEaseOutDuration = 5000;
		Bindings.CallFunction<float>(*TestActor, 8.f);  // The struct is different so it should now fail.
		// Everything else should be untouched (this checks for memory corruption)
		UTEST_EQUAL("", Easing.AutoEaseInDuration, 5000);
		UTEST_EQUAL("", Easing.AutoEaseOutDuration, 5000);
		UTEST_EQUAL("", Easing.EaseIn, TScriptInterface<IMovieSceneEasingFunction>());
		UTEST_EQUAL("", Easing.bManualEaseIn, false);
		UTEST_EQUAL("", Easing.ManualEaseInDuration, 0);
		UTEST_EQUAL("", Easing.EaseOut, TScriptInterface<IMovieSceneEasingFunction>());
		UTEST_EQUAL("", Easing.bManualEaseOut, false);
		UTEST_EQUAL("", Easing.ManualEaseOutDuration, 0);
	}

	// SingleInstancedStruct.Second property
	{
		ATestMovieSceneArrayPropertiesActor* TestActor = NewObject<ATestMovieSceneArrayPropertiesActor>();

		FTestMovieSceneStruct& TestStruct = TestActor->SingleInstancedStruct.InitializeAs<FTestMovieSceneStruct>();
		TestStruct.First = 2.f;
		TestStruct.Second = 4.f;

		FTrackInstancePropertyBindings Bindings(TEXT("Second"), TEXT("SingleInstancedStruct.Second"));
		UTEST_EQUAL("", TestStruct.Second, 4.f);
		Bindings.CallFunction<float>(*TestActor, 8.f);
		UTEST_EQUAL("", TestStruct.Second, 8.f);
		UTEST_VOLATILE_BINDING("", Bindings, *TestActor);

		FTestMovieSceneStruct2& TestStruct2 = TestActor->SingleInstancedStruct.InitializeAs<FTestMovieSceneStruct2>();
		TestStruct2.Second = 12.f;
		Bindings.CallFunction<float>(*TestActor, 18.f);  // The struct is different but duck typing should let us key this.
		UTEST_EQUAL("", TestStruct2.Second, 18.f);
		UTEST_VOLATILE_BINDING("", Bindings, *TestActor);
	}

	// SingleInstancedStruct.MultipleIntegers property
	{
		ATestMovieSceneArrayPropertiesActor* TestActor = NewObject<ATestMovieSceneArrayPropertiesActor>();

		FTestMovieSceneStruct& TestStruct = TestActor->SingleInstancedStruct.InitializeAs<FTestMovieSceneStruct>();
		TestStruct.MultipleIntegers.Add(3);
		TestStruct.MultipleIntegers.Add(4);

		FTrackInstancePropertyBindings Bindings(TEXT("MultipleIntegers[1]"), TEXT("SingleInstancedStruct.MultipleIntegers[1]"));
		UTEST_EQUAL("", TestStruct.MultipleIntegers[1], 4);
		Bindings.CallFunction<int32>(*TestActor, 8);
		UTEST_EQUAL("", TestStruct.MultipleIntegers[1], 8);
		UTEST_VOLATILE_BINDING("", Bindings, *TestActor);
	}

	// SingleInstancedStruct.MultipleVectors.X property
	{
		ATestMovieSceneArrayPropertiesActor* TestActor = NewObject<ATestMovieSceneArrayPropertiesActor>();

		FTestMovieSceneStruct& TestStruct = TestActor->SingleInstancedStruct.InitializeAs<FTestMovieSceneStruct>();
		TestStruct.MultipleVectors.Add(FVector(2, 3, 4));
		TestStruct.MultipleVectors.Add(FVector(5, 6, 7));

		FTrackInstancePropertyBindings Bindings(TEXT("X"), TEXT("SingleInstancedStruct.MultipleVectors[1].X"));
		UTEST_EQUAL("", TestStruct.MultipleVectors[1].X, 5.0);
		Bindings.CallFunction<double>(*TestActor, 10.0);
		UTEST_EQUAL("", TestStruct.MultipleVectors[1].X, 10.0);
		UTEST_VOLATILE_BINDING("", Bindings, *TestActor);
	}

	// MultipleInstancedStructs.Second property
	{
		ATestMovieSceneArrayPropertiesActor* TestActor = NewObject<ATestMovieSceneArrayPropertiesActor>();

		TestActor->MultipleInstancedStructs.Reserve(2);
		TestActor->MultipleInstancedStructs.Emplace();
		TestActor->MultipleInstancedStructs.Emplace();

		FTestMovieSceneStruct& TestStruct = TestActor->MultipleInstancedStructs[1].InitializeAs<FTestMovieSceneStruct>();
		TestStruct.First = 5.f;
		TestStruct.Second = 9.f;

		FTrackInstancePropertyBindings Bindings(TEXT("Second"), TEXT("MultipleInstancedStructs[1].Second"));
		UTEST_EQUAL("", TestStruct.Second, 9.f);
		Bindings.CallFunction<float>(*TestActor, 12.f);
		UTEST_EQUAL("", TestStruct.Second, 12.f);
		UTEST_VOLATILE_BINDING("", Bindings, *TestActor);
	}

	// MultipleInstancedStructs.MultipleIntegers property
	{
		ATestMovieSceneArrayPropertiesActor* TestActor = NewObject<ATestMovieSceneArrayPropertiesActor>();

		TestActor->MultipleInstancedStructs.Reserve(2);
		TestActor->MultipleInstancedStructs.Emplace();
		TestActor->MultipleInstancedStructs.Emplace();

		FTestMovieSceneStruct& TestStruct = TestActor->MultipleInstancedStructs[1].InitializeAs<FTestMovieSceneStruct>();
		TestStruct.MultipleIntegers.Add(8);
		TestStruct.MultipleIntegers.Add(9);

		FTrackInstancePropertyBindings Bindings(TEXT("MultipleIntegers[1]"), TEXT("MultipleInstancedStructs[1].MultipleIntegers[1]"));
		UTEST_EQUAL("", TestStruct.MultipleIntegers[1], 9);
		Bindings.CallFunction<int32>(*TestActor, 12);
		UTEST_EQUAL("", TestStruct.MultipleIntegers[1], 12);
		UTEST_VOLATILE_BINDING("", Bindings, *TestActor);
	}

	// MultipleInstancedStructs.MultipleVectors.Y property
	{
		ATestMovieSceneArrayPropertiesActor* TestActor = NewObject<ATestMovieSceneArrayPropertiesActor>();

		TestActor->MultipleInstancedStructs.Reserve(2);
		TestActor->MultipleInstancedStructs.Emplace();
		TestActor->MultipleInstancedStructs.Emplace();

		FTestMovieSceneStruct& TestStruct = TestActor->MultipleInstancedStructs[1].InitializeAs<FTestMovieSceneStruct>();
		TestStruct.MultipleVectors.Add(FVector(5, 4, 3));
		TestStruct.MultipleVectors.Add(FVector(2, 1, 0));

		FTrackInstancePropertyBindings Bindings(TEXT("Y"), TEXT("MultipleInstancedStructs[1].MultipleVectors[1].Y"));
		UTEST_EQUAL("", TestStruct.MultipleVectors[1].Y, 1.0);
		Bindings.CallFunction<double>(*TestActor, 11.0);
		UTEST_EQUAL("", TestStruct.MultipleVectors[1].Y, 11.0);
		UTEST_VOLATILE_BINDING("", Bindings, *TestActor);
	}

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS
