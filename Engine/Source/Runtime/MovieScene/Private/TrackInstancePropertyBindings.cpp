// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackInstancePropertyBindings.h"

#include "EntitySystem/MovieSceneIntermediatePropertyValue.h"
#include "MovieSceneFwd.h"
#include "String/ParseTokens.h"
#include "StructUtils/InstancedStruct.h"
#include "StructUtils/PropertyBag.h"

namespace UE::MovieScene
{

struct FPropertyResolutionStep
{
	FProperty* Property = nullptr;
	int32 ArrayIndex = INDEX_NONE;
	void* ContainerAddress = nullptr;
};

struct FPropertyResolutionState
{
	TArray<FPropertyResolutionStep> PropertySteps;
	bool bIsVolatile = false;
	bool bIsValid = true;

	const FPropertyResolutionStep* GetLastStep() const
	{
		if (!PropertySteps.IsEmpty())
		{
			return &PropertySteps.Last();
		}
		return nullptr;
	}
};

FPropertyResolutionStep FindProperty(void* BasePointer, UStruct* InStruct, FStringView PropertyName)
{
	FPropertyResolutionStep PropertyStep;
	PropertyStep.ContainerAddress = BasePointer;
	PropertyStep.Property = FindFProperty<FProperty>(InStruct, FName(PropertyName, FNAME_Find));
	return PropertyStep;
}

FPropertyResolutionStep FindPropertyAndArrayIndex(void* BasePointer, UStruct* InStruct, FStringView PropertyName)
{
	// Calculate the array index if possible.
	int32 ArrayIndex = -1;
	if (PropertyName.Len() > 0 && PropertyName[PropertyName.Len() - 1] == ']')
	{
		int32 OpenIndex = 0;
		if (PropertyName.FindLastChar('[', OpenIndex))
		{
			// We have a property name of the form "Foo[123]". Resolve the property itself ("Foo") and then
			// parse the array element index (123).
			FStringView TruncatedPropertyName(PropertyName.GetData(), OpenIndex);
			FPropertyResolutionStep PropertyStep = FindProperty(BasePointer, InStruct, TruncatedPropertyName);

			const int32 NumberLength = PropertyName.Len() - OpenIndex - 2;
			if (NumberLength > 0 && NumberLength <= 10)
			{
				TCHAR NumberBuffer[11];
				FMemory::Memzero(NumberBuffer);
				FMemory::Memcpy(NumberBuffer, &PropertyName[OpenIndex + 1], sizeof(TCHAR) * NumberLength);
				LexFromString(PropertyStep.ArrayIndex, NumberBuffer);
			}

			return PropertyStep;
		}
	}

	// No index found in this property name, just find the property normally.
	return FindProperty(BasePointer, InStruct, PropertyName);
}

void ResolvePropertyRecursive(void* BasePointer, UStruct* InStruct, TArrayView<const FStringView> InPropertyNames, uint32 Index, FPropertyResolutionState& OutResolutionState)
{
	// If we need to resovle a property on an instanced struct or property bag, we need to first dive into the correct struct.
	void* ActualBasePointer = BasePointer;
	UStruct* ActualStruct = InStruct;
	{
		static FProperty* const PropertyBagValueProperty = FInstancedPropertyBag::StaticStruct()->FindPropertyByName(TEXT("Value"));

		// Instanced structs are technically just a memory buffer with no real sub-properties, but
		// they do have sub-properties if we ask them about their "logical" struct type. Let's do that,
		// which makes it possible to animate the properties inside.
		if (InStruct == FInstancedStruct::StaticStruct())
		{
			// Instanced structs may be reallocated, flag this property path as volatile.
			OutResolutionState.bIsVolatile = true;

			FInstancedStruct* InstancedStruct = (FInstancedStruct*)BasePointer;
			const UScriptStruct* InstancedStructType = InstancedStruct->GetScriptStruct();
			uint8* InstancedStructMemory = InstancedStruct->GetMutableMemory();

			ActualBasePointer = (void*)InstancedStructMemory;
			ActualStruct = const_cast<UScriptStruct*>(InstancedStructType);
		}
		// As above but for a property bag, unless the property path is specifically targeting its inner 
		// instanced struct, in which case we don't need to do anything, we'll end up in the previous code
		// block on the next iteration.
		else if (InStruct->IsChildOf<FInstancedPropertyBag>() && InPropertyNames[Index] != PropertyBagValueProperty->GetFName())
		{
			// Property bags may be reallocated, flag this property path as volatile.
			OutResolutionState.bIsVolatile = true;

			FInstancedPropertyBag* PropertyBag = (FInstancedPropertyBag*)BasePointer;

			// Add a step for diving into the instanced struct, so we don't need to handle a special case later.
			OutResolutionState.PropertySteps.Add(FPropertyResolutionStep{ PropertyBagValueProperty, INDEX_NONE, (void*)PropertyBag });

			const UPropertyBag* PropertyBagType = PropertyBag->GetPropertyBagStruct();
			uint8* PropertyBagMemory = PropertyBag->GetMutableValue().GetMemory();

			ActualBasePointer = (void*)PropertyBagMemory;
			ActualStruct = const_cast<UPropertyBag*>(PropertyBagType);
		}
	}

	// Find the property on the given struct.
	const FPropertyResolutionStep PropertyStep = FindPropertyAndArrayIndex(ActualBasePointer, ActualStruct, InPropertyNames[Index]);
	if (!PropertyStep.Property)
	{
		OutResolutionState.bIsValid = false;
		return;
	}

	const bool bHasMoreSteps = InPropertyNames.IsValidIndex(Index + 1);

	if (PropertyStep.ArrayIndex != INDEX_NONE)
	{
		// We found that this segment of the property path reaches an element inside an array.
		if (FArrayProperty* ArrayProp = CastField<FArrayProperty>(PropertyStep.Property))
		{
			FScriptArrayHelper ArrayHelper(ArrayProp, ArrayProp->ContainerPtrToValuePtr<void>(ActualBasePointer));
			if (ArrayHelper.IsValidIndex(PropertyStep.ArrayIndex))
			{
				// Arrays may be resized, flag this property path as volatile.
				OutResolutionState.bIsVolatile = true;

				OutResolutionState.PropertySteps.Add(PropertyStep);

				FStructProperty* InnerStructProp = CastField<FStructProperty>(ArrayProp->Inner);
				// InnerStructProp is null for arrays of basic types like floats, integers, etc.
				if (InnerStructProp && bHasMoreSteps)
				{
					// Move the BasePointer to the array element and keep resolving the property path on it.
					void* ArrayElement = ArrayHelper.GetRawPtr(PropertyStep.ArrayIndex);
					ResolvePropertyRecursive(ArrayElement, InnerStructProp->Struct, InPropertyNames, Index + 1, OutResolutionState);
				}
				else
				{
					// The property path ends here (e.g. "Foo.Bar[1]").
					return;
				}
			}
			else
			{
				// The property path goes into an invalid index of the array.
				OutResolutionState.bIsValid = false;
			}
		}
		else
		{
			UE_LOGF(LogMovieScene, Error, "Mismatch in property evaluation. %ls is not of type: %ls", *PropertyStep.Property->GetName(), *FArrayProperty::StaticClass()->GetName());
			OutResolutionState.bIsValid = false;
		}
	}
	else if (FStructProperty* StructProp = CastField<FStructProperty>(PropertyStep.Property))
	{
		// This segment of the property path reaches a struct FProperty.
		OutResolutionState.PropertySteps.Add(PropertyStep);

		if (bHasMoreSteps)
		{
			// Move the BasePointer to the struct and keep resolving the property path on it.
			void* StructContainer = StructProp->ContainerPtrToValuePtr<void>(ActualBasePointer);
			ResolvePropertyRecursive(StructContainer, StructProp->Struct, InPropertyNames, Index + 1, OutResolutionState);
		}
		else
		{
			// We stop at the struct, probably because we can directly animate it, like a vector/rotator/etc.
			check(StructProp->GetName() == InPropertyNames[Index]);
		}
	}
	else
	{
		// This segment of the property path reaches something else... probably a final float/integer/etc.
		OutResolutionState.PropertySteps.Add(PropertyStep);
	}
}

void ResolveProperty(const UObject& InObject, FStringView PropertyPath, FPropertyResolutionState& OutResolutionState)
{
	using namespace UE::String;

	TArray<FStringView, TInlineAllocator<4>> PropertyNames;

	// Parse property paths into component parts separated by '.'
	auto ProcessToken = [&PropertyNames](FStringView Token) { PropertyNames.Emplace(Token); };
	ParseTokens(PropertyPath, TEXT('.'), TFunctionRef<void(FWideStringView)>(ProcessToken), EParseTokensOptions::IgnoreCase | EParseTokensOptions::SkipEmpty);
	
	if (IsValid(&InObject) && PropertyNames.Num() > 0)
	{
		ResolvePropertyRecursive((void*)&InObject, InObject.GetClass(), PropertyNames, 0, OutResolutionState);
	}
}

FVolatileProperty BuildVolatileProperty(const UObject* Object, FStringView PropertyPath, const FPropertyResolutionState& ResolutionState)
{
	FVolatileProperty VolatileProperty;
	VolatileProperty.RootContainer = Object;
	VolatileProperty.PropertyPath = PropertyPath;
	VolatileProperty.LeafProperty = ResolutionState.PropertySteps.Last().Property;

	void* BaseContainer = (void*)VolatileProperty.RootContainer;

	for (int32 StepIndex = 0; StepIndex < ResolutionState.PropertySteps.Num(); ++StepIndex)
	{
		const FPropertyResolutionStep& ResolutionStep(ResolutionState.PropertySteps[StepIndex]);

		// Here are some situations we need to handle. The property name in brackets is the current
		// resolution step.
		//
		// Obj.<Value> : add a fixed offset step
		//
		// Obj.<Foo>.Bar : don't do anything, wait until next step to add a fixed offset step
		//
		// Obj.<Value[1]> : add a fixed offset step, an array jump, and another fixed offset step
		//
		// Obj.<Foo[1]>.Bar : add a fixed offset step, an array jump, another fixed offset step, and 
		//		reset the base address to the array element.
		//
		// Obj.<InstancedStruct> : add a fixed offset step
		//
		// Obj.<InstancedStruct>.Foo : add a fixed offset step, an instanced struct property jump,
		//		and reset the base address to that property's value address.
		//
		// Obj.<InstancedStructs[1]> : add a fixed offset step, an array jump, and another fixed 
		//		offset step.
		//
		// Obj.<InstancedStructs[1]>.Foo : add a fixed offset step, an array jump, another fixed
		//		offset step, an instanced struct property jump, and reset the base address to that
		//		property's value address.
		//

		const bool bIsLastStep = (StepIndex == ResolutionState.PropertySteps.Num() - 1);

		bool bIsInstancedStruct = false;
		if (FStructProperty* StructProp = CastField<FStructProperty>(ResolutionStep.Property))
		{
			if (StructProp->Struct == FInstancedStruct::StaticStruct())
			{
				bIsInstancedStruct = true;
			}
		}
		else if (FArrayProperty* ArrayProp = CastField<FArrayProperty>(ResolutionStep.Property))
		{
			if (FStructProperty* InnerStructProp = CastField<FStructProperty>(ArrayProp->Inner))
			{
				if (InnerStructProp->Struct == FInstancedStruct::StaticStruct())
				{
					bIsInstancedStruct = true;
				}
			}
		}

		// If this is the last step, force-add an entry so that we can resolve the leaf container inside
		// the GetLeafContainerAddress function.
		if (bIsLastStep)
		{
			FVolatilePropertyStep LeafContainerStep;
			LeafContainerStep.SetContainerOffset((uintptr_t)ResolutionStep.ContainerAddress - (uintptr_t)BaseContainer);
			VolatileProperty.PropertySteps.Add(LeafContainerStep);

			// Set the leaf container step index, and make further offsets based off this leaf container.
			VolatileProperty.LeafContainerStepIndex = VolatileProperty.PropertySteps.Num() - 1;
			BaseContainer = ResolutionStep.ContainerAddress;
		}

		// If this is the last step before the end, or the step before jumping into an instanced struct 
		// or an array, add a step to jump to the value so far. Otherwise, continue to follow the property
		// path and we'll compute the offset later when needed.
		if (bIsLastStep || bIsInstancedStruct || ResolutionStep.ArrayIndex != INDEX_NONE)
		{
			void* ValuePtr = ResolutionStep.Property->ContainerPtrToValuePtr<void>(ResolutionStep.ContainerAddress);
			const uint32 ValueOffset = ((uintptr_t)ValuePtr - (uintptr_t)BaseContainer);
			if (ValueOffset > 0)
			{
				FVolatilePropertyStep ValueJumpStep;
				ValueJumpStep.SetContainerOffset(ValueOffset);
				VolatileProperty.PropertySteps.Add(ValueJumpStep);
			}
		}

		// If the current step is accessing a value in an array, we need to add some steps for that.
		if (ResolutionStep.ArrayIndex != INDEX_NONE)
		{
			// Add a step to jump into the array.
			FVolatilePropertyStep ArrayJumpStep;
			ArrayJumpStep.SetCheckArrayIndex(ResolutionStep.ArrayIndex);
			VolatileProperty.PropertySteps.Add(ArrayJumpStep);

			// Add a step to jump from the beginning of the array to the actual element.
			FArrayProperty* ArrayProp = CastFieldChecked<FArrayProperty>(ResolutionStep.Property);
			FScriptArrayHelper ArrayHelper(ArrayProp, ArrayProp->ContainerPtrToValuePtr<void>(ResolutionStep.ContainerAddress));
			void* ArrayDataPtr = ArrayHelper.GetRawPtr(0);
			void* RawElementPtr = ArrayHelper.GetRawPtr(ResolutionStep.ArrayIndex);
			FVolatilePropertyStep ElementJumpStep;
			ElementJumpStep.SetContainerOffset((uintptr_t)RawElementPtr - (uintptr_t)ArrayDataPtr);
			VolatileProperty.PropertySteps.Add(ElementJumpStep);

			// Reset the base container so that offsets are now calculated from the array element.
			BaseContainer = RawElementPtr;
		}

		// If the current step leads to an instanced struct and we have more steps after that, we need
		// to add a step for jumping into the struct's memory buffer and checking that the next step is
		// valid for that instanced struct's type.
		if (bIsInstancedStruct && !bIsLastStep)
		{
			const FPropertyResolutionStep& NextStep = ResolutionState.PropertySteps[StepIndex + 1];

			FVolatilePropertyStep InstancedStructStep;
			InstancedStructStep.SetCheckStruct(CastChecked<UScriptStruct>(NextStep.Property->GetOwnerStruct()));
			VolatileProperty.PropertySteps.Add(InstancedStructStep);

			// Also reset the base container so that offsets are now calculated from the instanced struct.
			BaseContainer = NextStep.ContainerAddress;
		}
	}

	ensure(VolatileProperty.LeafContainerStepIndex >= 0);

	return VolatileProperty;
}

void* FVolatilePropertyStep::ResolveAddress(void* ContainerAddress, bool& bNeedsRecaching) const
{
	switch (Data.GetIndex())
	{
		case 0:
			{
				// Just offset the pointer.
				const uint32 ContainerOffset = Data.Get<uint32>();
				return ((uint8*)ContainerAddress + ContainerOffset);
			}
		case 1:
			{
				// Jump into the array's memory if we know the index we will jump to next is valid
				// for this array.
				const int32 CheckArrayIndex = Data.Get<int32>();
				FScriptArray* ScriptArray = (FScriptArray*)ContainerAddress;
				if (ScriptArray->IsValidIndex(CheckArrayIndex))
				{
					return ScriptArray->GetData();
				}
				return nullptr;
			}
		case 2:
			{
				// Jump into the instanced struct's memory after checking that it's the type we
				// expect. If not, return null.
				FInstancedStruct* InstancedStruct = (FInstancedStruct*)ContainerAddress;
				const UScriptStruct* InstancedStructType = InstancedStruct->GetScriptStruct();
				const UScriptStruct* CheckStructType = Data.Get<TWeakObjectPtr<UScriptStruct>>().Get();
				if (ensure(CheckStructType) && CheckStructType == InstancedStructType)
				{
					return InstancedStruct->GetMutableMemory();
				}
				bNeedsRecaching = true;
				return nullptr;
			}
		default:
			{
				return nullptr;
			}
	}
}

void* FVolatileProperty::ResolvePropertySteps(bool bStopAtLeafStep) const
{
	bool bNeedsRecaching = false;
	void* Address = ResolvePropertyStepsImpl(bStopAtLeafStep, bNeedsRecaching);
	if (bNeedsRecaching && RootContainer && !PropertyPath.IsEmpty())
	{
		FPropertyResolutionState ResolutionState;
		ResolveProperty(*RootContainer, PropertyPath, ResolutionState);
		FVolatileProperty* MutableThis = const_cast<FVolatileProperty*>(this);
		if (ResolutionState.bIsValid)
		{
			// If the path resolves with the new object trail, let's accept that. This happens for example
			// with a new instanced struct that looks the same as the previous one, such as a re-generated 
			// property bag.
			FVolatileProperty NewVolatileProperty = BuildVolatileProperty(RootContainer, PropertyPath, ResolutionState);
			ensure(NewVolatileProperty.RootContainer == RootContainer);
			ensure(NewVolatileProperty.PropertyPath == PropertyPath);
			*MutableThis = MoveTemp(NewVolatileProperty);

			// Re-resolve.
			bNeedsRecaching = false;
			Address = ResolvePropertyStepsImpl(bStopAtLeafStep, bNeedsRecaching);
			ensure(!bNeedsRecaching);
		}
		else
		{
			// The path doesn't resolve anymore. Leave it failing, but disable its ability to re-resolve 
			// itself, so that we don't try to re-resolve it every time we call into it.
			MutableThis->PropertyPath.Reset();
		}
	}
	return Address;
}

void* FVolatileProperty::ResolvePropertyStepsImpl(bool bStopAtLeafStep, bool& bNeedsRecaching) const
{
	if (RootContainer)
	{
		void* CurAddress = (void*)RootContainer;
		const int32 NumSteps = (bStopAtLeafStep ? LeafContainerStepIndex + 1 : PropertySteps.Num());
		for (int32 Index = 0; Index < NumSteps; ++Index)
		{
			const FVolatilePropertyStep& PropertyStep = PropertySteps[Index];
			CurAddress = PropertyStep.ResolveAddress(CurAddress, bNeedsRecaching);
			if (CurAddress == nullptr || bNeedsRecaching)
			{
				break;
			}
		}
		return CurAddress;
	}
	return nullptr;
}

}  // namespace UE::MovieScene

TOptional<TPair<const FProperty*, UE::MovieScene::FSourcePropertyValue>> FTrackInstancePropertyBindings::StaticPropertyAndValue(const UObject* Object, FStringView InPropertyPath)
{
	using namespace UE::MovieScene;

	checkf(Object, TEXT("No object specified"));

	FPropertyResolutionState ResolutionState;
	UE::MovieScene::ResolveProperty(*Object, InPropertyPath, ResolutionState);

	if (ResolutionState.bIsValid)
	{
		if (const FPropertyResolutionStep* LastStep = ResolutionState.GetLastStep())
		{
			const FProperty* Property = LastStep->Property;
			if (ensure(Property && LastStep->ContainerAddress && LastStep->ArrayIndex == INDEX_NONE))
			{
				const void* PropertyAddress = Property->ContainerPtrToValuePtr<void>(LastStep->ContainerAddress);

				// Bool property values are stored in a bit field so using a straight cast of the pointer to get their value does not
				// work.  Instead use the actual property to get the correct value.
				if (const FBoolProperty* BoolProperty = CastField<const FBoolProperty>(Property))
				{
					return MakeTuple(Property, FSourcePropertyValue::FromValue(BoolProperty->GetPropertyValue(PropertyAddress)));
				}
				// Object properties might have various different types of storage, but we always expose them as a raw ptr
				else if (const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(Property))
				{
					return MakeTuple(Property, FSourcePropertyValue::FromValue(ObjectProperty->GetObjectPropertyValue(PropertyAddress)));
				}

				return MakeTuple(Property, FSourcePropertyValue::FromAddress(PropertyAddress, *Property));
			}
		}
	}

	return TOptional<TPair<const FProperty*, UE::MovieScene::FSourcePropertyValue>>();
}

TOptional<UE::MovieScene::FSourcePropertyValue> FTrackInstancePropertyBindings::StaticValue(const UObject* Object, FStringView InPropertyPath)
{
	TOptional<TPair<const FProperty*, UE::MovieScene::FSourcePropertyValue>> Result = StaticPropertyAndValue(Object, InPropertyPath);
	if (Result.IsSet())
	{
		return MoveTemp(Result->Value);
	}
	return TOptional<UE::MovieScene::FSourcePropertyValue>();
}

FProperty* FTrackInstancePropertyBindings::FResolvedPropertyAndFunction::GetValidProperty() const
{
	if (const FCachedProperty* CachedProperty = ResolvedProperty.TryGet<FCachedProperty>())
	{
		return CachedProperty->GetValidProperty();
	}
	else if (const FVolatileProperty* VolatileProperty = ResolvedProperty.TryGet<FVolatileProperty>())
	{
		return VolatileProperty->GetValidProperty();
	}
	else
	{
		return nullptr;
	}
}

void* FTrackInstancePropertyBindings::FResolvedPropertyAndFunction::GetContainerAddress() const
{
	if (const FCachedProperty* CachedProperty = ResolvedProperty.TryGet<FCachedProperty>())
	{
		return CachedProperty->ContainerAddress;
	}
	else if (const FVolatileProperty* VolatileProperty = ResolvedProperty.TryGet<FVolatileProperty>())
	{
		return VolatileProperty->GetLeafContainerAddress();
	}
	else
	{
		return nullptr;
	}
}

FTrackInstancePropertyBindings::FTrackInstancePropertyBindings(FName InPropertyName, const FString& InPropertyPath)
	: PropertyPath(InPropertyPath)
	, PropertyName(InPropertyName)
{
}

FProperty* FTrackInstancePropertyBindings::FindProperty(const UObject* Object, FStringView InPropertyPath)
{
	using namespace UE::MovieScene;
	FPropertyResolutionState ResolutionState;
	UE::MovieScene::ResolveProperty(*Object, InPropertyPath, ResolutionState);
	if (ResolutionState.bIsValid)
	{
		if (const FPropertyResolutionStep* LastStep = ResolutionState.GetLastStep())
		{
			return LastStep->Property;
		}
	}
	return nullptr;
}

FTrackInstancePropertyBindings::FResolvedPropertyAndFunction FTrackInstancePropertyBindings::FindPropertyAndFunction(const UObject* Object, FStringView InPropertyPath)
{
	using namespace UE::MovieScene;
	FPropertyResolutionState ResolutionState;
	UE::MovieScene::ResolveProperty(*Object, InPropertyPath, ResolutionState);

	if (!ResolutionState.bIsValid || ResolutionState.PropertySteps.IsEmpty())
	{
		return FResolvedPropertyAndFunction();
	}
	
	FResolvedPropertyAndFunction PropAndFunction;
	if (!ResolutionState.bIsVolatile)
	{
		// No volatility was found while resolving this property path. Just use the tail property and
		// container address, they should be fixed.
		const FPropertyResolutionStep& LastStep(ResolutionState.PropertySteps.Last());
		FCachedProperty CachedProperty{ LastStep.Property, LastStep.ContainerAddress, LastStep.ArrayIndex };
		PropAndFunction.ResolvedProperty = TVariant<FCachedProperty, FVolatileProperty>(TInPlaceType<FCachedProperty>(), CachedProperty);
	}
	else
	{
		// We found some volatility while resolving this property path, such as an array or an instanced
		// struct. Let's compress the steps as much as possible, keeping only the steps where we need to
		// jump into some other memory buffer.
		FVolatileProperty VolatileProperty = BuildVolatileProperty(Object, InPropertyPath, ResolutionState);
		PropAndFunction.ResolvedProperty = TVariant<FCachedProperty, FVolatileProperty>(TInPlaceType<FVolatileProperty>(), VolatileProperty);
	}
	return PropAndFunction;
}

const FTrackInstancePropertyBindings::FResolvedPropertyAndFunction& FTrackInstancePropertyBindings::FindOrAdd(const UObject& InObject)
{
	FObjectKey ObjectKey(&InObject);

	const FResolvedPropertyAndFunction* PropAndFunction = RuntimeObjectToFunctionMap.Find(ObjectKey);
	if (PropAndFunction && (
				PropAndFunction->SetterFunction.IsValid() ||
				PropAndFunction->GetValidProperty()))
	{
		return *PropAndFunction;
	}

	CacheBinding(InObject);
	return RuntimeObjectToFunctionMap.FindChecked(ObjectKey);
}

void FTrackInstancePropertyBindings::CallFunctionForEnum(UObject& InRuntimeObject, int64 PropertyValue)
{
	const FResolvedPropertyAndFunction& PropAndFunction = FindOrAdd(InRuntimeObject);

	FProperty* Property = PropAndFunction.GetValidProperty();
	if (Property && Property->HasSetter())
	{
		Property->CallSetter(&InRuntimeObject, &PropertyValue);
	}
	else if (UFunction* SetterFunction = PropAndFunction.SetterFunction.Get())
	{
		InvokeSetterFunction(&InRuntimeObject, SetterFunction, PropertyValue);
	}
	else if (Property)
	{
		if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
		{
			FNumericProperty* UnderlyingProperty = EnumProperty->GetUnderlyingProperty();
			void* ValueAddr = EnumProperty->ContainerPtrToValuePtr<void>(PropAndFunction.GetContainerAddress());
			UnderlyingProperty->SetIntPropertyValue(ValueAddr, PropertyValue);
		}
		else
		{
			UE_LOGF(LogMovieScene, Error, "Mismatch in property evaluation. %ls is not of type: %ls", *Property->GetName(), *FEnumProperty::StaticClass()->GetName());
		}
	}
}

void FTrackInstancePropertyBindings::CacheBinding(const UObject& Object)
{
	FResolvedPropertyAndFunction PropAndFunction = FindPropertyAndFunction(&Object, PropertyPath);
	{
		FString PropertyVarName = PropertyName.ToString();

		// If this is a bool property, strip off the 'b' so that the "Set" functions to be 
		// found are, for example, "SetHidden" instead of "SetbHidden"
		FProperty* Property = PropAndFunction.GetValidProperty();
		if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
		{
			PropertyVarName.RemoveFromStart("b", ESearchCase::CaseSensitive);
		}

		static const FString Set("Set");
		const FString FunctionString = Set + PropertyVarName;

		UFunction* SetterFunction = Object.FindFunction(FName(*FunctionString));
		if (SetterFunction && SetterFunction->NumParms >= 1)
		{
			PropAndFunction.SetterFunction = SetterFunction;
		}
	}

	RuntimeObjectToFunctionMap.Add(FObjectKey(&Object), PropAndFunction);
}

FProperty* FTrackInstancePropertyBindings::GetProperty(const UObject& Object)
{
	const FResolvedPropertyAndFunction& PropAndFunction = FindOrAdd(Object);
	return PropAndFunction.GetValidProperty();
}

bool FTrackInstancePropertyBindings::HasValidBinding(const UObject& Object)
{
	const FResolvedPropertyAndFunction& PropAndFunction = FindOrAdd(Object);
	return PropAndFunction.GetValidProperty() != nullptr;
}

const UStruct* FTrackInstancePropertyBindings::GetPropertyStruct(const UObject& Object)
{
	const FResolvedPropertyAndFunction& PropAndFunction = FindOrAdd(Object);

	if (FProperty* Property = PropAndFunction.GetValidProperty())
	{
		if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			return StructProperty->Struct;
		}
		return nullptr;
	}

	return nullptr;
}

int64 FTrackInstancePropertyBindings::GetCurrentValueForEnum(const UObject& Object)
{
	const FResolvedPropertyAndFunction& PropAndFunction = FindOrAdd(Object);

	FProperty* Property = PropAndFunction.GetValidProperty();

	if (Property)
	{
		if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
		{
			FNumericProperty* UnderlyingProperty = EnumProperty->GetUnderlyingProperty();
			void* ValueAddr = EnumProperty->ContainerPtrToValuePtr<void>(PropAndFunction.GetContainerAddress());
			int64 Result = UnderlyingProperty->GetSignedIntPropertyValue(ValueAddr);
			return Result;
		}
		else
		{
			UE_LOGF(LogMovieScene, Error, "Mismatch in property evaluation. %ls is not of type: %ls", *Property->GetName(), *FEnumProperty::StaticClass()->GetName());
		}
	}

	return 0;
}

// Explicit specializations for bools.

template<> void FTrackInstancePropertyBindings::CallFunction<bool>(UObject& InRuntimeObject, TCallTraits<bool>::ParamType PropertyValue)
{
	const FResolvedPropertyAndFunction& PropAndFunction = FindOrAdd(InRuntimeObject);

	FProperty* Property = PropAndFunction.GetValidProperty();

	if (Property && Property->HasSetter())
	{
		Property->CallSetter(&InRuntimeObject, &PropertyValue);
	}
	else if (UFunction* SetterFunction = PropAndFunction.SetterFunction.Get())
	{
		InvokeSetterFunction(&InRuntimeObject, SetterFunction, PropertyValue);
	}
	else if (Property)
	{
		if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
		{
			uint8* ValuePtr = BoolProperty->ContainerPtrToValuePtr<uint8>(PropAndFunction.GetContainerAddress());
			BoolProperty->SetPropertyValue(ValuePtr, PropertyValue);
		}
		else
		{
			UE_LOGF(LogMovieScene, Error, "Mismatch in property evaluation. %ls is not of type: %ls", *Property->GetName(), *FBoolProperty::StaticClass()->GetName());
		}
	}
}

template<> bool FTrackInstancePropertyBindings::TryGetPropertyValue<bool>(const FResolvedPropertyAndFunction& PropAndFunction, bool& OutValue)
{
	if (FProperty* Property = PropAndFunction.GetValidProperty())
	{
		if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
		{
			const uint8* ValuePtr = BoolProperty->ContainerPtrToValuePtr<uint8>(PropAndFunction.GetContainerAddress());
			OutValue = BoolProperty->GetPropertyValue(ValuePtr);
			return true;
		}
		else
		{
			UE_LOGF(LogMovieScene, Error, "Mismatch in property evaluation. %ls is not of type: %ls", *Property->GetName(), *FBoolProperty::StaticClass()->GetName());
		}
	}

	return false;
}

template<> void FTrackInstancePropertyBindings::SetCurrentValue<bool>(UObject& Object, TCallTraits<bool>::ParamType InValue)
{
	const FResolvedPropertyAndFunction& PropAndFunction = FindOrAdd(Object);

	if (FProperty* Property = PropAndFunction.GetValidProperty())
	{
		if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
		{
			uint8* ValuePtr = BoolProperty->ContainerPtrToValuePtr<uint8>(PropAndFunction.GetContainerAddress());
			BoolProperty->SetPropertyValue(ValuePtr, InValue);
		}
	}
}

// Explicit specializations for object pointers.

template<> void FTrackInstancePropertyBindings::CallFunction<UObject*>(UObject& InRuntimeObject, UObject* PropertyValue)
{
	const FResolvedPropertyAndFunction& PropAndFunction = FindOrAdd(InRuntimeObject);

	FProperty* Property = PropAndFunction.GetValidProperty();

	if (Property && Property->HasSetter())
	{
		Property->CallSetter(&InRuntimeObject, &PropertyValue);
	}
	else if (UFunction* SetterFunction = PropAndFunction.SetterFunction.Get())
	{
		InvokeSetterFunction(&InRuntimeObject, SetterFunction, PropertyValue);
	}
	else if (Property)
	{
		if (FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Property))
		{
			uint8* ValuePtr = ObjectProperty->ContainerPtrToValuePtr<uint8>(PropAndFunction.GetContainerAddress());
			ObjectProperty->SetObjectPropertyValue(ValuePtr, PropertyValue);
		}
		else
		{
			UE_LOGF(LogMovieScene, Error, "Mismatch in property evaluation. %ls is not of type: %ls", *Property->GetName(), *FObjectPropertyBase::StaticClass()->GetName());
		}
	}
}

template<> bool FTrackInstancePropertyBindings::TryGetPropertyValue<UObject*>(const FResolvedPropertyAndFunction& PropAndFunction, UObject*& OutValue)
{
	if (FProperty* Property = PropAndFunction.GetValidProperty())
	{
		if (FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(PropAndFunction.GetValidProperty()))
		{
			const uint8* ValuePtr = ObjectProperty->ContainerPtrToValuePtr<uint8>(PropAndFunction.GetContainerAddress());
			OutValue = ObjectProperty->GetObjectPropertyValue(ValuePtr);
			return true;
		}
		else
		{
			UE_LOGF(LogMovieScene, Error, "Mismatch in property evaluation. %ls is not of type: %ls", *Property->GetName(), *FObjectPropertyBase::StaticClass()->GetName());
		}
	}

	return false;
}

template<> void FTrackInstancePropertyBindings::SetCurrentValue<UObject*>(UObject& Object, UObject* InValue)
{
	const FResolvedPropertyAndFunction& PropAndFunction = FindOrAdd(Object);

	if (FProperty* Property = PropAndFunction.GetValidProperty())
	{
		if (FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Property))
		{
			uint8* ValuePtr = ObjectProperty->ContainerPtrToValuePtr<uint8>(PropAndFunction.GetContainerAddress());
			ObjectProperty->SetObjectPropertyValue(ValuePtr, InValue);
		}
	}
}

