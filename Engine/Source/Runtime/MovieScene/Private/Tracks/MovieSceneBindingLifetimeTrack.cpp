// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneBindingLifetimeTrack.h"
#include "MovieSceneCommonHelpers.h"
#include "Sections/MovieSceneBindingLifetimeSection.h"
#include "Serialization/ObjectReader.h"
#include "Serialization/ObjectWriter.h"
#include "MovieScene.h"
#include "MovieSceneTimeHelpers.h"
#include "EntitySystem/BuiltInComponentTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneBindingLifetimeTrack)

#define LOCTEXT_NAMESPACE "MovieSceneBindingLifetimeTrack"


/* UMovieSceneBindingLifetimeTrack structors
 *****************************************************************************/

UMovieSceneBindingLifetimeTrack::UMovieSceneBindingLifetimeTrack(const FObjectInitializer& Obj)
	: Super(Obj)
{
#if WITH_EDITORONLY_DATA
	TrackTint = FColor(26, 117, 49, 150);
	bSupportsConditions = false;
#endif

}


bool UMovieSceneBindingLifetimeTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneBindingLifetimeSection::StaticClass();
}

UMovieSceneSection* UMovieSceneBindingLifetimeTrack::CreateNewSection()
{
	return NewObject<UMovieSceneBindingLifetimeSection>(this, NAME_None, RF_Transactional);
}


bool UMovieSceneBindingLifetimeTrack::HasSection(const UMovieSceneSection& Section) const
{
	return Sections.ContainsByPredicate([&](const UMovieSceneSection* In) { return In == &Section; });
}


void UMovieSceneBindingLifetimeTrack::AddSection(UMovieSceneSection& Section)
{
	Sections.Add(&Section);
}


void UMovieSceneBindingLifetimeTrack::RemoveSection(UMovieSceneSection& Section)
{
	Sections.RemoveAll([&](const UMovieSceneSection* In) { return In == &Section; });
}

void UMovieSceneBindingLifetimeTrack::RemoveSectionAt(int32 SectionIndex)
{
	Sections.RemoveAt(SectionIndex);
}

void UMovieSceneBindingLifetimeTrack::RemoveAllAnimationData()
{
	Sections.Empty();
}


bool UMovieSceneBindingLifetimeTrack::IsEmpty() const
{
	return Sections.Num() == 0;
}


const TArray<UMovieSceneSection*>& UMovieSceneBindingLifetimeTrack::GetAllSections() const
{
	return Sections;
}

#if WITH_EDITORONLY_DATA

FText UMovieSceneBindingLifetimeTrack::GetDisplayName() const
{
	return LOCTEXT("TrackName", "Binding Lifetime");
}

#endif

void UMovieSceneBindingLifetimeTrack::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponentTypes = FBuiltInComponentTypes::Get();

		OutImportedEntity->AddBuilder(
			FEntityBuilder()
			.Add(BuiltInComponentTypes->GenericObjectBinding, Params.GetObjectBindingID())
			.AddConditional(BuiltInComponentTypes->BindingLifetime, FMovieSceneBindingLifetimeComponentData{ EMovieSceneBindingLifetimeState::Inactive }, Params.GetObjectBindingID().IsValid())
		);
}

TArray<FFrameNumberRange> UMovieSceneBindingLifetimeTrack::CalculateInverseLifetimeRange(const TArray<FFrameNumberRange>& Ranges)
{
	TArray<FFrameNumberRange> InverseLifetimeRange;
	InverseLifetimeRange.Add(FFrameNumberRange::All());
	// Iterate through our sections, removing them from the range

	auto RemoveRangeFromSet = [&](FFrameNumberRange SectionRange) {
		for (int32 Index = 0; Index < InverseLifetimeRange.Num(); ++Index)
		{
			const FFrameNumberRange& Current = InverseLifetimeRange[Index];
			if (Current.Overlaps(SectionRange))
			{
				// Special case that difference doesn't handle well
				if (!SectionRange.HasLowerBound() && !SectionRange.HasUpperBound())
				{
					InverseLifetimeRange.Reset();
					return;
				}

				TArray<FFrameNumberRange> SplitRanges = FFrameNumberRange::Difference(Current, SectionRange);
				for (const FFrameNumberRange& NewRange : SplitRanges)
				{
					// Splitting infinite ranges keeps an infinite range, which we don't want to keep
					if (!NewRange.HasLowerBound() && !NewRange.HasUpperBound())
					{
						continue;
					}

					InverseLifetimeRange.Add(NewRange);
				}
				InverseLifetimeRange.RemoveAtSwap(Index--);
			}
		}
	};

	TArray<TRange<FFrameNumber>> SectionRanges;
	for (const FFrameNumberRange& Range : Ranges)
	{
		if (Algo::AnyOf(SectionRanges, [&](const TRange<FFrameNumber> OtherRange) { return !OtherRange.IsDegenerate() && !Range.IsDegenerate() && OtherRange.Overlaps(Range); }))
		{
			// Lifetime Range sections have managed to overlap, which should not be allowed.
			ensure(false);
			InverseLifetimeRange.Reset();
			return InverseLifetimeRange;
		}
		SectionRanges.Add(Range);
		RemoveRangeFromSet(Range);
	}
	return InverseLifetimeRange;
}

bool UMovieSceneBindingLifetimeTrack::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	const FMovieSceneTrackEvaluationField& LocalEvaluationField = GetEvaluationField();
	TArray<FFrameNumberRange> Ranges;
	Algo::Transform(LocalEvaluationField.Entries, Ranges, [](const FMovieSceneTrackEvaluationFieldEntry& Entry) { return Entry.Range; });

	TArray<FFrameNumberRange> InverseLifetimeRange = CalculateInverseLifetimeRange(Ranges);
	// Add an entity for each section of the inverse range to define areas where our lifetime is inactive.
	for (const FFrameNumberRange& InverseRange : InverseLifetimeRange)
	{
		OutFieldBuilder->AddPersistentEntity(InverseRange, this, 0, OutFieldBuilder->AddMetaData(InMetaData));
	}

	for (const FMovieSceneTrackEvaluationFieldEntry& Entry : LocalEvaluationField.Entries)
	{
		UMovieSceneBindingLifetimeSection* BindingLifetimeSection = Cast<UMovieSceneBindingLifetimeSection>(Entry.Section);
		if (BindingLifetimeSection)
		{
			FFrameNumberRange SectionEffectiveRange = FFrameNumberRange::Intersection(EffectiveRange, Entry.Range);
			if (!SectionEffectiveRange.IsEmpty())
			{
				FMovieSceneEvaluationFieldEntityMetaData SectionMetaData = InMetaData;
				SectionMetaData.Flags = Entry.Flags;
				if (Entry.Section)
				{
					SectionMetaData.Condition = MovieSceneHelpers::GetSequenceCondition(this, Entry.Section, true);
				}
				BindingLifetimeSection->ExternalPopulateEvaluationField(SectionEffectiveRange, SectionMetaData, OutFieldBuilder);
			}
		}
	}

	return true;
}


#undef LOCTEXT_NAMESPACE



