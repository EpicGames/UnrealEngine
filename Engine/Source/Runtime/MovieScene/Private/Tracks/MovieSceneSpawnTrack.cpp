// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneSpawnTrack.h"
#include "MovieSceneCommonHelpers.h"
#include "Sections/MovieSceneBoolSection.h"
#include "Sections/MovieSceneSpawnSection.h"
#include "Evaluation/MovieSceneEvaluationTrack.h"
#include "Compilation/IMovieSceneTemplateGenerator.h"
#include "Serialization/ObjectReader.h"
#include "Serialization/ObjectWriter.h"
#include "MovieScene.h"
#include "MovieSceneTimeHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSpawnTrack)

#define LOCTEXT_NAMESPACE "MovieSceneSpawnTrack"


/* UMovieSceneSpawnTrack structors
 *****************************************************************************/

UMovieSceneSpawnTrack::UMovieSceneSpawnTrack(const FObjectInitializer& Obj)
	: Super(Obj)
{
#if WITH_EDITORONLY_DATA
	TrackTint = FColor(43, 43, 155, 65);
	bSupportsConditions = false;
#endif

}

void UMovieSceneSpawnTrack::PostLoad()
{
	TArray<uint8> Bytes;

	for (int32 Index = 0; Index < Sections.Num(); ++Index)
	{
		UMovieSceneBoolSection* BoolSection = ExactCast<UMovieSceneBoolSection>(Sections[Index]);
		if (BoolSection)
		{
			BoolSection->ConditionalPostLoad();
			Bytes.Reset();

			FObjectWriter(BoolSection, Bytes);
			UMovieSceneSpawnSection* NewSection = NewObject<UMovieSceneSpawnSection>(this, NAME_None, RF_Transactional);
			FObjectReader(NewSection, Bytes);

			Sections[Index] = NewSection;
		}
	}

	Super::PostLoad();
}

/* UMovieSceneTrack interface
 *****************************************************************************/

void UMovieSceneSpawnTrack::PopulateSpawnedRangeMask(const TRange<FFrameNumber>& InOverlap, TArray<TRange<FFrameNumber>, TInlineAllocator<1>>& OutRanges) const
{
	if (Sections.Num() > 1)
	{
		// Multiple sections - can't optimize this
		OutRanges.Add(InOverlap);
		return;
	}
	else if (Sections.Num() == 0 || !Sections[0]->IsActive())
	{
		// No active sections - mask out everything
		return;
	}

	const UMovieSceneSpawnSection* SpawnSection = Cast<const UMovieSceneSpawnSection>(Sections[0]);
	const FMovieSceneBoolChannel&  BoolCurve    = SpawnSection->GetChannel();
	const TRange<FFrameNumber>     SectionRange = SpawnSection->GetTrueRange();
	const TRange<FFrameNumber>     MaskedRange = TRange<FFrameNumber>::Intersection(InOverlap, SectionRange);
	if (MaskedRange.IsEmpty())
	{
		return;
	}

	// Only add the valid section ranges to the tree
	TArrayView<const FFrameNumber> Times  = BoolCurve.GetTimes();
	TArrayView<const bool>         Values = BoolCurve.GetValues();

	if (Times.Num() == 0)
	{
		if (BoolCurve.GetDefault().Get(false))
		{
			OutRanges.Add(MaskedRange);
		}
	}
	else
	{

		TRangeBound<FFrameNumber> StartBound = MaskedRange.GetLowerBound();

		// Find the effective key
		int32 Index = FMath::Min(StartBound.IsOpen() ? 0 : Algo::UpperBound(Times, UE::MovieScene::DiscreteInclusiveLower(StartBound)), Times.Num()-1);
		
		bool bIsSpawned = Values[StartBound.IsOpen() ? 0 : FMath::Max(0, Algo::UpperBound(Times, UE::MovieScene::DiscreteInclusiveLower(StartBound))-1)];
		for ( ; Index < Times.Num(); ++Index)
		{
			if (!MaskedRange.Contains(Times[Index]))
			{
				break;
			}

			if (bIsSpawned != Values[Index])
			{
				if (bIsSpawned)
				{
					TRange<FFrameNumber> LastRange(StartBound, TRangeBound<FFrameNumber>::Exclusive(Times[Index]));
					OutRanges.Add(LastRange);
				}

				bIsSpawned = Values[Index];

				if (bIsSpawned)
				{
					StartBound = TRangeBound<FFrameNumber>::Inclusive(Times[Index]);
				}
			}
		}

		TRange<FFrameNumber> TailRange(StartBound, MaskedRange.GetUpperBound());
		if (!TailRange.IsEmpty() && bIsSpawned)
		{
			OutRanges.Add(TailRange);
		}
	}
}

bool UMovieSceneSpawnTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneSpawnSection::StaticClass();
}

UMovieSceneSection* UMovieSceneSpawnTrack::CreateNewSection()
{
	return NewObject<UMovieSceneSpawnSection>(this, NAME_None, RF_Transactional);
}


bool UMovieSceneSpawnTrack::HasSection(const UMovieSceneSection& Section) const
{
	return Sections.ContainsByPredicate([&](const UMovieSceneSection* In){ return In == &Section; });
}


void UMovieSceneSpawnTrack::AddSection(UMovieSceneSection& Section)
{
	Sections.Add(&Section);
}


void UMovieSceneSpawnTrack::RemoveSection(UMovieSceneSection& Section)
{
	Sections.RemoveAll([&](const UMovieSceneSection* In) { return In == &Section; });
}

void UMovieSceneSpawnTrack::RemoveSectionAt(int32 SectionIndex)
{
	Sections.RemoveAt(SectionIndex);
}

void UMovieSceneSpawnTrack::RemoveAllAnimationData()
{
	Sections.Empty();
}


bool UMovieSceneSpawnTrack::IsEmpty() const
{
	return Sections.Num() == 0;
}


const TArray<UMovieSceneSection*>& UMovieSceneSpawnTrack::GetAllSections() const
{
	return Sections;
}

#if WITH_EDITOR

ECookOptimizationFlags UMovieSceneSpawnTrack::GetCookOptimizationFlags() const
{
	ECookOptimizationFlags CookOptimizationFlags = Super::GetCookOptimizationFlags();

	// Remove the object if the track is muted
	if (CookOptimizationFlags == ECookOptimizationFlags::RemoveTrack)
	{
		return ECookOptimizationFlags::RemoveObject;
	}

	return CookOptimizationFlags;
}

#endif

#if WITH_EDITORONLY_DATA

FText UMovieSceneSpawnTrack::GetDisplayName() const
{
	return LOCTEXT("TrackName", "Spawned");
}

#endif


#undef LOCTEXT_NAMESPACE

