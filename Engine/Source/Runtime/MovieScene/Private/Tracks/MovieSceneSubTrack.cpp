// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneSubTrack.h"
#include "Evaluation/MovieSceneEvaluationField.h"
#include "Evaluation/MovieSceneSequenceTransform.h"
#include "Evaluation/MovieSceneSequenceHierarchy.h"
#include "MovieSceneSequence.h"
#include "Sections/MovieSceneSubSection.h"
#include "MovieScene.h"
#include "Compilation/MovieSceneSegmentCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSubTrack)


#define LOCTEXT_NAMESPACE "MovieSceneSubTrack"


/* UMovieSceneSubTrack interface
 *****************************************************************************/
UMovieSceneSubTrack::UMovieSceneSubTrack( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	TrackTint = FColor(180, 0, 40, 65);
	RowHeight = 50;
#endif

	BuiltInTreePopulationMode = ETreePopulationMode::Blended;

	SupportedBlendTypes.Add(EMovieSceneBlendType::Absolute);
}

UMovieSceneSubSection* UMovieSceneSubTrack::AddSequenceOnRow(UMovieSceneSequence* Sequence, FFrameNumber StartTime, int32 Duration, int32 RowIndex)
{
	Modify();

	UMovieSceneSubSection* NewSection = CastChecked<UMovieSceneSubSection>(CreateNewSection());
	{
		NewSection->SetSequence(Sequence);
		NewSection->SetRange(TRange<FFrameNumber>(StartTime, StartTime + Duration));
	}

	// If no given row index, put it on the next available row
	if (RowIndex == INDEX_NONE)
	{
		RowIndex = 0;
		NewSection->SetRowIndex(RowIndex);
		while (NewSection->OverlapsWithSections(Sections) != nullptr)
		{
			++RowIndex;
			NewSection->SetRowIndex(RowIndex);
		}
	}

	NewSection->SetRowIndex(RowIndex);

	// If this overlaps with any sections, move out all the sections that are beyond this row
	if (NewSection->OverlapsWithSections(Sections))
	{
		for (UMovieSceneSection* OtherSection : Sections)
		{
			if (OtherSection != NewSection && OtherSection->GetRowIndex() >= RowIndex)
			{
				OtherSection->SetRowIndex(OtherSection->GetRowIndex()+1);
			}
		}
	}

	Sections.Add(NewSection);

#if WITH_EDITORONLY_DATA
	if (Sequence)
	{
		NewSection->TimecodeSource = Sequence->GetEarliestTimecodeSource();
	}
#endif

	return NewSection;
}

bool UMovieSceneSubTrack::ContainsSequence(const UMovieSceneSequence& Sequence, bool Recursively, const UMovieSceneSection* SectionToSkip) const
{
	UMovieSceneSequence* OuterSequence = GetTypedOuter<UMovieSceneSequence>();
	if (OuterSequence == &Sequence)
	{
		return true;
	}

	for (const auto& Section : Sections)
	{
		const auto SubSection = CastChecked<UMovieSceneSubSection>(Section);

		if (SubSection == SectionToSkip)
		{
			continue;
		}

		// is the section referencing the sequence?
		const UMovieSceneSequence* SubSequence = SubSection->GetSequence();

		if (SubSequence == nullptr)
		{
			continue;
		}

		if (SubSequence == &Sequence)
		{
			return true;
		}

		if (!Recursively)
		{
			continue;
		}

		// does the section have sub-tracks referencing the sequence?
		const UMovieScene* SubMovieScene = SubSequence->GetMovieScene();

		if (SubMovieScene == nullptr)
		{
			continue;
		}

		UMovieSceneSubTrack* SubSubTrack = SubMovieScene->FindTrack<UMovieSceneSubTrack>();

		if ((SubSubTrack != nullptr) && SubSubTrack->ContainsSequence(Sequence))
		{
			return true;
		}
	}

	return false;
}

TArray<UMovieSceneSection*, TInlineAllocator<4>> UMovieSceneSubTrack::FindAllSections(FFrameNumber Time) const
{
	TArray<UMovieSceneSection*, TInlineAllocator<4>> OverlappingSections;

	for (UMovieSceneSection* Section : Sections)
	{
		if (MovieSceneHelpers::IsSectionKeyable(Section) && Section->GetRange().Contains(Time))
		{
			OverlappingSections.Add(Section);
		}
	}

	Algo::Sort(OverlappingSections, MovieSceneHelpers::SortOverlappingSections);

	return OverlappingSections;
}

#if WITH_EDITOR

UMovieSceneSection* UMovieSceneSubTrack::FindSection(FFrameNumber Time) const
{
	TArray<UMovieSceneSection*, TInlineAllocator<4>> OverlappingSections = FindAllSections(Time);

	if (OverlappingSections.Num())
	{
		if (SectionToKey && OverlappingSections.Contains(SectionToKey))
		{
			return SectionToKey;
		}
		else
		{
			return OverlappingSections[0];
		}
	}

	return nullptr;
}

UMovieSceneSection* UMovieSceneSubTrack::FindOrExtendSection(FFrameNumber Time, float& OutWeight)
{
	return FindSection(Time);
}

UMovieSceneSection* UMovieSceneSubTrack::FindOrAddSection(FFrameNumber Time, bool& bSectionAdded)
{
	bSectionAdded = false;

	UMovieSceneSection* FoundSection = FindSection(Time);
	if (FoundSection)
	{
		return FoundSection;
	}

	// Add a new section that starts and ends at the same time
	UMovieSceneSection* NewSection = CreateNewSection();
	ensureAlwaysMsgf(NewSection->HasAnyFlags(RF_Transactional), TEXT("CreateNewSection must return an instance with RF_Transactional set! (pass RF_Transactional to NewObject)"));
	NewSection->SetFlags(RF_Transactional);
	NewSection->SetRange(TRange<FFrameNumber>::Inclusive(Time, Time));

	Sections.Add(NewSection);
	
	bSectionAdded = true;

	return NewSection;
}

#endif

/* UMovieSceneTrack interface
 *****************************************************************************/

void UMovieSceneSubTrack::AddSection(UMovieSceneSection& Section)
{
	if (Section.IsA<UMovieSceneSubSection>())
	{
		Sections.Add(&Section);
	}
}


bool UMovieSceneSubTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneSubSection::StaticClass();
}

UMovieSceneSection* UMovieSceneSubTrack::CreateNewSection()
{
	return NewObject<UMovieSceneSubSection>(this, NAME_None, RF_Transactional);
}


const TArray<UMovieSceneSection*>& UMovieSceneSubTrack::GetAllSections() const 
{
	return Sections;
}


bool UMovieSceneSubTrack::HasSection(const UMovieSceneSection& Section) const
{
	return Sections.Contains(&Section);
}


bool UMovieSceneSubTrack::IsEmpty() const
{
	return Sections.Num() == 0;
}


void UMovieSceneSubTrack::RemoveAllAnimationData()
{
	Sections.Empty();
}


void UMovieSceneSubTrack::RemoveSection(UMovieSceneSection& Section)
{
	Sections.Remove(&Section);
}

void UMovieSceneSubTrack::RemoveSectionAt(int32 SectionIndex)
{
	Sections.RemoveAt(SectionIndex);
}


bool UMovieSceneSubTrack::SupportsMultipleRows() const
{
	return true;
}

#if WITH_EDITORONLY_DATA
FText UMovieSceneSubTrack::GetDefaultDisplayName() const
{
	return LOCTEXT("TrackName", "Subsequences");
}

void UMovieSceneSubTrack::SetSectionToKey(UMovieSceneSection* Section)
{
	SectionToKey = Section;
}
#endif


#undef LOCTEXT_NAMESPACE

