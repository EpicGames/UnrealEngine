// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneTimeWarpTrack.h"
#include "Evaluation/MovieSceneSequenceTransform.h"
#include "MovieScene.h"
#include "Sections/MovieSceneTimeWarpSection.h"
#include "Variants/MovieSceneTimeWarpGetter.h"
#include "Decorations/IMovieSceneDecoration.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneTimeWarpTrack)


UMovieSceneTimeWarpTrack::UMovieSceneTimeWarpTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bSupportsConditions = false;

	// Timewarp should always exist at the top
	SortingOrder = -10000;
#endif
}

bool UMovieSceneTimeWarpTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneTimeWarpSection::StaticClass();
}

UMovieSceneSection* UMovieSceneTimeWarpTrack::CreateNewSection()
{
	return NewObject<UMovieSceneTimeWarpSection>(this, NAME_None, RF_Transactional);
}

void UMovieSceneTimeWarpTrack::OnAddedToMovieSceneImpl(UMovieScene* InMovieScene)
{
	UMovieSceneTimeWarpDecoration* TimeWarp = InMovieScene->GetOrCreateDecoration<UMovieSceneTimeWarpDecoration>();
	TimeWarp->AddTimeWarpSource(this);
}

void UMovieSceneTimeWarpTrack::OnRemovedFromMovieSceneImpl()
{
	UMovieScene* MovieScene = GetTypedOuter<UMovieScene>();
	if (MovieScene)
	{
		if (UMovieSceneTimeWarpDecoration* TimeWarp = MovieScene->FindDecoration<UMovieSceneTimeWarpDecoration>())
		{
			TimeWarp->RemoveTimeWarpSource(this);

			if (!TimeWarp->HasAnySources())
			{
				MovieScene->RemoveDecoration<UMovieSceneTimeWarpDecoration>();
			}
		}
	}
}

FMovieSceneNestedSequenceTransform UMovieSceneTimeWarpTrack::GenerateTimeWarpTransform()
{
	return GenerateTransform();
}

bool UMovieSceneTimeWarpTrack::IsTimeWarpActive() const
{
	return bIsActiveTimeWarp && !IsEvalDisabled();
}

void UMovieSceneTimeWarpTrack::SetIsTimeWarpActive(bool bInActive)
{
	bIsActiveTimeWarp = bInActive;
}

int32 UMovieSceneTimeWarpTrack::GetTimeWarpSortOrder() const
{
#if WITH_EDITORONLY_DATA
	return GetSortingOrder();
#else
	return 0;
#endif
}

FMovieSceneNestedSequenceTransform UMovieSceneTimeWarpTrack::GenerateTransform() const
{
	if (IsEvalDisabled())
	{
		return FMovieSceneNestedSequenceTransform();
	}

	for (UMovieSceneSection* Section : GetAllSections())
	{
		UMovieSceneTimeWarpSection* TimeWarpSection = Cast<UMovieSceneTimeWarpSection>(Section);
		if (TimeWarpSection && TimeWarpSection->IsActive() && !this->IsRowEvalDisabled(TimeWarpSection->GetRowIndex()))
		{
			return TimeWarpSection->GenerateTransform();
		}
	}

	return FMovieSceneNestedSequenceTransform();
}

void UMovieSceneTimeWarpTrack::RemoveAllAnimationData()
{
	Sections.Empty();
}

bool UMovieSceneTimeWarpTrack::HasSection(const UMovieSceneSection& Section) const
{
	return Sections.Contains(&Section);
}

void UMovieSceneTimeWarpTrack::AddSection(UMovieSceneSection& Section)
{
	Sections.AddUnique(&Section);
}

void UMovieSceneTimeWarpTrack::RemoveSection(UMovieSceneSection& Section)
{
	Sections.Remove(&Section);
}

void UMovieSceneTimeWarpTrack::RemoveSectionAt(int32 SectionIndex)
{
	Sections.RemoveAt(SectionIndex);
}

bool UMovieSceneTimeWarpTrack::IsEmpty() const
{
	return Sections.Num() == 0;
}

const TArray<UMovieSceneSection*>& UMovieSceneTimeWarpTrack::GetAllSections() const
{
	return Sections;
}

#if WITH_EDITORONLY_DATA
FText UMovieSceneTimeWarpTrack::GetDisplayName() const
{
	for (UMovieSceneSection* Section : Sections)
	{
		UMovieSceneTimeWarpSection* TimeWarpSection = Cast<UMovieSceneTimeWarpSection>(Section);
		UMovieSceneTimeWarpGetter* Custom = (TimeWarpSection && TimeWarpSection->TimeWarp.GetType() == EMovieSceneTimeWarpType::Custom) ? TimeWarpSection->TimeWarp.AsCustom() : nullptr;

		if (Custom)
		{
			return Custom->GetClass()->GetDisplayNameText();
		}
	}

	return NSLOCTEXT("MovieSceneTimeWarpTrack", "DefaultLabel", "Time Warp");
}

FText UMovieSceneTimeWarpTrack::GetDisplayNameToolTipText(const FMovieSceneLabelParams& LabelParams) const
{
	return NSLOCTEXT("MovieSceneTimeWarpTrack", "DefaultToolTip", "Controls the playback time warping for this sequence and all its subsequences. Does not affect audio or engine-wide time dilation.");
}

FName UMovieSceneTimeWarpTrack::GetTrackName() const
{
	return "Time Warp";
}
#endif
