// Copyright Epic Games, Inc. All Rights Reserved.

#include "Variants/MovieSceneNumericVariant.h"
#include "Variants/MovieSceneNumericVariantGetter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneNumericVariant)


bool operator==(const FMovieSceneNumericVariant& A, const FMovieSceneNumericVariant& B)
{
	if (A.IsLiteral() != B.IsLiteral())
	{
		return false;
	}
	else if (A.IsLiteral())
	{
		return *reinterpret_cast<const double*>(A.Data) == *reinterpret_cast<const double*>(B.Data);
	}
	else if (A.GetTypeBits() == B.GetTypeBits())
	{
		if (A.IsCustomPtr())
		{
			// Deep compare if possible.
			UMovieSceneNumericVariantGetter* PtrA = A.GetCustomPtr();
			UMovieSceneNumericVariantGetter* PtrB = B.GetCustomPtr();
			if (PtrA == PtrB)
			{
				return true;
			}

			if (PtrA && PtrB)
			{
				UClass* Class = PtrA->GetClass();
				if (Class == PtrB->GetClass())
				{
					for (TFieldIterator<FProperty> It(Class); It; ++It)
					{
						for (int32 i=0; i<It->ArrayDim; i++)
						{
							if (!It->Identical_InContainer(PtrA,PtrB,i,PPF_None))
							{
								return false;
							}
						}
					}
					return PtrA->AreNativePropertiesIdenticalTo(PtrB);
				}
			}
		}
		else
		{
			// Bitwise compare
			return 
				(*reinterpret_cast<const uint64*>(A.Data) & FMovieSceneNumericVariant::PAYLOAD_Bits) == 
				(*reinterpret_cast<const uint64*>(B.Data) & FMovieSceneNumericVariant::PAYLOAD_Bits);
		}
	}
	return false;
}

bool operator!=(const FMovieSceneNumericVariant& A, const FMovieSceneNumericVariant& B)
{
	return !(A == B);
}

FMovieSceneNumericVariant::FMovieSceneNumericVariant()
{
	Set(0.0);
}

FMovieSceneNumericVariant::FMovieSceneNumericVariant(double InValue)
{
	Set(InValue);
}

FMovieSceneNumericVariant::FMovieSceneNumericVariant(UMovieSceneNumericVariantGetter* InGetter)
{
	Set(InGetter);
}

FMovieSceneNumericVariant FMovieSceneNumericVariant::ShallowCopy() const
{
	// Bitwise copy
	FMovieSceneNumericVariant New(NoInit);
	FMemory::Memcpy(New.Data, this->Data, sizeof(Data));

#if UE_MOVIESCENE_WEAKNUMERICVARIANT_CHECKS
	New.WeakCustomGetter = this->WeakCustomGetter;
#endif
	return New;
}

FMovieSceneNumericVariant FMovieSceneNumericVariant::DeepCopy(UObject* NewOuter) const
{
	if (this->IsCustomPtr())
	{
		UMovieSceneNumericVariantGetter* Getter = this->GetCustomPtr();
		if (Getter)
		{
			Getter = DuplicateObject(Getter, NewOuter);
		}

		FMovieSceneNumericVariant New(Getter);

		if (this->HasCustomWeakPtrFlag())
		{
			New.MakeWeakUnsafe();
		}

		return New;
	}

	return ShallowCopy();
}

void FMovieSceneNumericVariant::Set(double InLiteralValue)
{
	*reinterpret_cast<double*>(Data) = InLiteralValue;

#if UE_MOVIESCENE_WEAKNUMERICVARIANT_CHECKS
	WeakCustomGetter = nullptr;
#endif
}

void FMovieSceneNumericVariant::Set(UMovieSceneNumericVariantGetter* InDynamicValue)
{
	if (InDynamicValue)
	{
		InDynamicValue->ReferenceToSelf = InDynamicValue;
	}

	uint64 NewValue = reinterpret_cast<const uint64>(InDynamicValue);
	checkf((NewValue & ~PAYLOAD_Bits) == 0, TEXT("Unable to store a pointer outside of a 48 bit address space in this container"));
	*reinterpret_cast<uint64*>(Data) = NewValue | TAGGED_Bits | TYPE_CustomPtr;
}

void FMovieSceneNumericVariant::SetWeakUnsafe(UMovieSceneNumericVariantGetter* InDynamicValue)
{
	Set(InDynamicValue);
	MakeWeakUnsafe();
}


void FMovieSceneNumericVariant::MakeWeakUnsafe()
{
	if (IsCustomPtr())
	{
#if UE_MOVIESCENE_WEAKNUMERICVARIANT_CHECKS
		WeakCustomGetter = GetCustomPtr();
#endif
		*reinterpret_cast<uint64*>(Data) |= CUSTOMPTR_Weak;
	}
}

double FMovieSceneNumericVariant::Get() const
{
	if (IsLiteral())
	{
		return GetLiteral();
	}

	if (IsCustomPtr())
	{
		UMovieSceneNumericVariantGetter* Getter = GetCustomPtr();
		if (Getter)
		{
			return Getter->GetValue();
		}
	}

	return 0.0;
}

void FMovieSceneNumericVariant::SetTypeBits(uint8 InType)
{
	check(!IsLiteral() && (InType & 0x7) == InType); // Must fit into the type mask

	const uint64 TypeMask = uint64(InType & 0x7) << 48;
	*reinterpret_cast<uint64*>(Data) |= (TypeMask & TYPE_Bits);

#if UE_MOVIESCENE_WEAKNUMERICVARIANT_CHECKS
	WeakCustomGetter = nullptr;
#endif
}

uint8 FMovieSceneNumericVariant::GetTypeBits() const
{
	check(!IsLiteral());

	const uint64 Type = (*reinterpret_cast<const uint64*>(Data)) & TYPE_Bits;
	return uint8(Type >> 48);
}

bool FMovieSceneNumericVariant::HasCustomWeakPtrFlag() const
{
	check(!IsLiteral() && (GetTypeBits() == TYPE_CustomPtr));
	return (*reinterpret_cast<const uint64*>(Data) & CUSTOMPTR_Weak) != 0ull;
}

UMovieSceneNumericVariantGetter* FMovieSceneNumericVariant::GetCustomPtr() const
{
	check(IsCustomPtr());

	const uint64 PtrValue = (*reinterpret_cast<const uint64*>(Data) & PAYLOAD_Bits) & ~CUSTOMPTR_FlagBits;

	UMovieSceneNumericVariantGetter* Ptr = reinterpret_cast<UMovieSceneNumericVariantGetter*>(PtrValue);

#if UE_MOVIESCENE_WEAKNUMERICVARIANT_CHECKS
	if (Ptr && HasCustomWeakPtrFlag() && !ensureMsgf(WeakCustomGetter.Get() != nullptr, TEXT("Weakly referenced getter stored in FMovieSceneNumericVariant has been destroyed!")))
	{
		return nullptr;
	}
#endif

	return Ptr;
}

bool FMovieSceneNumericVariant::Serialize(FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		SerializeCustom(Ar, [this](FArchive& Ar, uint8& TypeBits, void* InOutData){

			uint8 Type = 0;
			Ar << Type;

			if (Type == 0)
			{
				UMovieSceneNumericVariantGetter* Getter = nullptr;
				Ar << Getter;

				Set(Getter);
			}
			else
			{
				uint8 Payload[6];
				Ar.Serialize(Payload, sizeof(Payload));
				FMemory::Memcpy(InOutData, Payload, sizeof(Payload));
				TypeBits = Type;
			}
		});
	}
	else
	{
		SerializeCustom(Ar, [this](FArchive& Ar, uint8& TypeBits, void* InOutData){
			uint8 Type = GetTypeBits();

			if (Ar.GetArchiveState().IsSaving())
			{
				Ar << Type;
			}

			if (Type == 0)
			{
				UMovieSceneNumericVariantGetter* Getter = GetCustomPtr();
				Ar << Getter;
			}
			else
			{
				uint8 Payload[6];
				FMemory::Memcpy(Payload, InOutData, sizeof(Payload));
				Ar.Serialize(Payload, sizeof(Payload));
			}
		});
	}

	return true;
}

bool FMovieSceneNumericVariant::SerializeCustom(FArchive& Ar, TFunctionRef<void(FArchive&, uint8&, void*)> InCustomSerializer)
{
	if (Ar.IsLoading())
	{
		bool bIsLiteral = true;
		Ar << bIsLiteral;

		if (bIsLiteral)
		{
			Ar << GetLiteralRef();
		}
		else
		{
			// First off, initialize this type to be a custom type specified by the tagged bits
			uint64* Value = reinterpret_cast<uint64*>(Data);
			*Value = TAGGED_Bits;

			// Pass the type bits through to the serializer
			uint8 TypeBits = 0;
			InCustomSerializer(Ar, TypeBits, Data);

			// Assign type bits
			SetTypeBits(TypeBits);
		}
	}
	else
	{
		bool bIsLiteral = IsLiteral();

		if (Ar.GetArchiveState().IsSaving())
		{
			Ar << bIsLiteral;
		}

		if (bIsLiteral)
		{
			Ar << GetLiteralRef();
		}
		else
		{
			uint8 TypeBits = GetTypeBits();
			InCustomSerializer(Ar, TypeBits, Data);
		}
	}
	return true;
}

bool FMovieSceneNumericVariant::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == NAME_DoubleProperty)
	{
		double Value = 0.0;
		Slot << Value;
		Set(Value);
		return true;
	}
	if (Tag.Type == NAME_FloatProperty)
	{
		float Value = 0.f;
		Slot << Value;
		Set(Value);
		return true;
	}

	// int64 and uint64 are not supported in this variant without loss of precision

	if (Tag.Type == NAME_ByteProperty)
	{
		int32 Value = 0;
		Slot << Value;
		Set(Value);
		return true;
	}
	if (Tag.Type == NAME_Int32Property || Tag.Type == NAME_IntProperty)
	{
		int32 Value = 0;
		Slot << Value;
		Set(Value);
		return true;
	}
	if (Tag.Type == NAME_Int16Property)
	{
		int16 Value = 0;
		Slot << Value;
		Set(Value);
		return true;
	}
	if (Tag.Type == NAME_Int8Property)
	{
		int8 Value = 0;
		Slot << Value;
		Set(Value);
		return true;
	}

	if (Tag.Type == NAME_UInt32Property)
	{
		uint32 Value = 0;
		Slot << Value;
		Set(Value);
		return true;
	}
	if (Tag.Type == NAME_UInt16Property)
	{
		uint16 Value = 0;
		Slot << Value;
		Set(Value);
		return true;
	}
	if (Tag.Type == NAME_ByteProperty)
	{
		uint8 Value = 0;
		Slot << Value;
		Set(Value);
		return true;
	}

	return false;
}

bool FMovieSceneNumericVariant::Identical(const FMovieSceneNumericVariant* Other, uint32 PortFlags) const
{
	return *this == *Other;
}

void FMovieSceneNumericVariant::AddStructReferencedObjects(FReferenceCollector& Collector)
{
	if (!IsLiteral() && GetTypeBits() == TYPE_CustomPtr)
	{
		UMovieSceneNumericVariantGetter* Getter = GetCustomPtr();

		const bool bReportReference = !HasCustomWeakPtrFlag();
		if (bReportReference && Getter)
		{
			const FProperty* Property = Collector.GetSerializedProperty();
			Collector.AddReferencedObject(Getter->ReferenceToSelf, nullptr, Property);
		}
	}
}

bool FMovieSceneNumericVariant::ExportTextItem(FString& ValueStr, const FMovieSceneNumericVariant& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	return false;
}

bool FMovieSceneNumericVariant::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText, FArchive* InSerializingArchive)
{
	return false;
}

void FMovieSceneNumericVariant::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	
}
