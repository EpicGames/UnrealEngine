// Copyright Epic Games, Inc. All Rights Reserved.

#include "Variants/MovieScenePlayRateCurve.h"
#include "Variants/MovieSceneTimeWarpVariant.h"
#include "Variants/MovieSceneTimeWarpVariantPayloads.h"
#include "Channels/MovieScenePiecewiseCurveUtils.inl"
#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneInterpolation.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "MovieSceneTimeHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScenePlayRateCurve)


UMovieScenePlayRateCurve::UMovieScenePlayRateCurve()
{
	PlayRate.Domain = UE::MovieScene::ETimeWarpChannelDomain::PlayRate;

	OnSignatureChanged().AddUObject(this, &UMovieScenePlayRateCurve::InvalidateTimeWarp);
}

#if WITH_EDITOR
bool UMovieScenePlayRateCurve::Modify(bool bAlwaysMarkDirty)
{
	InvalidateTimeWarp();
	return Super::Modify(bAlwaysMarkDirty);
}
#endif

void UMovieScenePlayRateCurve::InitializeDefaults(UMovieSceneSection* Section)
{
	using namespace UE::MovieScene;

	PlayRate.SetDefault(1.0);
}

void UMovieScenePlayRateCurve::PostLoad()
{
	Super::PostLoad();

	// Migrate the legacy bManualPlaybackStart bool to AnchorMode. Order-independent: this does not depend
	// on the curve's outer chain, so it doesn't matter whether UMovieSceneTimeWarpSection::PostLoad has
	// already reparented the getter onto the sequence by the time we run.
	if (bManualPlaybackStart && AnchorMode == EMovieScenePlayRateCurveAnchorMode::SectionRelative)
	{
		AnchorMode = EMovieScenePlayRateCurveAnchorMode::Manual;
		bManualPlaybackStart = false;
	}
}


void UMovieScenePlayRateCurve::InvalidateTimeWarp()
{
	bUpToDate = false;
}

const UE::MovieScene::FPiecewiseCurve& UMovieScenePlayRateCurve::GetTimeWarpCurve() const
{
	if (!bUpToDate)
	{
		UMovieSceneSequence* Sequence  = GetTypedOuter<UMovieSceneSequence>();
		UMovieScene*        MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;

		IntegratedTimeWarp = PlayRate.AsPiecewiseCurve().Integral();

		FFrameTime IntegralStartTime = PlaybackStartFrame;
		if (AnchorMode == EMovieScenePlayRateCurveAnchorMode::SequencePlaybackStart && MovieScene)
		{
			IntegralStartTime = MovieScene->GetPlaybackRange().GetLowerBoundValue();
		}

		// Make the integral curve relative to the play start
		double IntegralOffset = 0.0;
		if (IntegratedTimeWarp.Evaluate(IntegralStartTime, IntegralOffset))
		{
			IntegratedTimeWarp.Offset(-IntegralOffset);
		}

		bUpToDate = true;
	}

	return IntegratedTimeWarp;
}



EMovieSceneChannelProxyType UMovieScenePlayRateCurve::PopulateChannelProxy(FMovieSceneChannelProxyData& OutProxyData, EAllowTopLevelChannels AllowTopLevel)
{
#if WITH_EDITOR
	FMovieSceneChannelMetaData ChannelMetaData;
	ChannelMetaData.Name = "PlayRate";
	ChannelMetaData.bCanCollapseToTrack = (AllowTopLevel == EAllowTopLevelChannels::Yes);
	ChannelMetaData.DisplayText = NSLOCTEXT("MovieScenePlayRateCurve", "PlayRateCurve_Label", "Play Rate");
	ChannelMetaData.WeakOwningObject = this;
	ChannelMetaData.bRelativeToSection = true;
	ChannelMetaData.bSortEmptyGroupsLast = false;
	ChannelMetaData.SortOrder = 0;
	OutProxyData.Add(PlayRate, ChannelMetaData);

#else
	OutProxyData.Add(PlayRate);
#endif

	return EMovieSceneChannelProxyType::Static;
}

bool UMovieScenePlayRateCurve::DeleteChannel(FMovieSceneTimeWarpVariant& OutVariant, FName ChannelName)
{
	if (ChannelName == "PlayRate")
	{
		OutVariant.Set(1.0);
		return true;
	}
	return false;
}

TRange<FFrameTime> UMovieScenePlayRateCurve::ComputeTraversedHull(const TRange<FFrameTime>& Range) const
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Interpolation;

	const FPiecewiseCurve& TimeWarp = GetTimeWarpCurve();

	TRange<FFrameTime> Result = Range;

	if (TimeWarp.Values.Num() == 0)
	{
		return Result;
	}

	FFrameTime StartTime = Range.GetLowerBound().IsOpen() ? FFrameTime(std::numeric_limits<int32>::lowest()) : Range.GetLowerBoundValue();
	FFrameTime EndTime   = Range.GetUpperBound().IsOpen() ? FFrameTime(std::numeric_limits<int32>::max())    : Range.GetUpperBoundValue();

	FInterpolationExtents Extents = ComputePiecewiseExtents(FPiecewiseCurveData{&TimeWarp}, StartTime, EndTime);
	if (Extents.MinValue > Extents.MaxValue)
	{
		return Result;
	}

	check(Extents.MinValue <= Extents.MaxValue);

	// Maintain bound exclusivity if possible
	if (Result.GetLowerBound().IsOpen())
	{
		Result.SetLowerBound(TRangeBound<FFrameTime>::Inclusive(FFrameTime::FromDecimal(Extents.MinValue))); 
	}
	else
	{
		Result.SetLowerBoundValue(FFrameTime::FromDecimal(Extents.MinValue));
	}

	if (Result.GetUpperBound().IsOpen())
	{
		Result.SetUpperBound(TRangeBound<FFrameTime>::Inclusive(FFrameTime::FromDecimal(Extents.MaxValue))); 
	}
	else
	{
		Result.SetUpperBoundValue(FFrameTime::FromDecimal(Extents.MaxValue));
	}

	return Result;
}

FFrameTime UMovieScenePlayRateCurve::RemapTime(FFrameTime In) const
{
	double OutValue = 0.0;
	GetTimeWarpCurve().Evaluate(In, OutValue);
	return FFrameTime::FromDecimal(OutValue);
}

TOptional<FFrameTime> UMovieScenePlayRateCurve::InverseRemapTimeCycled(FFrameTime InValue, FFrameTime InTimeHint, const UE::MovieScene::FInverseTransformTimeParams& Params) const
{
	return GetTimeWarpCurve().InverseEvaluate(InValue.AsDecimal(), InTimeHint, Params.Flags);
}

bool UMovieScenePlayRateCurve::InverseRemapTimeWithinRange(FFrameTime InTime, FFrameTime RangeStart, FFrameTime RangeEnd, const TFunctionRef<bool(FFrameTime)>& VisitorCallback) const
{
	return GetTimeWarpCurve().InverseEvaluateBetween(InTime.AsDecimal(), RangeStart, RangeEnd, VisitorCallback);
}

void UMovieScenePlayRateCurve::ScaleBy(double UnwarpedScaleFactor)
{
	Modify();
	Dilate(&PlayRate, 0, UnwarpedScaleFactor);
	bUpToDate = false;
}

UE::MovieScene::ETimeWarpChannelDomain UMovieScenePlayRateCurve::GetDomain() const
{
	return UE::MovieScene::ETimeWarpChannelDomain::PlayRate;
}

