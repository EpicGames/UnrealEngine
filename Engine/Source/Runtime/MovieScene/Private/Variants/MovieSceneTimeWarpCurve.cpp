// Copyright Epic Games, Inc. All Rights Reserved.

#include "Variants/MovieSceneTimeWarpCurve.h"
#include "Variants/MovieSceneTimeWarpVariant.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneInterpolation.h"
#include "MovieSceneTransformTypes.h"
#include "MovieSceneTimeHelpers.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneTimeWarpCurve)


UMovieSceneTimeWarpCurve::UMovieSceneTimeWarpCurve()
{
	Channel.Domain = UE::MovieScene::ETimeWarpChannelDomain::Time;
}

void UMovieSceneTimeWarpCurve::InitializeDefaults(UMovieSceneSection* Section)
{
	using namespace UE::MovieScene;

	UMovieSceneSequence* Sequence = GetTypedOuter<UMovieSceneSequence>();
	UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;

	if (MovieScene)
	{
		FFrameNumber StartFrame = DiscreteInclusiveLower(MovieScene->GetPlaybackRange());
		FFrameNumber EndFrame   = DiscreteExclusiveUpper(MovieScene->GetPlaybackRange());

		if (Section)
		{
			if (Section->HasStartFrame())
			{
				StartFrame = 0;
				if (Section->HasEndFrame())
				{
					EndFrame = Section->GetExclusiveEndFrame() - Section->GetInclusiveStartFrame();
				}
			}
			else if (Section->HasEndFrame())
			{
				EndFrame = Section->GetExclusiveEndFrame();
			}
		}

		TMovieSceneChannelData<FMovieSceneDoubleValue> ChannelData = Channel.GetData();

		FMovieSceneDoubleValue Value(0.0);
		Value.Value = StartFrame.Value;
		Value.InterpMode = RCIM_Linear;

		ChannelData.AddKey(StartFrame, Value);

		Value.Value = EndFrame.Value;
		ChannelData.AddKey(EndFrame, Value);

		Channel.PreInfinityExtrap  = RCCE_Constant;
		Channel.PostInfinityExtrap = RCCE_Constant;
	}
}

EMovieSceneChannelProxyType UMovieSceneTimeWarpCurve::PopulateChannelProxy(FMovieSceneChannelProxyData& OutProxyData, EAllowTopLevelChannels AllowTopLevel)
{
#if WITH_EDITOR
	FMovieSceneChannelMetaData ChannelMetaData;
	ChannelMetaData.Name = "TimeWarp";
	ChannelMetaData.bCanCollapseToTrack = (AllowTopLevel == EAllowTopLevelChannels::Yes);
	ChannelMetaData.DisplayText = NSLOCTEXT("MovieSceneTimeWarpCurve", "TimeWarpCurve_Label", "Time Warp");
	ChannelMetaData.WeakOwningObject = this;
	ChannelMetaData.bRelativeToSection = true;
	ChannelMetaData.bSortEmptyGroupsLast = false;
	ChannelMetaData.SortOrder = 0;
	OutProxyData.Add(Channel, ChannelMetaData);

#else
	OutProxyData.Add(Channel);
#endif

	return EMovieSceneChannelProxyType::Static;
}

bool UMovieSceneTimeWarpCurve::DeleteChannel(FMovieSceneTimeWarpVariant& OutVariant, FName ChannelName)
{
	if (ChannelName == "TimeWarp")
	{
		OutVariant.Set(1.0);
		return true;
	}
	return false;
}

TRange<FFrameTime> UMovieSceneTimeWarpCurve::ComputeTraversedHull(const TRange<FFrameTime>& Range) const
{
	TArrayView<const FFrameNumber>           Times  = Channel.GetData().GetTimes();
	TArrayView<const FMovieSceneDoubleValue> Values = Channel.GetData().GetValues();

	if (Values.Num() == 0)
	{
		FFrameTime Time = FFrameTime::FromDecimal(Channel.GetDefault().Get(0.0));
		return TRange<FFrameTime>::Inclusive(Time, Time);
	}
	else if (Values.Num() == 1)
	{
		FFrameTime Time = FFrameTime::FromDecimal(Values[0].Value);
		return TRange<FFrameTime>::Inclusive(Time, Time);
	}

	TRange<FFrameTime> Result = Range;

	FFrameTime StartTime = Range.GetLowerBound().IsOpen() ? FFrameTime(std::numeric_limits<int32>::lowest()) : Range.GetLowerBoundValue();
	FFrameTime EndTime   = Range.GetUpperBound().IsOpen() ? FFrameTime(std::numeric_limits<int32>::max())    : Range.GetUpperBoundValue();

	UE::MovieScene::Interpolation::FInterpolationExtents Extents = Channel.ComputeExtents(StartTime, EndTime);

	check(Extents.MinValue <= Extents.MaxValue);

	// Maintain bound exclusivity if possible
	if (Result.GetLowerBound().IsOpen())
	{
		Result.SetLowerBound(TRangeBound<FFrameTime>::Inclusive(FFrameTime::FromDecimal(Extents.MinValue))); 
	}
	else
	{
		Result.SetLowerBoundValue(FFrameTime::FromDecimal(Extents.MinValue));
	}

	if (Result.GetUpperBound().IsOpen())
	{
		Result.SetUpperBound(TRangeBound<FFrameTime>::Inclusive(FFrameTime::FromDecimal(Extents.MaxValue))); 
	}
	else
	{
		Result.SetUpperBoundValue(FFrameTime::FromDecimal(Extents.MaxValue));
	}

	return Result;
}

FFrameTime UMovieSceneTimeWarpCurve::RemapTime(FFrameTime In) const
{
	double OutValue = 0.0;
	Channel.Evaluate(In, OutValue);
	return FFrameTime::FromDecimal(OutValue);
}

TOptional<FFrameTime> UMovieSceneTimeWarpCurve::InverseRemapTimeCycled(FFrameTime InValue, FFrameTime InTimeHint, const UE::MovieScene::FInverseTransformTimeParams& Params) const
{
	TOptional<FFrameTime> FrameTime = Channel.InverseEvaluate(InValue.AsDecimal(), InTimeHint, Params.Flags);
	if (FrameTime)
	{
		return FrameTime;
	}

	const int32          CycleCount = Channel.GetCycleCount(InTimeHint);
	TRange<FFrameNumber> CycleRange = Channel.GetCycleRange(CycleCount);

	if (CycleCount != 0 && CycleRange.GetLowerBound().IsClosed() && CycleRange.GetUpperBound().IsClosed())
	{
		UE::MovieScene::Interpolation::FInterpolationExtents Extents =
			Channel.ComputeExtents(CycleRange.GetLowerBoundValue(), CycleRange.GetUpperBoundValue());

		const double CycleRangeDiff = Extents.MaxValue - Extents.MinValue;

		if (CycleRangeDiff != 0.0)
		{
			double ValueAsDecimal = InValue.AsDecimal();

			if (ValueAsDecimal > Extents.MaxValue)
			{
				const int32 CycleOffset = FMath::FloorToInt((ValueAsDecimal - Extents.MinValue) / CycleRangeDiff);

				ValueAsDecimal -= CycleOffset*CycleRangeDiff;
				InTimeHint     += CycleRange.Size<FFrameNumber>() * CycleOffset;

				return Channel.InverseEvaluate(ValueAsDecimal, InTimeHint, Params.Flags);
			}
			else if (ValueAsDecimal < Extents.MinValue)
			{
				const int32 CycleOffset = FMath::FloorToInt((ValueAsDecimal - Extents.MinValue) / CycleRangeDiff);

				// CycleOffset should be negative
				ValueAsDecimal -= CycleOffset*CycleRangeDiff;
				InTimeHint     += CycleRange.Size<FFrameNumber>() * CycleOffset;

				return Channel.InverseEvaluate(ValueAsDecimal, InTimeHint, Params.Flags);
			}
		}
	}

	return TOptional<FFrameTime>();
}

bool UMovieSceneTimeWarpCurve::InverseRemapTimeWithinRange(FFrameTime InTime, FFrameTime RangeStart, FFrameTime RangeEnd, const TFunctionRef<bool(FFrameTime)>& VisitorCallback) const
{
	return Channel.InverseEvaluateBetween(InTime.AsDecimal(), RangeStart, RangeEnd, VisitorCallback);
}

void UMovieSceneTimeWarpCurve::ScaleBy(double UnwarpedScaleFactor)
{
	Modify();
	Dilate(&Channel, 0, UnwarpedScaleFactor);
}

UE::MovieScene::ETimeWarpChannelDomain UMovieSceneTimeWarpCurve::GetDomain() const
{
	return UE::MovieScene::ETimeWarpChannelDomain::Time;
}
