// Copyright Epic Games, Inc. All Rights Reserved.

#include "Variants/MovieSceneTimeWarpGetter.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "MovieSceneSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneTimeWarpGetter)

UMovieSceneTimeWarpGetter::UMovieSceneTimeWarpGetter()
{
	// Allow time-warps to be accessible across different packages so that 
	//     we can store them directly inside FMovieSceneSequenceTransforms
	SetFlags(RF_Public);

	bMuted = false;
}

EMovieSceneChannelProxyType UMovieSceneTimeWarpGetter::PopulateChannelProxy(FMovieSceneChannelProxyData& OutProxyData, EAllowTopLevelChannels AllowTopLevel)
{
	return EMovieSceneChannelProxyType::Static;
}

bool UMovieSceneTimeWarpGetter::DeleteChannel(FMovieSceneTimeWarpVariant& OutVariant, FName ChannelName)
{
	return false;
}

UE::MovieScene::FChannelOwnerCapabilities UMovieSceneTimeWarpGetter::GetCapabilities(FName ChannelName) const
{
	UE::MovieScene::FChannelOwnerCapabilities Capabilities;
	Capabilities.bSupportsMute = true;
	return Capabilities;
}

bool UMovieSceneTimeWarpGetter::IsMuted(FName ChannelName) const
{
	return bMuted;
}

void UMovieSceneTimeWarpGetter::SetIsMuted(FName ChannelName, bool bIsMuted)
{
	Modify();
	bMuted = bIsMuted;
}

bool UMovieSceneTimeWarpGetter::IsMuted() const
{
	return bMuted;
}

void UMovieSceneTimeWarpGetter::SetIsMuted(bool bIsMuted)
{
	Modify();
	bMuted = bIsMuted;
}
