// Copyright Epic Games, Inc. All Rights Reserved.

#include "Variants/MovieSceneTimeWarpVariant.h"
#include "Variants/MovieSceneTimeWarpVariantPayloads.h"
#include "Variants/MovieSceneTimeWarpGetter.h"
#include "MovieScene.h"
#include "MovieSceneTimeHelpers.h"

#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneTimeWarpVariant)


void FMovieSceneTimeWarpVariant::Set(double InLiteralPlayRate)
{
	if (FMath::IsNearlyZero(InLiteralPlayRate))
	{
		Set(FMovieSceneTimeWarpFixedFrame{0});
	}
	else
	{
		Variant.Set(InLiteralPlayRate);
	}
}

void FMovieSceneTimeWarpVariant::Set(const FMovieSceneTimeWarpFixedFrame& InValue)
{
	Variant.SetTypedData(InValue, (uint8)EMovieSceneTimeWarpType::FixedTime - 1);
}

void FMovieSceneTimeWarpVariant::Set(const FMovieSceneTimeWarpFrameRate& InValue)
{
	Variant.SetTypedData(InValue, (uint8)EMovieSceneTimeWarpType::FrameRate - 1);
}

void FMovieSceneTimeWarpVariant::Set(const FMovieSceneTimeWarpLoop& InValue)
{
	Variant.SetTypedData(InValue, (uint8)EMovieSceneTimeWarpType::Loop - 1);
}

void FMovieSceneTimeWarpVariant::Set(const FMovieSceneTimeWarpClamp& InValue)
{
	Variant.SetTypedData(InValue, (uint8)EMovieSceneTimeWarpType::Clamp - 1);
}

void FMovieSceneTimeWarpVariant::Set(const FMovieSceneTimeWarpLoopFloat& InValue)
{
	Variant.SetTypedData(InValue, (uint8)EMovieSceneTimeWarpType::LoopFloat - 1);
}

void FMovieSceneTimeWarpVariant::Set(const FMovieSceneTimeWarpClampFloat& InValue)
{
	Variant.SetTypedData(InValue, (uint8)EMovieSceneTimeWarpType::ClampFloat - 1);
}

double FMovieSceneTimeWarpVariant::AsFixedPlayRate() const
{
	check(Variant.IsLiteral());
	return Variant.GetLiteral();
}

float FMovieSceneTimeWarpVariant::AsFixedPlayRateFloat() const
{
	check(Variant.IsLiteral());
	return Variant.GetLiteralAsFloat();
}

FMovieSceneTimeWarpFixedFrame FMovieSceneTimeWarpVariant::AsFixedTime() const
{
	check(GetType() == EMovieSceneTimeWarpType::FixedTime);
	return Variant.UnsafePayloadCast<FMovieSceneTimeWarpFixedFrame>();
}

FMovieSceneTimeWarpFrameRate FMovieSceneTimeWarpVariant::AsFrameRate() const
{
	check(GetType() == EMovieSceneTimeWarpType::FrameRate);
	return Variant.UnsafePayloadCast<FMovieSceneTimeWarpFrameRate>();
}

FMovieSceneTimeWarpLoop FMovieSceneTimeWarpVariant::AsLoop() const
{
	check(GetType() == EMovieSceneTimeWarpType::Loop);
	return Variant.UnsafePayloadCast<FMovieSceneTimeWarpLoop>();
}

FMovieSceneTimeWarpClamp FMovieSceneTimeWarpVariant::AsClamp() const
{
	check(GetType() == EMovieSceneTimeWarpType::Clamp);
	return Variant.UnsafePayloadCast<FMovieSceneTimeWarpClamp>();
}

FMovieSceneTimeWarpLoopFloat FMovieSceneTimeWarpVariant::AsLoopFloat() const
{
	check(GetType() == EMovieSceneTimeWarpType::LoopFloat);
	return Variant.UnsafePayloadCast<FMovieSceneTimeWarpLoopFloat>();
}

FMovieSceneTimeWarpClampFloat FMovieSceneTimeWarpVariant::AsClampFloat() const
{
	check(GetType() == EMovieSceneTimeWarpType::ClampFloat);
	return Variant.UnsafePayloadCast<FMovieSceneTimeWarpClampFloat>();
}

UMovieSceneTimeWarpGetter* FMovieSceneTimeWarpVariant::AsCustom() const
{
	check(GetType() == EMovieSceneTimeWarpType::Custom);
	return static_cast<UMovieSceneTimeWarpGetter*>(Variant.GetCustomPtr());
}

void FMovieSceneTimeWarpVariant::Set(UMovieSceneTimeWarpGetter* InDynamicValue)
{
	Variant.Set(InDynamicValue);
}

void FMovieSceneTimeWarpVariant::ScaleBy(double ScaleFactor)
{
	switch(GetType())
	{
	case EMovieSceneTimeWarpType::FixedPlayRate:
		{
			const double NewPlayRate = AsFixedPlayRate() * ScaleFactor;
			// Use Set() here in order to properly handle (near-)zero play rates
			Set(NewPlayRate);
		}
		break;
	case EMovieSceneTimeWarpType::Custom:
		if (UMovieSceneTimeWarpGetter* Custom = AsCustom())
		{
			Custom->ScaleBy(ScaleFactor);
		}
		break;

	case EMovieSceneTimeWarpType::FixedTime:
		break;

	case EMovieSceneTimeWarpType::FrameRate:
		break;

	case EMovieSceneTimeWarpType::Loop:
		{
			FMovieSceneTimeWarpLoop Loop = AsLoop();
			Loop.Duration = (Loop.Duration * ScaleFactor);
			Set(Loop);
		}
		break;

	case EMovieSceneTimeWarpType::Clamp:
		{
			FMovieSceneTimeWarpClamp Clamp = AsClamp();
			Clamp.Max = (Clamp.Max * ScaleFactor);
			Set(Clamp);
		}
		break;

	case EMovieSceneTimeWarpType::LoopFloat:
		{
			FMovieSceneTimeWarpLoopFloat Loop = AsLoopFloat();
			Loop.Duration *= ScaleFactor;
			Set(Loop);
		}
		break;

	case EMovieSceneTimeWarpType::ClampFloat:
		{
			FMovieSceneTimeWarpClampFloat Clamp = AsClampFloat();
			Clamp.Max *= ScaleFactor;
			Set(Clamp);
		}
		break;
	}
}


FFrameTime FMovieSceneTimeWarpVariant::RemapTime(FFrameTime InTime) const
{
	switch(GetType())
	{
	case EMovieSceneTimeWarpType::FixedPlayRate:
		return InTime * AsFixedPlayRate();

	case EMovieSceneTimeWarpType::Custom:
		if (UMovieSceneTimeWarpGetter* Custom = AsCustom())
		{
			return Custom->RemapTime(InTime);
		}
		return InTime;

	case EMovieSceneTimeWarpType::FixedTime:
		return AsFixedTime().FrameNumber;

	case EMovieSceneTimeWarpType::FrameRate:
		return ConvertFrameTime(InTime, FFrameRate(1, 1), AsFrameRate().GetFrameRate());

	case EMovieSceneTimeWarpType::Loop:
		return AsLoop().LoopTime(InTime);

	case EMovieSceneTimeWarpType::Clamp:
		return AsClamp().Clamp(InTime);

	case EMovieSceneTimeWarpType::LoopFloat:
		return AsLoopFloat().LoopTime(InTime);

	case EMovieSceneTimeWarpType::ClampFloat:
		return AsClampFloat().Clamp(InTime);
	}

	return InTime;
}

bool FMovieSceneTimeWarpVariant::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	return Variant.SerializeCustom(Ar, [this](FArchive& InAr, uint8& TypeBits, void* DataPtr){

		const bool bIsLoading = InAr.GetArchiveState().IsLoading();

		EMovieSceneTimeWarpType Type = this->GetType();
		InAr << Type;

		if (bIsLoading)
		{
			TypeBits = (uint8)Type - 1;
		}

		switch(Type)
		{
		case EMovieSceneTimeWarpType::Custom:
			if (bIsLoading)
			{
				UMovieSceneNumericVariantGetter* Custom = nullptr;
				InAr << Custom;
				Variant.Set(Custom);
			}
			else
			{
				UMovieSceneNumericVariantGetter* Custom = this->AsCustom();
				InAr << Custom;
			}
			break;

		case EMovieSceneTimeWarpType::FixedTime:
			FMovieSceneTimeWarpFixedFrame::StaticStruct()->SerializeItem(InAr, DataPtr, nullptr);
			break;

		case EMovieSceneTimeWarpType::FrameRate:
			if (bIsLoading)
			{
				FMovieSceneTimeWarpFrameRate* Rate = static_cast<FMovieSceneTimeWarpFrameRate*>(DataPtr);

				FFrameRate FrameRate;
				TBaseStructure<FFrameRate>::Get()->SerializeItem(InAr, &FrameRate, nullptr);

				*Rate = FMovieSceneTimeWarpFrameRate(FrameRate);
			}
			else
			{
				FMovieSceneTimeWarpFrameRate* Rate = static_cast<FMovieSceneTimeWarpFrameRate*>(DataPtr);

				FFrameRate FrameRate = Rate->GetFrameRate();
				TBaseStructure<FFrameRate>::Get()->SerializeItem(InAr, &FrameRate, nullptr);
			}
			break;

		case EMovieSceneTimeWarpType::Loop:
			FMovieSceneTimeWarpLoop::StaticStruct()->SerializeItem(InAr, DataPtr, nullptr);
			break;

		case EMovieSceneTimeWarpType::Clamp:
			FMovieSceneTimeWarpClamp::StaticStruct()->SerializeItem(InAr, DataPtr, nullptr);
			break;

		case EMovieSceneTimeWarpType::LoopFloat:
			FMovieSceneTimeWarpLoopFloat::StaticStruct()->SerializeItem(InAr, DataPtr, nullptr);
			break;

		case EMovieSceneTimeWarpType::ClampFloat:
			FMovieSceneTimeWarpClampFloat::StaticStruct()->SerializeItem(InAr, DataPtr, nullptr);
			break;
		}
	});
}

bool FMovieSceneTimeWarpVariant::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == NAME_DoubleProperty)
	{
		double Value = 0.0;
		Slot << Value;
		Set(Value);
		return true;
	}
	if (Tag.Type == NAME_FloatProperty)
	{
		float Value = 0.f;
		Slot << Value;
		Set(Value);
		return true;
	}

	// int64 and uint64 are not supported in this variant without loss of precision

	if (Tag.Type == NAME_ByteProperty)
	{
		int32 Value = 0;
		Slot << Value;
		Set(Value);
		return true;
	}
	if (Tag.Type == NAME_Int32Property || Tag.Type == NAME_IntProperty)
	{
		int32 Value = 0;
		Slot << Value;
		Set(Value);
		return true;
	}
	if (Tag.Type == NAME_Int16Property)
	{
		int16 Value = 0;
		Slot << Value;
		Set(Value);
		return true;
	}
	if (Tag.Type == NAME_Int8Property)
	{
		int8 Value = 0;
		Slot << Value;
		Set(Value);
		return true;
	}

	if (Tag.Type == NAME_UInt32Property)
	{
		uint32 Value = 0;
		Slot << Value;
		Set(Value);
		return true;
	}
	if (Tag.Type == NAME_UInt16Property)
	{
		uint16 Value = 0;
		Slot << Value;
		Set(Value);
		return true;
	}
	if (Tag.Type == NAME_ByteProperty)
	{
		uint8 Value = 0;
		Slot << Value;
		Set(Value);
		return true;
	}

	return false;
}

bool FMovieSceneTimeWarpVariant::ExportTextItem(FString& ValueStr, const FMovieSceneTimeWarpVariant& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	EMovieSceneTimeWarpType Type = this->GetType();
	ValueStr += UEnum::GetValueAsString(Type);

	switch(Type)
	{
	case EMovieSceneTimeWarpType::FixedPlayRate:
		{
			FMovieSceneFixedPlayRateStruct Struct = {this->AsFixedPlayRate()};
			FMovieSceneFixedPlayRateStruct::StaticStruct()->ExportText(ValueStr, &Struct, nullptr, Parent, PortFlags, ExportRootScope);
			return true;
		}

	case EMovieSceneTimeWarpType::Custom:
		{
			FMovieSceneCustomTimeWarpGetterStruct Struct = {this->AsCustom()};
			FMovieSceneCustomTimeWarpGetterStruct::StaticStruct()->ExportText(ValueStr, &Struct, nullptr, Parent, PortFlags, ExportRootScope);
			return true;
		}

	case EMovieSceneTimeWarpType::FixedTime:
		{
			FMovieSceneTimeWarpFixedFrame Struct = this->AsFixedTime();
			FMovieSceneTimeWarpFixedFrame::StaticStruct()->ExportText(ValueStr, &Struct, nullptr, Parent, PortFlags, ExportRootScope);
			return true;
		}

	case EMovieSceneTimeWarpType::FrameRate:
		{
			FMovieSceneTimeWarpFrameRate Struct = this->AsFrameRate();
			FMovieSceneTimeWarpFrameRate::StaticStruct()->ExportText(ValueStr, &Struct, nullptr, Parent, PortFlags, ExportRootScope);
			return true;
		}

	case EMovieSceneTimeWarpType::Loop:
		{
			FMovieSceneTimeWarpLoop Struct = this->AsLoop();
			FMovieSceneTimeWarpLoop::StaticStruct()->ExportText(ValueStr, &Struct, nullptr, Parent, PortFlags, ExportRootScope);
			return true;
		}

	case EMovieSceneTimeWarpType::Clamp:
		{
			FMovieSceneTimeWarpClamp Struct = this->AsClamp();
			FMovieSceneTimeWarpClamp::StaticStruct()->ExportText(ValueStr, &Struct, nullptr, Parent, PortFlags, ExportRootScope);
			return true;
		}

	case EMovieSceneTimeWarpType::LoopFloat:
		{
			FMovieSceneTimeWarpLoopFloat Struct = this->AsLoopFloat();
			FMovieSceneTimeWarpLoopFloat::StaticStruct()->ExportText(ValueStr, &Struct, nullptr, Parent, PortFlags, ExportRootScope);
			return true;
		}

	case EMovieSceneTimeWarpType::ClampFloat:
		{
			FMovieSceneTimeWarpClampFloat Struct = this->AsClampFloat();
			FMovieSceneTimeWarpClampFloat::StaticStruct()->ExportText(ValueStr, &Struct, nullptr, Parent, PortFlags, ExportRootScope);
			return true;
		}
	default:
		ensureMsgf(false, TEXT("Unimplemented type found when exporting text!"));
		return false;
	}
}

bool FMovieSceneTimeWarpVariant::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText)
{
	FNameBuilder TypeEnumEntry;

	TStringView<TCHAR> EnumToken = TEXTVIEW("EMovieSceneTimeWarpType::");
	if (FCString::Strncmp(EnumToken.GetData(), Buffer, EnumToken.Len()) != 0)
	{
		return false;
	}

	Buffer += EnumToken.Len();
	TypeEnumEntry += EnumToken;
	if (const TCHAR* TypeNameEnd = FPropertyHelpers::ReadToken(Buffer, TypeEnumEntry))
	{
		Buffer = TypeNameEnd;
	}
	else
	{
		return false;
	}

	FName EnumEntryName(TypeEnumEntry.ToView(), FNAME_Find);
	if (EnumEntryName.IsNone())
	{
		return false;
	}

	UEnum* Enum = StaticEnum<EMovieSceneTimeWarpType>();
	check(Enum);

	const int64 EnumValue = Enum->GetValueByName(EnumEntryName);
	if (EnumValue == INDEX_NONE)
	{
		return false;
	}

	EMovieSceneTimeWarpType NewType = (EMovieSceneTimeWarpType)EnumValue;

	switch(NewType)
	{
	case EMovieSceneTimeWarpType::FixedPlayRate:
		{
			FMovieSceneFixedPlayRateStruct Struct;
			if (const TCHAR* Result = FMovieSceneFixedPlayRateStruct::StaticStruct()->ImportText(Buffer, &Struct, Parent, PortFlags, ErrorText, []{ return FMovieSceneFixedPlayRateStruct::StaticStruct()->GetName(); }))
			{
				Buffer = Result;
				Set(Struct.PlayRate);
				return true;
			}
			return false;
		}

	case EMovieSceneTimeWarpType::Custom:
		{
			FMovieSceneCustomTimeWarpGetterStruct Struct;
			if (const TCHAR* Result = FMovieSceneCustomTimeWarpGetterStruct::StaticStruct()->ImportText(Buffer, &Struct, Parent, PortFlags, ErrorText, []{ return FMovieSceneCustomTimeWarpGetterStruct::StaticStruct()->GetName(); }))
			{
				Buffer = Result;
				Set(Struct.Object);
				return true;
			}
			return false;
		}

	case EMovieSceneTimeWarpType::FixedTime:
		{
			FMovieSceneTimeWarpFixedFrame Struct;
			if (const TCHAR* Result = FMovieSceneTimeWarpFixedFrame::StaticStruct()->ImportText(Buffer, &Struct, Parent, PortFlags, ErrorText, []{ return FMovieSceneTimeWarpFixedFrame::StaticStruct()->GetName(); }))
			{
				Buffer = Result;
				Set(Struct);
				return true;
			}
			return false;
		}

	case EMovieSceneTimeWarpType::FrameRate:
		{
			FMovieSceneTimeWarpFrameRate Struct;
			if (const TCHAR* Result = FMovieSceneTimeWarpFrameRate::StaticStruct()->ImportText(Buffer, &Struct, Parent, PortFlags, ErrorText, []{ return FMovieSceneTimeWarpFrameRate::StaticStruct()->GetName(); }))
			{
				Buffer = Result;
				Set(Struct);
				return true;
			}
			return false;
		}

	case EMovieSceneTimeWarpType::Loop:
		{
			FMovieSceneTimeWarpLoop Struct;
			if (const TCHAR* Result = FMovieSceneTimeWarpLoop::StaticStruct()->ImportText(Buffer, &Struct, Parent, PortFlags, ErrorText, []{ return FMovieSceneTimeWarpLoop::StaticStruct()->GetName(); }))
			{
				Buffer = Result;
				Set(Struct);
				return true;
			}
			return false;
		}

	case EMovieSceneTimeWarpType::Clamp:
		{
			FMovieSceneTimeWarpClamp Struct;
			if (const TCHAR* Result = FMovieSceneTimeWarpClamp::StaticStruct()->ImportText(Buffer, &Struct, Parent, PortFlags, ErrorText, []{ return FMovieSceneTimeWarpClamp::StaticStruct()->GetName(); }))
			{
				Buffer = Result;
				Set(Struct);
				return true;
			}
			return false;
		}

	case EMovieSceneTimeWarpType::LoopFloat:
		{
			FMovieSceneTimeWarpLoopFloat Struct;
			if (const TCHAR* Result = FMovieSceneTimeWarpLoopFloat::StaticStruct()->ImportText(Buffer, &Struct, Parent, PortFlags, ErrorText, []{ return FMovieSceneTimeWarpLoopFloat::StaticStruct()->GetName(); }))
			{
				Buffer = Result;
				Set(Struct);
				return true;
			}
			return false;
		}

	case EMovieSceneTimeWarpType::ClampFloat:
		{
			FMovieSceneTimeWarpClampFloat Struct;
			if (const TCHAR* Result = FMovieSceneTimeWarpClampFloat::StaticStruct()->ImportText(Buffer, &Struct, Parent, PortFlags, ErrorText, []{ return FMovieSceneTimeWarpClampFloat::StaticStruct()->GetName(); }))
			{
				Buffer = Result;
				Set(Struct);
				return true;
			}
			return false;
		}
	default:
		ensureMsgf(false, TEXT("Unimplemented type found when importing text!"));
		return false;
	}
}
