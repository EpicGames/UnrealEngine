// Copyright Epic Games, Inc. All Rights Reserved.

#include "Variants/MovieSceneTimeWarpVariantPayloads.h"
#include "MovieSceneTimeHelpers.h"
#include "MovieSceneTransformTypes.h"
#include <type_traits>

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneTimeWarpVariantPayloads)

static_assert(std::is_trivially_copyable_v<FMovieSceneTimeWarpFixedFrame>, "FMovieSceneTimeWarpFixedFrame must be trivially copyable");
static_assert(std::is_trivially_copyable_v<FMovieSceneTimeWarpFrameRate>, "FMovieSceneTimeWarpFrameRate must be trivially copyable");
static_assert(std::is_trivially_copyable_v<FMovieSceneTimeWarpLoop>, "FMovieSceneTimeWarpLoop must be trivially copyable");
static_assert(std::is_trivially_copyable_v<FMovieSceneTimeWarpClamp>, "FMovieSceneTimeWarpClamp must be trivially copyable");
static_assert(std::is_trivially_copyable_v<FMovieSceneTimeWarpLoopFloat>, "FMovieSceneTimeWarpLoopFloat must be trivially copyable");
static_assert(std::is_trivially_copyable_v<FMovieSceneTimeWarpClampFloat>, "FMovieSceneTimeWarpClampFloat must be trivially copyable");


FFrameTime FMovieSceneTimeWarpLoop::LoopTime(FFrameTime InTime) const
{
	int32 Unused;
	return LoopTime(InTime, Unused);
}

FFrameTime FMovieSceneTimeWarpLoop::LoopTime(FFrameTime InTime, int32& OutLoop) const
{
	const int32 Frame = InTime.FrameNumber.Value;
	const int32 Dur   = Duration.Value;

	// Make sure to compute negative loops correctly by subtracting 1 for any negative time
	//    This results in the equivalent of floor( double(time) / double(duration) )
	const uint32 FrameAsBits = *reinterpret_cast<const uint32*>(&Frame);
	const uint32 SignAsBits  = (FrameAsBits&0x80000000)>>31;    // Yields 0 for +ve or 1 for -ve times by shifting the sign bit
	const int32  Sign        = -static_cast<int32>(SignAsBits); // Yields 0 for +ve or -1 for -ve times

	OutLoop = Frame / Dur + Sign;

	// Maintain subframe
	InTime.FrameNumber = Frame - Dur*OutLoop;
	return InTime;
}

TRange<FFrameTime> FMovieSceneTimeWarpLoop::ComputeTraversedHull(const TRange<FFrameTime>& Range) const
{
	int32 StartLoop = 0;
	int32 EndLoop   = 0;

	TRangeBound<FFrameTime> LoopStart = TRangeBound<FFrameTime>::Inclusive(0);
	TRangeBound<FFrameTime> LoopEnd   = TRangeBound<FFrameTime>::Exclusive(Duration);

	if (Range.IsEmpty())
	{
		// Empty range of 0
		return TRange<FFrameTime>(0, 0);
	}
	else if (Range.GetLowerBound().IsOpen() || Range.GetUpperBound().IsOpen())
	{
		return TRange<FFrameTime>(LoopStart, LoopEnd);
	}

	TRangeBound<FFrameTime> WarpedStart = Range.GetLowerBound();
	TRangeBound<FFrameTime> WarpedEnd   = Range.GetUpperBound();

	WarpedStart.SetValue(LoopTime(WarpedStart.GetValue(), StartLoop));
	WarpedEnd.SetValue(LoopTime(WarpedEnd.GetValue(), EndLoop));

	// Do not loop exlusive end frames
	if (WarpedEnd.GetValue() == 0 && WarpedEnd.IsExclusive())
	{
		--EndLoop;
		WarpedEnd = LoopEnd;
	}

	if (StartLoop == EndLoop)
	{
		return TRange<FFrameTime>(WarpedStart, WarpedEnd);
	}

	const int32 NumCompleteLoops = EndLoop - StartLoop - 1;
	if (NumCompleteLoops >= 1)
	{
		return TRange<FFrameTime>(LoopStart, LoopEnd);
	}

	// If the range crosses a loop boundary and the end time is > the start time, we have traversed a full loop
	if (WarpedEnd.GetValue() > WarpedStart.GetValue())
	{
		return TRange<FFrameTime>(LoopStart, LoopEnd);
	}

	// Technically there are 2 disjointed ranges that were traversed, but this api can only return 1 so we just return the most recent one
	return TRange<FFrameTime>(LoopStart, WarpedEnd);
}

TOptional<FFrameTime> FMovieSceneTimeWarpLoop::InverseRemapTimeCycled(FFrameTime InValue, FFrameTime InTimeHint, const UE::MovieScene::FInverseTransformTimeParams& Params) const
{
	if ((InValue.FrameNumber >= 0 && InValue.FrameNumber < Duration) ||
		 EnumHasAnyFlags(Params.Flags, UE::MovieScene::EInverseEvaluateFlags::Cycle))
	{
		int32 HintCycle = 0;
		FFrameTime LoopedHint = LoopTime(InTimeHint, HintCycle);

		FFrameTime Difference(InValue - LoopedHint);
		int32 DifferenceCycle = 0;
		FFrameTime LoopedDiff = LoopTime(Difference, DifferenceCycle);

		const int32 Length = Duration.Value;
		// Get the result within the correct loop according to the hint
		return Length*HintCycle + Length*DifferenceCycle + LoopedHint + LoopedDiff;
	}
	return TOptional<FFrameTime>();
}

bool FMovieSceneTimeWarpLoop::InverseRemapTimeWithinRange(FFrameTime InTime, FFrameTime RangeStart, FFrameTime RangeEnd, const TFunctionRef<bool(FFrameTime)>& VisitorCallback) const
{
	int32 Length = Duration.Value;

	int32 InputLoop = 0;
	int32 StartLoop = 0;
	int32 EndLoop   = 0;

	FFrameTime LoopedInput = LoopTime(InTime,     InputLoop);
	FFrameTime StartTime   = LoopTime(RangeStart, StartLoop);
	FFrameTime EndTime     = LoopTime(RangeEnd,   EndLoop);

	if (StartLoop > EndLoop || (StartLoop == EndLoop && EndTime < StartTime) )
	{
		Swap(StartLoop, EndLoop);
		Swap(StartTime, EndTime);
	}

	int32 LoopIndex = InputLoop;
	FFrameTime Result  = LoopedInput + FFrameTime(Length*LoopIndex);

	// Handle with the start loop
	if (InputLoop != StartLoop || LoopedInput >= StartTime)
	{
		if (!VisitorCallback(Result))
		{
			return false;
		}
	}

	++LoopIndex;
	for ( ; LoopIndex < EndLoop; ++LoopIndex)
	{
		Result += FFrameTime(Length);
		if (!VisitorCallback(Result))
		{
			return false;
		}
	}

	// Handle trailing loop
	if (EndLoop != StartLoop && LoopedInput < EndTime)
	{
		Result += FFrameTime(Length);
		if (!VisitorCallback(Result))
		{
			return false;
		}
	}

	return true;
}

bool FMovieSceneTimeWarpLoop::ExtractBoundariesWithinRange(const TRange<FFrameTime>& Range, const TFunctionRef<bool(FFrameTime)>& InVisitor) const
{
	const int32 Start = Range.GetLowerBound().IsClosed() ? Range.GetLowerBoundValue().FrameNumber.Value : MIN_int32;
	const int32 End   = Range.GetUpperBound().IsClosed() ? Range.GetUpperBoundValue().FrameNumber.Value : MAX_int32;

	int32 LoopIndex = 0;
	int32 EndLoop   = 0;

	LoopTime(Start, LoopIndex);
	LoopTime(End, EndLoop);

	for ( ; LoopIndex <= EndLoop; ++LoopIndex)
	{
		FFrameTime StartResult = FFrameTime(Duration*LoopIndex);

		if (StartResult.FrameNumber.Value >= Start)
		{
			if (!InVisitor(StartResult))
			{
				return false;
			}
		}
	}

	return true;
}

FFrameTime FMovieSceneTimeWarpClamp::Clamp(FFrameTime InTime) const
{
	if (InTime < 0)
	{
		return FFrameTime(0);
	}
	if (InTime > Max)
	{
		return Max;
	}
	return InTime;
}

TRange<FFrameTime> FMovieSceneTimeWarpClamp::ComputeTraversedHull(const TRange<FFrameTime>& Range) const
{
	TRange<FFrameTime> Result = Range;
	if (!Range.GetLowerBound().IsOpen())
	{
		Result.SetLowerBoundValue(Clamp(Range.GetLowerBoundValue()));
	}
	if (!Range.GetUpperBound().IsOpen())
	{
		Result.SetUpperBoundValue(Clamp(Range.GetUpperBoundValue()));
	}
	return Result;
}


FFrameTime FMovieSceneTimeWarpLoopFloat::LoopTime(FFrameTime InTime) const
{
	int32 Unused;
	return LoopTime(InTime, Unused);
}

FFrameTime FMovieSceneTimeWarpLoopFloat::LoopTime(FFrameTime InTime, int32& OutLoop) const
{
	const double Time = InTime.AsDecimal();

	OutLoop = FMath::FloorToInt(Time / Duration);
	return FFrameTime::FromDecimal(Time - Duration*OutLoop);
}

TRange<FFrameTime> FMovieSceneTimeWarpLoopFloat::ComputeTraversedHull(const TRange<FFrameTime>& Range) const
{
	int32 StartLoop = 0;
	int32 EndLoop   = 0;

	FFrameTime LoopStart = 0;
	FFrameTime LoopEnd = FFrameTime::FromDecimal(Duration);

	if (Range.GetLowerBound().IsOpen() || Range.GetUpperBound().IsOpen())
	{
		return TRange<FFrameTime>(LoopStart, LoopEnd);
	}

	const FFrameTime WarpedStart = LoopTime(Range.GetLowerBoundValue(), StartLoop);
	const FFrameTime WarpedEnd   = LoopTime(Range.GetUpperBoundValue(), EndLoop);

	if (StartLoop == EndLoop)
	{
		TRange<FFrameTime> Result = Range;
		Result.SetLowerBoundValue(WarpedStart);
		Result.SetUpperBoundValue(WarpedEnd);
		return Result;
	}

	const int32 NumCompleteLoops = EndLoop - StartLoop - 1;
	if (NumCompleteLoops >= 1)
	{
		return TRange<FFrameTime>(LoopStart, LoopEnd);
	}

	// If the range crosses a loop boundary and the end time is > the start time, we have traversed a full loop
	if (WarpedEnd > WarpedStart)
	{
		return TRange<FFrameTime>(LoopStart, LoopEnd);
	}

	// Technically there are 2 disjointed ranges that were traversed, but this api can only return 1 so we just return the most recent one
	return TRange<FFrameTime>(LoopStart, WarpedEnd);
}

TOptional<FFrameTime> FMovieSceneTimeWarpLoopFloat::InverseRemapTimeCycled(FFrameTime InValue, FFrameTime InTimeHint, const UE::MovieScene::FInverseTransformTimeParams& Params) const
{
	if (InValue.FrameNumber >= 0 && InValue <= FFrameTime::FromDecimal(Duration))
	{
		int32 HintCycle = 0;
		LoopTime(InTimeHint, HintCycle);

		// Get the result within the correct loop according to the hint
		double Result = FMath::Fmod(InValue.AsDecimal(), Duration) + Duration*HintCycle;
		return FFrameTime::FromDecimal(Result);
	}
	return TOptional<FFrameTime>();
}

bool FMovieSceneTimeWarpLoopFloat::InverseRemapTimeWithinRange(FFrameTime InTime, FFrameTime RangeStart, FFrameTime RangeEnd, const TFunctionRef<bool(FFrameTime)>& VisitorCallback) const
{
	ensure(RangeStart < RangeEnd);

	FFrameTime Length = FFrameTime::FromDecimal(Duration);

	int32 InputLoop = 0;
	int32 StartLoop = 0;
	int32 EndLoop   = 0;

	FFrameTime LoopedInput = LoopTime(InTime,     InputLoop);
	FFrameTime StartTime   = LoopTime(RangeStart, StartLoop);
	FFrameTime EndTime     = LoopTime(RangeEnd,   EndLoop);

	int32 LoopIndex = InputLoop;
	FFrameTime Result  = LoopedInput + Length*LoopIndex;

	// Handle with the start loop
	if (InputLoop != StartLoop || LoopedInput >= StartTime)
	{
		if (!VisitorCallback(Result))
		{
			return false;
		}
	}

	++LoopIndex;
	for ( ; LoopIndex < EndLoop; ++LoopIndex)
	{
		Result += Length;
		if (!VisitorCallback(Result))
		{
			return false;
		}
	}

	// Handle trailing loop
	if (EndLoop != StartLoop && LoopedInput < EndTime)
	{
		Result += Length;
		if (!VisitorCallback(Result))
		{
			return false;
		}
	}
	
	return true;
}

bool FMovieSceneTimeWarpLoopFloat::ExtractBoundariesWithinRange(const TRange<FFrameTime>& Range, const TFunctionRef<bool(FFrameTime)>& InVisitor) const
{
	const double Start = Range.GetLowerBound().IsClosed() ? Range.GetLowerBoundValue().AsDecimal() : double(MIN_int32);
	const double End   = Range.GetUpperBound().IsClosed() ? Range.GetUpperBoundValue().AsDecimal() : double(MAX_int32);

	int32 LoopIndex = FMath::FloorToInt(Start / Duration);
	int32 EndLoop = FMath::FloorToInt(End / Duration);

	for (; LoopIndex <= EndLoop; ++LoopIndex)
	{
		const double Result = Duration*LoopIndex;
		if (Result >= Start)
		{
			if (!InVisitor(FFrameTime::FromDecimal(Result)))
			{
				return false;
			}
		}
	}

	return true;
}

FFrameTime FMovieSceneTimeWarpClampFloat::Clamp(FFrameTime InTime) const
{
	if (InTime < 0)
	{
		return FFrameTime(0);
	}
	if (InTime.AsDecimal() > Max)
	{
		return FFrameTime::FromDecimal(Max);
	}
	return InTime;
}

TRange<FFrameTime> FMovieSceneTimeWarpClampFloat::ComputeTraversedHull(const TRange<FFrameTime>& Range) const
{
	TRange<FFrameTime> Result = Range;
	if (!Range.GetLowerBound().IsOpen())
	{
		Result.SetLowerBoundValue(Clamp(Range.GetLowerBoundValue()));
	}
	if (!Range.GetUpperBound().IsOpen())
	{
		Result.SetUpperBoundValue(Clamp(Range.GetUpperBoundValue()));
	}
	return Result;
}

FMovieSceneTimeWarpFrameRate::FMovieSceneTimeWarpFrameRate()
	: FMovieSceneTimeWarpFrameRate(FFrameRate())
{}

FMovieSceneTimeWarpFrameRate::FMovieSceneTimeWarpFrameRate(FFrameRate InRate)
{
	constexpr int32 SignBit32   = 0x80000000;
	constexpr int32 SignBit24   = 0x00800000;
	constexpr int32 InvalidBits = 0x7F800000;

	int32 Numerator   = InRate.Numerator;
	int32 Denominator = InRate.Denominator;

	// Do not allow 8 most significant bits, offset by the sign bit (our sign bit becomes bit index 23)
	ensureMsgf((Numerator & InvalidBits) == 0, TEXT("Found invalid numerator: %d, in frame rate: %s"), Numerator, *InRate.ToPrettyText().ToString());
	ensureMsgf((Denominator & InvalidBits) == 0, TEXT("Found invalid denominator: %d, in frame rate: %s"), Denominator, *InRate.ToPrettyText().ToString());

	// Move the sign bit
	Numerator   |= ( (Numerator   & SignBit32) >> 8 );
	Denominator |= ( (Denominator & SignBit32) >> 8 );

	// Copy LSBs from 32 bits to 24 bits
	FMemory::Memcpy(FrameRateNumerator,   &Numerator,   sizeof(FrameRateNumerator));
	FMemory::Memcpy(FrameRateDenominator, &Denominator, sizeof(FrameRateDenominator));
}

FFrameRate FMovieSceneTimeWarpFrameRate::GetFrameRate() const
{
	constexpr int32 SignBit24 = 0x00800000;

	int32 Numerator   = 0;
	int32 Denominator = 0;

	// Copy LSBs from 24 bits to 32 bits
	FMemory::Memcpy(&Numerator,   FrameRateNumerator,   sizeof(FrameRateNumerator));
	FMemory::Memcpy(&Denominator, FrameRateDenominator, sizeof(FrameRateDenominator));

	// Move the sign bit
	Numerator   = ((Numerator  & SignBit24) << 8) | (Numerator  & ~SignBit24);
	Denominator = ((Denominator & SignBit24) << 8) | (Denominator & ~SignBit24);

	return FFrameRate(Numerator, Denominator);
}
