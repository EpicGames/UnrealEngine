// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"

#include "ActorForWorldTransforms.generated.h"

//Description of an actor selected parts we can find world transforms on 
USTRUCT(BlueprintType)
struct  FActorForWorldTransforms
{
	GENERATED_BODY()

	FActorForWorldTransforms() : Actor(nullptr), Component(nullptr), SocketName(NAME_None) {};

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Actor")
	TWeakObjectPtr<AActor> Actor;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Actor")	
	TWeakObjectPtr<USceneComponent> Component;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Actor")
	FName SocketName;

	bool operator ==(const FActorForWorldTransforms& Other) const
	{
		if (Actor == Other.Actor)
		{
			if (Component.IsValid())
			{
				if (Component == Other.Component)
				{
					if (SocketName == Other.SocketName)
					{
						return true;
					}
				}
			}
			else if(Other.Component.IsValid() == false)
			{
				return true;
			}
		}
		return false;
	}
};
