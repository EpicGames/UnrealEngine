// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneSequenceID.h"
#include "MovieSceneBindingProxy.h"
#include "Textures/SlateIcon.h"
#include "MovieSceneCustomBinding.generated.h"

namespace UE
{
	namespace MovieScene
	{
		struct FSharedPlaybackState;
	}
}

class UMovieSceneSequence;
class UMovieScene;
struct FGuid;
struct FMovieSceneBindingResolveParams;
struct FSlateBrush;
struct FMovieSceneBindingReference;
class UMovieSceneSpawnableBindingBase;

USTRUCT(BlueprintType)
struct FMovieSceneBindingResolveResult
{
	GENERATED_BODY()

	/** The resolved objects */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "General")
	TArray<TObjectPtr<UObject>> Objects;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "General", meta = (DeprecatedProperty, DeprecationMessage = "This property is deprecated, please use the array of Objects instead."))
	TObjectPtr<UObject> Object = nullptr;
};

/*
* Blueprint-specific resolution context for custom bindings.
*/
USTRUCT(BlueprintType)
struct FMovieSceneBindingResolveContext
{
	GENERATED_BODY()

	/* The world context*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Default")
	TObjectPtr<UObject> WorldContext;
	
	/* Binding for the bound object currently evaluating this condition if applicable (BindingId will be invalid for conditions on global tracks/sections). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Default")
	FMovieSceneBindingProxy Binding;
};

/**
 * A custom binding. Allows users to define their own binding resolution types, including dynamic 'Replaceable' bindings with previews in editor, as well as Spawnable types.
 */
UCLASS(abstract, DefaultToInstanced, EditInlineNew, MinimalAPI)
class UMovieSceneCustomBinding
	: public UObject
{
public:

	GENERATED_BODY()

	static MOVIESCENE_API const int32 BaseEnginePriority;
	static MOVIESCENE_API const int32 BaseCustomPriority;

	UFUNCTION(BlueprintCallable, Category = "Sequencer|Binding")
	static int32 GetBaseEnginePriority() { return BaseEnginePriority; }

	UFUNCTION(BlueprintCallable, Category = "Sequencer|Binding")
	static int32 GetBaseCustomPriority() { return BaseCustomPriority; }

	/* Must be implemented.
	* Resolve the custom binding based on the passed in context. May return an existing UObject or spawn a new one. 
	*/
	virtual FMovieSceneBindingResolveResult ResolveBinding(const FMovieSceneBindingResolveParams& ResolveParams, int32 BindingIndex, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const PURE_VIRTUAL(UMovieSceneCustomBinding::ResolveBinding, return FMovieSceneBindingResolveResult(););

	/* Returns whether this binding type will spawn an object in the current context. This will be true for Spawnables always, and true for Replaceables in Editor.*/
	virtual bool WillSpawnObject(TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const { return false; }


	/**
	  * @return Custom binding priority in order to sort the list of custom binding types.
	  * If several custom binding types support the creation of bindings from the same object types, the one with the highest priority will be picked.
	  */
	virtual int32 GetCustomBindingPriority() const { return BaseEnginePriority; }

	/*
	* Must be implemented. Called by Sequencer to determine whether this custom binding type supports binding the given object.
	* If true is returned, a new binding may be created using CreateNewCustomBinding.
	*/
	virtual bool SupportsBindingCreationFromObject(const UObject* SourceObject) const PURE_VIRTUAL(UMovieSceneCustomBinding::SupportsBindingCreationFromObject, return false;);

	/*
	* Must be implemented.
	* Called by Sequencer on each Custom Binding class CDO if it supports a UObject type to try to create a new instanced custom binding. 
	* If the derived custom spawnable type supports the passed in object type, this should return a new UMovieSceneCustomBinding instance parented to the passed in OwnerMovieScene.
	* See UMovieSceneSpawnableActorBinding for an example of how to implement.
	*/
	virtual UMovieSceneCustomBinding* CreateNewCustomBinding(UObject* SourceObject, UMovieScene& OwnerMovieScene) PURE_VIRTUAL(UMovieSceneCustomBinding::CreateNewCustomBinding, return nullptr;);

	/*
	* Optional method that can be overridden to return a desired name for the binding. This may be used by sequencer to name the possessable containing the binding.
	*/
	virtual FString GetDesiredBindingName() const { return FString(); }

	/*
	* For custom bindings inheriting from UMovieSceneSpawnableBindingBase, returns this object cast to UMovieSceneSpawnableBindingBase.
	* For custom bindings inheriting from UMovieSceneReplaceableBinding, returns the inner UMovieSceneSpawnableBindingBase* in editor, or nullptr in runtime.
	*/
	virtual const UMovieSceneSpawnableBindingBase* AsSpawnable(TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const { return nullptr; }

	/*
	* For custom bindings inheriting from UMovieSceneSpawnableBindingBase, returns this object cast to UMovieSceneSpawnableBindingBase.
	* For custom bindings inheriting from UMovieSceneReplaceableBinding, returns the inner UMovieSceneSpawnableBindingBase* in editor, or nullptr in runtime.
	*/
	UMovieSceneSpawnableBindingBase* AsSpawnable(TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) { return const_cast<UMovieSceneSpawnableBindingBase*>(const_cast<const UMovieSceneCustomBinding*>(this)->AsSpawnable(SharedPlaybackState)); }

	/*
	* Must be implemented.
	* Should return the most specific relevant class of the bound object. Used to populate the FMovieScenePossessable bound object class.
	*/
	virtual UClass* GetBoundObjectClass() const PURE_VIRTUAL(UMovieSceneCustomBinding::GetBoundObjectClass, return UObject::StaticClass(););


#if WITH_EDITOR

	/*
	* Called by Sequencer upon creating a new custom binding or converting a binding to use this type. Can be used by custom binding types to add required track types, etc.
	*/
	virtual void SetupDefaults(UObject* SpawnedObject, FGuid ObjectBindingId, UMovieScene& OwnerMovieScene) {}
	
	/*
	* Allows the custom binding to optionally provide a custom icon overlay for the object binding track.
	*/
	virtual FSlateIcon GetBindingTrackCustomIconOverlay() const { return FSlateIcon(); }

	/*
	* Allows the custom binding to optionally provide a custom tooltip to show when hovering over the icon area in the object binding track.
	*/
	virtual FText GetBindingTrackIconTooltip() const { return FText(); }

	/*
	* Called by UI code to see if this custom binding type supports conversions from the presented binding, including any current bound or spawned object as reference.
	*/
	virtual bool SupportsConversionFromBinding(const FMovieSceneBindingReference& BindingReference, const UObject* SourceObject) const { return false; }

	/*
	* Called during binding conversion to create a new binding of this type from a selected binding, if supported.
	*/
	virtual UMovieSceneCustomBinding* CreateCustomBindingFromBinding(const FMovieSceneBindingReference& BindingReference, UObject* SourceObject, UMovieScene& OwnerMovieScene) { return nullptr; }

	/*
	* Must be implemented. Used by the UI to describe this binding type during conversions, etc.
	*/
	virtual FText GetBindingTypePrettyName() const PURE_VIRTUAL(UMovieSceneCustomBinding::GetBindingTypePrettyName, return FText(););


	/*
	* Called by UI code to see if this custom binding supports converting to a possessable.
	*/
	virtual bool CanConvertToPossessable(const FGuid& Guid, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const { return true; }

	/*
	* Called by UI code when the binding has recently been added or modified in the case anything needs to be initialized or modified based on this. 
	*/
	virtual void OnBindingAddedOrChanged(UMovieScene& OwnerMovieScene) {}

#endif

};
