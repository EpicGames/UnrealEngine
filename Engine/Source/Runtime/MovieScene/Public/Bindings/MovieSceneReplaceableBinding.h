// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Bindings/MovieSceneCustomBinding.h"
#include "MovieSceneSequenceID.h"
#include "MovieSceneSpawnableBinding.h"
#include "Templates/SubclassOf.h"
#include "MovieSceneReplaceableBinding.generated.h"

#define UE_API MOVIESCENE_API

namespace UE
{
	namespace MovieScene
	{
		struct FSharedPlaybackState;
	}
}
struct FGuid;
class UMovieScene;
struct FSlateBrush;
class UMovieSceneSpawnableBindingBase;

/**
 * The base class for custom replaceable bindings. A replaceable binding uses an internal custom spawnable at editor time to produce a preview object,
 * while in editor will use some other mechanism to dynamically bind an object to the track. Different replaceable types can choose different combinations
 * of how to create a spawnable for preview vs. how to dynamically bind an object at runtime.
 * UMovieSceneReplaceableActorBinding as an example is the simplest type of replaceable binding and provides no method for binding at runtime and relies on the LevelSequenceActor's Binding Override 
 * mechanism to bind an actor at runtime.
 */
UCLASS(MinimalAPI, Abstract)
class UMovieSceneReplaceableBindingBase
	: public UMovieSceneCustomBinding
{
public:

	GENERATED_BODY()

public:

	/* UMovieSceneCustomBinding overrides */
	UE_API virtual bool SupportsBindingCreationFromObject(const UObject* SourceObject) const override;
	UE_API virtual UClass* GetBoundObjectClass() const override;
#if WITH_EDITOR
	UE_API virtual	void SetupDefaults(UObject* SpawnedObject, FGuid ObjectBindingId, UMovieScene& OwnerMovieScene) override;
	UE_API virtual FSlateIcon GetBindingTrackCustomIconOverlay() const override;
	UE_API virtual FText GetBindingTrackIconTooltip() const override;

	UE_API virtual bool SupportsConversionFromBinding(const FMovieSceneBindingReference& BindingReference, const UObject* SourceObject) const override;
	UE_API virtual UMovieSceneCustomBinding* CreateCustomBindingFromBinding(const FMovieSceneBindingReference& BindingReference, UObject* SourceObject, UMovieScene& OwnerMovieScene) override;
#endif

#if WITH_EDITORONLY_DATA
	// Optional Editor-only preview object
	UPROPERTY(Instanced, VisibleAnywhere, BlueprintReadOnly, Category="Editor")
	TObjectPtr<UMovieSceneSpawnableBindingBase> PreviewSpawnable = nullptr;
#endif

public:

	/*
	*  Note that we choose to implement CreateCustomBinding here rather than in subclasses.
	*  Instead we rely on subclasses to implement CreateInnerSpawnable and InitReplaceableBinding which we call here.
	*/
	UE_API UMovieSceneCustomBinding* CreateNewCustomBinding(UObject* SourceObject, UMovieScene& OwnerMovieScene) override final;

protected:
	/** UObject overrides */
	UE_API virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;

	/*
	* Must be implemented. Called during non-editor/runtime to resolve the binding dynamically. In editor worlds/Sequencer will instead use the PreviewSpawnable binding to spawn a preview object.
	* If no object is returned, Sequencer's BindingOverrides can still be used to dynamically bind the object. See UMovieSceneReplaceableActorBinding for an example.
	*/
	virtual FMovieSceneBindingResolveResult ResolveRuntimeBindingInternal(const FMovieSceneBindingResolveParams& ResolveParams, int32 BindingIndex, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const PURE_VIRTUAL(UMovieSceneReplaceableBindingBase::ResolveRuntimeBindingInternal, return FMovieSceneBindingResolveResult(););

	/* Called from CreateNewCustomBinding to create the inner spawnable used for sequencer preview. By default will just call GetInnerSpawnableClass and call CreateNewCustomBinding on that. */
	UE_API virtual UMovieSceneSpawnableBindingBase* CreateInnerSpawnable(UObject* SourceObject, UMovieScene& OwnerMovieScene);

	/*
	* Must be implemented and return a non abstract spawnable binding class inheriting from UMovieSceneSpawnableBindingBase to use for the preview for this replaceable binding.
	*/
	virtual TSubclassOf<UMovieSceneSpawnableBindingBase> GetInnerSpawnableClass() const PURE_VIRTUAL(UMovieSceneReplaceableBindingBase::GetInnerSpawnableClass, return nullptr;);

	/* Must be implemented. Called from CreateNewCustomBinding to allow the replaceable to initialize any data members from the source object. */
	virtual void InitReplaceableBinding(UObject* SourceObject, UMovieScene & OwnerMovieScene) PURE_VIRTUAL(UMovieSceneReplaceableBindingBase::InitReplaceableBinding, return;);

protected:

	/* UMovieSceneCustomBinding overrides*/
	UE_API bool WillSpawnObject(TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const override final;
	UE_API FMovieSceneBindingResolveResult ResolveBinding(const FMovieSceneBindingResolveParams& ResolveParams, int32 BindingIndex, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const override final;
	UE_API const UMovieSceneSpawnableBindingBase* AsSpawnable(TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const override final;

};

#undef UE_API
