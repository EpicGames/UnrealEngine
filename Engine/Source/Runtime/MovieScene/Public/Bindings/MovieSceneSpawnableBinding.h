// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Bindings/MovieSceneCustomBinding.h"
#include "MovieSceneSequenceID.h"
#include "MovieSceneSpawnable.h"
#include "MovieSceneSpawnableBinding.generated.h"

#define UE_API MOVIESCENE_API

namespace UE
{
	namespace MovieScene
	{
		struct FSharedPlaybackState;
	}
}
struct FGuid;
class UMovieScene;
struct FSlateBrush;

/**
 * The base class for custom spawnable bindings. A spawnable binding will spawn an object upon resolution or return a cached previously spawned object.
 * UMovieSceneSpawnableActorBinding is the reimplementation of previous FMovieSceneSpawnable features and spawns an actor based on a saved template and actor class.
 * Otherwise, projects are free to implement their own Spawnable bindings by overriding this class. 
 * In doing so, they could choose to just override GetSpawnObjectClass, PostSpawnObject, and PreDestroyObject for example to do custom post-spawn setup on a character mesh,
 * or they could choose to fully override SpawnObject and DestroySpawnedObject and do their own custom logic for spawning completely.
 */
UCLASS(MinimalAPI, abstract)
class UMovieSceneSpawnableBindingBase
	: public UMovieSceneCustomBinding
{
public:

	GENERATED_BODY()

public:

	/* Called by the Movie Scene Spawner for this spawnable binding to spawn its object. */
	UE_API virtual UObject* SpawnObject(const FGuid& BindingId, int32 BindingIndex, UMovieScene& MovieScene, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState);

	/* Called  by the Movie Scene Spawner right before a spawned object with the specified ID and template ID is destroyed*/
	virtual void PreDestroyObject(UObject* Object, const FGuid& BindingId, int32 BindingIndex, FMovieSceneSequenceIDRef TemplateID) {}

	/* Called by the Movie Scene Spawner to destroy this previously spawned object */
	UE_API virtual void DestroySpawnedObject(UObject* Object);

	// Optional interface functions for spawnables that support object templates.

	/* Override and return true if the binding type supports object templates */
	virtual bool SupportsObjectTemplates() const { return false; }

	/* Override and return the object template if the binding type supports object templates*/
	virtual UObject* GetObjectTemplate() { return nullptr; }

	/**
	 * Sets the object template to the specified object directly.
	 * Used for Copy/Paste, typically you should use CopyObjectTemplate.
	 */
	virtual void SetObjectTemplate(UObject* InObjectTemplate) {}

	/**
	 * Copy the specified object into this spawnable's template
	 */
	virtual void CopyObjectTemplate(UObject* InSourceObject, UMovieSceneSequence& MovieSceneSequence) {}

public:
	/*
	* The spawn ownership setting for this spawnable, allowing spawnables to potentially outlast the lifetime of their sub sequence or sequence altogether.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Spawnable)
	ESpawnOwnership SpawnOwnership = ESpawnOwnership::InnerSequence;

	/** When enabled, this spawnable will always be respawned if it gets destroyed externally. When disabled, this object will only ever be spawned once for each binding lifetime section even if destroyed externally. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category= Spawnable)
	bool bContinuouslyRespawn = false;

public:

#if WITH_EDITOR
	/* UMovieSceneCustomBinding overrides */
	UE_API virtual	void SetupDefaults(UObject* SpawnedObject, FGuid ObjectBindingId, UMovieScene& OwnerMovieScene) override;
	UE_API virtual FSlateIcon GetBindingTrackCustomIconOverlay() const override;
	UE_API virtual FText GetBindingTrackIconTooltip() const override;
#endif
protected:

	
	/* Must be overridden. Handles the actual spawning of the object. Is overridden by UMovieSceneSpawnableActorBindingBase for example to handle actor-specific spawning. */
	virtual UObject* SpawnObjectInternal(UWorld* WorldContext, FName SpawnName, const FGuid& BindingId, int32 BindingIndex, UMovieScene& MovieScene, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState)  PURE_VIRTUAL(UMovieSceneSpawnableBindingBase::SpawnObjectInternal, return nullptr;);

	/* Must be overridden. Handles the actual destruction of the object. Is overridden by UMovieSceneSpawnableActorBindingBase for example to handle actor-specific destruction.*/
	virtual void DestroySpawnedObjectInternal(UObject* Object) PURE_VIRTUAL(UMovieSceneSpawnableBindingBase::DestroySpawnedObjectInternal, return;);

	// Helper functions used by various base implementations- can be overridden to customize spawn behavior.

	/* By default, objects will be spawned in Sequencer's current world context. However, derived classes can override for more specialized behavior*/
	UE_API virtual UWorld* GetWorldContext(TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const;

	/* Should returns the name of the object to be spawned if a custom name desired. If not specified, defaults to creating a unique name from the object class. */
	virtual FName GetSpawnName(const FGuid& BindingId, UMovieScene& MovieScene, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const { return NAME_None; }

	/* Can be used by derived classes to perform custom post spawn setup on an object. */
	virtual void PostSpawnObject(UObject* SpawnedObject, UWorld* WorldContext, const FGuid& BindingId, int32 BindingIndex, UMovieScene& MovieScene, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) {}

protected:

private:
	// Allows Replaceables to access private functions of their inner spawnables.
	friend class UMovieSceneReplaceableBindingBase;

	/* UMovieSceneCustomBinding overrides*/
	bool WillSpawnObject(TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const override final { return true; }
	UE_API FMovieSceneBindingResolveResult ResolveBinding(const FMovieSceneBindingResolveParams& ResolveParams, int32 BindingIndex, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const override final;
	UE_API const UMovieSceneSpawnableBindingBase* AsSpawnable(TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const override final;
};

#undef UE_API
