// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/InlineValue.h"
#include "UObject/Interface.h"
#include "Templates/SubclassOf.h"
#include "IMovieSceneChannelOverrideProvider.generated.h"

class UMovieSceneSectionChannelOverrideRegistry;

namespace UE
{
namespace MovieScene
{

/** Utility class to convert channel names to indices */
struct FChannelOverrideNames
{
	FChannelOverrideNames(int32 InIndexOffset, std::initializer_list<FName> InChannelNames)
		: ChannelNames(InChannelNames)
		, IndexOffset(InIndexOffset)
	{
	}

	int32 GetIndex(const FName& InChannelName) const
	{
		int32 Index = ChannelNames.Find(InChannelName);
		if (Index != INDEX_NONE)
		{
			Index += IndexOffset;
		}
		return Index;
	}

	FName GetChannelName(int32 InIndex) const
	{
		const int32 Index = InIndex - IndexOffset;
		if (ChannelNames.IsValidIndex(Index))
		{
			return ChannelNames[Index];
		}
		return NAME_None;
	}

	TArray<FName> ChannelNames;
	int32 IndexOffset = 0;
};

/**
 *
 */
struct FChannelOverrideProviderTraits
{
	virtual ~FChannelOverrideProviderTraits() = default;

	/**
	 * Gets the channel ID for a given channel.
	 */
	virtual FName GetDefaultChannelTypeName(FName ChannelName) const = 0;

	/**
	 * Convert a channel name to an override entity ID.
	 */
	virtual int32 GetChannelOverrideEntityID(FName ChannelName) const = 0;

	/**
	 * Convert an override entity ID to a channel name.
	 */
	virtual FName GetChannelOverrideName(int32 EntityID) const = 0;
};

using FChannelOverrideProviderTraitsHandle = TInlineValue<FChannelOverrideProviderTraits, 64>;

/**
 *
 */
template<typename DefaultChannelType, int OverrideEntityID = 10>
struct TSingleChannelOverrideProviderTraits : FChannelOverrideProviderTraits
{
	FName SingleChannelName;

	TSingleChannelOverrideProviderTraits(FName InChannelName)
		: SingleChannelName(InChannelName)
	{
	}

	virtual FName GetDefaultChannelTypeName(FName ChannelName) const override
	{
		return DefaultChannelType::StaticStruct()->GetFName();
	}

	virtual int32 GetChannelOverrideEntityID(FName ChannelName) const override
	{
		ensure(ChannelName == SingleChannelName);
		return OverrideEntityID;
	}

	virtual FName GetChannelOverrideName(int32 EntityID) const override
	{
		ensure(EntityID == OverrideEntityID);
		return SingleChannelName;
	}
};

template<typename... OverrideChannelTypes>
struct TNamedChannelOverrideProviderTraits : FChannelOverrideProviderTraits
{
	const FChannelOverrideNames* ChannelNames;
	FName ChannelTypes[sizeof...(OverrideChannelTypes)];

	TNamedChannelOverrideProviderTraits(const FChannelOverrideNames* InChannelNames)
		: ChannelNames(InChannelNames)
		, ChannelTypes{ OverrideChannelTypes::StaticStruct()->GetFName()... }
	{}

	virtual FName GetDefaultChannelTypeName(FName ChannelName) const override
	{
		const int32 ChannelIndex = ChannelNames->GetIndex(ChannelName);
		check(ChannelIndex >= 0 && ChannelIndex < sizeof(ChannelTypes));
		return ChannelTypes[ChannelIndex];
	}

	virtual int32 GetChannelOverrideEntityID(FName ChannelName) const override
	{
		return ChannelNames->GetIndex(ChannelName);
	}

	virtual FName GetChannelOverrideName(int32 EntityID) const override
	{
		return ChannelNames->GetChannelName(EntityID);
	}
};

}  // namespace MovieScene
}  // namespace UE

UINTERFACE(MinimalAPI)
class UMovieSceneChannelOverrideProvider : public UInterface
{
public:

	GENERATED_BODY()
};

/**
 * Interface to be added to UMovieSceneSection types when they contain entity data
 */
class IMovieSceneChannelOverrideProvider
{
public:

	using FChannelOverrideProviderTraitsHandle = UE::MovieScene::FChannelOverrideProviderTraitsHandle;

	GENERATED_BODY()

	/**
	 * Gets the channel override container.
	 */
	virtual UMovieSceneSectionChannelOverrideRegistry* GetChannelOverrideRegistry(bool bCreateIfMissing) = 0;

	/**
	 * Gets the naming/indexing traits for this provider.
	 */
	virtual FChannelOverrideProviderTraitsHandle GetChannelOverrideProviderTraits() const = 0;

	/**
	 * Called when channel overrides have been added or removed. Should invalidate the channel proxy.
	 */
	virtual void OnChannelOverridesChanged() = 0;
};

