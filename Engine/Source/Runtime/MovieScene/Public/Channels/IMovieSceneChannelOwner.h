// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/Interface.h"
#include "IMovieSceneChannelOwner.generated.h"

namespace UE::MovieScene
{
	struct FChannelOwnerCapabilities
	{
		FChannelOwnerCapabilities()
			: bSupportsMute (false)
		{}

		uint8 bSupportsMute : 1;
	};
} // namespace UE::MovieScene


/**
 * Interface that can be added to a channel owner to implement various opt-in behaviors for that channel
 */
UINTERFACE(MinimalAPI)
class UMovieSceneChannelOwner : public UInterface
{
public:

	GENERATED_BODY()
};

class IMovieSceneChannelOwner
{
public:

	GENERATED_BODY()

	/**
	 * Retrieve the capabilities for the channel on this interface
	 **/
	virtual UE::MovieScene::FChannelOwnerCapabilities GetCapabilities(FName ChannelName) const
	{
		return UE::MovieScene::FChannelOwnerCapabilities();
	}

	/**
	 * Check whether the specified channel is muted.
	 * @note: Only valid to be called if GetCapabilities(ChannelName).bSupportsMute == true
	 */
	virtual bool IsMuted(FName ChannelName) const
	{
		return false;
	}

	/**
	 * Set the mute state for the specified channel.
	 * @note: Only valid to be called if GetCapabilities(ChannelName).bSupportsMute == true
	 */
	virtual void SetIsMuted(FName ChannelName, bool bIsMuted)
	{
	}
};

