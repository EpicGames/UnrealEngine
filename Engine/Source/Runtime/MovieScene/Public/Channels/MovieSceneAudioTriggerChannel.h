// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannel.h"
#include "Channels/MovieSceneChannelEditorData.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreTypes.h"
#include "Evaluation/MovieScenePlayback.h"
#include "Math/Range.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameTime.h"
#include "Misc/Optional.h"
#include "MovieSceneChannelData.h"
#include "MovieSceneChannelTraits.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/ObjectMacros.h"

#include "MovieSceneAudioTriggerChannel.generated.h"

struct FFrameRate;
struct FKeyHandle;

struct FMoveSceneAudioTriggerState
{	
	TOptional<int32> PreviousIndex;
	TOptional<FFrameTime> PreviousUpdateTime;
};

USTRUCT()
struct FMovieSceneAudioTriggerChannel : public FMovieSceneChannel
{
	GENERATED_BODY()

	FMovieSceneAudioTriggerChannel() = default;

	/**
	 * Access a mutable interface for this channel's data
	 *
	 * @return An object that is able to manipulate this channel's data
	 */
	inline TMovieSceneChannelData<bool> GetData()
	{
		return TMovieSceneChannelData<bool>(&Times, &Values, this, &KeyHandles);
	}

	/**
	 * Access a constant interface for this channel's data
	 *
	 * @return An object that is able to interrogate this channel's data
	 */
	inline TMovieSceneChannelData<const bool> GetData() const
	{
		return TMovieSceneChannelData<const bool>(&Times, &Values);
	}

	/**
	 * Const access to this channel's times
	 */
	inline TArrayView<const FFrameNumber> GetTimes() const
	{
		return Times;
	}

	/**
	 * Check whether this channel has any data
	 */
	inline bool HasAnyData() const
	{
		return Times.Num() != 0;
	}

	/**
	 * Evaluate this channel
	 *
	 * @param InContext   Scene Context with time and directionality necessary to compute triggers
	 * @param InState    A persistent state regard if the trigger has fired
	 * @param OutTriggered   A Value that if set to true combined with the function returning true means there's a trigger
	 * @return true if the channel was evaluated successfully, false otherwise
	 */
	MOVIESCENE_API bool EvaluatePossibleTriggers(const FMovieSceneContext& InContext, FMoveSceneAudioTriggerState& InState, bool& OutTriggered) const;

public:

	// ~ FMovieSceneChannel Interface
	MOVIESCENE_API virtual void GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles) override;
	MOVIESCENE_API virtual void GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes) override;
	MOVIESCENE_API virtual void SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes) override;
	MOVIESCENE_API virtual void DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles) override;
	MOVIESCENE_API virtual void DeleteKeys(TArrayView<const FKeyHandle> InHandles) override;
	MOVIESCENE_API virtual void DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore) override;
	MOVIESCENE_API virtual void RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer) override;
	MOVIESCENE_API virtual TRange<FFrameNumber> ComputeEffectiveRange() const override;
	MOVIESCENE_API virtual int32 GetNumKeys() const override;
	MOVIESCENE_API virtual void Reset() override;
	MOVIESCENE_API virtual void Offset(FFrameNumber DeltaPosition) override;
	MOVIESCENE_API virtual void Optimize(const FKeyDataOptimizationParams& InParameters) override;
	MOVIESCENE_API virtual FKeyHandle GetHandle(int32 Index) override;
	MOVIESCENE_API virtual int32 GetIndex(FKeyHandle Handle) override;
private:

	UPROPERTY(meta = (KeyTimes))
	TArray<FFrameNumber> Times;

	// These are all 1s, but are required for the templates to bind correctly
	UPROPERTY(meta = (KeyValues))
	TArray<bool> Values;

	/** This needs to be a UPROPERTY so it gets saved into editor transactions but transient so it doesn't get saved into assets. */
	UPROPERTY(Transient)
	FMovieSceneKeyHandleMap KeyHandles;
};

inline bool EvaluateChannel(const FMovieSceneAudioTriggerChannel* InChannel, FFrameTime InTime, bool& OutValue)
{
	return false;
}

template<>
struct TMovieSceneChannelTraits<FMovieSceneAudioTriggerChannel> : TMovieSceneChannelTraitsBase<FMovieSceneAudioTriggerChannel>
{
	enum { SupportsDefaults = false };

#if WITH_EDITORONLY_DATA

	/** Bool channels can have external values (ie, they can get their values from external objects for UI purposes) */
	typedef TMovieSceneExternalValue<bool> ExtendedEditorDataType;

#endif
};
