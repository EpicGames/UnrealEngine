// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannelEditorData.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreTypes.h"
#include "HAL/PlatformCrt.h"
#include "Math/Range.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameTime.h"
#include "Misc/Optional.h"
#include "MovieSceneChannel.h"
#include "MovieSceneChannelData.h"
#include "MovieSceneChannelTraits.h"
#include "Serialization/StructuredArchive.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"
#include "Curves/RealCurve.h"

#include "MovieSceneBoolChannel.generated.h"

struct FFrameRate;
struct FKeyHandle;
struct FPropertyTag;


USTRUCT()
struct FMovieSceneBoolChannel : public FMovieSceneChannel
{
	GENERATED_BODY()

	typedef bool CurveValueType;

	FMovieSceneBoolChannel()
		: PreInfinityExtrap(RCCE_Constant), PostInfinityExtrap(RCCE_Constant), DefaultValue(false), bHasDefaultValue(false)
	{}

	/**
	 * Serialize this type from another
	 */
	MOVIESCENE_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	/**
	 * Access a mutable interface for this channel's data
	 *
	 * @return An object that is able to manipulate this channel's data
	 */
	virtual inline TMovieSceneChannelData<bool> GetData()
	{
		return TMovieSceneChannelData<bool>(&Times, &Values, this, &KeyHandles);
	}

	/**
	 * Access a constant interface for this channel's data
	 *
	 * @return An object that is able to interrogate this channel's data
	 */
	virtual inline TMovieSceneChannelData<const bool> GetData() const
	{
		return TMovieSceneChannelData<const bool>(&Times, &Values);
	}

	/**
	 * Const access to this channel's times
	 */
	inline TArrayView<const FFrameNumber> GetTimes() const
	{
		return Times;
	}

	/**
	 * Const access to this channel's values
	 */
	inline TArrayView<const bool> GetValues() const
	{
		return Values;
	}

	/**
	 * Check whether this channel has any data
	 */
	inline bool HasAnyData() const
	{
		return Times.Num() != 0 || bHasDefaultValue == true;
	}

	/**
	 * Evaluate this channel
	 *
	 * @param InTime     The time to evaluate at
	 * @param OutValue   A value to receive the result
	 * @return true if the channel was evaluated successfully, false otherwise
	 */
	MOVIESCENE_API virtual bool Evaluate(FFrameTime InTime, bool& OutValue) const;

	/**
	 * Add keys with these times to channel. The number of elements in both arrays much match or nothing is added.
	 * @param InTimes Times to add
	 * @param InValues Values to add
	 */
	inline void AddKeys(const TArray<FFrameNumber>& InTimes, const TArray<bool>& InValues)
	{
		check(InTimes.Num() == InValues.Num());
		Times.Append(InTimes);
		Values.Append(InValues);
	}

public:

	// ~ FMovieSceneChannel Interface
	MOVIESCENE_API virtual void GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles) override;
	MOVIESCENE_API virtual void GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes) override;
	MOVIESCENE_API virtual void SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes) override;
	MOVIESCENE_API virtual void DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles) override;
	MOVIESCENE_API virtual void DeleteKeys(TArrayView<const FKeyHandle> InHandles) override;
	MOVIESCENE_API virtual void DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore) override;
	MOVIESCENE_API virtual void RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer) override;
	MOVIESCENE_API virtual TRange<FFrameNumber> ComputeEffectiveRange() const override;
	MOVIESCENE_API virtual int32 GetNumKeys() const override;
	MOVIESCENE_API virtual void Reset() override;
	MOVIESCENE_API virtual void Offset(FFrameNumber DeltaPosition) override;
	MOVIESCENE_API virtual void Optimize(const FKeyDataOptimizationParams& InParameters) override;
	MOVIESCENE_API virtual void ClearDefault() override;
	MOVIESCENE_API virtual FKeyHandle GetHandle(int32 Index) override;
	MOVIESCENE_API virtual int32 GetIndex(FKeyHandle Handle) override;

public:

	/**
	 * Set this channel's default value that should be used when no keys are present
	 *
	 * @param InDefaultValue The desired default value
	 */
	inline void SetDefault(bool InDefaultValue)
	{
		bHasDefaultValue = true;
		DefaultValue = InDefaultValue;
	}

	/**
	 * Get this channel's default value that will be used when no keys are present
	 *
	 * @return (Optional) The channel's default value
	 */
	inline TOptional<bool> GetDefault() const
	{
		return bHasDefaultValue ? TOptional<bool>(DefaultValue) : TOptional<bool>();
	}

	/**
	 * Remove this channel's default value causing the channel to have no effect where no keys are present
	 */
	inline void RemoveDefault()
	{
		bHasDefaultValue = false;
	}
public:
	/** Pre-infinity extrapolation state, bool channel only supports constant, cycle and oscillate */
	UPROPERTY()
	TEnumAsByte<ERichCurveExtrapolation> PreInfinityExtrap;

	/** Post-infinity extrapolation state, bool channel only supports  constant, cycle and oscillate */
	UPROPERTY()
	TEnumAsByte<ERichCurveExtrapolation> PostInfinityExtrap;

protected:

	UPROPERTY(meta=(KeyTimes))
	TArray<FFrameNumber> Times;

	UPROPERTY()
	bool DefaultValue;

	UPROPERTY()
	bool bHasDefaultValue;

	UPROPERTY(meta=(KeyValues))
	TArray<bool> Values;

	/** This needs to be a UPROPERTY so it gets saved into editor transactions but transient so it doesn't get saved into assets. */
	UPROPERTY(Transient)
	FMovieSceneKeyHandleMap KeyHandles;
};

template<>
struct TStructOpsTypeTraits<FMovieSceneBoolChannel> : public TStructOpsTypeTraitsBase2<FMovieSceneBoolChannel>
{
	enum { WithStructuredSerializeFromMismatchedTag = true };
};

template<>
struct TMovieSceneChannelTraits<FMovieSceneBoolChannel> : TMovieSceneChannelTraitsBase<FMovieSceneBoolChannel>
{
#if WITH_EDITORONLY_DATA

	/** Bool channels can have external values (ie, they can get their values from external objects for UI purposes) */
	typedef TMovieSceneExternalValue<bool> ExtendedEditorDataType;

#endif
};
