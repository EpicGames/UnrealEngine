// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannelEditorData.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Math/Range.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameTime.h"
#include "Misc/Optional.h"
#include "MovieSceneChannel.h"
#include "MovieSceneChannelData.h"
#include "MovieSceneChannelTraits.h"
#include "Serialization/StructuredArchive.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "Curves/RealCurve.h"

#include "MovieSceneByteChannel.generated.h"

struct FFrameRate;
struct FKeyHandle;
struct FPropertyTag;


USTRUCT()
struct FMovieSceneByteChannel : public FMovieSceneChannel
{
	GENERATED_BODY()

	typedef uint8 CurveValueType;

	FMovieSceneByteChannel()
		: PreInfinityExtrap(RCCE_Constant)
		, PostInfinityExtrap(RCCE_Constant)
		, DefaultValue(0)
		, bHasDefaultValue(false)
		, Enum(nullptr)
		, KeyHandles()
	{}

	/**
	 * Serialize this type from another
	 */
	MOVIESCENE_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	/**
	 * Access a mutable interface for this channel's data
	 *
	 * @return An object that is able to manipulate this channel's data
	 */
	inline TMovieSceneChannelData<uint8> GetData()
	{
		return TMovieSceneChannelData<uint8>(&Times, &Values, this, &KeyHandles);
	}

	/**
	 * Access a constant interface for this channel's data
	 *
	 * @return An object that is able to interrogate this channel's data
	 */
	inline TMovieSceneChannelData<const uint8> GetData() const
	{
		return TMovieSceneChannelData<const uint8>(&Times, &Values);
	}

	/**
	 * Const access to this channel's times
	 */
	inline TArrayView<const FFrameNumber> GetTimes() const
	{
		return Times;
	}

	/**
	 * Const access to this channel's values
	 */
	inline TArrayView<const uint8> GetValues() const
	{
		return Values;
	}

	/**
	 * Check whether this channel has any data
	 */
	inline bool HasAnyData() const
	{
		return Times.Num() != 0 || bHasDefaultValue == true;
	}

	/**
	 * Evaluate this channel
	 *
	 * @param InTime     The time to evaluate at
	 * @param OutValue   A value to receive the result
	 * @return true if the channel was evaluated successfully, false otherwise
	 */
	MOVIESCENE_API bool Evaluate(FFrameTime InTime, uint8& OutValue) const;

	/**
	 * Add keys with these times to channel. The number of elements in both arrays much match or nothing is added.
	 * @param InTimes Times to add
	 * @param InValues Values to add
	 */
	inline void AddKeys(const TArray<FFrameNumber>& InTimes, const TArray<uint8>& InValues)
	{
		check(InTimes.Num() == InValues.Num());
		Times.Append(InTimes);
		Values.Append(InValues);
	}

public:

	// ~ FMovieSceneChannel Interface
	MOVIESCENE_API virtual void GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles) override;
	MOVIESCENE_API virtual void GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes) override;
	MOVIESCENE_API virtual void SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes) override;
	MOVIESCENE_API virtual void DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles) override;
	MOVIESCENE_API virtual void DeleteKeys(TArrayView<const FKeyHandle> InHandles) override;
	MOVIESCENE_API virtual void DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore) override;
	MOVIESCENE_API virtual void RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer) override;
	MOVIESCENE_API virtual TRange<FFrameNumber> ComputeEffectiveRange() const override;
	MOVIESCENE_API virtual int32 GetNumKeys() const override;
	MOVIESCENE_API virtual void Reset() override;
	MOVIESCENE_API virtual void Offset(FFrameNumber DeltaPosition) override;
	MOVIESCENE_API virtual void Optimize(const FKeyDataOptimizationParams& InParameters) override;
	MOVIESCENE_API virtual void ClearDefault() override;

public:

	/**
	 * Set this channel's default value that should be used when no keys are present
	 *
	 * @param InDefaultValue The desired default value
	 */
	inline void SetDefault(uint8 InDefaultValue)
	{
		bHasDefaultValue = true;
		DefaultValue = InDefaultValue;
	}

	/**
	 * Get this channel's default value that will be used when no keys are present
	 *
	 * @return (Optional) The channel's default value
	 */
	inline TOptional<uint8> GetDefault() const
	{
		return bHasDefaultValue ? TOptional<uint8>(DefaultValue) : TOptional<uint8>();
	}

	/**
	 * Remove this channel's default value causing the channel to have no effect where no keys are present
	 */
	inline void RemoveDefault()
	{
		bHasDefaultValue = false;
	}


public:

	UEnum* GetEnum() const
	{
		return Enum;
	}

	void SetEnum(UEnum* InEnum)
	{
		Enum = InEnum;
	}

public:
	/** Pre-infinity extrapolation state, byte channel only supports constant, cycle and oscillate */
	UPROPERTY()
	TEnumAsByte<ERichCurveExtrapolation> PreInfinityExtrap;

	/** Post-infinity extrapolation state, byte channel only supports constant, cycle and oscillate */
	UPROPERTY()
	TEnumAsByte<ERichCurveExtrapolation> PostInfinityExtrap;

private:

	UPROPERTY(meta=(KeyTimes))
	TArray<FFrameNumber> Times;

	UPROPERTY()
	uint8 DefaultValue;

	UPROPERTY()
	bool bHasDefaultValue;

	UPROPERTY(meta=(KeyValues))
	TArray<uint8> Values;

	UPROPERTY()
	TObjectPtr<UEnum> Enum;

	/** This needs to be a UPROPERTY so it gets saved into editor transactions but transient so it doesn't get saved into assets. */
	UPROPERTY(Transient)
	FMovieSceneKeyHandleMap KeyHandles;
};

template<>
struct TStructOpsTypeTraits<FMovieSceneByteChannel> : public TStructOpsTypeTraitsBase2<FMovieSceneByteChannel>
{
	enum { WithStructuredSerializeFromMismatchedTag = true };
};

template<>
struct TMovieSceneChannelTraits<FMovieSceneByteChannel> : TMovieSceneChannelTraitsBase<FMovieSceneByteChannel>
{
#if WITH_EDITOR

	/** Byte channels can have external values (ie, they can get their values from external objects for UI purposes) */
	typedef TMovieSceneExternalValue<uint8> ExtendedEditorDataType;

#endif
};
