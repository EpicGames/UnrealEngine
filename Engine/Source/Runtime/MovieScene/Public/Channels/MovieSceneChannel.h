// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Math/Range.h"
#include "Misc/CoreDefines.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "Curves/KeyHandle.h"
#include "UObject/ObjectMacros.h"

#include "MovieSceneChannel.generated.h"

struct FFrameNumber;
struct FFrameRate;
struct FKeyDataOptimizationParams;
struct FMovieSceneChannel;

namespace UE::MovieScene
{
	struct IRetimingInterface;
}

/*
*  Events that are fired when adding, deleting or moving keys
*/
/**
*  Item that's sent  when an event is fired when we delete or add a key
*/
struct FKeyAddOrDeleteEventItem
{
	FKeyAddOrDeleteEventItem(int32 InIndex, FFrameNumber InFrame) : Index(InIndex), Frame(InFrame) {};
	int32 Index;
	FFrameNumber Frame;;
};

/**
*  Item that's sent when an event is fired when we move a key in time
*/
struct FKeyMoveEventItem
{
	FKeyMoveEventItem(int32 InIndex, FFrameNumber InFrame, int32 InNewIndex, FFrameNumber InNewFrame) : Index(InIndex), Frame(InFrame), NewIndex(InNewIndex), NewFrame(InNewFrame) {};
	int32 Index;
	FFrameNumber Frame;
	int32 NewIndex;
	FFrameNumber NewFrame;
};
/*
*  Class to help with pre/post cycling of values
*/
namespace UE
{
namespace MovieScene
{
struct FCycleParams
{
	double ValueOffset;
	FFrameTime Time;
	int32 CycleCount;
	int32 Duration;
	bool bMirrorCurve;

	FCycleParams(FFrameTime InTime, int32 InDuration)
		: ValueOffset(0.0)
		, Time(InTime)
		, CycleCount(0)
		, Duration(InDuration)
		, bMirrorCurve(false)
	{}

	inline void ComputePreValueOffset(double FirstValue, double LastValue)
	{
		// CycleCount is negative for pre-extrap
		ValueOffset = (LastValue - FirstValue) * CycleCount;
	}
	inline void ComputePostValueOffset(double FirstValue, double LastValue)
	{
		ValueOffset = (LastValue - FirstValue) * CycleCount;
	}
	inline bool ShouldMirrorCurve() const
	{
		return bMirrorCurve;
	}
	inline void Oscillate(int32 MinFrame, int32 MaxFrame)
	{
		if (FMath::Abs(CycleCount) % 2 == 1)
		{
			bMirrorCurve = true;

			const int64 Offset = int64(MaxFrame) - int64(Time.FrameNumber.Value);
			Time = FFrameTime(MinFrame) + FFrameTime(static_cast<int32>(Offset)) - FFrameTime(0, Time.GetSubFrame());
		}
	}
};
MOVIESCENE_API  FCycleParams CycleTime(FFrameNumber MinFrame, FFrameNumber MaxFrame, FFrameTime InTime);

} // namespace MovieScene
} // namespace UE
/*
* Note if any Channel uses these delegate's they need a custom serializer to make sure they stick around on undo/redo. Dynamic delegates are too heavy.
*/
DECLARE_MULTICAST_DELEGATE_TwoParams(FMovieSceneChannelDataKeyAddedEvent, FMovieSceneChannel*, const TArray<FKeyAddOrDeleteEventItem>& Items);
DECLARE_MULTICAST_DELEGATE_TwoParams(FMovieSceneChannelDataKeyDeletedEvent, FMovieSceneChannel*, const TArray<FKeyAddOrDeleteEventItem>& Items);
DECLARE_MULTICAST_DELEGATE_TwoParams(FMovieSceneChannelDataKeyMovedEvent, FMovieSceneChannel*, const TArray<FKeyMoveEventItem>& Items);

USTRUCT()
struct FMovieSceneChannel
{
	GENERATED_BODY()

	FMovieSceneChannel() {}

	virtual ~FMovieSceneChannel() = default;

	/**
	 * Get the time for the specified key handle
	 *
	 * @param InHandle              The handle of the key to get the time for
	 * @param OutKeyTime            Out parameter to receive the key's time
	 */
	MOVIESCENE_API void GetKeyTime(const FKeyHandle InHandle, FFrameNumber& OutKeyTime);

	/**
	 * Set the time for the specified key handle
	 *
	 * @param InHandle              The handle of the key to get the time for
	 * @param InKeyTime             The new time for the key
	 */
	MOVIESCENE_API void SetKeyTime(const FKeyHandle InHandle, const FFrameNumber InKeyTime);

	/**
	 * Get key information pertaining to all keys that exist within the specified range
	 *
	 * @param WithinRange           The range within which to return key information
	 * @param OutKeyTimes           (Optional) Array to receive key times
	 * @param OutKeyHandles         (Optional) Array to receive key handles
	 */
	virtual void GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles)
	{}

	/**
	 * Get all key times for the specified key handles
	 *
	 * @param InHandles             Array of handles to get times for
	 * @param OutKeyTimes           Pre-sized array of key times to set. Invalid key handles will not assign to this array. Must match size of InHandles
	 */
	virtual void GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes)
	{}

	/**
	 * Set key times for the specified key handles
	 *
	 * @param InHandles             Array of handles to get times for
	 * @param InKeyTimes            Array of times to apply - one per handle
	 */
	virtual void SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes)
	{}

	/**
	 * Duplicate the keys for the specified key handles
	 *
	 * @param InHandles             Array of handles to duplicate
	 * @param OutKeyTimes           Pre-sized array to receive duplicated key handles. Invalid key handles will not be assigned to this array. Must match size of InHandles
	 */
	virtual void DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles)
	{}

	/**
	 * Delete the keys for the specified key handles
	 *
	 * @param InHandles             Array of handles to delete
	 */
	virtual void DeleteKeys(TArrayView<const FKeyHandle> InHandles)
	{}

	/**
	 * Delete keys before or after a specified time
	 *
	 * @param InTime				Delete keys after this time
	 * @param bDeleteKeysBefore     Whether to delete keys before the specified time
	 */
	virtual void DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore)
	{}

	/**
	 * Called when the frame resolution of this channel is to be changed.
	 *
	 * @param SourceRate      The previous frame resolution that the channel is currently in
	 * @param DestinationRate The desired new frame resolution. All keys should be transformed into this time-base.
	 */
	UE_DEPRECATED(5.6, "Please use RemapTimes")
	MOVIESCENE_API virtual void ChangeFrameResolution(FFrameRate SourceRate, FFrameRate DestinationRate);

	/**
	 * Remap the times contained within this channel using a custom remapper
	 */
	virtual void RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer)
	{}

	/**
	 * Compute the effective range of this channel, for example, the extents of its key times
	 *
	 * @return A range that represents the greatest range of times occupied by this channel, in the sequence's frame resolution
	 */
	virtual TRange<FFrameNumber> ComputeEffectiveRange() const
	{
		return TRange<FFrameNumber>::Empty();
	}

	/**
	 * Get the total number of keys on this channel
	 *
	 * @return The number of keys on this channel
	 */
	virtual int32 GetNumKeys() const
	{
		return 0;
	}

	/**
	 * Reset this channel back to its original state
	 */
	virtual void Reset()
	{}

	/**
	 * Offset the keys within this channel by a given delta position
	 *
	 * @param DeltaPosition   The number of frames to offset by, in the sequence's frame resolution
	 */
	virtual void Offset(FFrameNumber DeltaPosition)
	{}

	/**
	 * Optimize this channel by removing any redundant data according to the specified parameters
	 *
	 * @param InParameters    Parameter struct specifying how to optimize the channel
	 */
	virtual void Optimize(const FKeyDataOptimizationParams& InParameters)
	{}

	/**
	 * Clear all the default value on this channel
	 */
	virtual void ClearDefault()
	{}

	/**
	 * Perform a possibly heavy operation after an edit change 
	 *
	 */
	virtual void PostEditChange() {}

	/**
	* Retrieve a key handle for the specified key time index
	*
	* @param Index          The index to retrieve
	* @return A key handle that identifies the key at the specified index, regardless of re-ordering
	*/
	virtual FKeyHandle GetHandle(int32 Index)
	{
		return FKeyHandle::Invalid();
	}

	/**
	 * Attempt to retrieve the index of key from its handle
	 *
	 * @param Handle         The handle to retrieve
	 * @return The index of the key, or INDEX_NONE
	 */
	virtual int32 GetIndex(FKeyHandle Handle)
	{
		return INDEX_NONE;
	}

public:

	/**
	* Get delegate that's called when a key is added
	*@return Returns key added event delegate
	*/
	FMovieSceneChannelDataKeyAddedEvent& OnKeyAddedEvent() { return KeyAddedEvent; }
	/**
	* Get delegate that's called when a key is deleted
	*@return Returns key deleted event delegate
	*/
	FMovieSceneChannelDataKeyDeletedEvent& OnKeyDeletedEvent() { return KeyDeletedEvent; }
	/**
	* Get delegate that's called when a key is moved
	*@return Returns key moved event delegate
	*/
	FMovieSceneChannelDataKeyMovedEvent& OnKeyMovedEvent() { return KeyMovedEvent; }

protected:
	/** Broadcasts a notification whenever key is added */
	FMovieSceneChannelDataKeyAddedEvent KeyAddedEvent;

	/** Broadcasts a notification whenever key is deleted */
	FMovieSceneChannelDataKeyDeletedEvent KeyDeletedEvent;

	/** Broadcasts a notification whenever key is moved */
	FMovieSceneChannelDataKeyMovedEvent KeyMovedEvent;

};
