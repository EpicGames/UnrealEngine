// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannel.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreTypes.h"
#include "Curves/KeyHandle.h"
#include "Math/Range.h"
#include "Math/RangeBound.h"
#include "Misc/AssertionMacros.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameTime.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"
#include "IMovieSceneRetimingInterface.h"
#include "Algo/RemoveIf.h"

#include "MovieSceneChannelData.generated.h"

struct FFrameRate;
struct FKeyDataOptimizationParams;

namespace UE::MovieScene
{
	template<typename ValueType>
	void OnRemapChannelKeyTime(const FMovieSceneChannel* Channel, const IRetimingInterface& Retimer, FFrameNumber PreviousTime, FFrameNumber CurrentTime, ValueType& InOutValue)
	{}
}

/** A map of key handles that is copyable, but does not copy data on copy */
USTRUCT()
struct FMovieSceneKeyHandleMap : public FKeyHandleLookupTable
{
	GENERATED_BODY()

public:
	FMovieSceneKeyHandleMap() = default;
	FMovieSceneKeyHandleMap(const FMovieSceneKeyHandleMap& RHS){}
	FMovieSceneKeyHandleMap& operator=(const FMovieSceneKeyHandleMap& RHS)
	{
		Reset();
		return *this;
	}
};

template<>
struct TStructOpsTypeTraits<FMovieSceneKeyHandleMap>
	: public TStructOpsTypeTraitsBase2<FMovieSceneKeyHandleMap>
{
	enum
	{
		WithSerializer = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

namespace UE
{
namespace MovieScene
{
	/**
	 * Evaluate the specified time array by finding the two indices that are adjacent to the supplied time
	 *
	 * @param InTimes        A sorted array of frame numbers
	 * @param InTime         The time to find within the array
	 * @param OutIndex1      The first time in the array that's >= InTime, or INDEX_NONE if there are none
	 * @param OutIndex2      OutIndex1 + 1 if it is a valid index in the array, INDEX_NONE otherwise
	 */
	MOVIESCENE_API void EvaluateTime(TArrayView<const FFrameNumber> InTimes, FFrameTime InTime, int32& OutIndex1, int32& OutIndex2);


	/**
	 * Evaluate the specified time array by finding the two indices and an interpolation value that are adjacent to the supplied time
	 *
	 * @param InTimes        A sorted array of frame numbers
	 * @param InTime         The time to find within the array
	 * @param OutIndex1      The first time in the array that's >= InTime, or INDEX_NONE if there are none
	 * @param OutIndex2      OutIndex1 + 1 if it is a valid index in the array, INDEX_NONE otherwise
	 * @param OutInterp      A value from 0.0 -> 1.0 specifying how a linear interpolation value from index 1 to index 2
	 */
	MOVIESCENE_API void EvaluateTime(TArrayView<const FFrameNumber> InTimes, FFrameTime InTime, int32& OutIndex1, int32& OutIndex2, double& OutInterp);


	/**
	 * Find the range of times that fall around PredicateTime +/- InTolerance up to a maximum
	 *
	 * @param InTimes        A sorted array of frame numbers
	 * @param PredicateTime  The time around which to search
	 * @param InTolerance    The tolerance range to search around PredicateTime with
	 * @param MaxNum         A maximum number of times to find, starting with those closest to the predicate time
	 * @param OutMin         The earliest index that met the conditions of the search
	 * @param OutMax         The latest index that met the conditions of the search
	 */
	MOVIESCENE_API void FindRange(TArrayView<const FFrameNumber> InTimes, FFrameNumber PredicateTime, FFrameNumber InTolerance, int32 MaxNum, int32& OutMin, int32& OutMax);

} // namespace MovieScene
} // namespace UE


/**
 * Base class channel data utility that provides a consistent interface to a sorted array of times and handles.
 * Complete access should be through TMovieSceneChannelData that allows mutation of the data
 */
struct FMovieSceneChannelData
{
	/**
	 * Read-only access to this channel's key times.
	 */
	inline TArrayView<const FFrameNumber> GetTimes() const
	{
		return *Times;
	}

	/**
	 * Mutable access to this channel's key times.
	 * @note: *Warning*: any usage *must* keep times sorted. Any reordering of times will not be reflected in the values array.
	 */
	inline TArrayView<FFrameNumber> GetTimes()
	{
		return *Times;
	}

	/**
	 * Retrieve a key handle for the specified key time index
	 *
	 * @param Index          The index to retrieve
	 * @return A key handle that identifies the key at the specified index, regardless of re-ordering
	 */
	MOVIESCENE_API FKeyHandle GetHandle(int32 Index);

	/**
	 * Attempt to retrieve the index of key from its handle
	 *
	 * @param Handle         The handle to retrieve
	 * @return The index of the key, or INDEX_NONE
	 */
	MOVIESCENE_API int32 GetIndex(FKeyHandle Handle);

	/**
	 * Attempt to find a key at a given time and tolerance
	 *
	 * @param InTime         The time at which to search
	 * @param InTolerance    A tolerance of frame numbers to allow either side of the specified time
	 * @return The index of the key closest to InTime and within InTolerance, or INDEX_NONE
	 */
	MOVIESCENE_API int32 FindKey(FFrameNumber InTime, FFrameNumber InTolerance = 0);

	/**
	 * Find the range of keys that fall around InTime +/- InTolerance up to a maximum
	 *
	 * @param InTime         The time around which to search
	 * @param MaxNum         A maximum number of times to find, starting with those closest to the predicate time
	 * @param OutMin         The earliest index that met the conditions of the search
	 * @param OutMax         The latest index that met the conditions of the search
	 * @param InTolerance    The tolerance range to search around PredicateTime with
	 */
	MOVIESCENE_API void FindKeys(FFrameNumber InTime, int32 MaxNum, int32& OutMinIndex, int32& OutMaxIndex, int32 InTolerance);

	/**
	 * Compute the total time range of the channel data.
	 *
	 * @return The range of this channel data
	 */
	MOVIESCENE_API TRange<FFrameNumber> GetTotalRange() const;

	/**
	 * Convert the frame resolution of a movie scene channel by moving the key times to the equivalent frame time
	 *
	 * @param SourceRate      The frame rate the channel is currently in
	 * @param DestinationRate The new frame rate to convert the channel to
	 */
	UE_DEPRECATED(5.6, "Please use RemapTimes")
	MOVIESCENE_API void ChangeFrameResolution(FFrameRate SourceRate, FFrameRate DestinationRate);

	/**
	 * Get all the keys in the given range. Resulting arrays must be the same size where indices correspond to both arrays.
	 *
	 * @param WithinRange        The bounds to get keys for
	 * @param OutKeyTimes        Array to receive all key times within the given range
	 * @param OutKeyHandles      Array to receive all key handles within the given range
	 */
	MOVIESCENE_API void GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles);

	/**
	 * Get key times for a number of keys in the channel data
	 *
	 * @param InHandles          Array of key handles that should have their times set
	 * @param OutKeyTimes        Array of times that should be set for each key handle. Must be exactly the size of InHandles
	 */
	MOVIESCENE_API void GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes);

	/**
	 * Offset the channel data by a given delta time
	 *
	 * @param DeltaTime     The time to offset by
	 */
	MOVIESCENE_API void Offset(FFrameNumber DeltaTime);

protected:

	/**
	 * Constructor that takes a non-owning pointer to an array of times and a key handle map
	 *
	 * @param InTimes        A pointer to an array that should be operated on by this class. Externally owned.
	 * @param InKeyHandles   A key handle map used for persistent, order independent identification of keys
	 * @param InChannel      A pointer to the owning channel.
	 */
	MOVIESCENE_API FMovieSceneChannelData(FMovieSceneChannel* InChannel, TArray<FFrameNumber>* InTimes, FKeyHandleLookupTable* InKeyHandles);

	UE_DEPRECATED(5.5, "Constructor that takes an optional FMovieSceneChannel is now deprecated. FMovieSceneChannel is now required.")
	MOVIESCENE_API FMovieSceneChannelData(TArray<FFrameNumber>* InTimes, FKeyHandleLookupTable* InKeyHandles, FMovieSceneChannel* InChannel = nullptr);

	/**
	 * Move the key at index KeyIndex to a new time
	 *
	 * @return The index of the key in its new position
	 */
	MOVIESCENE_API int32 MoveKeyInternal(int32 KeyIndex, FFrameNumber InNewTime);

	/**
	 * Add a new key at the specified time
	 *
	 * @return The index of the key in its new position
	 */
	MOVIESCENE_API int32 AddKeyInternal(FFrameNumber InTime);

	/**
	 * Replaces the key handle at the given Index with NewHandle. Fails if the handle is already in use.
	 *
	 * @param KeyIndices	The index of the key to replace
	 * @param NewHandles	The key handle instance that should be at the given index.
	 */
	MOVIESCENE_API void ReplaceKeyHandlesInternal(TConstArrayView<int32> KeyIndices, TConstArrayView<FKeyHandle> NewHandles);

protected:

	/** Pointer to an external array of sorted times. Must be kept in sync with a corresponding value array. */
	TArray<FFrameNumber>* Times;

	/** Pointer to an external key handle map */
	FKeyHandleLookupTable* KeyHandles;

	/** Optional Pointer to the owning FMovieSceneChannel, should be set if the add,move, and delete callbacks are needed */
	FMovieSceneChannel* OwningChannel;

};

/**
 * Templated channel data utility class that provides a consistent interface for interacting with a channel's keys and values.
 * Assumes that the supplied time and value arrays are already sorted ascendingly by time and are the same size.
 * This class will maintain those invariants throughout its lifetime.
 */
template<typename ValueType>
struct TMovieSceneChannelData : FMovieSceneChannelData
{
	typedef typename TCallTraits<ValueType>::ParamType ParamType;


	/**
	 * Constructor that takes a non-owning pointer to an array of times and values, and a key handle map
	 *
	 * @param InTimes        A pointer to an array of times that should be operated on by this class. Externally owned.
	 * @param InValues       A pointer to an array of values that should be operated on by this class. Externally owned.
	 * @param InKeyHandles   A key handle map used for persistent, order independent identification of keys
	 * @param InChannel A option point to the owning channel, should be set if the move,add, delete delegates are utilizaed
	 */
	UE_DEPRECATED(5.5, "Constructor that takes an optional FMovieSceneChannel is now deprecated. FMovieSceneChannel is now required.")
	TMovieSceneChannelData(TArray<FFrameNumber>* InTimes, TArray<ValueType>* InValues, FKeyHandleLookupTable* InKeyHandles, FMovieSceneChannel* InChannel = nullptr)
		: FMovieSceneChannelData(InChannel, InTimes, InKeyHandles), Values(InValues) 
	{
		check(Times && Values);
	}


	/**
	 * Constructor that takes a non-owning pointer to an array of times and values, and a key handle map
	 *
	 * @param InTimes        A pointer to an array of times that should be operated on by this class. Externally owned.
	 * @param InValues       A pointer to an array of values that should be operated on by this class. Externally owned.
	 * @param InChannel      A pointer to the owning channel.
	 * @param InKeyHandles   A key handle map used for persistent, order independent identification of keys
	 */
	TMovieSceneChannelData(TArray<FFrameNumber>* InTimes, TArray<ValueType>* InValues, FMovieSceneChannel* InChannel, FKeyHandleLookupTable* InKeyHandles)
		: FMovieSceneChannelData(InChannel, InTimes, InKeyHandles), Values(InValues)
	{
		check(Times && Values);
	}
	

	/**
	 * Conversion to a constant version of this class
	 */
	operator TMovieSceneChannelData<const ValueType>()
	{
		return TMovieSceneChannelData<const ValueType>(Times, Values);
	}

	/**
	 * Read-only access to this channel's values
	 */
	inline TArrayView<const ValueType> GetValues() const
	{
		return *Values;
	}

	/**
	 * Mutable access to this channel's values
	 */
	inline TArrayView<ValueType> GetValues()
	{
		return *Values;
	}

	/**
	 * Add a new key at a given time
	 *
	 * @param InTime         The time at which to add the new key
	 * @param InValue        The value of the new key
	 * @return The index of the newly added key
	 */
	int32 AddKey(FFrameNumber InTime, ParamType InValue)
	{
		int32 KeyIndex = AddKeyInternal(InTime);
		Values->Insert(InValue, KeyIndex);
		if (OwningChannel && OwningChannel->OnKeyAddedEvent().IsBound())
		{
			TArray<FKeyAddOrDeleteEventItem> Items;
			Items.Add(FKeyAddOrDeleteEventItem(KeyIndex, InTime));
			OwningChannel->OnKeyAddedEvent().Broadcast(OwningChannel, Items);
			
		}
		return KeyIndex;
	}

	/**
	 * Move the key at index KeyIndex to a new time
	 *
	 * @param KeyIndex       The index of the key to move
	 * @param NewTime        The time to move the key to
	 * @return The index of the key in its new position
	 */

	int32 MoveKey(int32 KeyIndex, FFrameNumber NewTime, bool bRemoveDuplicateKeys = false)
	{
		return MoveKey(KeyIndex, NewTime, bRemoveDuplicateKeys, [](const FKeyHandle& KeyHandle){ return true; });
	}

	template<typename TCanRemove> requires std::is_invocable_r_v<bool, TCanRemove, const FKeyHandle&>
	int32 MoveKey(int32 KeyIndex, FFrameNumber NewTime, bool bRemoveDuplicateKeys, TCanRemove&& CanRemove)
	{
		if (bRemoveDuplicateKeys)
		{
			FKeyHandle KeyHandle = GetHandle(KeyIndex);

			TArray<FKeyHandle> KeysToRemove;
			TArray<FFrameNumber> DummyTimes;
			GetKeys(TRange<FFrameNumber>(NewTime, NewTime), &DummyTimes, &KeysToRemove);
			KeysToRemove.Remove(KeyHandle);
			KeysToRemove.SetNum(Algo::RemoveIf(KeysToRemove, CanRemove));

			if (KeysToRemove.Num() > 0)
			{
				DeleteKeys(KeysToRemove);
			}

			KeyIndex = GetIndex(KeyHandle);
		}

		if (KeyIndex == INDEX_NONE)
		{
			return KeyIndex;
		}

		int32 NewIndex = MoveKeyInternal(KeyIndex, NewTime);
		if (NewIndex != KeyIndex)
		{
			// We have to remove the key and re-add it in the right place
			// This could probably be done better by just shuffling up/down the items that need to move, without ever changing the size of the array
			ValueType OldValue = (*Values)[KeyIndex];
			Values->RemoveAt(KeyIndex, EAllowShrinking::No);
			Values->Insert(OldValue, NewIndex);
		}

		return NewIndex;
	}

	/**
	 * Move the key at index KeyIndex to a new time
	 *
	 * @param KeyIndex       The index of the key to move
	 * @param NewTime        The time to move the key to
	 * @return The index of the key in its new position
	 */
	int32 SetKeyTime(int32 KeyIndex, FFrameNumber InNewTime)
	{
		return MoveKey(KeyIndex, InNewTime);
	}



	/**
	 * Remap the times of all the keys in this channel using an abstract retimer
	 *
	 * @param Retimer         Custom retimer object that can convert from an old time to a new time
	 */
	void RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer)
	{
		using namespace UE::MovieScene;

		const int32 Num = Times->Num();
		if (Num == 0)
		{
			return;
		}

		bool bIsSorted = true;
		{
			int32 Index = 0;

			// Remap the first time
			{
				FFrameNumber& NewTime  = (*Times)[Index];
				FFrameNumber  OldTime  = NewTime;

				NewTime = Retimer.RemapTime(OldTime);
				OnRemapChannelKeyTime(OwningChannel, Retimer, OldTime, NewTime, (*Values)[Index]);
			}
			++Index;

			// Remap others keeping track of whether we need resorting
			for ( ; Index < Num; ++Index)
			{
				FFrameNumber& NewTime  = (*Times)[Index];
				FFrameNumber  OldTime  = NewTime;
				FFrameNumber  Previous = (*Times)[Index-1];

				NewTime = Retimer.RemapTime(OldTime);
				bIsSorted &= NewTime >= Previous;

				OnRemapChannelKeyTime(OwningChannel, Retimer, OldTime, NewTime, (*Values)[Index]);
			}
		}

		if (!bIsSorted)
		{
			// Have to re-sort all the keys...
			TArray<int32, TInlineAllocator<128>> Indices;
			for (int32 Index = 0; Index < Num; ++Index)
			{
				Indices.Add(Index);
			}

			Algo::Sort(Indices, [this](int32 A, int32 B){
				return (*Times)[A] < (*Times)[B];
			});

			for (int32 Index = 0; Index < Num; ++Index)
			{
				KeyHandles->MoveHandle(Index, Indices[Index]);
			}

			// Move all the keys by repeatedly swapping until we get the correct index
			for (int32 Index = 0; Index < Num; ++Index)
			{
				while (Indices[Index] != Index)
				{
					const int32 DestIndex = Indices[Index];
					Swap((*Times)[Index], (*Times)[DestIndex]);
					Swap((*Values)[Index], (*Values)[DestIndex]);
					Swap(Indices[Index], Indices[DestIndex]);
				}
			}
		}
	}

	/**
	 * Replaces the key handle at the given Index with NewHandle. Fails if the handle is already in use.
	 *
	 * @param KeyIndex	The index of the key to replace
	 * @param NewHandle	The key handle instance that should be at the given index.
	 */
	void ReplaceKeyHandle(int32 KeyIndex, FKeyHandle NewHandle)
	{
		ReplaceKeyHandles(TConstArrayView<int32>(&KeyIndex, 1), TConstArrayView<FKeyHandle>(&NewHandle, 1));
	}
	/**
	 * Replaces the key handles at the given KeyIndices with NewHandles. Handles already in use are skipped.
	 *
	 * @param KeyIndices	The indidces of the keys to replace
	 * @param NewHandles	The key handle instances that should be at the given indices.
	 */
	void ReplaceKeyHandles(TConstArrayView<int32> KeyIndices, TConstArrayView<FKeyHandle> NewHandles)
	{
		ReplaceKeyHandlesInternal(KeyIndices, NewHandles);
	}

	/**
	 * Remove the key at a given index
	 *
	 * @param KeyIndex       The index of the key to remove
	 */
	void RemoveKey(int32 KeyIndex)
	{
		check(Times->IsValidIndex(KeyIndex));
		if (OwningChannel && OwningChannel->OnKeyDeletedEvent().IsBound())
		{
			const FFrameNumber Time = (*Times)[KeyIndex];
			TArray<FKeyAddOrDeleteEventItem> Items;
			Items.Add(FKeyAddOrDeleteEventItem(KeyIndex, Time));
			OwningChannel->OnKeyDeletedEvent().Broadcast(OwningChannel, Items);
		}
		Times->RemoveAt(KeyIndex, EAllowShrinking::No);
		Values->RemoveAt(KeyIndex, EAllowShrinking::No);

		if (KeyHandles)
		{
			KeyHandles->DeallocateHandle(KeyIndex);
		}
	}

	/**
	 * Set the value of the key at InTime to InValue, adding a new key if necessary
	 *
	 * @param InTime         The time at which to add the new key
	 * @param InValue        The value of the new key
	 * @return The handle of the key
	 */
	FKeyHandle UpdateOrAddKey(FFrameNumber InTime, ParamType InValue)
	{
		int32 ExistingKey = FindKey(InTime);
		if (ExistingKey != INDEX_NONE)
		{
			(*Values)[ExistingKey] = InValue;
		}
		else
		{
			ExistingKey = AddKey(InTime, InValue);
		}

		return GetHandle(ExistingKey);
	}

	/**
	 * Updates keys with these times in the channel. Will add or insert any missing keys, and replace values of existing keys.
	 * This assumes that the InTimes array is sorted, as it performs a tape merge to efficiently update the keys.
	 * 
	 * @param InTimes Times to update
	 * @param InValues Values to update
	 */
	void UpdateOrAddKeys(const TArrayView<const FFrameNumber> InTimes, const TArrayView<ValueType> InValues)
	{
		check(InTimes.Num() == InValues.Num());
		const int32 UpdateNum = InTimes.Num();
		int32 CurrIdx = 0;
		int32 UpdateIdx = 0;

		while (true)
		{
			if (UpdateIdx == UpdateNum)
			{
				// No more keys to insert
				break;
			}

			if (CurrIdx == Times->Num())
			{
				// Append the rest of the keys to the end
				Times->Append(InTimes.RightChop(UpdateIdx));
				Values->Append(InValues.RightChop(UpdateIdx));
				break;
			}
			if ((*Times)[CurrIdx] == InTimes[UpdateIdx])
			{
				// Replace value at the associated time
				(*Values)[CurrIdx] = InValues[UpdateIdx];
				CurrIdx++;
				UpdateIdx++;
				continue;
			}
			if ((*Times)[CurrIdx] > InTimes[UpdateIdx])
			{
				// Insert new time value pair
				Times->Insert(InTimes[UpdateIdx], CurrIdx);
				Values->Insert(InValues[UpdateIdx], CurrIdx);
				CurrIdx++;
				UpdateIdx++;
				continue;
			}
			if ((*Times)[CurrIdx] < InTimes[UpdateIdx])
			{
				// Not found insertion slot yet
				CurrIdx++;
				continue;
			}
		}
		KeyHandles->Reset();
		for (int32 Index = 0; Index < Times->Num(); ++Index)
		{
			KeyHandles->AllocateHandle(Index);
		}
	}

	/**
	 * Set key times for a number of keys in this channel data
	 *
	 * @param InHandles          Array of key handles that should have their times set
	 * @param InKeyTimes         Array of new times for each handle of the above array
	 */
	void SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes)
	{
		check(InHandles.Num() == InKeyTimes.Num());

		for (int32 Index = 0; Index < InHandles.Num(); ++Index)
		{
			const int32 KeyIndex = GetIndex(InHandles[Index]);
			if (KeyIndex != INDEX_NONE)
			{
				MoveKey(KeyIndex, InKeyTimes[Index]);
			}
		}
	}

	/**
	 * Duplicate a number of keys within this channel data
	 *
	 * @param InHandles          Array of key handles that should be duplicated
	 * @param OutNewHandles      Array view to receive key handles for each duplicated key. Must exactly mathc the size of InHandles.
	 */
	void DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles)
	{
		for (int32 Index = 0; Index < InHandles.Num(); ++Index)
		{
			const int32 KeyIndex = GetIndex(InHandles[Index]);
			if (KeyIndex == INDEX_NONE)
			{
				// we must add a handle even if the supplied handle does not relate to a key in this channel
				OutNewHandles[Index] = FKeyHandle::Invalid();
			}
			else
			{
				// Do not cache value and time arrays since they can be reallocated during this loop
				auto KeyCopy = (*Values)[KeyIndex];
				int32 NewKeyIndex = AddKey((*Times)[KeyIndex], MoveTemp(KeyCopy));
				OutNewHandles[Index] = GetHandle(NewKeyIndex);
			}
		}
	}

	/**
	 * Delete a number of keys from this channel data
	 *
	 * @param InHandles          Array of key handles that should be deleted
	 */
	void DeleteKeys(TArrayView<const FKeyHandle> InHandles)
	{
		for (int32 Index = 0; Index < InHandles.Num(); ++Index)
		{
			const int32 KeyIndex = GetIndex(InHandles[Index]);
			if (KeyIndex != INDEX_NONE)
			{
				RemoveKey(KeyIndex);
			}
		}
	}


	/**
	 * Delete keys before or after a specified time
	 *
	 * @param InTime				Delete keys after this time
	 * @param bDeleteKeysBefore     Whether to delete keys before the specified time
	 */
	void DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore)
	{
		TArray<FFrameNumber> OutKeyTimes;
		TArray<FKeyHandle> OutKeyHandles;
		GetKeys(TRange<FFrameNumber>::All(), &OutKeyTimes, &OutKeyHandles);

		TArray<FKeyHandle> KeysToRemove;
		for (int32 Index = 0; Index < OutKeyTimes.Num(); ++Index)
		{
			if (bDeleteKeysBefore)
			{
				if (OutKeyTimes[Index] < InTime)
				{
					KeysToRemove.Add(OutKeyHandles[Index]);
				}
			}
			else
			{
				if (OutKeyTimes[Index] > InTime)
				{
					KeysToRemove.Add(OutKeyHandles[Index]);
				}
			}
		}

		DeleteKeys(KeysToRemove);
	}

	/**
	 * Remove all the keys from this channel
	 */
	void Reset()
	{
		if (OwningChannel && OwningChannel->OnKeyDeletedEvent().IsBound())
		{
			TArray<FKeyAddOrDeleteEventItem> Items;
			for (int32 Index = 0; Index < Times->Num(); ++Index)
			{
				const FFrameNumber Time = (*Times)[Index];
				Items.Add(FKeyAddOrDeleteEventItem(Index, Time));
			}
			OwningChannel->OnKeyDeletedEvent().Broadcast(OwningChannel, Items);
		}
		Times->Reset();
		Values->Reset();
		if (KeyHandles)
		{
			KeyHandles->Reset();
		}
	}

private:

	/** Pointer to an external array of values, to be kept in sync with FMovieSceneChannelData::Times */
	TArray<ValueType>* Values;

};


/**
 * Specialization of TMovieSceneChannelData for const value types (limited read-only access to data)
 */
template<typename ValueType>
struct TMovieSceneChannelData<const ValueType>
{
	typedef typename TCallTraits<ValueType>::ParamType ParamType;


	/**
	 * Constructor that takes a non-owning pointer to an array of times and values, and a key handle map
	 *
	 * @param InTimes        A pointer to an array of times that should be operated on by this class. Externally owned.
	 * @param InValues       A pointer to an array of values that should be operated on by this class. Externally owned.
	 * @param InKeyHandles   A key handle map used for persistent, order independent identification of keys
	 */
	TMovieSceneChannelData(const TArray<FFrameNumber>* InTimes, const TArray<ValueType>* InValues)
		: Times(InTimes), Values(InValues)
	{
		check(Times && Values);
	}

	/**
	 * Read-only access to this channel's key times.
	 */
	inline TArrayView<const FFrameNumber> GetTimes() const
	{
		return *Times;
	}

	/**
	 * Read-only access to this channel's values
	 */
	inline TArrayView<const ValueType> GetValues() const
	{
		return *Values;
	}

	/**
	 * Attempt to find a key at a given time and tolerance
	 *
	 * @param InTime         The time at which to search
	 * @param InTolerance    A tolerance of frame numbers to allow either side of the specified time
	 * @return The index of the key closest to InTime and within InTolerance, INDEX_NONE or Times.Num()
	 */
	int32 FindKey(FFrameNumber InTime, FFrameNumber InTolerance = 0) const
	{
		int32 MinIndex = 0, MaxIndex = 0;
		UE::MovieScene::FindRange(*Times, InTime, InTolerance, 1, MinIndex, MaxIndex);
		return MinIndex;
	}

	/**
	 * Find the range of keys that fall around InTime +/- InTolerance up to a maximum
	 *
	 * @param InTime         The time around which to search
	 * @param MaxNum         A maximum number of times to find, starting with those closest to the predicate time
	 * @param OutMin         The earliest index that met the conditions of the search
	 * @param OutMax         The latest index that met the conditions of the search
	 * @param InTolerance    The tolerance range to search around PredicateTime with
	 */
	void FindKeys(FFrameNumber InTime, int32 MaxNum, int32& OutMinIndex, int32& OutMaxIndex, FFrameNumber InTolerance) const
	{
		UE::MovieScene::FindRange(*Times, InTime, InTolerance, MaxNum, OutMinIndex, OutMaxIndex);
	}

	/**
	 * Compute the total time range of the channel data.
	 *
	 * @return The range of this channel data
	 */
	TRange<FFrameNumber> GetTotalRange() const
	{
		return Times->Num() ? TRange<FFrameNumber>((*Times)[0], TRangeBound<FFrameNumber>::Inclusive((*Times)[Times->Num()-1])) : TRange<FFrameNumber>::Empty();
	}

private:

	/** Pointer to an external array of sorted times. Must be kept in sync with Values. */
	const TArray<FFrameNumber>* Times;

	/** Pointer to an external array of values, to be kept in sync with Times. */
	const TArray<ValueType>* Values;
};
