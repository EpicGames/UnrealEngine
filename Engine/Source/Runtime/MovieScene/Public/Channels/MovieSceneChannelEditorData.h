// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Misc/Optional.h"
#include "MovieSceneCommonHelpers.h"
#include "Templates/Function.h"
#include "UObject/NameTypes.h"
#include "Misc/FrameNumber.h"
#include "Misc/Attribute.h"

class FTrackInstancePropertyBindings;
class UMovieSceneSection;
class IMovieScenePlayer;
class UObject;
struct FMovieSceneSequenceID;
struct FFrameNumber;
struct FFrameRate;
struct FGuid;
struct FMovieSceneRootEvaluationTemplateInstance;

#if WITH_EDITOR

DECLARE_DELEGATE_RetVal_ThreeParams(FText, FGetMovieSceneTooltipText, IMovieScenePlayer*, FGuid, FMovieSceneSequenceID);
/**
 * Editor meta data for a channel of data within a movie scene section
 */
struct FMovieSceneChannelMetaData
{
	/*
	 * Default Constructor
	 */
	MOVIESCENE_API FMovieSceneChannelMetaData();

	/*
	 * Construction from a name and display text. Necessary when there is more than one channel.
	 *
	 * @param InName           The unique name of this channel within the section
	 * @param InDisplayText    Text to display on the sequencer node tree
	 * @param InGroup          (Optional) When not empty, specifies a name to group channels by
	 * @param bInEnabled        (Optional) When true the channel is enabled, if false it is not.
	 */
	MOVIESCENE_API FMovieSceneChannelMetaData(FName InName, FText InDisplayText, FText InGroup = FText(), bool bInEnabled = true);

	/*
	 * Set the identifiers for this editor data
	 *
	 * @param InName           The unique name of this channel within the section
	 * @param InDisplayText    Text to display on the sequencer node tree
	 * @param InGroup          (Optional) When not empty, specifies a name to group channels by
	 */
	MOVIESCENE_API void SetIdentifiers(FName InName, FText InDisplayText, FText InGroup = FText());

	/*
	 * Get property metadata that corresponds to the given key.
	 * 
	 * @param InKey The requested key to get metadata for
	 */
	MOVIESCENE_API FString GetPropertyMetaData(const FName& InKey) const;

	/*
	 * Get the amount that all of this channel's keys are offset by
	 */
	MOVIESCENE_API FFrameNumber GetOffsetTime(const UMovieSceneSection* InSection) const;

	/** Whether this channel is enabled or not */
	uint8 bEnabled : 1;
	/** True if this channel can be collapsed onto the top level track node */
	uint8 bCanCollapseToTrack : 1;
	/** True if this channel's times are defined relative to the section start time */
	uint8 bRelativeToSection : 1;
	/** A sort order for this channel. Channels are sorted by this order, then by name. Groups are sorted by the channel with the lowest sort order. */
	uint32 SortOrder;
	/** Whether this channel can be keyed */
	bool bCanBeKeyed;
	/** By default if a channel has no FText::Group specified, we put it last, by setting this to false we use SortIndex instead */
	bool bSortEmptyGroupsLast;
	/** Invert the value for display purposes */
	bool bInvertValue;
	/** Whether this channel can preserve the ratio to its grouped channels when editing */
	bool bCanPreserveRatio;
	/** Determines whether preserve ratio is enabled or not, not saved */
	mutable bool bPreserveRatio;

	/** This channel's unique name */
	FName Name;
	/**
	 * Path representation of a sub property relative to the class property (i.e. topmost property) but NOT including the topmost property itself.
	 * E.g. for FWidgetTransform, a sub-property path for the first channel would be "Translation.X"
	 */
	FName SubPropertyPath;
	/**
	 * Path representation of a sub-property relative to a class property (i.e. topmost property) but NOT including the topmost property itself.
	 * This should be used when a Channel can be used by multiple struct sources and happen to have different property names for these.
	 * The prime example of this is FTransform vs FEulerTransform. FTransform uses "Translation" vs FEulerTransform uses "Location"
	 */
	TMap<FName, FName> SubPropertyPathMap;
	/** Text to display on this channel's key area node */
	FText DisplayText;
	/** Delegate to get a dynamic tooltip for the key area node */
	FGetMovieSceneTooltipText GetTooltipTextDelegate;
	/** Name to group this channel with others of the same group name */
	FText Group;
	/** Delegate to get a dynamic tooltip for the group */
	FGetMovieSceneTooltipText GetGroupTooltipTextDelegate;
	/** Intent name */
	FText IntentName;
	/* Optional. If unspecified IKeyArea::CreateCurveEditorModel will create a fallback. */
	FText LongIntentNameFormat;
	/** Optional color to draw underneath the keys on this channel */
	TOptional<FLinearColor> Color;
	/** Property meta data */
	TMap<FName, FString> PropertyMetaData;
	/** Optional object that owns this channel. By default the section owns the channel but sometimes channels live inside other nested objects instead. */
	TWeakObjectPtr<> WeakOwningObject;
	/** Key offset */
	TAttribute<FFrameNumber> KeyOffset;
};


/**
 * Typed external value that can be used to define how to access the current value on an object for any given channel of data. Typically defined as the extended editor data for many channel types through TMovieSceneChannelTraits::ExtendedEditorDataType.
 */
template<typename T>
struct TMovieSceneExternalValue
{
	/**
	 * Defaults to an undefined function (no external value)
	 */
	TMovieSceneExternalValue()
	{}

	/**
	 * Helper constructor that defines an external value as the same type as the template type.
	 * Useful for passthrough external values of the same type (ie, a float channel that animates a float property)
	 *
	 * @return A new external value that gets the value from the object as a T
	 */
	static TMovieSceneExternalValue<T> Make()
	{
		TMovieSceneExternalValue<T> Result;
		Result.OnGetExternalValue = GetValue;
		return Result;
	}

	/**
	 * Static definition that retrieves the current value of InObject as a T
	 *
	 * @param InObject      The object to retrieve the property from
	 * @param Bindings      (Optional) Pointer to the property bindings structure that represents the property itself
	 * @return (Optiona) The current value of the property on InObject, or nothing if there were no bindings
	 */
	static TOptional<T> GetValue(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		return Bindings ? Bindings->GetCurrentValue<T>(InObject) : TOptional<T>();
	}

	/** Function to invoke to get the current value of the property of an object */
	TFunction<TOptional<T>(UObject&, FTrackInstancePropertyBindings*)> OnGetExternalValue;

	/** Optional Function To Get Current Value and Weight, needed for setting keys on blended sections */
	TFunction<void (UObject*, UMovieSceneSection*,  FFrameNumber, FFrameRate, FMovieSceneRootEvaluationTemplateInstance&, T&, float&) > OnGetCurrentValueAndWeight;
};


/**
 * Commonly used channel display names and colors
 */
struct FCommonChannelData
{
	static MOVIESCENE_API const FText ChannelX;
	static MOVIESCENE_API const FText ChannelY;
	static MOVIESCENE_API const FText ChannelZ;
	static MOVIESCENE_API const FText ChannelW;

	static MOVIESCENE_API const FText ChannelR;
	static MOVIESCENE_API const FText ChannelG;
	static MOVIESCENE_API const FText ChannelB;
	static MOVIESCENE_API const FText ChannelA;

	static MOVIESCENE_API const FLinearColor RedChannelColor;
	static MOVIESCENE_API const FLinearColor GreenChannelColor;
	static MOVIESCENE_API const FLinearColor BlueChannelColor;

	static MOVIESCENE_API const FName TooltipText;
	static MOVIESCENE_API const FName GroupDisplayName;
};


#endif // WITH_EDITOR
