// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Misc/InlineValue.h"
#include "Algo/Sort.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Channels/MovieSceneChannelEditorData.h"

template<typename> struct TMovieSceneChannelTraits;

#if WITH_EDITOR

/**
 * Base entry type for use in FMovieSceneChannelProxy that stores editor meta-data and extended editor data for each channel of a given type (one entry per type).
 */
struct FMovieSceneChannelEditorDataEntry
{
	/**
	 * Templated constructor that uses the channel parameter to construct
	 * the editor data array from the correct editor data type
	 */
	template<typename ChannelType>
	explicit FMovieSceneChannelEditorDataEntry(const ChannelType& Channel)
	{
		ConstructExtendedEditorDataArray<ChannelType>((typename TMovieSceneChannelTraits<ChannelType>::ExtendedEditorDataType*)nullptr);
	}

	/**
	 * Get the common editor data for all channels
	 */
	TArrayView<const FMovieSceneChannelMetaData> GetMetaData() const
	{
		return MetaDataArray;
	}

	/**
	 * Access the extended editor data for a specific channel
	 */
	const void* GetExtendedEditorData(int32 ChannelIndex) const
	{
		if (ExtendedEditorDataArray.IsValid())
		{
			return ExtendedEditorDataArray->GetChannel(ChannelIndex);
		}
		return nullptr;
	}

protected:

	/**
	 * Add new editor data for the specified channel type at the last index in the array
	 */
	template<typename ChannelType>
	void AddMetaData(const FMovieSceneChannelMetaData& MetaData)
	{
		static_assert(std::is_same_v<typename TMovieSceneChannelTraits<ChannelType>::ExtendedEditorDataType, void>, "Must supply extended editor data according to the channel's traits.");

		// Add the editor meta-data
		FMovieSceneChannelMetaData& AddedMetaData = MetaDataArray.Add_GetRef(MetaData);

		// Automatically disable keying for channel types that don't support it (e.g., default-only channels)
		if constexpr (!TMovieSceneChannelTraits<ChannelType>::SupportsKeying)
		{
			AddedMetaData.bCanBeKeyed = false;
		}
	}

	/**
	 * Add new editor data for the specified channel type at the last index in the arrays
	 */
	template<typename ChannelType, typename ExtendedEditorDataType>
	void AddMetaData(const FMovieSceneChannelMetaData& MetaData, ExtendedEditorDataType&& InExtendedEditorData)
	{
		// Add the editor meta-data
		FMovieSceneChannelMetaData& AddedMetaData = MetaDataArray.Add_GetRef(MetaData);

		// Automatically disable keying for channel types that don't support it (e.g., default-only channels)
		if constexpr (!TMovieSceneChannelTraits<ChannelType>::SupportsKeying)
		{
			AddedMetaData.bCanBeKeyed = false;
		}

		// Add the extended channel-type specific editor data
		auto& TypedImpl = static_cast<TMovieSceneExtendedEditorDataArray<ChannelType>&>(ExtendedEditorDataArray.GetValue());
		TypedImpl.Data.Add(Forward<ExtendedEditorDataType>(InExtendedEditorData));
	}

	/**
	 * Set the extended channel-type specific editor data on an already added channel entry
	 */
	template<typename ChannelType, typename ExtendedEditorDataType>
	void SetExtendedEditorData(int32 EntryIndex, ExtendedEditorDataType&& InExtendedEditorData)
	{
		// Set the extended channel-type specific editor data
		auto& TypedImpl = static_cast<TMovieSceneExtendedEditorDataArray<ChannelType>&>(ExtendedEditorDataArray.GetValue());
		TypedImpl.Data[EntryIndex] = Forward<ExtendedEditorDataType>(InExtendedEditorData);
	}

	/**
	 * Access the extended editor data for channels stored in this entry
	 */
	template<typename ChannelType>
	TArrayView<const typename TMovieSceneChannelTraits<ChannelType>::ExtendedEditorDataType> GetAllExtendedEditorData() const
	{
		static_assert(!std::is_same_v<typename TMovieSceneChannelTraits<ChannelType>::ExtendedEditorDataType, void>, "This channel type does not define any extended editor data.");

		const auto& TypedImpl = static_cast<const TMovieSceneExtendedEditorDataArray<ChannelType>&>(ExtendedEditorDataArray.GetValue());
		return TypedImpl.Data;
	}

private:

	/** Construct the extended editor data container for channel types that require it */
	template<typename ChannelType, typename ExtendedEditorDataType>
	void ConstructExtendedEditorDataArray(ExtendedEditorDataType*)
	{
		ExtendedEditorDataArray = TMovieSceneExtendedEditorDataArray<ChannelType>();
	}

	template<typename ChannelType>
	void ConstructExtendedEditorDataArray(void*)
	{
	}

private:

	/** Base editor data, one per channel */
	TArray<FMovieSceneChannelMetaData, TInlineAllocator<1>> MetaDataArray;

	/**
	 * We store the array behind an interface whose access is via void*
	 * Typed access is only permitted using the original ChannelType templated methods
	 * to ensure safe casting of the array
	 */
	struct IMovieSceneExtendedEditorDataArray
	{
		virtual ~IMovieSceneExtendedEditorDataArray() = default;
		virtual const void* GetChannel(int32 Index) const = 0;
	};

	template<typename ChannelType>
	struct TMovieSceneExtendedEditorDataArray : IMovieSceneExtendedEditorDataArray
	{
		typedef typename TMovieSceneChannelTraits<ChannelType>::ExtendedEditorDataType ExtendedEditorDataType;

		virtual const void* GetChannel(int32 Index) const { return &Data[Index]; }

		/** The actual editor data */
		TArray<ExtendedEditorDataType> Data;
	};

	/** Extended editor data, one per channel, defined by TMovieSceneChannelTraits::ExtendedEditorDataType. Unused if ExtendedEditorDataType is void.  */
	TInlineValue<IMovieSceneExtendedEditorDataArray, 8> ExtendedEditorDataArray;
};

#else // WITH_EDITOR

/** Empty stub in non-editor bulds */
struct FMovieSceneChannelEditorDataEntry
{
	template<typename ChannelType> FMovieSceneChannelEditorDataEntry(const ChannelType&){}
};

#endif // WITH_EDITOR
