// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

struct FMovieSceneChannel;
struct FMovieSceneChannelMetaData;
struct FMovieSceneChannelProxy;
template<typename> struct TMovieSceneChannelHandle;
template<typename> struct TMovieSceneChannelTraits;

/**
 * Handle to a specific channel in a UMovieSceneSection. Will become nullptr when the FMovieSceneChannelProxy it was created with is reallocated.
 */
struct FMovieSceneChannelHandle
{
	/**
	 * Default constructor
	 */
	MOVIESCENE_API FMovieSceneChannelHandle();

	/**
	 * Construction from a weak channel proxy, the channel's type, and its index
	 */
	MOVIESCENE_API FMovieSceneChannelHandle(TWeakPtr<FMovieSceneChannelProxy> InWeakChannelProxy, FName InChannelTypeName, int32 InChannelIndex);

public:

	/**
	 * Compare this channel handle for equality against another. Will return false for a handle that is to a different proxy even if it might have pointed to the same data (ie, one handle has expired)
	 */
	friend bool operator==(const FMovieSceneChannelHandle& A, const FMovieSceneChannelHandle& B)
	{
		if (A.ChannelIndex == B.ChannelIndex && A.ChannelTypeName == B.ChannelTypeName)
		{
			FMovieSceneChannelProxy* ProxyA = A.WeakChannelProxy.Pin().Get();
			FMovieSceneChannelProxy* ProxyB = B.WeakChannelProxy.Pin().Get();

			// Cannot be equal if either handle is expired
			return ProxyA && ProxyA == ProxyB;
		}
		return false;
	}

	/**
	 * Compare this channel handle for inequality against another. Will return true for a handle that is to a different proxy even if it might have pointed to the same data (ie, one handle has expired)
	 */
	friend bool operator!=(const FMovieSceneChannelHandle& A, const FMovieSceneChannelHandle& B)
	{
		return !(A == B);
	}

	template<typename ChannelType>
	static void TrackChannelTypeName()
	{
#if DO_CHECK
		TrackChannelTypeNameInternal(ChannelType::StaticStruct());
#endif
	}

public:

	/**
	 * Cast this handle to a handle of a related type. Callee is responsible for ensuring that the type's are compatible.
	 */
	template<typename OtherChannelType>
	TMovieSceneChannelHandle<OtherChannelType> Cast() const;

	/**
	 * Access this channel's type identifier
	 */
	MOVIESCENE_API FName GetChannelTypeName() const;

	/**
	 * Access this channel's index
	 */
	MOVIESCENE_API int32 GetChannelIndex() const;

	/**
	 * Get the channel pointer this handle represents.
	 *
	 * @return the channel's pointer, or nullptr if the proxy it was created with is no longer alive.
	 */
	MOVIESCENE_API FMovieSceneChannel* Get() const;

	/**
	 * Attempt to access the proxy for this channel. Will return nullptr if it is no longer valid.
	 */
	FMovieSceneChannelProxy* GetChannelProxy() const
	{
		return WeakChannelProxy.Pin().Get();
	}

#if WITH_EDITOR

	/**
	 * Get the meta data associated with this channel
	 *
	 * @return the channel's meta data, or nullptr if the proxy it was created with is no longer alive.
	 */
	MOVIESCENE_API const FMovieSceneChannelMetaData* GetMetaData() const;

	/**
	 * Get the extended editor data associated with this channel
	 *
	 * @return the channel's extended editor data, or nullptr if the proxy it was created with is no longer alive.
	 */
	MOVIESCENE_API const void* GetExtendedEditorData() const;

#endif // WITH_EDITOR

private:

#if DO_CHECK
	MOVIESCENE_API static void TrackChannelTypeNameInternal(UStruct* ChannelType);
	MOVIESCENE_API static UStruct* GetChannelTypeByName(FName ChannelName);
	MOVIESCENE_API bool IsCastValidInternal(UStruct* OtherType) const;

	template<typename OtherChannelType>
	bool IsCastValid() const
	{
		return IsCastValidInternal(OtherChannelType::StaticStruct());
	}
#endif // DO_CHECK

private:

	/** Weak pointer to the channel, proxy alisased to the channel proxy's shared reference controller to ensure it becomes null when the proxy is re-allocated */
	TWeakPtr<FMovieSceneChannelProxy> WeakChannelProxy;

	/** The type name for the channel in the proxy */
	FName ChannelTypeName;

	/** The index of the channel within the typed channels array */
	int32 ChannelIndex;
};


/**
 * Forward declaration of a typed channel handle.
 * Typed channel handles replicate the inheritance hierarchy of their channels in order that overload resolution on them can happen without user-conversion or templating
 */
template<typename ChannelType> struct TMovieSceneChannelHandle;

/** Base implementation that is just the untyped handle type */
template<> struct TMovieSceneChannelHandle<FMovieSceneChannel> : FMovieSceneChannelHandle
{
protected:
	TMovieSceneChannelHandle()
	{}

	TMovieSceneChannelHandle(TWeakPtr<FMovieSceneChannelProxy> InWeakChannelProxy, FName InName, int32 InChannelIndex)
		: FMovieSceneChannelHandle(InWeakChannelProxy, InName, InChannelIndex)
	{}
};

/**
 * Handle to a specific channel in a UMovieSceneSection. Will become nullptr when the FMovieSceneChannelProxy it was created with is reallocated.
 * The handle class hierarchy matches that of the channel itself so that functions can be overloaded for any given base/derived handle type
 */
template<typename ChannelType>
struct TMovieSceneChannelHandle : TMovieSceneChannelHandle<typename ChannelType::Super>
{
	TMovieSceneChannelHandle()
	{}

	/**
	 * Construction from a weak channel proxy, and the channel's index
	 */
	TMovieSceneChannelHandle(TWeakPtr<FMovieSceneChannelProxy> InWeakChannelProxy, int32 InChannelIndex)
		: TMovieSceneChannelHandle<typename ChannelType::Super>(InWeakChannelProxy, ChannelType::StaticStruct()->GetFName(), InChannelIndex)
	{}

	/**
	 * Get the channel pointer this handle represents.
	 *
	 * @return the channel's pointer, or nullptr if the proxy it was created with is no longer alive.
	 */
	ChannelType* Get() const
	{
		return static_cast<ChannelType*>(FMovieSceneChannelHandle::Get());
	}

#if WITH_EDITOR

	/**
	 * Get the extended editor data associated with this channel
	 *
	 * @return the channel's extended editor data, or nullptr if the proxy it was created with is no longer alive.
	 */
	const typename TMovieSceneChannelTraits<ChannelType>::ExtendedEditorDataType* GetExtendedEditorData() const
	{
		return static_cast<const typename TMovieSceneChannelTraits<ChannelType>::ExtendedEditorDataType*>(FMovieSceneChannelHandle::GetExtendedEditorData());
	}

#endif // WITH_EDITOR

protected:

	TMovieSceneChannelHandle(TWeakPtr<FMovieSceneChannelProxy> InWeakChannelProxy, FName InChannelTypeName, int32 InChannelIndex)
		: TMovieSceneChannelHandle<typename ChannelType::Super>(InWeakChannelProxy, InChannelTypeName, InChannelIndex)
	{}

	friend FMovieSceneChannelHandle;
};


template<typename OtherChannelType>
TMovieSceneChannelHandle<OtherChannelType> FMovieSceneChannelHandle::Cast() const
{
#if DO_CHECK
	checkf(IsCastValid<OtherChannelType>(), TEXT("Invalid cast between unrelated channel types"));
#endif

	return TMovieSceneChannelHandle<OtherChannelType>(WeakChannelProxy, ChannelTypeName, ChannelIndex);
}
