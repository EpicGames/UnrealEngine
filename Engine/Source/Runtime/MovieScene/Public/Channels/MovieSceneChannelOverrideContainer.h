// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "Evaluation/MovieSceneEvaluationField.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "MovieSceneSection.h"
#include "MovieSceneSignedObject.h"
#include "Misc/InlineValue.h"

#include "MovieSceneChannelOverrideContainer.generated.h"

class UMovieSceneEntitySystemLinker;
struct FMovieSceneChannel;
struct FMovieSceneChannelMetaData;
struct FMovieSceneChannelProxyData;
struct FMovieSceneEntityComponentFieldBuilder;

namespace UE
{
namespace MovieScene
{

/**
 * Entity import parameters for a channel override.
 */
struct FChannelOverrideEntityImportParams
{
	FName ChannelName;
	UE::MovieScene::FComponentTypeID ResultComponent;
};

}  // namespace MovieScene
}  // namespace UE

/**
 * A wrapper to implement polymorphism for FMovieSceneChannel.
 */
UCLASS(Abstract, MinimalAPI)
class UMovieSceneChannelOverrideContainer : public UMovieSceneSignedObject
{
	GENERATED_BODY()

public:

	/** Initialize this container 
	 *
	 * @param InChannel		The channel being overriden
	 */
	virtual void InitializeOverride(FMovieSceneChannel* InChannel) {}

	/** Returns whether this container's underlying channel can be used as an override to the given channel type */
	virtual bool SupportsOverride(FName DefaultChannelTypeName) const { return false; }

	/** Imports the entity for this channel */
	virtual void ImportEntityImpl(
			const UE::MovieScene::FChannelOverrideEntityImportParams& OverrideParams, 
			const UE::MovieScene::FEntityImportParams& ImportParams, UE::MovieScene::FImportedEntity* OutImportedEntity) {};

	/** Gets the underlying channel */
	virtual const FMovieSceneChannel* GetChannel() const { return nullptr; }
	/** Gets the underlying channel */
	virtual FMovieSceneChannel* GetChannel() { return nullptr; }

#if WITH_EDITOR
	/** Caches the channel proxy for this channel */
	virtual FMovieSceneChannelHandle AddChannelProxy(FName ChannelName, FMovieSceneChannelProxyData& ProxyData, const FMovieSceneChannelMetaData& MetaData) { return FMovieSceneChannelHandle(); }
#else
	virtual void AddChannelProxy(FName ChannelName, FMovieSceneChannelProxyData& ProxyData) {}
#endif

public:

	using FOverrideCandidates = TArray<TSubclassOf<UMovieSceneChannelOverrideContainer>, TInlineAllocator<8>>;

	/** Get a list of channel overrides that can work in the place of the given channel type */
	static MOVIESCENE_API void GetOverrideCandidates(FName InDefaultChannelTypeName, FOverrideCandidates& OutCandidates);
};

