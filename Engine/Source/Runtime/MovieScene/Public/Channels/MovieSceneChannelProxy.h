// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/BinarySearch.h"
#include "Channels/MovieSceneChannelEditorDataEntry.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Misc/AssertionMacros.h"
#include "Misc/InlineValue.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/NameTypes.h"

struct FMovieSceneChannel;
struct FMovieSceneChannelMetaData;
struct FMovieSceneChannelProxy;
struct FMovieSceneChannelProxyData;

/**
 * An entry within FMovieSceneChannelProxy that contains all channels (and editor data) for any given channel type
 */
struct FMovieSceneChannelEntry : FMovieSceneChannelEditorDataEntry
{
	/** 
	 * Get the type name of the channels stored in this entry
	 */
	FName GetChannelTypeName() const
	{
		return ChannelTypeName;
	}

	/** 
	 * Access all the channels contained within this entry
	 */
	TArrayView<FMovieSceneChannel* const> GetChannels() const
	{
		return Channels;
	}

#if WITH_EDITOR

	/**
	 * Access extended typed editor data for channels stored in this entry
	 */
	template<typename ChannelType>
	TArrayView<const typename TMovieSceneChannelTraits<ChannelType>::ExtendedEditorDataType> GetAllExtendedEditorData() const
	{
		check(ChannelType::StaticStruct()->GetFName() == ChannelTypeName);
		return FMovieSceneChannelEditorDataEntry::GetAllExtendedEditorData<ChannelType>();
	}

#endif

private:

	// only FMovieSceneChannelProxyData and FMovieSceneChannelProxy can create entries
	friend FMovieSceneChannelProxyData;
	friend FMovieSceneChannelProxy;

	/** Templated constructor from the channel and its ID */
	template<typename ChannelType>
	explicit FMovieSceneChannelEntry(FName InChannelTypeName, const ChannelType& Channel)
		: FMovieSceneChannelEditorDataEntry(Channel)
		, ChannelTypeName(InChannelTypeName)
	{}

	/** The name of the channel's struct type */
	FName ChannelTypeName;

	/** Pointers to the channels that this entry contains. Pointers are assumed to stay alive as long as this entry is. If channels are reallocated, a new channel proxy should be created */
	TArray<FMovieSceneChannel*> Channels;
};




/**
 * Construction helper that is required to create a new FMovieSceneChannelProxy from multiple channels
 */
struct FMovieSceneChannelProxyData
{
#if WITH_EDITOR

	/**
	 * Add a new channel to the proxy. Channel's address is stored internally as a voic* and should exist as long as the channel proxy does
	 *
	 * @param InChannel          The channel to add to this proxy. Should live for as long as the proxy does. Any re-allocation should be accompanied with a re-creation of the proxy
	 * @param InMetaData         The editor meta data to be associated with this channel
	 */
	template<typename ChannelType>
	void Add(ChannelType& InChannel, const FMovieSceneChannelMetaData& InMetaData)
	{
		static_assert(std::is_same_v<typename TMovieSceneChannelTraits<ChannelType>::ExtendedEditorDataType, void>, "Must supply extended editor data according to the channel's traits.");
		static_assert(!std::is_same_v<ChannelType, FMovieSceneChannel>, "Cannot add channels by their base FMovieSceneChannel type.");

		// Add the channel
		const int32 ChannelTypeIndex = AddInternal(InChannel);
		// Add the editor data at the same index
		Entries[ChannelTypeIndex].AddMetaData<ChannelType>(InMetaData);
	}

	/**
	 * Add a new channel to the proxy. Channel's address is stored internally as a voic* and should exist as long as the channel proxy does
	 *
	 * @param InChannel          The channel to add to this proxy. Should live for as long as the proxy does. Any re-allocation should be accompanied with a re-creation of the proxy
	 * @param InMetaData         The editor meta data to be associated with this channel
	 */
	template<typename ChannelType>
	FMovieSceneChannelHandle AddWithDefaultEditorData(ChannelType& InChannel, const FMovieSceneChannelMetaData& InMetaData)
	{
		static_assert(!std::is_same_v<typename TMovieSceneChannelTraits<ChannelType>::ExtendedEditorDataType, void>, "This method is for channels with typed editor data. You *must* call SetExtendedEditorData afterwards.");
		static_assert(!std::is_same_v<ChannelType, FMovieSceneChannel>, "Cannot add channels by their base FMovieSceneChannel type.");

		// Add the channel
		const int32 ChannelTypeIndex = AddInternal(InChannel);
		// Add a default editor data at the same index, hopefully the caller will set it afterwards
		Entries[ChannelTypeIndex].AddMetaData<ChannelType>(InMetaData, typename TMovieSceneChannelTraits<ChannelType>::ExtendedEditorDataType());
		// Return index usable for SetExtendedEditorData
		const FName ChannelTypeName = ChannelType::StaticStruct()->GetFName();
		return FMovieSceneChannelHandle(nullptr, ChannelTypeName, Entries[ChannelTypeIndex].GetChannels().Num() - 1);
	}

	/**
	 * Add a new channel to the proxy. Channel's address is stored internally as a voic* and should exist as long as the channel proxy does
	 *
	 * @param InChannel             The channel to add to this proxy. Should live for as long as the proxy does. Any re-allocation should be accompanied with a re-creation of the proxy
	 * @param InMetaData            The editor meta data to be associated with this channel
	 * @param InExtendedEditorData  Additional editor data to be associated with this channel as per its traits
	 */
	template<typename ChannelType, typename ExtendedEditorDataType>
	void Add(ChannelType& InChannel, const FMovieSceneChannelMetaData& InMetaData, ExtendedEditorDataType&& InExtendedEditorData)
	{
		static_assert(!std::is_same_v<typename TMovieSceneChannelTraits<ChannelType>::ExtendedEditorDataType, void>, "Must supply typed editor data according to the channel's traits. Define TMovieSceneChannelTraits<ChannelType>::ExtendedEditorDataType to use this function.");
		static_assert(!std::is_same_v<ChannelType, FMovieSceneChannel>, "Cannot add channels by their base FMovieSceneChannel type.");

		// Add the channel
		const int32 ChannelTypeIndex = AddInternal(InChannel);
		// Add the editor data at the same index
		Entries[ChannelTypeIndex].AddMetaData<ChannelType>(InMetaData, Forward<ExtendedEditorDataType>(InExtendedEditorData));
	}

	template<typename ChannelType, typename ExtendedEditorDataType>
	void SetExtendedEditorData(FMovieSceneChannelHandle ChannelHandle, ExtendedEditorDataType&& InExtendedEditorData)
	{
		// Find the entry
		// TODO: we rely on ChannelType's extended editor data to be the same as the overriden extended editor data!
		const FName ChannelTypeName = ChannelHandle.GetChannelTypeName();
		FMovieSceneChannelEntry* Entry = Entries.FindByPredicate([=](const FMovieSceneChannelEntry& CurEntry)
				{ return CurEntry.ChannelTypeName == ChannelTypeName; });
		if (ensure(Entry))
		{
			Entry->SetExtendedEditorData<ChannelType>(ChannelHandle.GetChannelIndex(), InExtendedEditorData);
		}
	}

#else

	/**
	 * Add a new channel to the proxy. Channel's address is stored internally as a voic* and should exist as long as the channel proxy does
	 *
	 * @param InChannel    The channel to add to this proxy. Should live for as long as the proxy does. Any re-allocation should be accompanied with a re-creation of the proxy
	 */
	template<typename ChannelType>
	void Add(ChannelType& InChannel)
	{
		AddInternal(InChannel);
	}

#endif

private:

	/**
	 * Implementation that adds a channel to an entry, creating a new entry for this channel type if necessary
	 *
	 * @param InChannel    The channel to add to this proxy. Should live for as long as the proxy does. Any re-allocation should be accompanied with a re-creation of the proxy
	 */
	template<typename ChannelType>
	int32 AddInternal(ChannelType& InChannel);

	friend struct FMovieSceneChannelProxy;
	/** Array of entryies, one per channel type. Inline allocation space for one entry since most sections only have one channel type. */
	TArray<FMovieSceneChannelEntry, TInlineAllocator<1>> Entries;
};


/**
 * Proxy type stored inside UMovieSceneSection for access to all its channels. Construction via either a single channel, or a FMovieSceneChannelProxyData structure
 * This proxy exists as a generic accessor to any channel data existing in derived types
 */
struct FMovieSceneChannelProxy : TSharedFromThis<FMovieSceneChannelProxy>
{
public:

	FSimpleMulticastDelegate OnDestroy;

	/** Default construction - emtpy proxy */
	FMovieSceneChannelProxy(){}

	~FMovieSceneChannelProxy()
	{
		OnDestroy.Broadcast();
	}

	/**
	 * Construction via multiple channels
	 */
	FMovieSceneChannelProxy(FMovieSceneChannelProxyData&& InChannels)
		: Entries(MoveTemp(InChannels.Entries))
	{}

	/** Not copyable or moveable to ensure that previously retrieved pointers remain valid for the lifetime of this object. */
	FMovieSceneChannelProxy(const FMovieSceneChannelProxy&) = delete;
	FMovieSceneChannelProxy& operator=(const FMovieSceneChannelProxy&) = delete;

	FMovieSceneChannelProxy(FMovieSceneChannelProxy&&) = delete;
	FMovieSceneChannelProxy& operator=(FMovieSceneChannelProxy&&) = delete;

public:

	/**
	 * Const access to all the entries in this proxy
	 *
	 * @return Array view of all this proxy's entries (channels grouped by type)
	 */
	TArrayView<const FMovieSceneChannelEntry> GetAllEntries() const
	{
		return Entries;
	}

	/**
	 * Find an entry by its channel type name
	 *
	 * @return A pointer to the channel, or nullptr
	 */
	MOVIESCENE_API const FMovieSceneChannelEntry* FindEntry(FName ChannelTypeName) const;

	/**
	 * Find the index of the specified channel ptr in this proxy
	 *
	 * @param ChannelTypeName  The type name of the channel
	 * @param ChannelPtr       The channel pointer to find
	 * @return The index of the channel if found, else INDEX_NONE
	 */
	MOVIESCENE_API int32 FindIndex(FName ChannelTypeName, const FMovieSceneChannel* ChannelPtr) const;

	/**
	 * Get all channels of the specified type
	 *
	 * @return A possibly empty array view of all the channels in this proxy that match the template type
	 */
	template<typename ChannelType>
	TArrayView<ChannelType*> GetChannels() const;

	/**
	 * Get the channel for the specified index of a particular type.
	 *
	 * @return A pointer to the channel, or nullptr if the index was invalid, or the type was not present
	 */
	template<typename ChannelType>
	ChannelType* GetChannel(int32 ChannelIndex) const;

	/**
	 * Get the channel for the specified index of a particular type.
	 *
	 * @return A pointer to the channel, or nullptr if the index was invalid, or the type was not present
	 */
	MOVIESCENE_API FMovieSceneChannel* GetChannel(FName ChannelTypeName, int32 ChannelIndex) const;

	/**
	 * Returns the total number of channels
	 * @return The total number of channels
	 */
	MOVIESCENE_API int32 NumChannels() const;

	/**
	 * Make a channel handle out for the specified index and channel type name
	 *
	 * @return A handle to the supplied channel that will become nullptr when the proxy is reallocated, or nullptr if the index or channel type name are invalid.
	 */
	MOVIESCENE_API FMovieSceneChannelHandle MakeHandle(FName ChannelTypeName, int32 Index);

	/**
	 * Make a channel handle out for the specified index and templated channel type
	 *
	 * @return A handle to the supplied channel that will become nullptr when the proxy is reallocated, or nullptr if the index or channel type name are invalid.
	 */
	template<typename ChannelType>
	TMovieSceneChannelHandle<ChannelType> MakeHandle(int32 Index)
	{
		FName ChannelTypeName = ChannelType::StaticStruct()->GetFName();
		return MakeHandle(ChannelTypeName, Index).template Cast<ChannelType>();
	}

	/**
	 * Make a new channel handle for the same channel type name and index for this channel proxy
	 *
	 * @return A handle to the supplied channel that will become nullptr when the proxy is reallocated, or nullptr if the index or channel type name are invalid.
	 */
	template<typename ChannelType>
	TMovieSceneChannelHandle<ChannelType> CopyHandle(TMovieSceneChannelHandle<ChannelType> InOtherHandle)
	{
		return MakeHandle(InOtherHandle.GetChannelTypeName(), InOtherHandle.GetChannelIndex()).template Cast<ChannelType>();
	}

#if !WITH_EDITOR

	/**
	 * Construction via a single channel, and its editor data
	 * Channel's address is stored internally as a voic* and should exist as long as this channel proxy does.
	 */
	template<typename ChannelType>
	FMovieSceneChannelProxy(ChannelType& InChannel);

#else

	/**
	 * Construction via a single channel, and its editor data
	 * Channel's address is stored internally as a voic* and should exist as long as this channel proxy does.
	 */
	template<typename ChannelType>
	FMovieSceneChannelProxy(ChannelType& InChannel, const FMovieSceneChannelMetaData& InMetaData);

	/**
	 * Construction via a single channel, and its editor data
	 * Channel's address is stored internally as a voic* and should exist as long as this channel proxy does.
	 */
	template<typename ChannelType, typename ExtendedEditorDataType>
	FMovieSceneChannelProxy(ChannelType& InChannel, const FMovieSceneChannelMetaData& InMetaData, ExtendedEditorDataType&& InExtendedEditorData);


	/**
	 * Access all the editor meta data for the templated channel type
	 *
	 * @return A potentially empty array view for all the editor meta data for the template channel type
	 */
	template<typename ChannelType>
	TArrayView<const FMovieSceneChannelMetaData> GetMetaData() const;

	/**
	 * Get the channel with the specified name, assuming it is of a given type
	 *
	 * @return A typed handle to the channel, or an invalid handle if no channel of that name was found, or it wasn't of the specified type
	 */
	template<typename ChannelType>
	TMovieSceneChannelHandle<ChannelType> GetChannelByName(FName ChannelName) const;

	/**
	 * Get the channel with the specified name
	 *
	 * @return A handle to the channel, or an invalid handle if no channel of that name was found
	 */
	MOVIESCENE_API FMovieSceneChannelHandle GetChannelByName(FName ChannelName) const;

	/**
	 * Access all the extended data for the templated channel type
	 *
	 * @return A potentially empty array view for all the extended editor data for the template channel type
	 */
	template<typename ChannelType>
	TArrayView<const typename TMovieSceneChannelTraits<ChannelType>::ExtendedEditorDataType> GetAllExtendedEditorData() const;

#endif  // !WITH_EDITOR

private:

	/** Do not expose shared-ownership semantics of this object */
	using TSharedFromThis::AsShared;

	/** Array of channel entries, one per channel type. Should never be changed or reallocated after construction to keep pointers alive. */
	TArray<FMovieSceneChannelEntry, TInlineAllocator<1>> Entries;

#if WITH_EDITOR

	/** Populate the named channel table */
	MOVIESCENE_API void EnsureHandlesByNamePopulated() const;

	/** Lazy-created lookup table between a channel name and a channel handle */
	mutable TMap<FName, FMovieSceneChannelHandle> HandlesByName;
	/** Whether the named channel table has been populated */
	mutable bool bHandlesByNamePopulated = false;

#endif // WITH_EDITOR
};


/**
 * Implementation that adds a channel to an entry, creating a new entry for this channel type if necessary
 *
 * @param InChannel    The channel to add to this proxy. Should live for as long as the proxy does. Any re-allocation should be accompanied with a re-creation of the proxy
 */
template<typename ChannelType>
int32 FMovieSceneChannelProxyData::AddInternal(ChannelType& InChannel)
{
	FMovieSceneChannelHandle::TrackChannelTypeName<ChannelType>();

	// Find the entry for this channel's type
	FName ChannelTypeName = ChannelType::StaticStruct()->GetFName();

	// Find the first entry that has a >= channel ID
	int32 ChannelTypeIndex = Algo::LowerBoundBy(Entries, ChannelTypeName, &FMovieSceneChannelEntry::GetChannelTypeName, FNameLexicalLess());

	// If the index we found isn't valid, or it's not the channel we want, we need to add a new entry there
	if (ChannelTypeIndex >= Entries.Num() || Entries[ChannelTypeIndex].GetChannelTypeName() != ChannelTypeName)
	{
		Entries.Insert(FMovieSceneChannelEntry(ChannelTypeName, InChannel), ChannelTypeIndex);
	}

	check(Entries.IsValidIndex(ChannelTypeIndex));

	// Add the channel to the channels array
	Entries[ChannelTypeIndex].Channels.Add(&InChannel);
	return ChannelTypeIndex;
}


/**
 * Get all channels of the specified type
 *
 * @return A possibly empty array view of all the channels in this proxy that match the template type
 */
template<typename ChannelType>
TArrayView<ChannelType*> FMovieSceneChannelProxy::GetChannels() const
{
	FName ChannelTypeName = ChannelType::StaticStruct()->GetFName();
	const FMovieSceneChannelEntry* FoundEntry = FindEntry(ChannelTypeName);

	if (FoundEntry)
	{
		return TArrayView<ChannelType*>((ChannelType**)(FoundEntry->Channels.GetData()), FoundEntry->Channels.Num());
	}
	return TArrayView<ChannelType*>();
}


/**
 * Get the channel for the specified index of a particular type.
 *
 * @return A pointer to the channel, or nullptr if the index was invalid, or the type was not present
 */
template<typename ChannelType>
ChannelType* FMovieSceneChannelProxy::GetChannel(int32 ChannelIndex) const
{
	TArrayView<ChannelType*> Channels = GetChannels<ChannelType>();
	return Channels.IsValidIndex(ChannelIndex) ? Channels[ChannelIndex] : nullptr;
}

#if !WITH_EDITOR


/**
 * Construction via a single channel, and its editor data
 * Channel's address is stored internally as a voic* and should exist as long as this channel proxy does.
 */
template<typename ChannelType>
FMovieSceneChannelProxy::FMovieSceneChannelProxy(ChannelType& InChannel)
{
	static_assert(!std::is_same_v<ChannelType, FMovieSceneChannel>, "Cannot add channels by their base FMovieSceneChannel type..");

	const FName ChannelTypeName = ChannelType::StaticStruct()->GetFName();
	Entries.Add(FMovieSceneChannelEntry(ChannelTypeName, InChannel));
	Entries[0].Channels.Add(&InChannel);
}


#else	// !WITH_EDITOR


/**
 * Construction via a single channel, and its editor data
 * Channel's address is stored internally as a voic* and should exist as long as this channel proxy does.
 */
template<typename ChannelType>
FMovieSceneChannelProxy::FMovieSceneChannelProxy(ChannelType& InChannel, const FMovieSceneChannelMetaData& InMetaData)
{
	static_assert(!std::is_same_v<ChannelType, FMovieSceneChannel>, "Cannot add channels by their base FMovieSceneChannel type..");
	static_assert(std::is_same_v<typename TMovieSceneChannelTraits<ChannelType>::ExtendedEditorDataType, void>, "Must supply typed editor data according to the channel's traits.");

	const FName ChannelTypeName = ChannelType::StaticStruct()->GetFName();
	Entries.Add(FMovieSceneChannelEntry(ChannelTypeName, InChannel));
	Entries[0].Channels.Add(&InChannel);
	Entries[0].AddMetaData<ChannelType>(InMetaData);
}


/**
 * Construction via a single channel, its editor data, and its extended editor data. Compulsary for channel types with a TMovieSceneChannelTraits<ChannelType>::ExtendedEditorDataType.
 * Channel's address is stored internally as a voic* and should exist as long as this channel proxy does.
 */
template<typename ChannelType, typename ExtendedEditorDataType>
FMovieSceneChannelProxy::FMovieSceneChannelProxy(ChannelType& InChannel, const FMovieSceneChannelMetaData& InMetaData, ExtendedEditorDataType&& InExtendedEditorDataType)
{
	static_assert(!std::is_same_v<ChannelType, FMovieSceneChannel>, "Cannot add channels by their base FMovieSceneChannel type..");
	static_assert(!std::is_same_v<typename TMovieSceneChannelTraits<ChannelType>::ExtendedEditorDataType, void>, "Must supply typed editor data according to the channel's traits. Define TMovieSceneChannelTraits<ChannelType>::ExtendedEditorDataType to use this function.");

	const FName ChannelTypeName = ChannelType::StaticStruct()->GetFName();
	Entries.Add(FMovieSceneChannelEntry(ChannelTypeName, InChannel));
	Entries[0].Channels.Add(&InChannel);
	Entries[0].AddMetaData<ChannelType>(InMetaData, Forward<ExtendedEditorDataType>(InExtendedEditorDataType));
}


/**
 * Access all the extended data for the templated channel type
 *
 * @return A potentially empty array view for all the extended editor data for the template channel type
 */
template<typename ChannelType>
TArrayView<const typename TMovieSceneChannelTraits<ChannelType>::ExtendedEditorDataType> FMovieSceneChannelProxy::GetAllExtendedEditorData() const
{
	FName ChannelTypeName = ChannelType::StaticStruct()->GetFName();
	const FMovieSceneChannelEntry* FoundEntry = FindEntry(ChannelTypeName);

	if (FoundEntry)
	{
		return FoundEntry->GetAllExtendedEditorData<ChannelType>();
	}
	return TArrayView<const typename TMovieSceneChannelTraits<ChannelType>::ExtendedEditorDataType>();
}


/**
 * Access all the editor meta data for the templated channel type
 *
 * @return A potentially empty array view for all the editor meta data for the template channel type
 */
template<typename ChannelType>
TArrayView<const FMovieSceneChannelMetaData> FMovieSceneChannelProxy::GetMetaData() const
{
	FName ChannelTypeName = ChannelType::StaticStruct()->GetFName();
	const FMovieSceneChannelEntry* FoundEntry = FindEntry(ChannelTypeName);

	if (FoundEntry)
	{
		return FoundEntry->GetMetaData();
	}
	return TArrayView<const FMovieSceneChannelMetaData>();
}


/**
 * Get the channel for the specified sort index of a particular type.
 *
 * @return A pointer to the channel, or nullptr if the index was invalid, or the type was not present
 */
template<typename ChannelType>
TMovieSceneChannelHandle<ChannelType> FMovieSceneChannelProxy::GetChannelByName(FName ChannelName) const
{
	const FMovieSceneChannelHandle UntypedHandle = GetChannelByName(ChannelName);
	const FName ChannelTypeName = ChannelType::StaticStruct()->GetFName();
	// Invalid handles will have a type name of 'None', so we will always fail this test and retrun a null typed handle
	if (UntypedHandle.GetChannelTypeName() == ChannelTypeName)
	{
		return UntypedHandle.Cast<ChannelType>();
	}
	return TMovieSceneChannelHandle<ChannelType>();
}


#endif	// !WITH_EDITOR
