// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameTime.h"
#include "Misc/FrameRate.h"
#include "Misc/Optional.h"
#include "MovieSceneChannelEditorData.h"
#include "MovieSceneSection.h"


enum class EMovieSceneKeyInterpolation : uint8;

/**
 * Parameter structure passed to UE::MovieScene::Optimize that defines optimization parameters
 */
struct FKeyDataOptimizationParams
{
	/** An arbitrary tolerance under which keys should be optimizied */
	float Tolerance = KINDA_SMALL_NUMBER;

	/** Whether to automatically set interpolation tangents or not */
	bool bAutoSetInterpolation = false;

	/** A range inside which to optimize keys */
	TRange<FFrameNumber> Range = TRange<FFrameNumber>::All();

	/** The display rate to use for calculating tangents in non-normalized space */
	FFrameRate DisplayRate;
};

/**
 * Traits structure to be specialized for any channel type passed to FMovieSceneChannelProxy
 */
template<typename ChannelType>
struct TMovieSceneChannelTraitsBase
{
	enum { SupportsDefaults = true };

	/** Whether this channel type supports keying. Default-only channels should set this to false. */
	enum { SupportsKeying = true };

#if WITH_EDITOR

	/** Type that specifies what editor data should be associated with ChannelType. Void (default) implies no extended data. */
	typedef void ExtendedEditorDataType;

#endif
};

template<typename ValueType>
struct FMovieSceneChannelTraitsTransform
{
	ValueType Offset = {};
	ValueType Scale = {};
	ValueType Pivot = {};
};

/**
 * Traits structure to be specialized for any channel type passed to FMovieSceneChannelProxy
 */
template<typename ChannelType>
struct TMovieSceneChannelTraits : TMovieSceneChannelTraitsBase<ChannelType>
{
};

template<typename ChannelType>
using TGetMovieSceneChannelTraits = TMovieSceneChannelTraits<typename std::remove_cv_t<ChannelType>>;

namespace UE
{
namespace MovieScene
{
	/**
	 * Called to evaluate a channel. Overload with specific channel types for custom behaviour.
	 *
	 * @param InChannel   The channel to evaluate
	 * @param InTime      The time to evaluate at
	 * @param OutValue    Value to receive the result
	 * @return true if the channel was evaluated successfully, false otherwise
	 */
	template<typename ChannelType, typename ValueType>
	bool EvaluateChannel(const ChannelType* InChannel, FFrameTime InTime, ValueType& OutValue)
	{
		return InChannel->Evaluate(InTime, OutValue);
	}

	/**
	 * Called to evaluate a channel. Overload with specific channel types for custom behaviour.
	 *
	 * @param InSection	  The section that contains the channel
	 * @param InChannel   The channel to evaluate
	 * @param InTime      The time to evaluate at
	 * @param OutValue    Value to receive the result
	 * @return true if the channel was evaluated successfully, false otherwise
	 */
	template<typename ChannelType, typename ValueType>
	bool EvaluateChannel(const UMovieSceneSection* InSection, const ChannelType* InChannel, FFrameTime InTime, ValueType& OutValue)
	{
		return EvaluateChannel(InChannel, InTime, OutValue);
	}

	/**
	 * Called to assign a specific value in a channel.
	 *
	 * @param InChannel     The channel the value is contained within
	 * @param InKeyHandle   The handle of the key to assign to
	 * @param InValue       The new value
	 * @return If the value was successfully assigned. Will return false if the key handle doesn't belong to that channel.
	 */
	template<typename ChannelType, typename ValueType>
	bool AssignValue(ChannelType* InChannel, FKeyHandle InKeyHandle, ValueType&& InValue)
	{
		auto ChannelData = InChannel->GetData();
		int32 ValueIndex = ChannelData.GetIndex(InKeyHandle);

		if (ValueIndex != INDEX_NONE)
		{
			ChannelData.GetValues()[ValueIndex] = Forward<ValueType>(InValue);
			return true;
		}

		// The key handle doesn't belong to this channel if it can't be found in the channel data, so report failure.
		return false;
	}

	template<typename ChannelType, typename ValueType>
	bool GetKeyValue(ChannelType* InChannel, FKeyHandle InKeyHandle, ValueType& OutValue)
	{
		auto ChannelData = InChannel->GetData();
		int32 ValueIndex = ChannelData.GetIndex(InKeyHandle);
		
		if (ValueIndex != INDEX_NONE)
		{
			OutValue = ChannelData.GetValues()[ValueIndex];
			return true;
		}

		// The key handle doesn't belong to this channel if it can't be found in the channel data, so report failure.
		return false;
	}

	template<typename ChannelType>
	EMovieSceneKeyInterpolation GetInterpolationMode(ChannelType* InChannel, const FFrameNumber& InTime, EMovieSceneKeyInterpolation DefaultInterpolationMode)
	{
		return DefaultInterpolationMode;
	}

	/**
	 * Add a key to a channel, or update an existing key if one already exists at this time
	 *
	 * @param InChannel     The channel the value is contained within
	 * @param InTime        The time to add or update the key at
	 * @param InValue       The new value
	 * @return A handle to the key that was added
	 */
	template<typename ChannelType, typename ValueType>
	FKeyHandle AddKeyToChannel(ChannelType* InChannel, FFrameNumber InTime, ValueType&& Value, EMovieSceneKeyInterpolation Interpolation)
	{
		auto ChannelInterface = InChannel->GetData();
		int32 ExistingIndex = ChannelInterface.FindKey(InTime);

		FKeyHandle Handle = FKeyHandle::Invalid();

		if (ExistingIndex != INDEX_NONE)
		{
			Handle = ChannelInterface.GetHandle(ExistingIndex);
			AssignValue(InChannel, Handle, Forward<ValueType>(Value));
		}
		else
		{
			ExistingIndex = ChannelInterface.AddKey(InTime, Forward<ValueType>(Value));
			Handle = ChannelInterface.GetHandle(ExistingIndex);
		}

		return Handle;
	}

	/**
	 * Check whether the specified value already exists at the specified time
	 *
	 * @param InChannel     The channel to check
	 * @param InTime        The time to check for
	 * @param InValue       The value to check - compared against the curve's existing value at this time
	 * @return true if this value already exists at the time, false otherwise
	 */
	template<typename ChannelType, typename ValueType>
	bool ValueExistsAtTime(const ChannelType* InChannel, FFrameNumber InTime, const ValueType& InValue)
	{
		ValueType ExistingValue{};
		return EvaluateChannel(InChannel, InTime, ExistingValue) && ExistingValue == InValue;
	}

	/**
	 * Set a channel's default value
	 *
	 * @param InChannel      The channel to set the default on
	 * @param DefaultValue   The new default value
	 */
	template<typename ChannelType, typename ValueType>
	typename TEnableIf<TGetMovieSceneChannelTraits<ChannelType>::SupportsDefaults>::Type SetChannelDefault(ChannelType* Channel, ValueType&& DefaultValue)
	{
		Channel->SetDefault(DefaultValue);
	}
	template<typename ChannelType, typename ValueType>
	typename TEnableIf<!TGetMovieSceneChannelTraits<ChannelType>::SupportsDefaults>::Type SetChannelDefault(ChannelType* Channel, ValueType&& DefaultValue)
	{}

	/**
	* Removes a channel's default value
	*
	* @param InChannel      The channel to remove the default from.
	*/
	template<typename ChannelType>
	typename TEnableIf<TGetMovieSceneChannelTraits<ChannelType>::SupportsDefaults>::Type RemoveChannelDefault(ChannelType* Channel)
	{
		Channel->RemoveDefault();
	}
	template<typename ChannelType>
	typename TEnableIf<!TGetMovieSceneChannelTraits<ChannelType>::SupportsDefaults>::Type RemoveChannelDefault(ChannelType* Channel)
	{}

	/**
	* Gets the default value for the channel if set 
	*
	* @param InChannel      The channel to set the default on
	* @return	If there is a default value or not.
	*/
	template<typename ChannelType, typename ValueType>
	typename TEnableIf<TGetMovieSceneChannelTraits<ChannelType>::SupportsDefaults, bool>::Type GetChannelDefault(ChannelType* Channel, ValueType& OutDefaultValue)
	{
		if (Channel->GetDefault().IsSet())
		{
			OutDefaultValue = Channel->GetDefault().GetValue();
			return true;
		}
		return false;
	}

	template<typename ChannelType, typename ValueType>
	typename TEnableIf<!TGetMovieSceneChannelTraits<ChannelType>::SupportsDefaults, bool>::Type GetChannelDefault(ChannelType* Channel, ValueType& OutDefaultValue)
	{
		return false;
	}

	/**
	 * Optimize the specified channel by removing any redundant keys
	 *
	 * @param InChannel     The channel to optimize
	 * @param Params        Optimization parameters
	 */
	template<typename ChannelType>
	void Optimize(ChannelType* InChannel, const FKeyDataOptimizationParams& Params)
	{
		using namespace UE::MovieScene;

		auto ChannelInterface = InChannel->GetData();
		if (ChannelInterface.GetTimes().Num() > 1)
		{
			int32 StartIndex = 0;
			int32 EndIndex = 0;

			{
				TArrayView<const FFrameNumber> Times = ChannelInterface.GetTimes();
				StartIndex = Params.Range.GetLowerBound().IsClosed() ? Algo::LowerBound(Times, Params.Range.GetLowerBoundValue()) : 0;
				EndIndex   = Params.Range.GetUpperBound().IsClosed() ? Algo::UpperBound(Times, Params.Range.GetUpperBoundValue()) : Times.Num();
			}

			for (int32 Index = StartIndex; Index < EndIndex && Index < ChannelInterface.GetTimes().Num(); ++Index)
			{
				// Reget times and values as they may be reallocated
				FFrameNumber Time   = ChannelInterface.GetTimes()[Index];
				auto  OriginalValue = ChannelInterface.GetValues()[Index];

				// If the channel evaluates the same with this key removed, we can leave it out
				ChannelInterface.RemoveKey(Index);
				if (ValueExistsAtTime(InChannel, Time, OriginalValue))
				{
					Index--;
				}
				else
				{
					ChannelInterface.AddKey(Time, MoveTemp(OriginalValue));
				}
			}
		}
	}

	template<typename ValueType>
	void InvertValue(ValueType& InOutValue){}

	template<typename ValueType>
	void ReciprocalValue(ValueType& InOutValue) {}

	template<typename ValueType>
	void TransformValue(ValueType& InOutValue, const FMovieSceneChannelTraitsTransform<ValueType>& Transform) {}

} // namespace MovieScene
} // namespace UE
