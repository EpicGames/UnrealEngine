// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/EnumAsByte.h"
#include "Curves/RichCurve.h"
#include "KeyParams.h"
#include "Misc/FrameNumber.h"
#include "MovieSceneChannel.h"
#include "MovieSceneChannelData.h"
#include "MovieSceneChannelTraits.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"

#include "MovieSceneCurveChannelCommon.generated.h"

class FArchive;
template <typename T> struct TIsPODType;


/**
 * Tangents for curve channel control points.
 */
USTRUCT()
struct FMovieSceneTangentData
{
	GENERATED_BODY()

	FMovieSceneTangentData()
		: ArriveTangent(0.f)
		, LeaveTangent(0.f)
		, ArriveTangentWeight(0.f)
		, LeaveTangentWeight(0.f)
		, TangentWeightMode(RCTWM_WeightedNone)

	{}

	bool Serialize(FArchive& Ar);
	bool operator==(const FMovieSceneTangentData& Other) const;
	bool operator!=(const FMovieSceneTangentData& Other) const;
	friend FArchive& operator<<(FArchive& Ar, FMovieSceneTangentData& P)
	{
		P.Serialize(Ar);
		return Ar;
	}

	/** If RCIM_Cubic, the arriving tangent at this key */
	UPROPERTY(EditAnywhere, Category = "Key")
	float ArriveTangent;

	/** If RCIM_Cubic, the leaving tangent at this key */
	UPROPERTY(EditAnywhere, Category = "Key")
	float LeaveTangent;

	/** If RCTWM_WeightedArrive or RCTWM_WeightedBoth, the weight of the left tangent */
	UPROPERTY(EditAnywhere, Category = "Key")
	float ArriveTangentWeight;

	/** If RCTWM_WeightedLeave or RCTWM_WeightedBoth, the weight of the right tangent */
	UPROPERTY(EditAnywhere, Category = "Key")
	float LeaveTangentWeight;

	/** If RCIM_Cubic, the tangent weight mode */
	UPROPERTY(EditAnywhere, Category = "Key")
	TEnumAsByte<ERichCurveTangentWeightMode> TangentWeightMode;

	// This is required because TMovieSceneCurveChannelImpl<ChannelType>::Serialize dumps us as a byte array so we need padding to be initialized to avoid indeterminism in the cooked build
	uint8 UnserializedPaddingBytes[3] = {0};
};

// Make sure padding size matches our expectation
static_assert(sizeof(FMovieSceneTangentData) == 
	sizeof(FMovieSceneTangentData::ArriveTangent) + sizeof(FMovieSceneTangentData::LeaveTangent) + sizeof(FMovieSceneTangentData::ArriveTangentWeight) + 
	sizeof(FMovieSceneTangentData::LeaveTangentWeight) + sizeof(FMovieSceneTangentData::TangentWeightMode) + sizeof(FMovieSceneTangentData::UnserializedPaddingBytes),
	"Adjust padding size to avoid cooked build indeterminism with uninitialized padded data");


template<>
struct TIsPODType<FMovieSceneTangentData>
{
	enum { Value = true };
};


template<>
struct TStructOpsTypeTraits<FMovieSceneTangentData>
	: public TStructOpsTypeTraitsBase2<FMovieSceneTangentData>
{
	enum
	{
		WithSerializer = true,
		WithCopy = false,
		WithIdenticalViaEquality = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

