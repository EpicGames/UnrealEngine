// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannelEditorData.h"
#include "Channels/MovieSceneCurveChannelCommon.h"
#include "Channels/MovieScenePiecewiseCurve.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/EnumAsByte.h"
#include "CoreTypes.h"
#include "Curves/KeyHandle.h"
#include "Curves/RealCurve.h"
#include "Curves/RichCurve.h"
#include "HAL/PlatformCrt.h"
#include "KeyParams.h"
#include "Math/Range.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "Misc/Optional.h"
#include "MovieSceneChannel.h"
#include "MovieSceneChannelData.h"
#include "MovieSceneChannelTraits.h"
#include "Serialization/StructuredArchive.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"

#include "MovieSceneFloatChannel.generated.h"

class FArchive;
struct FPropertyTag;
template<typename> struct TMovieSceneCurveChannelImpl;
template <typename T> struct TIsPODType;

namespace UE::MovieScene
{
	MOVIESCENE_API void OnRemapChannelKeyTime(const FMovieSceneChannel* Channel, const IRetimingInterface& Retimer, FFrameNumber PreviousTime, FFrameNumber CurrentTime, FMovieSceneFloatValue& InOutValue);

	namespace Interpolation
	{
		struct FCachedInterpolation;
	}
}

USTRUCT()
struct FMovieSceneFloatValue
{
	GENERATED_BODY()

	FMovieSceneFloatValue()
		: Value(0.f), InterpMode(RCIM_Cubic), TangentMode(RCTM_Auto), PaddingByte(0)
	{}

	explicit FMovieSceneFloatValue(float InValue)
		: Value(InValue), InterpMode(RCIM_Cubic), TangentMode(RCTM_Auto), PaddingByte(0)
	{}

	bool Serialize(FArchive& Ar);
	MOVIESCENE_API bool operator==(const FMovieSceneFloatValue& Other) const;
	MOVIESCENE_API bool operator!=(const FMovieSceneFloatValue& Other) const;
	friend FArchive& operator<<(FArchive& Ar, FMovieSceneFloatValue& P)
	{
		P.Serialize(Ar);
		return Ar;
	}

	UPROPERTY(EditAnywhere, Category = "Key")
	float Value;

	UPROPERTY(EditAnywhere, Category = "Key")
	FMovieSceneTangentData Tangent;

	UPROPERTY(EditAnywhere, Category = "Key")
	TEnumAsByte<ERichCurveInterpMode> InterpMode;

	UPROPERTY(EditAnywhere, Category = "Key")
	TEnumAsByte<ERichCurveTangentMode> TangentMode;

	/**
	 * float value = 4 bytes
	 * tangent data = 4 floats + byte enum = 4*4 + 1 = 17 bytes, rounded up to 20 bytes on clang-win64
	 * interp and tangent modes = 2 byte enums = 2 bytes
	 * total = 26 bytes
	 */
	UPROPERTY()
	uint8 PaddingByte;

	// This is required because TMovieSceneCurveChannelImpl<ChannelType>::Serialize dumps us as a byte array so we need padding to be initialized to avoid indeterminism in the cooked build
	uint8 UnserializedPaddingBytes[1] = {0};
};
static_assert(
	sizeof(FMovieSceneFloatValue) ==
	sizeof(FMovieSceneFloatValue::Value) + sizeof(FMovieSceneFloatValue::Tangent) + sizeof(FMovieSceneFloatValue::InterpMode) + sizeof(FMovieSceneFloatValue::TangentMode) + sizeof(FMovieSceneFloatValue::PaddingByte) + sizeof(FMovieSceneFloatValue::UnserializedPaddingBytes),
	"Adjust padding size to avoid cooked build indeterminism with uninitialized padded data");

template<>
struct TIsPODType<FMovieSceneFloatValue>
{
	enum { Value = true };
};


template<>
struct TStructOpsTypeTraits<FMovieSceneFloatValue>
	: public TStructOpsTypeTraitsBase2<FMovieSceneFloatValue>
{
	enum
	{
		WithSerializer = true,
		WithCopy = false,
		WithIdenticalViaEquality = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};


USTRUCT()
struct FMovieSceneFloatChannel : public FMovieSceneChannel
{
	GENERATED_BODY()

	typedef float CurveValueType;
	typedef FMovieSceneFloatValue ChannelValueType;

	MOVIESCENE_API FMovieSceneFloatChannel();

	MOVIESCENE_API virtual ~FMovieSceneFloatChannel();

	/**
	 * Compute the value extents of this curve within the specified limits
	 * 
	 * @param StartTime     The start of the range within which to compute extents
	 * @param EndTime       The end of the range within which to compute extents
	 * @return The extents of this channel
	 */
	MOVIESCENE_API UE::MovieScene::Interpolation::FInterpolationExtents ComputeExtents(FFrameTime StartTime, FFrameTime EndTime) const;
	
	MOVIESCENE_API UE::MovieScene::FPiecewiseCurve AsPiecewiseCurve(bool bWithPreAndPostInfinityExtrap = true) const;

	/**
	 * Access a mutable interface for this channel's data
	 *
	 * @return An object that is able to manipulate this channel's data
	 */
	inline TMovieSceneChannelData<FMovieSceneFloatValue> GetData()
	{
		return TMovieSceneChannelData<FMovieSceneFloatValue>(&Times, &Values, this, &KeyHandles);
	}

	/**
	 * Access a constant interface for this channel's data
	 *
	 * @return An object that is able to interrogate this channel's data
	 */
	inline TMovieSceneChannelData<const FMovieSceneFloatValue> GetData() const
	{
		return TMovieSceneChannelData<const FMovieSceneFloatValue>(&Times, &Values);
	}

	/**
	 * Const access to this channel's times
	 */
	inline TArrayView<const FFrameNumber> GetTimes() const
	{
		return Times;
	}

	/**
	 * Const access to this channel's values
	 */
	inline TArrayView<const FMovieSceneFloatValue> GetValues() const
	{
		return Values;
	}

	/**
	* Evaluate this channel with the frame resolution 
	*
	* @param InTime     The time to evaluate at
	* @param InTime     The Frame Rate the time to evaluate at
	* @param OutValue   A value to receive the result
	* @return true if the channel was evaluated successfully, false otherwise
	*/
	MOVIESCENE_API bool Evaluate(FFrameTime InTime, float& OutValue) const;

	/**
	 * Retrieve a cached interpolation from this channel for the specified time
	 */
	MOVIESCENE_API UE::MovieScene::Interpolation::FCachedInterpolation GetInterpolationForTime(FFrameTime InTime) const;

	/**
	 * Set the channel's times and values to the requested values
	 */
	MOVIESCENE_API void Set(TArray<FFrameNumber> InTimes, TArray<FMovieSceneFloatValue> InValues);

	/**
	 * Set the channel's times and values to the requested values, but does not allocate key handles
	 */
	MOVIESCENE_API void SetKeysOnly(TArrayView<FFrameNumber> InTimes, TArrayView<FMovieSceneFloatValue> InValues);
public:

	// ~ FMovieSceneChannel Interface
	MOVIESCENE_API virtual void GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles) override;
	MOVIESCENE_API virtual void GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes) override;
	MOVIESCENE_API virtual void SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes) override;
	MOVIESCENE_API virtual void DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles) override;
	MOVIESCENE_API virtual void DeleteKeys(TArrayView<const FKeyHandle> InHandles) override;
	MOVIESCENE_API virtual void DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore) override;
	MOVIESCENE_API virtual void RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer) override;
	MOVIESCENE_API virtual TRange<FFrameNumber> ComputeEffectiveRange() const override;
	MOVIESCENE_API virtual int32 GetNumKeys() const override;
	MOVIESCENE_API virtual void Reset() override;
	MOVIESCENE_API virtual void Offset(FFrameNumber DeltaPosition) override;
	MOVIESCENE_API virtual void Optimize(const FKeyDataOptimizationParams& InParameters) override;
	MOVIESCENE_API virtual void ClearDefault() override;
	MOVIESCENE_API virtual void PostEditChange() override;
	MOVIESCENE_API virtual FKeyHandle GetHandle(int32 Index) override;
	MOVIESCENE_API virtual int32 GetIndex(FKeyHandle Handle) override;

public:

	/**
	 * Check whether this channel has any data
	 */
	inline bool HasAnyData() const
	{
		return Times.Num() != 0 || bHasDefaultValue == true;
	}

	/**
	 * Set this channel's default value that should be used when no keys are present
	 *
	 * @param InDefaultValue The desired default value
	 */
	inline void SetDefault(float InDefaultValue)
	{
		bHasDefaultValue = true;
		DefaultValue = InDefaultValue;
	}

	/**
	 * Get this channel's default value that will be used when no keys are present
	 *
	 * @return (Optional) The channel's default value
	 */
	inline TOptional<float> GetDefault() const
	{
		return bHasDefaultValue ? TOptional<float>(DefaultValue) : TOptional<float>();
	}

	/**
	 * Remove this channel's default value causing the channel to have no effect where no keys are present
	 */
	inline void RemoveDefault()
	{
		bHasDefaultValue = false;
	}

public:

	MOVIESCENE_API bool Serialize(FArchive& Ar);
#if WITH_EDITORONLY_DATA
	MOVIESCENE_API void PostSerialize(const FArchive& Ar);
#endif
	friend FArchive& operator<<(FArchive& Ar, FMovieSceneFloatChannel& Me)
	{
		Me.Serialize(Ar);
		return Ar;
	}

	/** Serialize this float function from a mismatching property tag (FRichCurve) */
	MOVIESCENE_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	MOVIESCENE_API int32 AddConstantKey(FFrameNumber InTime, float InValue);

	MOVIESCENE_API int32 AddLinearKey(FFrameNumber InTime, float InValue);

	MOVIESCENE_API int32 AddCubicKey(FFrameNumber InTime, float InValue, ERichCurveTangentMode TangentMode = RCTM_Auto, const FMovieSceneTangentData& Tangent = FMovieSceneTangentData());

	MOVIESCENE_API void AutoSetTangents(float Tension = 0.f);

	/** Get the channel's frame resolution */
	FFrameRate GetTickResolution() const { return TickResolution; }
	/** Set the channel's frame resolution */
	void SetTickResolution(FFrameRate InTickSolution) { TickResolution = InTickSolution; }

	/**
	 * Populate the specified array with times and values that represent the smooth interpolation of this channel across the specified range
	 *
	 * @param StartTimeSeconds      The first time in seconds to include in the resulting array
	 * @param EndTimeSeconds        The last time in seconds to include in the resulting array
	 * @param TimeThreshold         A small time threshold in seconds below which we should stop adding new points
	 * @param ValueThreshold        A small value threshold below which we should stop adding new points where the linear interpolation would suffice
	 * @param TickResolution        The tick resolution with which to interpret this channel's times
	 * @param InOutPoints           An array to populate with the evaluated points
	 */
	MOVIESCENE_API void PopulateCurvePoints(double StartTimeSeconds, double EndTimeSeconds, double TimeThreshold, float ValueThreshold, FFrameRate TickResolution, TArray<TTuple<double, double>>& InOutPoints) const;	

	/**
	* Add keys with these times to channel. The number of elements in both arrays much match or nothing is added.
	* Also assume that the times are greater than last time in the channel and are increasing. If not bad things can happen.
	* @param InTimes Times to add
	* @param InValues Values to add
	*/
	MOVIESCENE_API void AddKeys(const TArray<FFrameNumber>& InTimes, const TArray<FMovieSceneFloatValue>& InValues);

	MOVIESCENE_API void UpdateOrAddKeys(const TArrayView<const FFrameNumber> InTimes, const TArrayView<FMovieSceneFloatValue> InValues);

#if WITH_EDITORONLY_DATA
	/* Get whether to show this curve in the UI */
	MOVIESCENE_API bool GetShowCurve() const;
	/* Set whether to show this curve in the UI */
	MOVIESCENE_API void SetShowCurve(bool bInShowCurve);
#endif

public:

	/** Pre-infinity extrapolation state */
	UPROPERTY()
	TEnumAsByte<ERichCurveExtrapolation> PreInfinityExtrap;

	/** Post-infinity extrapolation state */
	UPROPERTY()
	TEnumAsByte<ERichCurveExtrapolation> PostInfinityExtrap;

private:

	UPROPERTY(meta=(KeyTimes))
	TArray<FFrameNumber> Times;

	UPROPERTY(meta=(KeyValues))
	TArray<FMovieSceneFloatValue> Values;

	UPROPERTY()
	float DefaultValue;

	UPROPERTY()
	bool bHasDefaultValue;

	/** This needs to be a UPROPERTY so it gets saved into editor transactions but transient so it doesn't get saved into assets. */
	UPROPERTY(Transient)
	FMovieSceneKeyHandleMap KeyHandles;

	UPROPERTY()
	FFrameRate TickResolution;

#if WITH_EDITORONLY_DATA

	UPROPERTY()
	bool bShowCurve;

#endif

	friend struct TMovieSceneCurveChannelImpl<FMovieSceneFloatChannel>;
	using FMovieSceneFloatChannelImpl = TMovieSceneCurveChannelImpl<FMovieSceneFloatChannel>;
};

template<>
struct TStructOpsTypeTraits<FMovieSceneFloatChannel> : public TStructOpsTypeTraitsBase2<FMovieSceneFloatChannel>
{
	enum 
	{ 
		WithStructuredSerializeFromMismatchedTag = true, 
	    WithSerializer = true,
#if WITH_EDITORONLY_DATA
		WithPostSerialize = true,
#endif
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

template<>
struct TMovieSceneChannelTraits<FMovieSceneFloatChannel> : TMovieSceneChannelTraitsBase<FMovieSceneFloatChannel>
{
#if WITH_EDITOR

	/** Float channels can have external values (ie, they can get their values from external objects for UI purposes) */
	typedef TMovieSceneExternalValue<float> ExtendedEditorDataType;

#endif
};

/**
 * Overload for getting the interpolation mode for a channel at a specified time, it could be the previous key's mode.See UE::MovieScene::GetInterpolationMode for default implementation.
 */
MOVIESCENE_API EMovieSceneKeyInterpolation GetInterpolationMode(FMovieSceneFloatChannel* InChannel, const FFrameNumber& InTime, EMovieSceneKeyInterpolation DefaultInterpolationMode);

/**
 * Overload for adding a new key to a float channel at a given time. See UE::MovieScene::AddKeyToChannel for default implementation.
 */
MOVIESCENE_API FKeyHandle AddKeyToChannel(FMovieSceneFloatChannel* Channel, FFrameNumber InFrameNumber, float InValue, EMovieSceneKeyInterpolation Interpolation);

/**
 * Overload for dilating float channel data. See UE::MovieScene::Dilate for default implementation.
 */
MOVIESCENE_API void Dilate(FMovieSceneFloatChannel* InChannel, FFrameNumber Origin, float DilationFactor);


/**
 * Overloads for common utility functions.
 */
MOVIESCENE_API bool ValueExistsAtTime(const FMovieSceneFloatChannel* InChannel, FFrameNumber InFrameNumber, float InValue);
MOVIESCENE_API bool ValueExistsAtTime(const FMovieSceneFloatChannel* InChannel, FFrameNumber InFrameNumber, const FMovieSceneFloatValue& InValue);
MOVIESCENE_API void AssignValue(FMovieSceneFloatChannel* InChannel, FKeyHandle InKeyHandle, float InValue);
MOVIESCENE_API void InvertValue(float& InOutValue);
MOVIESCENE_API void ReciprocalValue(float& InOutValue);
MOVIESCENE_API void TransformValue(float& InOutValue, const FMovieSceneChannelTraitsTransform<float>& Transform);
