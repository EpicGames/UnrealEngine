// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameTime.h"
#include "Misc/FrameRate.h"
#include "Misc/TVariant.h"


/**
 * This file contains structures that represent specific interpolation algorithms for either a continuous or discrete range.
 *
 * They are used by Sequencer evaluation to bypass expensive piecewise data searching every frame by caching the resulting
 * interpolation over the relevant time range which allows it to perform only the minimum computation required to find a result.
 *
 * FCachedInterpolation is a variant type that can represent any one of the supported interpolation modes in this file.
 */
namespace UE::MovieScene::Interpolation
{

struct FInvalidValue;
struct FConstantValue;
struct FLinearInterpolation;
struct FQuadraticInterpolation;
struct FCubicInterpolation;
struct FQuarticInterpolation;
struct FCubicBezierInterpolation;
struct FWeightedCubicInterpolation;


/**
 * Template structure used for representing an array of solutions with a minimum size
 * 
 * For example:
 * // An array with at least 3 elements, varifyable at compile-time
 * int32 Solve3(TInterpSolutions<double, 3> Solutions)
 * {
 *     Solutions[0] = 0.0;
 *     Solutions[1] = 1.0;
 *     Solutions[2] = 2.0;
 *     return 3;
 * }
 * int32 Solve4(TInterpSolutions<double, 4> Solutions);
 * 
 * double Solutions[8];
 * Solve3(Solutions);
 * Solve4(Solutions);
 */
template<typename T, int MinN>
struct TInterpSolutions
{
	/**
	 * Construction from a c-style array with a size of at least MinN
	 */
	template<int N>
	TInterpSolutions(T (&In)[N]) requires (N >= MinN)
		: Array(In)
	{}

	/**
	 * Conversion to another array of size <= this
	 */
	template<int OtherN>
	operator TInterpSolutions<T, OtherN>() const requires (OtherN <= MinN)
	{
		return TInterpSolutions<T, OtherN>(Array);
	}

	/**
	 * Access the nth element of this array
	 */
	T& operator[](int Index) const
	{
		check(Index<MinN);
		return Array[Index];
	}

private:

	template<typename U, int OtherN>
	friend struct TInterpSolutions;

	explicit TInterpSolutions(T* InArray)
		: Array(InArray)
	{}

	T* Array;
};



/**
 * Structure that represents the extents of a curve in y
 */
struct FInterpolationExtents
{
	double MinValue = std::numeric_limits<double>::max();
	double MaxValue = std::numeric_limits<double>::lowest();

	FFrameTime MinValueTime;
	FFrameTime MaxValueTime;

	MOVIESCENE_API bool IsValid() const;

	MOVIESCENE_API void AddPoint(double Value, FFrameTime Time);

	MOVIESCENE_API void Combine(const FInterpolationExtents& Other);
};

/**
 * Sentinal type that represents an invalid interpolation value.
 * Only used when a curve has no data whatsoever, and therefore cannot be evaluated
 */
struct FInvalidValue
{
};


/**
 * Structure representing a constant value.
 * Temporarily includes a flag to determine whether it needs to be re-evaluated for legacy fallback.
 */
struct FConstantValue
{
	/** The constant value */
	double Value;

	FFrameNumber Origin;

	UE_DEPRECATED(5.5, "Please provide an origin. This is required for Integral() to work correctly.")
	FConstantValue(double InValue)
		: FConstantValue(0, InValue)
	{}

	FConstantValue(FFrameNumber InOrigin, double InValue)
		: Value(InValue)
		, Origin(InOrigin)
	{}

	MOVIESCENE_API FConstantValue Derivative() const;
	MOVIESCENE_API FLinearInterpolation Integral(double ConstantOffset = 0.0) const;
};

/**
 * Structure representing a linear interpolation of the form f(t) = a(t-o) + b.
 */
struct FLinearInterpolation
{
	/** The coeffient 'a' in f(t) = a(t-o) + b */
	double Coefficient;
	/** The constant 'b' in f(t) = a(t-o) + b */
	double Constant;
	/** The origin 'o' in f(t) = a(t-o) + b */
	FFrameNumber Origin;

	FLinearInterpolation(FFrameNumber InOrigin, double InCoefficient, double InConstant)
		: Coefficient(InCoefficient)
		, Constant(InConstant)
		, Origin(InOrigin)
	{}

	/**
	 * Evaluate the expression
	 */
	MOVIESCENE_API double Evaluate(FFrameTime InTime) const;

	/**
	 * Attempt to solve this interpolation for x
	 */
	MOVIESCENE_API int32 Solve(double Value, TInterpSolutions<FFrameTime, 1> OutResults) const;

	/**
	 * Attempt to solve this interpolation for x within limits
	 */
	MOVIESCENE_API int32 SolveWithin(FFrameTime Start, FFrameTime End, double Value, TInterpSolutions<FFrameTime, 1> OutResults) const;

	/**
	 * Compute this expression's derivative
	 */
	MOVIESCENE_API FConstantValue Derivative() const;

	/**
	 * Compute this expression's integral with an optional constant offset
	 */
	MOVIESCENE_API FQuadraticInterpolation Integral(double ConstantOffset = 0.0) const;
};

/**
 * Structure representing a quadratic interpolation of the form f(x) = g(x-o) and g(x) = ax^2 + bx + c.
 */
struct FQuadraticInterpolation
{
	/** The coeffients a and b in g(x) = ax^2 + bx + c */
	double A, B;
	/** The constant 'c' in g(x) = ax^2 + bx + c */
	double Constant;
	/** The origin 'o' in f(x) = g(x-o) */
	FFrameNumber Origin;

	FQuadraticInterpolation(FFrameNumber InOrigin, double InA, double InB, double InConstant)
		: A(InA), B(InB)
		, Constant(InConstant)
		, Origin(InOrigin)
	{}

	/**
	 * Evaluate the expression
	 */
	MOVIESCENE_API double Evaluate(FFrameTime InTime) const;

	/**
	 * Attempt to solve this interpolation for x
	 */
	MOVIESCENE_API int32 Solve(double Value, TInterpSolutions<FFrameTime, 2> OutResults) const;

	/**
	 * Attempt to solve this interpolation for x within limits
	 */
	MOVIESCENE_API int32 SolveWithin(FFrameTime Start, FFrameTime End, double Value, TInterpSolutions<FFrameTime, 2> OutResults) const;

	/**
	 * Compute this expression's derivative
	 */
	MOVIESCENE_API FLinearInterpolation Derivative() const;

	/**
	 * Compute this expression's integral with an optional constant offset
	 */
	MOVIESCENE_API FCubicInterpolation Integral(double ConstantOffset = 0.0) const;
};

/**
 * Structure representing a cubic interpolation of the form f(x) = g(x-o) and g(x) = ax^3 + bx^2 + cx + d.
 */
struct FCubicInterpolation
{
	/** The coeffients a, b, and c in g(x) = ax^3 + bx^2 + cx + d */
	double A, B, C;
	/** The constant 'd' in g(x) = ax^3 + bx^2 + cx + d */
	double Constant;

	double DX;

	/** The origin 'o' in f(x) = g(x-o) */
	FFrameNumber Origin;

	FCubicInterpolation(FFrameNumber InOrigin, double InA, double InB, double InC, double InConstant, double InDX = 1.0)
		: A(InA), B(InB), C(InC)
		, Constant(InConstant)
		, DX(InDX)
		, Origin(InOrigin)
	{}

	/**
	 * Evaluate the expression
	 */
	MOVIESCENE_API double Evaluate(FFrameTime InTime) const;

	/**
	 * Attempt to solve this interpolation for x
	 */
	MOVIESCENE_API int32 Solve(double Value, TInterpSolutions<FFrameTime, 3> OutResults) const;

	/**
	 * Attempt to solve this interpolation for x within limits
	 */
	MOVIESCENE_API int32 SolveWithin(FFrameTime Start, FFrameTime End, double Value, TInterpSolutions<FFrameTime, 3> OutResults) const;

	/**
	 * Compute this expression's derivative
	 */
	MOVIESCENE_API FQuadraticInterpolation Derivative() const;

	/**
	 * Compute this expression's integral with an optional constant offset
	 */
	MOVIESCENE_API FQuarticInterpolation Integral(double ConstantOffset = 0.0) const;
};

/**
 * Structure representing a quartic interpolation of the form f(x) = g(x-o) and g(x) = ax^4 + bx^3 + cx^2 + dx + e.
 */
struct FQuarticInterpolation
{
	/** The coeffients a, b, c, and d in g(x) = ax^4 + bx^3 + cx^2 + dx + e */
	double A, B, C, D;
	/** The constant 'e' in g(x) = ax^4 + bx^3 + cx^2 + dx + e */
	double Constant;

	double DX;

	/** The origin 'o' in f(x) = g(x-o) */
	FFrameNumber Origin;

	FQuarticInterpolation(FFrameNumber InOrigin, double InA, double InB, double InC, double InD, double InConstant, double InDX = 1.0)
		: A(InA), B(InB), C(InC), D(InD)
		, Constant(InConstant)
		, DX(InDX)
		, Origin(InOrigin)
	{}

	/**
	 * Evaluate the expression
	 */
	MOVIESCENE_API double Evaluate(FFrameTime InTime) const;

	/**
	 * Attempt to solve this interpolation for x
	 */
	MOVIESCENE_API int32 Solve(double Value, TInterpSolutions<FFrameTime, 4> OutResults) const;

	/**
	 * Attempt to solve this interpolation for x within limits
	 */
	MOVIESCENE_API int32 SolveWithin(FFrameTime Start, FFrameTime End, double Value, TInterpSolutions<FFrameTime, 4> OutResults) const;

	/**
	 * Compute this expression's derivative
	 */
	MOVIESCENE_API FCubicInterpolation Derivative() const;
};

/**
 * A cubic bezier interpolation between 2 control points with tangents, represented as 4 control points on a Bezier curve
 */
struct FCubicBezierInterpolation
{
	/** The delta value between the two control points in the time-domain */
	double DX;
	/** The four control points that should be passed to BezierInterp */
	double P0, P1, P2, P3;
	/** The origin time of the first control point */
	FFrameNumber Origin;

	MOVIESCENE_API FCubicBezierInterpolation(
		FFrameNumber InOrigin,
		double InDX,
		double InStartValue,
		double InEndValue,
		double InStartTangent,
		double InEndTangent);

	MOVIESCENE_API FCubicInterpolation AsCubic() const;

	/**
	 * Evaluate the expression
	 */
	MOVIESCENE_API double Evaluate(FFrameTime InTime) const;

	/**
	 * Attempt to solve this interpolation for x
	 */
	MOVIESCENE_API int32 Solve(double InValue, TInterpSolutions<FFrameTime, 4> OutResults) const;

	/**
	 * Compute this expression's derivative
	 */
	MOVIESCENE_API FQuadraticInterpolation Derivative() const;

	/**
	 * Compute this expression's integral with an optional constant offset
	 */
	MOVIESCENE_API FQuarticInterpolation Integral(double ConstantOffset = 0.0) const;
};

/**
 * A weighted cubic bezier interpolation between 2 control points with weighted tangents
 */
struct FWeightedCubicInterpolation
{
	double DX;
	double StartKeyValue;
	double NormalizedStartTanDX;
	double StartKeyTanY;
	double StartWeight;

	double EndKeyValue;
	double NormalizedEndTanDX;
	double EndKeyTanY;
	double EndWeight;

	FFrameNumber Origin;

	FWeightedCubicInterpolation(
		FFrameRate TickResolution,
		FFrameNumber InOrigin,

		FFrameNumber StartTime, 
		double StartValue,
		double StartTangent,
		double StartTangentWeight,
		bool bStartIsWeighted,

		FFrameNumber EndTime, 
		double EndValue,
		double EndTangent,
		double EndTangentWeight,
		bool bEndIsWeighted);

	double Evaluate(FFrameTime InTime) const;


	/**
	 * Attempt to solve this interpolation for x
	 */
	MOVIESCENE_API int32 Solve(double Value, TInterpSolutions<FFrameTime, 4> OutResults) const;
};

/**
 * Simple 1 dimensional range based off a FFrameNumber to define the range within which a cached interpolation is valid
 */
struct FCachedInterpolationRange
{
	/** Make an empty range */
	static MOVIESCENE_API FCachedInterpolationRange Empty();
	/** Make finite range from InStart to InEnd, not including the end frame */
	static MOVIESCENE_API FCachedInterpolationRange Finite(FFrameNumber InStart, FFrameNumber InEnd);
	/** Make an infinite range */
	static MOVIESCENE_API FCachedInterpolationRange Infinite();

	/** Make a range that only contains the specified time */
	static MOVIESCENE_API FCachedInterpolationRange Only(FFrameNumber InTime);
	/** Make a range that covers all times from (and including) the specified start */
	static MOVIESCENE_API FCachedInterpolationRange From(FFrameNumber InStart);
	/** Make a range that covers all times up to (but not including) the specified end */
	static MOVIESCENE_API FCachedInterpolationRange Until(FFrameNumber InEnd);

	FFrameNumber Clamp(FFrameNumber In) const
	{
		return FMath::Clamp(
			In.Value,
			Start.Value,
			End.Value);
	}
	FFrameTime Clamp(FFrameTime In) const
	{
		return FMath::Clamp(
			In,
			FFrameTime(Start),
			FFrameTime(End));
	}

	MOVIESCENE_API bool Contains(FFrameNumber FrameNumber) const;

	MOVIESCENE_API bool IsEmpty() const;

	/** Inclusive start frame */
	FFrameNumber Start;
	/** Exclusive end frame (unless End == Max()) */
	FFrameNumber End;
};

/**
 * Variant structure that wraps an interpolation and the range within which it is valid.
 * ~96 bytes
 */
struct FCachedInterpolation
{
	/** Default construction to an invalid state. Calling Evaluate will always return false */
	MOVIESCENE_API FCachedInterpolation();
	/** Construction as a constant value */
	MOVIESCENE_API FCachedInterpolation(const FCachedInterpolationRange& InRange, const FConstantValue& Constant);
	/** Construction as a linear interpolation */
	MOVIESCENE_API FCachedInterpolation(const FCachedInterpolationRange& InRange, const FLinearInterpolation& Linear);
	/** Construction as a quadratic interpolation */
	MOVIESCENE_API FCachedInterpolation(const FCachedInterpolationRange& InRange, const FQuadraticInterpolation& Quadratic);
	/** Construction as a cubic interpolation */
	MOVIESCENE_API FCachedInterpolation(const FCachedInterpolationRange& InRange, const FCubicInterpolation& Cubic);
	/** Construction as a quartic interpolation */
	MOVIESCENE_API FCachedInterpolation(const FCachedInterpolationRange& InRange, const FQuarticInterpolation& Quartic);
	/** Construction as a cubic interpolation */
	MOVIESCENE_API FCachedInterpolation(const FCachedInterpolationRange& InRange, const FCubicBezierInterpolation& CubicBezier);
	/** Construction as a weighted cubic interpolation */
	MOVIESCENE_API FCachedInterpolation(const FCachedInterpolationRange& InRange, const FWeightedCubicInterpolation& WeightedCubic);

	/**
	 * Check whether this cache is valid (ie, has a valid range and interpolation)
	 */
	MOVIESCENE_API bool IsValid() const;

	/**
	 * Check whether this cache is still valid for the specified frame number.
	 * @return true if this interpolation is relevant to the frame, or false if it should be re-generated
	 */
	MOVIESCENE_API bool IsCacheValidForTime(FFrameNumber FrameNumber) const;

	/**
	 * Retrieve the range that this interpolation applies to
	 */
	MOVIESCENE_API FCachedInterpolationRange GetRange() const;

	/**
	 * Evaluate this interpolation for the specified frame time
	 *
	 * @param FrameTime  The time to evaluate at
	 * @param OutResult  Reference to recieve the resulting value. Only written to if this function returns true.
	 * @return true if this interpolation evaluated successfully (and the result written to OutResult), false otherwise.
	 */
	MOVIESCENE_API bool Evaluate(FFrameTime FrameTime, double& OutResult) const;

	MOVIESCENE_API TOptional<FCachedInterpolation> ComputeIntegral(double ConstantOffset = 0.0) const;

	MOVIESCENE_API TOptional<FCachedInterpolation> ComputeDerivative() const;

	MOVIESCENE_API int32 InverseEvaluate(double InValue, TInterpSolutions<FFrameTime, 4> OutResults) const;

	MOVIESCENE_API FInterpolationExtents ComputeExtents(FFrameTime From, FFrameTime To) const;
	MOVIESCENE_API FInterpolationExtents ComputeExtents() const;

	MOVIESCENE_API void Offset(double Amount);

private:

	/** Variant containing the actual interpolation implementation */
	TVariant<FInvalidValue,
		FConstantValue,
		FLinearInterpolation,
		FQuadraticInterpolation,
		FCubicInterpolation,
		FQuarticInterpolation,
		FCubicBezierInterpolation,
		FWeightedCubicInterpolation> Data;

	/** Structure representint the range of times this interpolation applies to */
	FCachedInterpolationRange Range;
};

} // namespace UE::MovieScene
