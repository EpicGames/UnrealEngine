// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameNumber.h"
#include "MovieSceneChannel.h"
#include "MovieSceneChannelData.h"
#include "MovieSceneChannelTraits.h"
#include "MovieSceneObjectPathChannel.generated.h"

/**
 * Key value type for object path channels that stores references to objects as both a hard and soft reference, to ensure compatability with both sub objects and async loading
 */
USTRUCT()
struct FMovieSceneObjectPathChannelKeyValue
{
	GENERATED_BODY()

	/** Default constructor */
	FMovieSceneObjectPathChannelKeyValue()
		: HardPtr(nullptr)
	{}

	/** Construction from an object pointer */
	MOVIESCENE_API FMovieSceneObjectPathChannelKeyValue(UObject* InObject);

	/**
	 * Assignment from a raw object pointer
	 */
	MOVIESCENE_API FMovieSceneObjectPathChannelKeyValue& operator=(UObject* NewObject);

	friend bool operator==(const FMovieSceneObjectPathChannelKeyValue& A, const FMovieSceneObjectPathChannelKeyValue& B)
	{
		return A.SoftPtr == B.SoftPtr;
	}

public:

	/**
	 * Legacy conversion from a TSoftObjectPtr<>
	 */
	MOVIESCENE_API bool SerializeFromMismatchedTag(FPropertyTag const& Tag, FStructuredArchive::FSlot Slot);
	
	/**
	 * Access the soft object pointer that this key should load
	 */
	const TSoftObjectPtr<>& GetSoftPtr() const
	{
		return SoftPtr;
	}

	/**
	 * Attempt to find this object either by returning the internally kept raw pointer, or by resolving (but not loading) the soft object path
	 */
	MOVIESCENE_API UObject* Get() const;

	UObject* GetValue() const
	{
		return Get();
	}

private:

	/** Persistent storage of the object by path (which allows us to support cross-level actor references, for instance) */
	UPROPERTY()
	TSoftObjectPtr<UObject> SoftPtr;

	/** Hard reference to the loaded object - relevant for any asset type which also hints the async loader to efficiently load the asset in advance */
	UPROPERTY()
	mutable TObjectPtr<UObject> HardPtr;
};

template<>
struct TStructOpsTypeTraits<FMovieSceneObjectPathChannelKeyValue> : public TStructOpsTypeTraitsBase2<FMovieSceneObjectPathChannelKeyValue>
{
	enum { WithStructuredSerializeFromMismatchedTag = true };
};

USTRUCT()
struct FMovieSceneObjectPathChannel : public FMovieSceneChannel
{
	GENERATED_BODY()

	typedef FMovieSceneObjectPathChannelKeyValue CurveValueType;

	/** Default constructor */
	FMovieSceneObjectPathChannel()
		: PropertyClass(nullptr)
	{}

	inline void SetPropertyClass(UClass* InPropertyClass)
	{
		PropertyClass = InPropertyClass;
	}

	inline UClass* GetPropertyClass() const
	{
		return PropertyClass;
	}

	/**
	 * Access a mutable interface for this channel's data
	 *
	 * @return An object that is able to manipulate this channel's data
	 */
	inline TMovieSceneChannelData<FMovieSceneObjectPathChannelKeyValue> GetData()
	{
		return TMovieSceneChannelData<FMovieSceneObjectPathChannelKeyValue>(&Times, &Values, this, &KeyHandles);
	}

	/**
	 * Access a constant interface for this channel's data
	 *
	 * @return An object that is able to interrogate this channel's data
	 */
	inline TMovieSceneChannelData<const FMovieSceneObjectPathChannelKeyValue> GetData() const
	{
		return TMovieSceneChannelData<const FMovieSceneObjectPathChannelKeyValue>(&Times, &Values);
	}

	/**
	 * Evaluate this channel
	 *
	 * @param InTime     The time to evaluate at
	 * @param OutValue   A value to receive the result
	 * @return true if the channel was evaluated successfully, false otherwise
	 */
	MOVIESCENE_API bool Evaluate(FFrameTime InTime, UObject*& OutValue) const;

public:

	//~ FMovieSceneChannel Interface
	MOVIESCENE_API virtual void GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles) override;
	MOVIESCENE_API virtual void GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes) override;
	MOVIESCENE_API virtual void SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes) override;
	MOVIESCENE_API virtual void DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles) override;
	MOVIESCENE_API virtual void DeleteKeys(TArrayView<const FKeyHandle> InHandles) override;
	MOVIESCENE_API virtual void DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore) override;
	MOVIESCENE_API virtual void RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer) override;
	MOVIESCENE_API virtual TRange<FFrameNumber> ComputeEffectiveRange() const override;
	MOVIESCENE_API virtual int32 GetNumKeys() const override;
	MOVIESCENE_API virtual void Reset() override;
	MOVIESCENE_API virtual void Offset(FFrameNumber DeltaPosition) override;
	MOVIESCENE_API virtual void Optimize(const FKeyDataOptimizationParams& InParameters) override;
	MOVIESCENE_API virtual void ClearDefault() override;
	MOVIESCENE_API virtual FKeyHandle GetHandle(int32 Index) override;
	MOVIESCENE_API virtual int32 GetIndex(FKeyHandle Handle) override;

public:

	/**
	 * Set this channel's default value that should be used when no keys are present
	 *
	 * @param InDefaultValue The desired default value
	 */
	inline void SetDefault(UObject* InDefaultValue)
	{
		DefaultValue = InDefaultValue;
	}
	MOVIESCENE_API void SetDefault(const FMovieSceneObjectPathChannelKeyValue& InDefaultValue);

	/**
	 * Get this channel's default value that will be used when no keys are present
	 *
	 * @return (Optional) The channel's default value
	 */
	inline const FMovieSceneObjectPathChannelKeyValue& GetDefault() const
	{
		return DefaultValue;
	}

	/**
	 * Remove this channel's default value causing the channel to have no effect where no keys are present
	 */
	inline void RemoveDefault()
	{
		DefaultValue = nullptr;
	}

private:

	UPROPERTY()
	TObjectPtr<UClass> PropertyClass;

	UPROPERTY(meta=(KeyTimes))
	TArray<FFrameNumber> Times;

	UPROPERTY(meta=(KeyValues))
	TArray<FMovieSceneObjectPathChannelKeyValue> Values;

	UPROPERTY()
	FMovieSceneObjectPathChannelKeyValue DefaultValue;

	/** This needs to be a UPROPERTY so it gets saved into editor transactions but transient so it doesn't get saved into assets. */
	UPROPERTY(Transient)
	FMovieSceneKeyHandleMap KeyHandles;
};

inline bool GetChannelDefault(const FMovieSceneObjectPathChannel* Channel, FMovieSceneObjectPathChannelKeyValue& OutDefaultValue)
{
	OutDefaultValue = Channel->GetDefault();
	return true;
}
inline bool EvaluateChannel(const FMovieSceneObjectPathChannel* InChannel, FFrameTime InTime, FMovieSceneObjectPathChannelKeyValue& OutValue)
{
	UObject* Object = nullptr;
	if (InChannel->Evaluate(InTime, Object))
	{
		OutValue = Object;
		return true;
	}
	return false;
}

template<>
struct TMovieSceneChannelTraits<FMovieSceneObjectPathChannel> : TMovieSceneChannelTraitsBase<FMovieSceneObjectPathChannel>
{
#if WITH_EDITOR

	/** Integer channels can have external values (ie, they can get their values from external objects for UI purposes) */
	typedef TMovieSceneExternalValue<UObject*> ExtendedEditorDataType;

#endif
};
