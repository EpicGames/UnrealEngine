// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "MovieSceneFwd.h"
#include "Channels/MovieSceneInterpolation.h"
#include "MovieSceneTransformTypes.h"

#define UE_API MOVIESCENE_API

struct FFrameTime;
enum ERichCurveExtrapolation : int;

namespace UE::MovieScene
{

namespace Interpolation
{
	struct FCachedInterpolation;
} // namespace Interpolation


/**
 * A piecewise curve represented as an array of interpolations.
 */
struct FPiecewiseCurve
{
	/**
	 * Compute the integral of this curve (ie, the cumulative area under this curve), returning the result as another piecewise curve
	 */
	MOVIESCENE_API FPiecewiseCurve Integral() const;


	/**
	 * Compute the integral of this curve (ie, the slope of this curve), returning the result as another piecewise curve
	 */
	MOVIESCENE_API FPiecewiseCurve Derivative() const;


	/**
	 * Offset this curve in the (y) direction by a certain amount.
	 * The resulting state of this curve, f(x), is g(x) = f(x) + c
	 * 
	 * @param Amount        The constant offset to apply to this curve
	 */
	MOVIESCENE_API void Offset(double Amount);


	/**
	 * Get the interpolation for a specific time
	 * 
	 * @param Time          The time (x-value) to retrieve an interpolation for
	 */
	MOVIESCENE_API Interpolation::FCachedInterpolation GetInterpolationForTime(FFrameTime Time) const;


	/**
	 * Evaluate this curve at the specified time
	 * 
	 * @param Time          The time (x-value) to evaluate at
	 * @param OutResult     Value to receieve the evaluated result
	 * @return true if the curve was evaluated successfully and OutResult was written to, false otherwise.
	 */
	MOVIESCENE_API bool Evaluate(FFrameTime Time, double& OutResult) const;


	/**
	 * Solve this curve for a given (y). Where more than one solution exists, TimeHint will be used to find the solution closest to the hint.
	 * 
	 * @param Value         The result to solve for
	 * @param TimeHint      Predicate time to use for locating the most relevant solution
	 * @param Flags         Flag structure used to control how to solve the curve
	 * @return The solution if one exists, false otherwise
	 */
	MOVIESCENE_API TOptional<FFrameTime> InverseEvaluate(double Value, FFrameTime TimeHint, EInverseEvaluateFlags Flags) const;


	/**
	 * Solve this curve for a given (y), only considering solutions that lie within a certain range.
	 * 
	 * @param Value         The result to solve for
	 * @param StartTime     Start time before which solutions will not be considered
	 * @param EndTime       End time after which solutions will not be considered
	 * @param Visitor       Callback that is invoked for each solution. Returning true allows the algorithm to continue, false will terminate the algorithm.
	 * @return False if any invocation of Visitor returned false, true otherwise.
	 */
	MOVIESCENE_API bool InverseEvaluateBetween(double Value, FFrameTime StartTime, FFrameTime EndTime, const TFunctionRef<bool(FFrameTime)>& Visitor) const;

public:

	TArray<Interpolation::FCachedInterpolation> Values;
};


struct FPiecewiseCurveData
{
	const FPiecewiseCurve* Channel;

	UE_API bool HasDefaultValue() const;
	UE_API double GetDefaultValue() const;
	UE_API double PreExtrapolate(const FFrameTime& Time) const;
	UE_API double PostExtrapolate(const FFrameTime& Time) const;
	UE_API int32 NumPieces() const;
	UE_API int32 GetIndexOfPieceByTime(const FFrameTime& Time) const;
	UE_API Interpolation::FCachedInterpolation GetPieceByIndex(int32 Index) const;
	UE_API Interpolation::FCachedInterpolation GetPieceByTime(const FFrameTime& Time) const;
	UE_API FFrameNumber GetFiniteStart() const;
	UE_API FFrameNumber GetFiniteEnd() const;
	UE_API ERichCurveExtrapolation GetPreExtrapolation() const;
	UE_API ERichCurveExtrapolation GetPostExtrapolation() const;
	UE_API double GetStartingValue() const;
	UE_API double GetEndingValue() const;
};


} // namespace UE::MovieScene

#undef UE_API
