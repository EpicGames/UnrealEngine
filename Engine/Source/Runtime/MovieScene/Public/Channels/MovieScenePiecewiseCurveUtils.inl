// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "MovieSceneFwd.h"
#include "Channels/MovieSceneInterpolation.h"
#include "MovieSceneTransformTypes.h"
#include "Curves/RichCurve.h"
#include "Channels/MovieSceneChannel.h"

struct FFrameTime;

namespace UE::MovieScene::Interpolation
{


template<typename PiecewiseDataType>
UE::MovieScene::Interpolation::FInterpolationExtents ComputeExtentsWithinBounds(const PiecewiseDataType& PiecewiseData, FFrameTime StartTime, FFrameTime EndTime)
{
	using namespace UE::MovieScene;

	const int32        NumPieces         = PiecewiseData.NumPieces();
	const FFrameNumber FiniteSplineStart = PiecewiseData.GetFiniteStart();
	const FFrameNumber FiniteSplineEnd   = PiecewiseData.GetFiniteEnd();

	check(NumPieces > 0 &&
		StartTime >= FiniteSplineStart && StartTime <= FiniteSplineEnd &&
		EndTime   >= FiniteSplineStart && EndTime   <= FiniteSplineEnd
	);

	// Start at the first key that is less than or equal to the start time (upper bound finds first key >, then we subtract 1)
	const int32 StartIndex = PiecewiseData.GetIndexOfPieceByTime(StartTime);

	Interpolation::FInterpolationExtents Extents;

	for (int32 PieceIndex = StartIndex; PieceIndex >= 0 && PieceIndex < NumPieces; ++PieceIndex)
	{
		Interpolation::FCachedInterpolation Interp = PiecewiseData.GetPieceByIndex(PieceIndex);

		if (Interp.GetRange().Start > EndTime)
		{
			break;
		}

		FFrameTime EvalTime1 = Interp.GetRange().Clamp(StartTime);
		FFrameTime EvalTime2 = Interp.GetRange().Clamp(EndTime);
		Extents.Combine(Interp.ComputeExtents(EvalTime1, EvalTime2));
	}

	return Extents;
}

template<typename PiecewiseDataType>
UE::MovieScene::Interpolation::FInterpolationExtents ComputePiecewiseExtents(const PiecewiseDataType& PiecewiseData, FFrameTime StartTime, FFrameTime EndTime)
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Interpolation;

	if (!ensure(StartTime <= EndTime))
	{
		return FInterpolationExtents{ 0, 0, 0, 0 };
	}
	
	check(StartTime <= EndTime);

	const int32 NumPieces = PiecewiseData.NumPieces();

	if (NumPieces == 0)
	{
		FInterpolationExtents Extents;
		if (PiecewiseData.HasDefaultValue())
		{
			Extents.AddPoint(PiecewiseData.GetDefaultValue(), StartTime);
			Extents.AddPoint(PiecewiseData.GetDefaultValue(), EndTime);
		}

		return Extents;
	}

	FFrameNumber FiniteSplineStart = PiecewiseData.GetFiniteStart();
	FFrameNumber FiniteSplineEnd   = PiecewiseData.GetFiniteEnd();

	FInterpolationExtents FinalExtents;

	// Deal with linear pre-post extrapolation
	if (PiecewiseData.GetPreExtrapolation() == RCCE_Linear && StartTime.FrameNumber.Value < FiniteSplineStart)
	{
		const int32 FirstKeyTime = FiniteSplineStart.Value;
		const int32 Min          = StartTime.FrameNumber.Value;
		const int32 Max          = FMath::Min(FirstKeyTime, EndTime.FrameNumber.Value);

		const double MinPreExtrap = PiecewiseData.PreExtrapolate(Min);
		const double MaxPreExtrap = PiecewiseData.PreExtrapolate(Max);

		FinalExtents.AddPoint(MinPreExtrap, Min);
		FinalExtents.AddPoint(MaxPreExtrap, Max);

		if (EndTime.FrameNumber.Value <= FirstKeyTime)
		{
			return FinalExtents;
		}

		// Clamp to the valid range
		StartTime = FiniteSplineStart;
	}

	if (PiecewiseData.GetPostExtrapolation() == RCCE_Linear && EndTime.FrameNumber.Value > FiniteSplineEnd)
	{
		const int32 LastKeyTime  = FiniteSplineEnd.Value;
		const int32 Min          = FMath::Max(LastKeyTime, StartTime.FrameNumber.Value);
		const int32 Max          = EndTime.FrameNumber.Value;

		const double MinPostExtrap = PiecewiseData.PostExtrapolate(Min);
		const double MaxPostExtrap = PiecewiseData.PostExtrapolate(Max);

		FinalExtents.AddPoint(MinPostExtrap, Min);
		FinalExtents.AddPoint(MaxPostExtrap, Max);

		if (StartTime.FrameNumber.Value >= LastKeyTime)
		{
			return FinalExtents;
		}

		// Clamp to the valid range
		EndTime = FiniteSplineEnd;
	}

	FCycleParams StartCycled = CycleTime(FiniteSplineStart, FiniteSplineEnd, StartTime);
	FCycleParams EndCycled   = CycleTime(FiniteSplineStart, FiniteSplineEnd, EndTime);

	const double StartValue = PiecewiseData.GetStartingValue();
	const double EndValue   = PiecewiseData.GetEndingValue();

	// Deal with offset cycles and oscillation on the start frame
	if (StartTime < FFrameTime(FiniteSplineStart))
	{
		switch (PiecewiseData.GetPreExtrapolation())
		{
		case RCCE_Linear:          StartCycled.CycleCount = 0;                                               break;
		case RCCE_Cycle:           break;
		case RCCE_CycleWithOffset: StartCycled.ComputePreValueOffset(StartValue, EndValue);                  break;
		case RCCE_Oscillate:       StartCycled.Oscillate(FiniteSplineStart.Value, FiniteSplineEnd.Value);    break;
		case RCCE_Constant:        StartCycled.Time = FiniteSplineStart; StartCycled.CycleCount = 0;         break;
		case RCCE_None:            StartCycled.Time = FiniteSplineStart; StartCycled.CycleCount = 0;         break;
		}
	}
	else if (StartTime > FFrameTime(FiniteSplineEnd))
	{
		switch (PiecewiseData.GetPostExtrapolation())
		{
		case RCCE_Linear:          StartCycled.CycleCount = 0;                                               break;
		case RCCE_Cycle:           break;
		case RCCE_CycleWithOffset: StartCycled.ComputePostValueOffset(StartValue, EndValue);                 break;
		case RCCE_Oscillate:       StartCycled.Oscillate(FiniteSplineStart.Value, FiniteSplineEnd.Value);    break;
		case RCCE_Constant:        StartCycled.Time = FiniteSplineEnd; StartCycled.CycleCount = 0;           break;
		case RCCE_None:            StartCycled.Time = FiniteSplineEnd; StartCycled.CycleCount = 0;           break;
		}
	}

	// Deal with offset cycles and oscillation on the end frame
	if (EndTime < FFrameTime(FiniteSplineStart))
	{
		switch (PiecewiseData.GetPreExtrapolation())
		{
		case RCCE_Linear:          EndCycled.CycleCount = 0;                                                 break;
		case RCCE_Cycle:           break;
		case RCCE_CycleWithOffset: EndCycled.ComputePreValueOffset(StartValue, EndValue);                    break;
		case RCCE_Oscillate:       EndCycled.Oscillate(FiniteSplineStart.Value, FiniteSplineEnd.Value);      break;
		case RCCE_Constant:        EndCycled.Time = FiniteSplineStart; EndCycled.CycleCount = 0;             break;
		case RCCE_None:            EndCycled.Time = FiniteSplineStart; EndCycled.CycleCount = 0;             break;
		}
	}
	else if (EndTime > FFrameTime(FiniteSplineEnd))
	{
		switch (PiecewiseData.GetPostExtrapolation())
		{
		case RCCE_Linear:          EndCycled.CycleCount = 0;                                                 break;
		case RCCE_Cycle:           break;
		case RCCE_CycleWithOffset: EndCycled.ComputePostValueOffset(StartValue, EndValue);                   break;
		case RCCE_Oscillate:       EndCycled.Oscillate(FiniteSplineStart.Value, FiniteSplineEnd.Value);      break;
		case RCCE_Constant:        EndCycled.Time = FiniteSplineEnd; EndCycled.CycleCount = 0;               break;
		case RCCE_None:            EndCycled.Time = FiniteSplineEnd; EndCycled.CycleCount = 0;               break;
		}
	}

	if (StartCycled.CycleCount != EndCycled.CycleCount)
	{
		const double     OffsetPerCycle = (EndValue        - StartValue);
		const FFrameTime CycleDuration  = (FiniteSplineEnd - FiniteSplineStart);

		FInterpolationExtents StartCycleExtents = ComputeExtentsWithinBounds(
			PiecewiseData,
			StartCycled.bMirrorCurve ? FiniteSplineStart : StartCycled.Time,
			StartCycled.bMirrorCurve ? StartCycled.Time  : FiniteSplineEnd);

		StartCycleExtents.MinValue     += StartCycled.ValueOffset;
		StartCycleExtents.MaxValue     += StartCycled.ValueOffset;
		StartCycleExtents.MinValueTime += StartCycled.CycleCount * CycleDuration;
		StartCycleExtents.MaxValueTime += StartCycled.CycleCount * CycleDuration;

		FInterpolationExtents EndCycleExtents = ComputeExtentsWithinBounds(
			PiecewiseData,
			EndCycled.bMirrorCurve ? EndCycled.Time  : FiniteSplineStart,
			EndCycled.bMirrorCurve ? FiniteSplineEnd : EndCycled.Time);

		EndCycleExtents.MinValue     += EndCycled.ValueOffset;
		EndCycleExtents.MaxValue     += EndCycled.ValueOffset;
		EndCycleExtents.MinValueTime += EndCycled.CycleCount * CycleDuration;
		EndCycleExtents.MaxValueTime += EndCycled.CycleCount * CycleDuration;

		FinalExtents.Combine(StartCycleExtents);
		FinalExtents.Combine(EndCycleExtents);

		if (EndCycled.CycleCount - StartCycled.CycleCount > 1)
		{
			const bool bHasPreExtrapCycles  = PiecewiseData.GetPreExtrapolation()  == RCCE_CycleWithOffset && StartCycled.CycleCount < 0;
			const bool bHasPostExtrapCycles = PiecewiseData.GetPostExtrapolation() == RCCE_CycleWithOffset && EndCycled.CycleCount   > 0;

			const int32 NumFullPreExtrapCycles  = bHasPreExtrapCycles  ? -(StartCycled.CycleCount - FMath::Min(EndCycled.CycleCount,   0))-1 : 0;
			const int32 NumFullPostExtrapCycles = bHasPostExtrapCycles ?  (EndCycled.CycleCount   - FMath::Max(StartCycled.CycleCount, 0))-1 : 0;

			FInterpolationExtents FullExtents = ComputeExtentsWithinBounds(PiecewiseData, FiniteSplineStart, FiniteSplineEnd);

			if (NumFullPreExtrapCycles + NumFullPostExtrapCycles > 0)
			{
				FInterpolationExtents PreExtents;
				FInterpolationExtents PostExtents;
				PostExtents.AddPoint(
					FullExtents.MaxValue     + OffsetPerCycle * NumFullPostExtrapCycles,
					FullExtents.MaxValueTime + CycleDuration  * NumFullPostExtrapCycles);
				PostExtents.AddPoint(
					FullExtents.MinValue     + OffsetPerCycle * NumFullPostExtrapCycles,
					FullExtents.MinValueTime + CycleDuration  * NumFullPostExtrapCycles);

				PreExtents.AddPoint(
					FullExtents.MaxValue     - OffsetPerCycle * NumFullPreExtrapCycles,
					FullExtents.MaxValueTime - CycleDuration  * NumFullPreExtrapCycles);
				PreExtents.AddPoint(
					FullExtents.MinValue     - OffsetPerCycle * NumFullPreExtrapCycles,
					FullExtents.MinValueTime - CycleDuration  * NumFullPreExtrapCycles);

				FullExtents.Combine(PreExtents);
				FullExtents.Combine(PostExtents);
			}

			FinalExtents.Combine(FullExtents);
		}
	}
	else if (EndCycled.Time == StartCycled.Time)
	{
		double Value = 0.0;

		if (StartCycled.Time < FiniteSplineStart)
		{
			Value = PiecewiseData.PreExtrapolate(StartCycled.Time);
		}
		else if (StartCycled.Time >= FiniteSplineEnd)
		{
			Value = PiecewiseData.PostExtrapolate(StartCycled.Time);
		}
		else
		{
			PiecewiseData.GetPieceByTime(StartCycled.Time).Evaluate(StartCycled.Time, Value);
		}

		FInterpolationExtents Extents;
		Extents.AddPoint(Value, StartTime);
		return Extents;
	}
	else
	{
		// Within the same cycle is easy - just compute the extents between the two bounds, and offset by the oscillation amount
		FInterpolationExtents Extents = ComputeExtentsWithinBounds(
			PiecewiseData,
			StartCycled.bMirrorCurve ? EndCycled.Time   : StartCycled.Time,
			StartCycled.bMirrorCurve ? StartCycled.Time : EndCycled.Time);

		Extents.MinValue += StartCycled.ValueOffset;
		Extents.MaxValue += StartCycled.ValueOffset;

		FinalExtents.Combine(Extents);
	}

	return FinalExtents;
}

} // namespace UE::MovieScene::Interpolation