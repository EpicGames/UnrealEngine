// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/Map.h"
#include "CoreMinimal.h"

#include "Channels/MovieSceneChannel.h"
#include "Channels/MovieSceneChannelOverrideContainer.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "Evaluation/MovieSceneEvaluationField.h"
#include "HAL/Platform.h"
#include "MovieSceneSection.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneSectionChannelOverrideRegistry.generated.h"

class UMovieSceneChannelOverrideContainer;
class UMovieSceneEntitySystemLinker;
class UMovieScenePropertyTrack;
struct FFrameNumber;
struct FMovieSceneChannel;
struct FMovieSceneEntityComponentFieldBuilder;
struct FMovieSceneEvaluationFieldEntityMetaData;
template <typename ElementType> class TRange;
template <typename T> struct TObjectPtr;

namespace UE
{
namespace MovieScene
{
	struct FEntityImportParams;
	struct FImportedEntity;
}
}

/**
* This object contains a map of actual channel overrides, where each override is a channel identifier and a channel container.
*/
UCLASS(MinimalAPI)
class UMovieSceneSectionChannelOverrideRegistry : public UObject
{
	GENERATED_BODY()

public:
	MOVIESCENE_API UMovieSceneSectionChannelOverrideRegistry();

	/** 
	* Add channel to the registry 
	* @param ChannelName		The name of the channel to override
	* @param Data				The container that owns a overriden channel instanse
	*/
	MOVIESCENE_API void AddChannel(FName ChannelName, UMovieSceneChannelOverrideContainer* ChannelContainer);

	/**
	* Returns if the channel is overriden
	* @param ChannelName	Name of the channel
	* @return	Whether this channel is overriden
	*/
	MOVIESCENE_API bool ContainsChannel(FName ChannelName) const;

	/**
	 * Returns the number of override channels in this registry
	 * @return	The number of override channels
	 */
	MOVIESCENE_API int32 NumChannels() const;

	/**
	 * Returns the channel override for a given name, or nullptr if not found
	 * @param ChannelName	Name of the channel
	 * @return	The channel override container
	 */
	MOVIESCENE_API UMovieSceneChannelOverrideContainer* GetChannel(FName ChannelName) const;

	/**
	 * Get all channel containers of a given type.
	 */
	template<typename ChannelContainerType>
	void GetChannels(TArray<ChannelContainerType*>& OutChannels) const
	{
		for (TPair<FName, TObjectPtr<UMovieSceneChannelOverrideContainer>> Pair : Overrides)
		{
			ChannelContainerType* TypedContainer = Cast<ChannelContainerType>(Pair.Value);
			if (TypedContainer)
			{
				OutChannels.Add(TypedContainer);
			}
		}
	}

	/**
	* Removes a channel from the registry
	* @param ChannelName		The name of the channel whose override to remove
	*/
	MOVIESCENE_API void RemoveChannel(FName ChannelName);

	/**
	* Forward ImportEntityImpl calls to an overriden channel
	*/
	MOVIESCENE_API void ImportEntityImpl(const UE::MovieScene::FChannelOverrideEntityImportParams& OverrideParams, const UE::MovieScene::FEntityImportParams& ImportParams, UE::MovieScene::FImportedEntity* OutImportedEntity);

	/**
	* Called when overridden channels should populate evaluation field
	*/
	MOVIESCENE_API void PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder, UMovieSceneSection& OwnerSection);

#if WITH_EDITOR

	/**
	 * Called by the owning section after it has been created for a paste operation.
	 */
	MOVIESCENE_API void OnPostPaste();

	/**
	 * Called when an undo/redo operation has affected this override registry.
	 */
	MOVIESCENE_API virtual void PostEditUndo() override;

#endif

private:

	/** Map of channel overrides. */
	UPROPERTY()
	TMap<FName, TObjectPtr<UMovieSceneChannelOverrideContainer>> Overrides;
};

namespace UE
{
namespace MovieScene 
{

/** Utility function to return whether a channel is overriden */
inline bool IsChannelOverriden(const UMovieSceneSectionChannelOverrideRegistry* OverrideRegistry, FName ChannelName)
{
	return OverrideRegistry && OverrideRegistry->ContainsChannel(ChannelName);
}

/** Utility function to return a channel override */
inline UMovieSceneChannelOverrideContainer* GetChannelOverride(const UMovieSceneSectionChannelOverrideRegistry* OverrideRegistry, FName ChannelName)
{
	return OverrideRegistry ? OverrideRegistry->GetChannel(ChannelName) : nullptr;
}

#if WITH_EDITOR

/**
 * Utility function for adding a possibly-overriden channel into a channel proxy, with some specific extended editor
 * data.
 */
template<typename ChannelType, typename ExtendedEditorDataType>
void AddChannelProxy(
		FMovieSceneChannelProxyData& ProxyData, const UMovieSceneSectionChannelOverrideRegistry* OverrideRegistry,
		FName ChannelName, ChannelType& DefaultChannel, 
		const FMovieSceneChannelMetaData& InMetaData, ExtendedEditorDataType&& InExtendedEditorData)
{
	UMovieSceneChannelOverrideContainer* OverrideContainer = GetChannelOverride(OverrideRegistry, ChannelName);
	if (OverrideContainer)
	{
		// Because the extended editor data is a strongly-typed templated parameter, we can't pass it through the
		// virtual methods of the channel containers. So channel containers will add an entry with default extended
		// editor data, and we set the correct data immediately after.
		FMovieSceneChannelHandle ChannelHandle = OverrideContainer->AddChannelProxy(ChannelName, ProxyData, InMetaData);
		ProxyData.SetExtendedEditorData<ChannelType>(ChannelHandle, InExtendedEditorData);
	}
	else
	{
		ProxyData.Add(DefaultChannel, InMetaData, InExtendedEditorData);
	}
}

/**
 * Utility function for making a channel proxy with one channel that is maybe overriden by another channel.
 */
template<typename ChannelType, typename ExtendedEditorDataType>
TSharedPtr<FMovieSceneChannelProxy> MakeChannelProxy(
	const UMovieSceneSectionChannelOverrideRegistry* OverrideRegistry,
	ChannelType& DefaultChannel,
	const FMovieSceneChannelMetaData& InMetaData, ExtendedEditorDataType&& InExtendedEditorData)
{
	// When there's only one channel, it usually has no name. Create a channel proxy by hand if
	// you need a named singled channel.
	const FName ChannelName(NAME_None);
	FMovieSceneChannelProxyData ProxyData;
	UMovieSceneChannelOverrideContainer* OverrideContainer = GetChannelOverride(OverrideRegistry, ChannelName);
	if (OverrideContainer)
	{
		FMovieSceneChannelHandle ChannelHandle = OverrideContainer->AddChannelProxy(ChannelName, ProxyData, InMetaData);
		ProxyData.SetExtendedEditorData<ChannelType>(ChannelHandle, InExtendedEditorData);
	}
	else
	{
		ProxyData.Add(DefaultChannel, InMetaData, InExtendedEditorData);
	}
	return MakeShared<FMovieSceneChannelProxy>(MoveTemp(ProxyData));
}

#else

/**
 * Utility function for adding a possibly-overriden channel into a channel proxy.
 */
template<typename ChannelType>
void AddChannelProxy(
		FMovieSceneChannelProxyData& ProxyData, const UMovieSceneSectionChannelOverrideRegistry* OverrideRegistry,
		FName ChannelName, ChannelType& DefaultChannel)
{
	UMovieSceneChannelOverrideContainer* OverrideContainer = GetChannelOverride(OverrideRegistry, ChannelName);
	if (OverrideContainer)
	{
		OverrideContainer->AddChannelProxy(ChannelName, ProxyData);
	}
	else
	{
		ProxyData.Add(DefaultChannel);
	}
}

/**
 * Utility function for making a channel proxy with one channel that is maybe overriden by another channel.
 */
template<typename ChannelType>
TSharedPtr<FMovieSceneChannelProxy> MakeChannelProxy(
	const UMovieSceneSectionChannelOverrideRegistry* OverrideRegistry,
	ChannelType& DefaultChannel)
{
	const FName ChannelName(NAME_None);
	FMovieSceneChannelProxyData ProxyData;
	UMovieSceneChannelOverrideContainer* OverrideContainer = GetChannelOverride(OverrideRegistry, ChannelName);
	if (OverrideContainer)
	{
		OverrideContainer->AddChannelProxy(ChannelName, ProxyData);
	}
	else
	{
		ProxyData.Add(DefaultChannel);
	}
	return MakeShared<FMovieSceneChannelProxy>(MoveTemp(ProxyData));
}

#endif

}  // namespace MovieScene
}  // namespace UE

