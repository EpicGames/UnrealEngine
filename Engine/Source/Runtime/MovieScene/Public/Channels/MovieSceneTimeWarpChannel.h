// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannelTraits.h"
#include "Channels/MovieSceneDoubleChannel.h"

#include "MovieSceneTimeWarpChannel.generated.h"

class UMovieScene;

namespace UE::MovieScene
{
	enum class ETimeWarpChannelDomain : uint8
	{
		Time,
		PlayRate,
	};
}


USTRUCT()
struct FMovieSceneTimeWarpChannel : public FMovieSceneDoubleChannel
{
	GENERATED_BODY()

	UE::MovieScene::ETimeWarpChannelDomain Domain;
};

MOVIESCENE_API void Dilate(FMovieSceneTimeWarpChannel* InChannel, FFrameNumber Origin, double DilationFactor);

inline EMovieSceneKeyInterpolation GetTimeWarpMode(FMovieSceneDoubleChannel* InChannel, const FFrameNumber& InTime, EMovieSceneKeyInterpolation DefaultInterpolationMode)
{
	return GetInterpolationMode(static_cast<FMovieSceneDoubleChannel*>(InChannel), InTime, DefaultInterpolationMode);
}
inline FKeyHandle AddKeyToChannel(FMovieSceneTimeWarpChannel* Channel, FFrameNumber InFrameNumber, double InValue, EMovieSceneKeyInterpolation Interpolation)
{
	return AddKeyToChannel(static_cast<FMovieSceneDoubleChannel*>(Channel), InFrameNumber, InValue, Interpolation);
}
inline bool ValueExistsAtTime(const FMovieSceneTimeWarpChannel* InChannel, FFrameNumber InFrameNumber, const FMovieSceneDoubleValue& InValue)
{
	return ValueExistsAtTime(static_cast<const FMovieSceneDoubleChannel*>(InChannel), InFrameNumber, InValue);
}
inline bool ValueExistsAtTime(const FMovieSceneTimeWarpChannel* InChannel, FFrameNumber InFrameNumber, double Value)
{
	return ValueExistsAtTime(static_cast<const FMovieSceneDoubleChannel*>(InChannel), InFrameNumber, Value);
}
inline bool ValueExistsAtTime(const FMovieSceneTimeWarpChannel* InChannel, FFrameNumber InFrameNumber, float Value)
{
	return ValueExistsAtTime(static_cast<const FMovieSceneDoubleChannel*>(InChannel), InFrameNumber, Value);
}
inline void AssignValue(FMovieSceneTimeWarpChannel* InChannel, FKeyHandle InKeyHandle, double InValue)
{
	AssignValue(static_cast<FMovieSceneDoubleChannel*>(InChannel), InKeyHandle, InValue);
}
inline void AssignValue(FMovieSceneTimeWarpChannel* InChannel, FKeyHandle InKeyHandle, float InValue)
{
	AssignValue(static_cast<FMovieSceneDoubleChannel*>(InChannel), InKeyHandle, InValue);
}
template<>
struct TMovieSceneChannelTraits<FMovieSceneTimeWarpChannel> : TMovieSceneChannelTraitsBase<FMovieSceneTimeWarpChannel>
{
	enum { SupportsDefaults = false };
};
