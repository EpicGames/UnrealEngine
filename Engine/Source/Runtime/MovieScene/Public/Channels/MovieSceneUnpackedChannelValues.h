// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/UnrealTemplate.h"
#include "Misc/InlineValue.h"
#include "KeyParams.h"
#include "Concepts/EqualityComparable.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneChannelTraits.h"

struct FMovieSceneChannel;

namespace UE::MovieScene
{

/**
 * Abstraction for interacting with a value on a specific channel, principally used for auto-key and keying operations in the Sequencer editor
 */
struct IChannelValue
{
	virtual ~IChannelValue() = default;

	/**
	 * Retrieve the channel pointer that this value relates to
	 */
	virtual FMovieSceneChannel* RetrieveChannel(FMovieSceneChannelProxy& Proxy) const = 0;

	/**
	 * Ask the channel whether the wrapped value already exists at the specified time
	 */
	virtual bool AlreadyExistsAtTime(const FMovieSceneChannel* InChannel, FFrameNumber InTime) const = 0;

	/**
	 * Ask the channel whether the wrapped value is already the default
	 */
	virtual bool IsAlreadyDefault(const FMovieSceneChannel* Channel) const = 0;

	/**
	 * Add a key at the specified time with a specified interpolation type
	 */
	virtual void AddKey(FMovieSceneChannel* Channel, FFrameNumber InTime, EMovieSceneKeyInterpolation InterpolationMode) const = 0;

	/**
	 * Set this channel value as the channel's default
	 */
	virtual void SetDefault(FMovieSceneChannel* Channel) const = 0;
};

template<typename ChannelType>
struct TChannelValue : IChannelValue
{
	using ValueType = typename ChannelType::CurveValueType;

	template<typename U>
	TChannelValue(U&& InValue)
		: Value(Forward<U>(InValue))
	{}

	virtual bool AlreadyExistsAtTime(const FMovieSceneChannel* InChannel, FFrameNumber InTime) const override
	{
		const ChannelType* TypedChannel = static_cast<const ChannelType*>(InChannel);
		return ValueExistsAtTime(TypedChannel, InTime, Value);
	}

	virtual bool IsAlreadyDefault(const FMovieSceneChannel* InChannel) const override
	{
		using namespace UE::MovieScene;

		if constexpr (TModels_V<CEqualityComparable, ValueType>)
		{
			const ChannelType* TypedChannel = static_cast<const ChannelType*>(InChannel);

			ValueType DefaultValue;
			return GetChannelDefault(TypedChannel, DefaultValue) && DefaultValue == Value;
		}
		else
		{
			return false;
		}
	}

	virtual void AddKey(FMovieSceneChannel* InChannel, FFrameNumber InTime, EMovieSceneKeyInterpolation InterpolationMode) const override
	{
		ChannelType* TypedChannel = static_cast<ChannelType*>(InChannel);

		InterpolationMode = GetInterpolationMode(TypedChannel, InTime, InterpolationMode);
		AddKeyToChannel(TypedChannel, InTime, Value, InterpolationMode);
	}

	virtual void SetDefault(FMovieSceneChannel* InChannel) const override
	{
		ChannelType* TypedChannel = static_cast<ChannelType*>(InChannel);
		SetChannelDefault(TypedChannel, Value);
	}

private:

	ValueType Value;
};

template<typename ChannelType>
struct TIndexedChannelValue : TChannelValue<ChannelType>
{
	template<typename U>
	TIndexedChannelValue(U&& InValue, int32 InChannelIndex)
		: TChannelValue<ChannelType>(Forward<U>(InValue))
		, ChannelIndex(InChannelIndex)
	{}

	virtual FMovieSceneChannel* RetrieveChannel(FMovieSceneChannelProxy& Proxy) const
	{
		return Proxy.GetChannel<ChannelType>(ChannelIndex);
	}

private:

	int32 ChannelIndex;
};



struct FUnpackedChannelValue
{
	template<typename ChannelType>
	FUnpackedChannelValue(TIndexedChannelValue<ChannelType>&& InChannelValue, FName InPropertyPath)
		: PropertyPath(InPropertyPath)
		, ChannelValue(MoveTemp(InChannelValue))
	{}

	const IChannelValue* operator->() const
	{
		return ChannelValue.GetPtr();
	}

	FName GetPropertyPath() const
	{
		return PropertyPath;
	}

private:

	FName PropertyPath;
	TInlineValue<IChannelValue, 24> ChannelValue;
};


#define UE_MOVIESCENE_UNPACKED_MEMBER(ChannelType, ChannelIndex, StructInstance, Member) \
	FUnpackedChannelValue(TIndexedChannelValue<ChannelType>(StructInstance.Member, ChannelIndex), FName(#Member))


/**
 * An array-like container of type-erased values for a collection of FMovieSceneChannels
 * This is used by track editors and property traits as a way of unpacking composite properties into their
 *    constituent channels and values.
 */
struct FUnpackedChannelValues
{
	/**
	 * Add another value to this collection.
	 * @note: Addition order is normally important for most properties since it
	 *        maps to the position of the channel in the channel list
	 */
	void Add(FUnpackedChannelValue&& InValue)
	{
		Values.Emplace(MoveTemp(InValue));
	}

	/**
	 * Return the total number of elements in this container
	 */
	int32 Num() const
	{
		return Values.Num();
	}

	/**
	 * Retrieve the value at the specified index.
	 * @note: Index must be a valid index into this container
	 */
	const FUnpackedChannelValue& operator[](int32 Index) const
	{
		return Values[Index];
	}

	/**
	 * Remove and return the value at the specified index
	 * @note: Every subsequent entry will be relocated to fill the resulting gap
	 */
	FUnpackedChannelValue StealAtIndex(int32 Index)
	{
		FUnpackedChannelValue Value = MoveTemp(Values[Index]);
		Values.RemoveAt(Index, 1, EAllowShrinking::No);
		return Value;
	}

	/**
	 * Retrieve all channel values as a mutable array view
	 */
	TArrayView<FUnpackedChannelValue> GetValues()
	{
		return Values;
	}

private:

	/** Array storage of sequential values */
	TArray<FUnpackedChannelValue, TInlineAllocator<16>> Values;
};

} // namespace UE::MovieScene
