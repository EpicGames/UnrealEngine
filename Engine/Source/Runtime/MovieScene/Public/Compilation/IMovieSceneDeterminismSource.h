// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "Containers/Array.h"
#include "Misc/FrameTime.h"
#include "Compilation/MovieSceneDeterminismFence.h"
#include "IMovieSceneDeterminismSource.generated.h"

/** Determinism data that is generated on compile of a UMovieSceneSequence */
USTRUCT()
struct FMovieSceneDeterminismData
{
	GENERATED_BODY()

	/**
	 * Array of times that should be treated as fences. Fences cannot be crossed in a single evaluation, and force the evaluation to be split into 2 separate parts.
	 * Duplicates are allowed during compilation, but will be removed in the final compiled data.
	 */
	UPROPERTY()
	TArray<FMovieSceneDeterminismFence> Fences;

	/** True if this sequence should include a fence on the lower bound of any sub sequence's that include it */
	UPROPERTY()
	bool bParentSequenceRequiresLowerFence = false;

	/** True if this sequence should include a fence on the upper bound of any sub sequence's that include it */
	UPROPERTY()
	bool bParentSequenceRequiresUpperFence = false;
};


UINTERFACE(MinimalAPI)
class UMovieSceneDeterminismSource : public UInterface
{
public:
	GENERATED_BODY()
};


/**
 * Interface that can be added to a UMovieSceneSequence, UMovieSceneTrack or UMovieSceneSection in order to provide
 * determinism fences during compilation
 */
class IMovieSceneDeterminismSource
{
public:
	GENERATED_BODY()

public:

	/**
	 * Called during compilation to populate determinism data for the specified local range
	 */
	virtual void PopulateDeterminismData(FMovieSceneDeterminismData& OutData, const TRange<FFrameNumber>& Range) const = 0;
};

