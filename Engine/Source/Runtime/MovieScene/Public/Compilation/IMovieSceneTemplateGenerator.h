// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MovieSceneSequenceID.h"
#include "MovieSceneFwd.h"
#include "Containers/ArrayView.h"
#include "Evaluation/MovieSceneSequenceTransform.h"

class UMovieSceneTrack;
struct FMovieSceneEvaluationFieldSegmentPtr;
struct FMovieSceneEvaluationTrack;
struct FMovieSceneSharedDataId;
struct FMovieSceneSubSequenceData;

enum class ESectionEvaluationFlags : uint8;

/** Abstract base class used to generate evaluation templates */
struct IMovieSceneTemplateGenerator
{
	/**
	 * Add a new track that is to be owned by this template
	 *
	 * @param InTrackTemplate			The track template to add
	 * @param SourceTrack				The originating track
	 */
	virtual void AddOwnedTrack(FMovieSceneEvaluationTrack&& InTrackTemplate, const UMovieSceneTrack& SourceTrack) = 0;

protected:
	virtual ~IMovieSceneTemplateGenerator() = default;
};

