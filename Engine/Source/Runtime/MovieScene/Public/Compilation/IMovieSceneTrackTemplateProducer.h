// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"
#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "IMovieSceneTrackTemplateProducer.generated.h"

class UObject;

enum class EMovieSceneCompletionMode : uint8;

class FArchive;
class UClass;
class UMovieSceneSection;
class UMovieSceneTrack;
struct FMovieSceneEvalTemplatePtr;
struct FMovieSceneEvaluationTrack;
struct IMovieSceneTemplateGenerator;


/** Track compiler arguments */
struct FMovieSceneTrackCompilerArgs
{
	MOVIESCENE_API FMovieSceneTrackCompilerArgs(UMovieSceneTrack* InTrack, IMovieSceneTemplateGenerator* InGenerator);

	/** The object binding ID that this track belongs to. */
	FGuid ObjectBindingId;

	EMovieSceneCompletionMode DefaultCompletionMode;

	UMovieSceneTrack* Track;

	/** The generator responsible for generating the template */
	IMovieSceneTemplateGenerator* Generator;

	struct FMovieSceneSequenceTemplateStore {};

	struct FMovieSceneTrackCompilationParams { 	bool bForEditorPreview, bDuringBlueprintCompile; };
};


/** Enumeration specifying the result of a compilation */
enum class EMovieSceneCompileResult : uint8
{
	/** The compilation was successful */
	Success,
	/** The compilation was not successful */
	Failure,
	/** No compilation routine was implemented */
	Unimplemented
};

UINTERFACE(MinimalAPI)
class UMovieSceneTrackTemplateProducer : public UInterface
{
public:
	GENERATED_BODY()
};

class IMovieSceneTrackTemplateProducer
{
public:
	GENERATED_BODY()


	//~ Methods relating to compilation 

public:

	/**
	 * Generate a template for this track
	 *
	 * @param Args 			Compilation arguments
	 */
	MOVIESCENE_API virtual void GenerateTemplate(const FMovieSceneTrackCompilerArgs& Args) const;

	/**
	 * Get a raw compiled copy of this track with no additional shared tracks or compiler parameters
	 */
	MOVIESCENE_API FMovieSceneEvaluationTrack GenerateTrackTemplate(UMovieSceneTrack* SourceTrack) const;


protected:

	/**
	 * Overridable user defined custom compilation method
	 *
	 * @param Track 		Destination track to compile into
	 * @param Args 			Compilation arguments
	 * @return Compilation result
	 */
	virtual EMovieSceneCompileResult CustomCompile(FMovieSceneEvaluationTrack& Track, const FMovieSceneTrackCompilerArgs& Args) const { return EMovieSceneCompileResult::Unimplemented; }

	/**
	 * Called after this track has been compiled, regardless of whether it was compiled through CustomCompile, or the default logic
	 *
	 * @param Track 		Destination track to compile into
	 * @param Args 			Compilation arguments
	 */
	virtual void PostCompile(FMovieSceneEvaluationTrack& Track, const FMovieSceneTrackCompilerArgs& Args) const {}

protected:

	/**
	 * Create a movie scene eval template for the specified section
	 *
	 * @param InSection		The section to create a template for
	 * @return A template, or null
	 */
	virtual FMovieSceneEvalTemplatePtr CreateTemplateForSection(const UMovieSceneSection& InSection) const = 0;

	/**
	 * Compile this movie scene track into an efficient runtime structure
	 *
	 * @param Track 		Destination track to compile into
	 * @param Args 			Compilation arguments
	 * @return Compilation result
	 */
	MOVIESCENE_API EMovieSceneCompileResult Compile(FMovieSceneEvaluationTrack& Track, const FMovieSceneTrackCompilerArgs& Args) const;

};
