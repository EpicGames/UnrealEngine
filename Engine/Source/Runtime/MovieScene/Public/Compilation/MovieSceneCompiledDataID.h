// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

struct FMovieSceneCompiledDataID
{
	FMovieSceneCompiledDataID()
		: Value(INDEX_NONE)
	{}

	explicit FMovieSceneCompiledDataID(int32 SuppliedIndex)
		: Value(SuppliedIndex)
	{}

	bool IsValid() const
	{
		return Value != INDEX_NONE;
	}

	int32 Value;
};