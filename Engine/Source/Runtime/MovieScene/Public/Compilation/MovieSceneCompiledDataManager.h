// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compilation/MovieSceneCompiledDataID.h"
#include "Compilation/MovieSceneDeterminismFence.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "CoreTypes.h"
#include "Evaluation/MovieSceneEvaluationField.h"
#include "Evaluation/MovieSceneEvaluationTemplate.h"
#include "Evaluation/MovieSceneSequenceHierarchy.h"
#include "HAL/CriticalSection.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/FrameTime.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "MovieSceneFwd.h"
#include "UObject/Object.h"
#include "UObject/ObjectKey.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneCompiledDataManager.generated.h"

class UMovieSceneSequence;
class UMovieSceneSubSection;
class UMovieSceneSubTrack;
class UMovieSceneTrack;
struct FCompileOnTheFlyData;
struct FFrameNumber;
struct FGatherParameters;
struct FMovieSceneBinding;
struct FMovieSceneEvaluationOperand;
struct FMovieSceneGatheredCompilerData;
struct FMovieSceneSequenceID;
struct FTrackGatherParameters;
template<typename DataType> struct TMovieSceneEvaluationTreeDataIterator;

namespace UE
{
namespace MovieScene
{
	struct FSubSequencePath;
}
}

UENUM(Flags)
enum class EMovieSceneSequenceCompilerMask : uint8
{
	Hierarchy               = 1 << 0,
	EvaluationTemplate      = 1 << 1,
	EvaluationTemplateField = 1 << 2,
	EntityComponentField    = 1 << 3,

	None                    = 0,
};
ENUM_CLASS_FLAGS(EMovieSceneSequenceCompilerMask);

/** Flags generated at compile time for a given sequence */
USTRUCT()
struct FMovieSceneCompiledSequenceFlagStruct
{
	GENERATED_BODY()

	FMovieSceneCompiledSequenceFlagStruct()
		: bParentSequenceRequiresLowerFence(0)
		, bParentSequenceRequiresUpperFence(0)
	{}

	/** True if this sequence should include a fence on the lower bound of any sub sequence's that include it */
	UPROPERTY()
	uint8 bParentSequenceRequiresLowerFence : 1;

	/** True if this sequence should include a fence on the upper bound of any sub sequence's that include it */
	UPROPERTY()
	uint8 bParentSequenceRequiresUpperFence : 1;
};


/** Used for serialization only */
UCLASS()
class UMovieSceneCompiledData : public UObject
{
public:
	GENERATED_BODY()

	UMovieSceneCompiledData();

	void Reset();

#if WITH_EDITORONLY_DATA
	static void AppendToClassSchema(FAppendToClassSchemaContext& Context);
#endif

private:
	friend class UMovieSceneCompiledDataManager;

	/** 352 Bytes */
	UPROPERTY()
	FMovieSceneEvaluationTemplate EvaluationTemplate;

	/** 352 Bytes */
	UPROPERTY()
	FMovieSceneSequenceHierarchy Hierarchy;

	/** 272 Bytes */
	UPROPERTY()
	FMovieSceneEntityComponentField EntityComponentField;

	/** 64 Bytes */
	UPROPERTY()
	FMovieSceneEvaluationField TrackTemplateField;

	/** 16 Bytes */
	UPROPERTY()
	TArray<FMovieSceneDeterminismFence> DeterminismFences;

	/** 16 bytes */
	UPROPERTY()
	FGuid CompiledSignature;

	/** 16 Bytes */
	UPROPERTY()
	FGuid CompilerVersion;

	/** 1 Byte */
	UPROPERTY()
	EMovieSceneSequenceCompilerMask AccumulatedMask;

	/** 1 Byte */
	UPROPERTY()
	EMovieSceneSequenceCompilerMask AllocatedMask;

	/** 1 Byte */
	UPROPERTY()
	EMovieSceneSequenceFlags AccumulatedFlags;

	/** 1 Byte */
	FMovieSceneCompiledSequenceFlagStruct CompiledFlags;
};


struct FMovieSceneCompiledDataEntry
{
	FMovieSceneCompiledDataEntry();

	MOVIESCENE_API UMovieSceneSequence* GetSequence() const;

	/** 16 Bytes */
	FGuid CompiledSignature;

	/** 16 Bytes */
	TArray<FMovieSceneDeterminismFence> DeterminismFences;

	/** 8 Bytes */
	FObjectKey SequenceKey;

	/** 4 Bytes */
	FMovieSceneCompiledDataID DataID;

	/** 1 Byte */
	EMovieSceneSequenceFlags AccumulatedFlags;

	/** 1 Byte */
	EMovieSceneSequenceCompilerMask AllocatedMask;

	/** 1 Byte */
	EMovieSceneSequenceCompilerMask AccumulatedMask;

	/** 1 Byte */
	FMovieSceneCompiledSequenceFlagStruct CompiledFlags;
};

UCLASS(MinimalAPI)
class UMovieSceneCompiledDataManager
	: public UObject
{
public:
	GENERATED_BODY()

	MOVIESCENE_API UMovieSceneCompiledDataManager();

#if WITH_EDITOR
	static MOVIESCENE_API UMovieSceneCompiledDataManager* GetPrecompiledData(EMovieSceneServerClientMask EmulatedMask = EMovieSceneServerClientMask::All);
#else
	static MOVIESCENE_API UMovieSceneCompiledDataManager* GetPrecompiledData();
#endif

	static MOVIESCENE_API void ReportSequenceDestroyed(UMovieSceneSequence* InSequence);

	MOVIESCENE_API UMovieSceneCompiledData* MakeCompiledData(UMovieSceneSequence* Sequence) const;

	EMovieSceneServerClientMask GetNetworkMask() const { return NetworkMask; }

	MOVIESCENE_API void SetEmulatedNetworkMask(EMovieSceneServerClientMask NewMask);

	MOVIESCENE_API void Reset(UMovieSceneSequence* Sequence);

	MOVIESCENE_API void DestroyAllData();

	MOVIESCENE_API FMovieSceneCompiledDataID GetDataID(UMovieSceneSequence* Sequence);

	MOVIESCENE_API FMovieSceneCompiledDataID FindDataID(UMovieSceneSequence* Sequence) const;

	MOVIESCENE_API void DestroyTemplate(FMovieSceneCompiledDataID DataID);

	MOVIESCENE_API bool IsDirty(UMovieSceneSequence* Sequence) const;

	MOVIESCENE_API bool IsDirty(FMovieSceneCompiledDataID CompiledDataID) const;

	MOVIESCENE_API bool IsDirty(const FMovieSceneCompiledDataEntry& Entry) const;

	MOVIESCENE_API bool ValidateEntry(FMovieSceneCompiledDataID DataID, UMovieSceneSequence* Sequence) const;

	/**
	 * Return a reference to a compiled data entry.
	 * WARNING: This reference will become invalid if any sequence in this manager is (re)compiled
	 */
	const FMovieSceneCompiledDataEntry& GetEntryRef(FMovieSceneCompiledDataID DataID) const
	{
		check(CompiledDataEntries.IsValidIndex(DataID.Value));
		return CompiledDataEntries[DataID.Value];
	}

	MOVIESCENE_API FMovieSceneCompiledDataID GetSubDataID(FMovieSceneCompiledDataID DataID, FMovieSceneSequenceID SubSequenceID);

	uint32 GetReallocationVersion() const
	{
		return ReallocationVersion;
	}

	const FMovieSceneSequenceHierarchy*    FindHierarchy(FMovieSceneCompiledDataID DataID) const { return Hierarchies.Find(DataID.Value);       }
	const FMovieSceneEvaluationTemplate*   FindTrackTemplate(FMovieSceneCompiledDataID DataID) const { return TrackTemplates.Find(DataID.Value);         }
	const FMovieSceneEvaluationField*      FindTrackTemplateField(FMovieSceneCompiledDataID DataID) const { return TrackTemplateFields.Find(DataID.Value);    }
	const FMovieSceneEntityComponentField* FindEntityComponentField(FMovieSceneCompiledDataID DataID) const { return EntityComponentFields.Find(DataID.Value);  }

	const FMovieSceneSequenceHierarchy&    GetHierarchyChecked(FMovieSceneCompiledDataID DataID) const { return Hierarchies.FindChecked(DataID.Value);           }
	const FMovieSceneEvaluationTemplate&   GetTrackTemplateChecked(FMovieSceneCompiledDataID DataID) const { return TrackTemplates.FindChecked(DataID.Value);        }
	const FMovieSceneEvaluationField&      GetTrackTemplateFieldChecked(FMovieSceneCompiledDataID DataID) const { return TrackTemplateFields.FindChecked(DataID.Value);   }
	const FMovieSceneEntityComponentField& GetEntityComponentFieldChecked(FMovieSceneCompiledDataID DataID) const { return EntityComponentFields.FindChecked(DataID.Value); }

	MOVIESCENE_API void Compile(FMovieSceneCompiledDataID DataID);

	MOVIESCENE_API void Compile(FMovieSceneCompiledDataID DataID, EMovieSceneServerClientMask InNetworkMask);

	MOVIESCENE_API FMovieSceneCompiledDataID Compile(UMovieSceneSequence* Sequence);

	MOVIESCENE_API void Compile(FMovieSceneCompiledDataID DataID, UMovieSceneSequence* Sequence);

	MOVIESCENE_API void Compile(FMovieSceneCompiledDataID DataID, UMovieSceneSequence* Sequence, EMovieSceneServerClientMask InNetworkMask);

	static MOVIESCENE_API bool CompileHierarchy(UMovieSceneSequence* Sequence, FMovieSceneSequenceHierarchy* InOutHierarchy, EMovieSceneServerClientMask NetworkMask);

	MOVIESCENE_API void CopyCompiledData(UMovieSceneSequence* Sequence);
	MOVIESCENE_API void LoadCompiledData(UMovieSceneSequence* Sequence);

public:

	// Internal API.

	bool CanMarkSignedObjectAsChangedDuringCook(UMovieSceneSequence* Sequence) const;

private:

	MOVIESCENE_API void Gather(const FMovieSceneCompiledDataEntry& Entry, UMovieSceneSequence* Sequence, const FTrackGatherParameters& Params, FMovieSceneGatheredCompilerData* OutCompilerData) const;

	MOVIESCENE_API void CompileTrack(FMovieSceneCompiledDataEntry* OutEntry, const FMovieSceneBinding* ObjectBinding, UMovieSceneTrack* Track, const FTrackGatherParameters& Params, TSet<FGuid>* OutCompiledSignatures, FMovieSceneGatheredCompilerData* OutCompilerData);

	MOVIESCENE_API void GatherTrack(const FMovieSceneBinding* ObjectBinding, UMovieSceneTrack* Track, const FTrackGatherParameters& Params, const FMovieSceneEvaluationTemplate* TrackTemplate, FMovieSceneGatheredCompilerData* OutCompilerData) const;

	MOVIESCENE_API void CompileSubSequences(const FMovieSceneSequenceHierarchy& Hierarchy, const FTrackGatherParameters& Params, FMovieSceneGatheredCompilerData* OutCompilerData);

	static MOVIESCENE_API bool CompileHierarchy(UMovieSceneSequence* Sequence, const FGatherParameters& Params, FMovieSceneSequenceHierarchy* InOutHierarchy);

	static MOVIESCENE_API bool CompileHierarchyImpl(UMovieSceneSequence* Sequence, const FGatherParameters& Params, const FMovieSceneEvaluationOperand& Operand, UE::MovieScene::FSubSequencePath* RootPath, FMovieSceneSequenceHierarchy* InOutHierarchy);

	static MOVIESCENE_API bool GenerateSubSequenceData(UMovieSceneSequence* SubSequence, const FGatherParameters& Params, const FMovieSceneEvaluationOperand& Operand, UE::MovieScene::FSubSequencePath* RootPath, FMovieSceneSequenceHierarchy* InOutHierarchy);
	static MOVIESCENE_API bool GenerateSubSequenceData(UMovieSceneSubTrack* SubTrack, const FGatherParameters& Params, const FMovieSceneEvaluationOperand& Operand, UE::MovieScene::FSubSequencePath* RootPath, FMovieSceneSequenceHierarchy* InOutHierarchy);

	static MOVIESCENE_API void PopulateSubSequenceTree(UMovieSceneSequence* SubSequence, const FGatherParameters& Params, UE::MovieScene::FSubSequencePath* RootPath, FMovieSceneSequenceHierarchy* InOutHierarchy);
	static MOVIESCENE_API void PopulateSubSequenceTree(UMovieSceneSubTrack* SubTrack, const FGatherParameters& Params, UE::MovieScene::FSubSequencePath* RootPath, FMovieSceneSequenceHierarchy* InOutHierarchy);

	static MOVIESCENE_API TOptional<FFrameNumber> GetLoopingSubSectionEndTime(const UMovieSceneSequence* InRootSequence, const UMovieSceneSubSection* SubSection, const FGatherParameters& Params);

	MOVIESCENE_API void CompileTrackTemplateField(FMovieSceneCompiledDataEntry* OutEntry, const FMovieSceneSequenceHierarchy& Hierarchy, FMovieSceneGatheredCompilerData* InCompilerData);

	MOVIESCENE_API void PopulateEvaluationGroup(const TArray<FCompileOnTheFlyData>& SortedCompileData, FMovieSceneEvaluationGroup* OutGroup);

	MOVIESCENE_API void PopulateMetaData(const FMovieSceneSequenceHierarchy& RootHierarchy, const TArray<FCompileOnTheFlyData>& SortedCompileData, TMovieSceneEvaluationTreeDataIterator<FMovieSceneSubSequenceTreeEntry> SubSequences, FMovieSceneEvaluationMetaData* OutMetaData);

	MOVIESCENE_API void ProcessTrack(FMovieSceneCompiledDataEntry* OutEntry, const FMovieSceneBinding* ObjectBinding, UMovieSceneTrack* Track, const FTrackGatherParameters& Params, FMovieSceneGatheredCompilerData* OutCompilerData, TSet<FGuid>* OutCompiledSignatures);

	MOVIESCENE_API void ProcessSubTrack(FMovieSceneCompiledDataEntry* OutEntry, UMovieSceneSubTrack* SubTrack, const FGuid& ObjectBindingId, const FTrackGatherParameters& Params, FMovieSceneGatheredCompilerData* OutCompilerData);

	MOVIESCENE_API void DestroyData(FMovieSceneCompiledDataID DataID);

	virtual void BeginDestroy() override;

private:

	FMovieSceneCompiledDataEntry* GetEntryPtr(FMovieSceneCompiledDataID DataID)
	{
		check(CompiledDataEntries.IsValidIndex(DataID.Value));
		return &CompiledDataEntries[DataID.Value];
	}

	const FMovieSceneCompiledDataEntry* GetEntryPtr(FMovieSceneCompiledDataID DataID) const
	{
		check(CompiledDataEntries.IsValidIndex(DataID.Value));
		return &CompiledDataEntries[DataID.Value];
	}

	MOVIESCENE_API void ConsoleVariableSink();

private:

	FCriticalSection AsyncLoadCriticalSection;

	friend struct FMovieSceneCompileDataManagerGenerator;

	TMap<FObjectKey, FMovieSceneCompiledDataID> SequenceToDataIDs;

	TSparseArray<FMovieSceneCompiledDataEntry>  CompiledDataEntries;

	UPROPERTY()
	TMap<int32, FMovieSceneSequenceHierarchy>    Hierarchies;

	UPROPERTY()
	TMap<int32, FMovieSceneEvaluationTemplate>   TrackTemplates;

	UPROPERTY()
	TMap<int32, FMovieSceneEvaluationField>      TrackTemplateFields;

	UPROPERTY()
	TMap<int32, FMovieSceneEntityComponentField> EntityComponentFields;

	FGuid CompilerVersion;

	uint32 ReallocationVersion;

	EMovieSceneServerClientMask NetworkMask;

	static TSet<UMovieSceneCompiledDataManager*> ActiveManagers;
};


