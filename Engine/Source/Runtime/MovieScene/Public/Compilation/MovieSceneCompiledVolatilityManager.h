// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/SortedMap.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Templates/UniquePtr.h"

class UMovieSceneCompiledDataManager;
struct FGuid;
struct FMovieSceneCompiledDataID;
struct FMovieSceneSequenceID;

namespace UE
{
namespace MovieScene
{

struct FSharedPlaybackState;

struct FCompiledDataVolatilityManager
{
	static TUniquePtr<FCompiledDataVolatilityManager> Construct(TSharedRef<const FSharedPlaybackState> SharedPlaybackState);

	FCompiledDataVolatilityManager(TSharedRef<const FSharedPlaybackState> SharedPlaybackState);

	bool ConditionalRecompile();

private:

	bool HasBeenRecompiled() const;

	bool HasSequenceBeenRecompiled(FMovieSceneCompiledDataID DataID, FMovieSceneSequenceID SequenceID) const;

	void UpdateCachedSignatures();

private:

	TWeakPtr<const FSharedPlaybackState> WeakSharedPlaybackState;
	TSortedMap<FMovieSceneSequenceID, FGuid, TInlineAllocator<16>> CachedCompilationSignatures;
};

} // namespace MovieScene
} // namespace UE
