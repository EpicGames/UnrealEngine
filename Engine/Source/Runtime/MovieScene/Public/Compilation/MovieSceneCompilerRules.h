// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compilation/MovieSceneSegmentCompiler.h"
#include "CoreMinimal.h"
#include "Misc/Optional.h"

struct FFrameNumber;
struct FMovieSceneSegment;
template <typename ElementType> class TRange;

namespace MovieSceneSegmentCompiler
{
	MOVIESCENE_API TOptional<FMovieSceneSegment> EvaluateNearestSegment(const TRange<FFrameNumber>& Range, const FMovieSceneSegment* PreviousSegment, const FMovieSceneSegment* NextSegment);

	MOVIESCENE_API bool AlwaysEvaluateSection(const FMovieSceneSectionData& InSectionData);

	MOVIESCENE_API void FilterOutUnderlappingSections(FSegmentBlendData& BlendData);

	MOVIESCENE_API void ChooseLowestRowIndex(FSegmentBlendData& BlendData);

	// Reduces the evaluated sections to only the section that resides last in the source data. Legacy behaviour from various track instances.
	MOVIESCENE_API void BlendSegmentLegacySectionOrder(FSegmentBlendData& BlendData);
}

/** Default track row segment blender for all tracks */
struct FDefaultTrackRowSegmentBlender : FMovieSceneTrackRowSegmentBlender
{
	virtual void Blend(FSegmentBlendData& BlendData) const override
	{
		// By default we only evaluate the section with the highest Z-Order if they overlap on the same row
		MovieSceneSegmentCompiler::FilterOutUnderlappingSections(BlendData);
	}
};

/** Track segment blender that evaluates the nearest segment in empty space */
struct FEvaluateNearestSegmentBlender : FMovieSceneTrackSegmentBlender
{
	FEvaluateNearestSegmentBlender()
	{
		bCanFillEmptySpace = true;
	}

	virtual TOptional<FMovieSceneSegment> InsertEmptySpace(const TRange<FFrameNumber>& Range, const FMovieSceneSegment* PreviousSegment, const FMovieSceneSegment* NextSegment) const
	{
		return MovieSceneSegmentCompiler::EvaluateNearestSegment(Range, PreviousSegment, NextSegment);
	}
};

struct FMovieSceneAdditiveCameraTrackBlender : public FMovieSceneTrackSegmentBlender
{
	virtual void Blend(FSegmentBlendData& BlendData) const override
	{
		// sort by start time to match application order of player camera
		BlendData.Sort(SortByStartTime);
	}

private:

	MOVIESCENE_API static bool SortByStartTime(const FMovieSceneSectionData& A, const FMovieSceneSectionData& B);
};
