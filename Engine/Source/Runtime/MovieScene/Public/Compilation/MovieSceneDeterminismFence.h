// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameNumber.h"
#include "MovieSceneDeterminismFence.generated.h"


/** Structure that defines a specific determinism fence */
USTRUCT()
struct FMovieSceneDeterminismFence
{
	GENERATED_BODY()

	FMovieSceneDeterminismFence()
		: bInclusive(false)
	{}

	FMovieSceneDeterminismFence(FFrameNumber InFrameNumber, bool bInInclusive = false)
		: FrameNumber(InFrameNumber)
		, bInclusive(bInInclusive)
	{}

	friend bool operator==(FMovieSceneDeterminismFence A, FMovieSceneDeterminismFence B)
	{
		return A.FrameNumber == B.FrameNumber && A.bInclusive == B.bInclusive;
	}

	/** True if this sequence should include a fence on the lower bound of any sub sequence's that include it */
	UPROPERTY()
	FFrameNumber FrameNumber;

	/**
	 * Default: false. When true, specifies that this fence should be evaluated exactly on the specified time.
	 * When false, all times up to, but not including FrameNumber will be evaluated.
	 * 
	 * Exclusive should be used for a fence at the end of a subsection with an exclusive time to ensure that the
	 *     sub-section is entirely evaluated before evaluation returns.
	 * 
	 * Inclusive should be used if an exact time must be evaluated (such as for testing purposes)
	 */
	UPROPERTY()
	uint8 bInclusive : 1;
};