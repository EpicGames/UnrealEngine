// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "CoreMinimal.h"
#include "Evaluation/MovieSceneSegment.h"
#include "HAL/Platform.h"
#include "MovieSceneSection.h"
#include "MovieSceneTrackEvaluationField.h"
#include "Templates/Function.h"

struct FMovieSceneTrackEvaluationData;
template <typename DataType> struct TMovieSceneEvaluationTree;

namespace UE
{
namespace MovieScene
{

struct FEvaluationTreePopulationRules
{
	/**
	 * Structure used by population rules for sorting sections before adding them to the evaluation tree.
	 */
	struct FSortedSection
	{
		const UMovieSceneSection& Section;
		int32 Index;

		FSortedSection(const UMovieSceneSection& InSection, int32 InSectionIndex) : Section(InSection), Index(InSectionIndex) {}

		int32 Row() const { return Section.GetRowIndex(); }
		int32 OverlapPriority() const { return Section.GetOverlapPriority(); }

		//bool operator<(const FSortedSection& Other) const { return SortByOverlapPriorityAndRow(*this, Other); }

		static bool SortByOverlapPriorityAndRow(const FSortedSection& A, const FSortedSection& B)
		{
			if (A.Row() == B.Row())
			{
				return A.OverlapPriority() > B.OverlapPriority();
			}
			return A.Row() < B.Row();
		}
	};

	using FSectionSortPredicate = TFunctionRef<bool(const FSortedSection&, const FSortedSection&)>;

	/**
	 * Adds all active and non-empty sections to the evaluation tree.
	 */
	MOVIESCENE_API static void Blended(TArrayView<UMovieSceneSection* const> Sections, TMovieSceneEvaluationTree<FMovieSceneTrackEvaluationData>& OutTree);

	/**
	 * As above, but with custom sorting for the sections before they're added to the evaluation tree.
	 */
	MOVIESCENE_API static void Blended(TArrayView<UMovieSceneSection* const> Sections, TMovieSceneEvaluationTree<FMovieSceneTrackEvaluationData>& OutTree, FSectionSortPredicate Predicate);

	/**
	 * Adds active and non-empty sections to the evaluation tree based on priority: top rows have priority over bottom rows,
	 * and overlapping sections have priority over underlapped sections.
	 * Sections that have a valid blend type are always added to the tree.
	 */
	MOVIESCENE_API static void HighPass(TArrayView<UMovieSceneSection* const> Sections, TMovieSceneEvaluationTree<FMovieSceneTrackEvaluationData>& OutTree);

	/**
	 * Adds active and non-empty sections to the evaluation tree based on priority: overlapping sections have priority over 
	 * underlapped sections.
	 * Sections that have a valid blend type are always added to the tree.
	 */
	MOVIESCENE_API static void HighPassPerRow(TArrayView<UMovieSceneSection* const> Sections, TMovieSceneEvaluationTree<FMovieSceneTrackEvaluationData>& OutTree);

	/**
	 * Adds active and non-empty sections to the evaluation tree based on the priority defined in the given predicate.
	 * For any time range, only the first section (based on the given predicate's sorting) will be added to the evaluation tree, and others will be discarded.
	 * Sections that have a valid blend type are always added to the tree.
	 */
	MOVIESCENE_API static void HighPassCustom(TArrayView<UMovieSceneSection* const> Sections, TMovieSceneEvaluationTree<FMovieSceneTrackEvaluationData>& OutTree, FSectionSortPredicate Predicate);

	/**
	 * Adds active and non-empty sections to the evaluation tree based on the priority defined in the given predicate.
	 * For any time range, only the first section of each row (based on the given predicate's sorting) will be added to the evaluation tree, and others will be discarded.
	 * Sections that have a valid blend type are always added to the tree.
	 */
	MOVIESCENE_API static void HighPassCustomPerRow(TArrayView<UMovieSceneSection* const> Sections, TMovieSceneEvaluationTree<FMovieSceneTrackEvaluationData>& OutTree, FSectionSortPredicate Predicate);

	/**
	 * Runs the logic to populate empty ranges in the field with the nearest section
	 */
	MOVIESCENE_API static void PopulateNearestSection(TArrayView<UMovieSceneSection* const> Sections, TMovieSceneEvaluationTree<FMovieSceneTrackEvaluationData>& OutTree);

private:
	static void SortSections(TArrayView<UMovieSceneSection* const> Sections, TArray<FSortedSection, TInlineAllocator<16>>& SortedSections, FSectionSortPredicate Predicate);
};

} // namespace MovieScene
} // namespace UE
