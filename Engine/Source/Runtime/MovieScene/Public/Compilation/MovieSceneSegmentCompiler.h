// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "CoreFwd.h"
#include "CoreTypes.h"
#include "Evaluation/MovieSceneSegment.h"
#include "HAL/PlatformCrt.h"
#include "Misc/InlineValue.h"
#include "Misc/Optional.h"

class UMovieSceneSection;
struct FFrameNumber;
template <typename ElementType> class TRange;
template <typename T> struct TIsContiguousContainer;

/**
 * Data structure supplied to segment blenders that includes information about the section
 */
struct FMovieSceneSectionData
{
	FMovieSceneSectionData()
		: Section(nullptr), TemplateIndex(INDEX_NONE), Flags(ESectionEvaluationFlags::None)
	{}

	/** Constructor */
	FMovieSceneSectionData(const UMovieSceneSection* InSection, int32 InTemplateIndex, ESectionEvaluationFlags InFlags)
		: Section(InSection), TemplateIndex(InTemplateIndex), Flags(InFlags)
	{}

	/** The section that is to be evaluated */
	const UMovieSceneSection* Section;
	/** The index of the template within the evaluation track */
	int32 TemplateIndex;
	/** Evaluation flags for the section */
	ESectionEvaluationFlags Flags;
};

/**
 * Container supplied to segment blenders that includes information about all sections to be evaluated.
 * Blenders are free to reorder and filter the elements as they desire.
 * Implemented as a named type to afford future changes without changing call-sites signatures.
 */
struct FSegmentBlendData : TArray<FMovieSceneSectionData, TInlineAllocator<4>>
{
	/**
	 * Add all the section data to the specified segment
	 *
	 * @param Segment The segment to populate with this data
	 */
	void AddToSegment(FMovieSceneSegment& Segment) const
	{
		for (const FMovieSceneSectionData& SectionData : *this)
		{
			Segment.Impls.Add(FSectionEvaluationData(SectionData.TemplateIndex, SectionData.Flags));
		}
	}
};

//~ trait to re-enable GetData on FSegmentBlendData
template<> struct TIsContiguousContainer<FSegmentBlendData>
{
	enum { Value = true };
};

/**
 * Structure that defines how to combine and sort overlapping sections on the same row.
 */
struct FMovieSceneTrackRowSegmentBlender
{
	virtual ~FMovieSceneTrackRowSegmentBlender() = default;

	/**
	 * Blend the specified data by performing some specific processing such as sorting or filtering 
	 *
	 * @param BlendData				Blend data to be modified to suit the implementation's will
	 */
	virtual void Blend(FSegmentBlendData& BlendData) const
	{
	}
};

/**
 * Structure that defines how to combine and sort sections in the same track row after processing on a per-row basis.
 */
struct FMovieSceneTrackSegmentBlender
{
	/** Default constructor */
	FMovieSceneTrackSegmentBlender()
	{
		bCanFillEmptySpace = false;
		bAllowEmptySegments = false;
	}

	virtual ~FMovieSceneTrackSegmentBlender() = default;

	/**
	 * Check whether the resulting segments be empty. When true, Empty segments will be allowed to be added to the resulting evaluation field.
	 * @note: use when evaluation of a track should always be performed, even if there are no sections at the current time
	 */
	bool AllowEmptySegments() const
	{
		return bAllowEmptySegments;
	}

	/**
	 * Check whether these rules can fill empty space between segments (such as to evaluate the nearest section)
	 * @note: use in combination with AllowEmptySegments when evaluation of a track should always be performed, even if there are no sections at the current time
	 */
	bool CanFillEmptySpace() const
	{
		return bCanFillEmptySpace;
	}

	/**
	 * Implementation function to insert empty space between two other segments or at the start/end
	 *
	 * @param Range					The time range for the potential new segment
	 * @param PreviousSegment		A pointer to the previous segment if there is one. Null if the empty space is before the first segment.
	 * @param Next					A pointer to the subsequent segment if there is one. Null if the empty space is after the last segment.
	 *
	 * @return An optional new segment to define
	 */
	virtual TOptional<FMovieSceneSegment> InsertEmptySpace(const TRange<FFrameNumber>& Range, const FMovieSceneSegment* PreviousSegment, const FMovieSceneSegment* NextSegment) const
	{
		return TOptional<FMovieSceneSegment>();
	}

	/**
	 * Blend the specified data by performing some specific processing such as sorting or filtering 
	 *
	 * @param BlendData				Blend data to be modified to suit the implementation's will
	 */
	virtual void Blend(FSegmentBlendData& BlendData) const
	{
	}

protected:

	/** Whether we allow empty segments to be evaluated or not */
	bool bAllowEmptySegments;

	/** Whether we can fill empty space (false signifies InsertEmptySpace will never be called) */
	bool bCanFillEmptySpace;
};


typedef TInlineValue<FMovieSceneTrackRowSegmentBlender, 16> FMovieSceneTrackRowSegmentBlenderPtr;
typedef TInlineValue<FMovieSceneTrackSegmentBlender, 16> FMovieSceneTrackSegmentBlenderPtr;
