// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Evaluation/MovieSceneAnimTypeID.h"
#include "HAL/Platform.h"
#include "Misc/InlineValue.h"
#include "Misc/Optional.h"
#include "Templates/Decay.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class UObject;
struct FMovieSceneBlendingAccumulator;
struct FMovieSceneContext;
template<typename T> struct TMovieSceneInterrogationIterator;

/** Key used for populating template interrogation data */
struct FMovieSceneInterrogationKey
{
	/** Unique identifier for the type of data that resulted from the interrogation - doesn't at all have to relate to any FMovieSceneAnimTypeIDs used at runtime */
	FMovieSceneAnimTypeID AnimTypeID;

	FMovieSceneInterrogationKey(FMovieSceneAnimTypeID InAnimTypeID)
		: AnimTypeID(InAnimTypeID)
	{}

	friend bool operator==(FMovieSceneInterrogationKey A, FMovieSceneInterrogationKey B)
	{
		return A.AnimTypeID == B.AnimTypeID;
	}
	friend bool operator!=(FMovieSceneInterrogationKey A, FMovieSceneInterrogationKey B)
	{
		return A.AnimTypeID != B.AnimTypeID;
	}

	/**
	* Access the interrogation key for transform data - any interrogation data stored with this key is guaranteed to be of type 'FTransform'
	* to be used by various editors and sections that reference transforms
	*/
	MOVIESCENE_API static FMovieSceneInterrogationKey GetTransformInterrogationKey();
};

/** Tokens that are stored in FMovieSceneInterrogationData */
struct IMovieSceneInterrogationToken
{
	FMovieSceneInterrogationKey Key;
	IMovieSceneInterrogationToken(FMovieSceneInterrogationKey InKey) : Key(InKey) {}
};

template<typename T>
struct TMovieSceneInterrogationToken : IMovieSceneInterrogationToken
{
	T Data;

	template<typename U> TMovieSceneInterrogationToken(U&& In, FMovieSceneInterrogationKey InKey)
		: IMovieSceneInterrogationToken(InKey)
		, Data(Forward<U>(In))
	{
	}
};

/**
 * Data structure that is passed to all tracks and templates when interrogating them for data.
 * Used by FMovieSceneEvaluationTrack::Interrogate
 */
struct FMovieSceneInterrogationData
{
	/**
	 * Add arbitrary data to the container under the specified key.
	 * Care should be taken by clients to ensure that data types that are added with specific keys are always accessed as the correct types
	 */
	template<typename T>
	void Add(T&& InData, FMovieSceneInterrogationKey Key)
	{
		TokenData.Add(TMovieSceneInterrogationToken<typename TDecay<T>::Type>(Forward<T>(InData), Key));
	}
	/**
	 * Iterate all data stored in this container
	 */
	TArray<TInlineValue<IMovieSceneInterrogationToken>>::TConstIterator Iterate() const
	{
		return TokenData.CreateConstIterator();
	}

	/**
	 * Iterate any data in this container that matches the specified key
	 * Care should be taken by clients to ensure that data types that are added with specific keys are always accessed as the correct types
	 */
	template<typename T>
	TMovieSceneInterrogationIterator<T> Iterate(FMovieSceneInterrogationKey Key) const
	{
		return TMovieSceneInterrogationIterator<T>(*this, Key);
	}

	MOVIESCENE_API void Finalize(const FMovieSceneContext& Context, UObject* BindingOverride);

	MOVIESCENE_API FMovieSceneBlendingAccumulator& GetAccumulator();

private:

	template<typename T> friend struct TMovieSceneInterrogationIterator;
	TArray<TInlineValue<IMovieSceneInterrogationToken>> TokenData;

	/** Optional accumulator that is allocated when required */
	TSharedPtr<FMovieSceneBlendingAccumulator> Accumulator;
};


template<typename DataType>
struct TMovieSceneInterrogationIterator
{
	TMovieSceneInterrogationIterator(const FMovieSceneInterrogationData& InContainer, FMovieSceneInterrogationKey InPredicateKey)
		: Container(InContainer)
		, PredicateKey(InPredicateKey)
		, Index(-1)
	{
		NextElement();
	}

	TMovieSceneInterrogationIterator& operator++()
	{
		NextElement();
		return *this;
	}

	const DataType& operator*() const
	{
		return static_cast<const TMovieSceneInterrogationToken<DataType>&>(Container.TokenData[Index].GetValue()).Data;
	}

	const DataType* operator->() const
	{
		return &static_cast<const TMovieSceneInterrogationToken<DataType>&>(Container.TokenData[Index].GetValue()).Data;
	}

	inline explicit operator bool() const
	{
		return Container.TokenData.IsValidIndex(Index);
	}

	inline bool operator!() const 
	{
		return !(bool)*this;
	}

	inline friend bool operator==(const TMovieSceneInterrogationIterator& LHS, const TMovieSceneInterrogationIterator& RHS) { return &LHS.Container == &RHS.Container && LHS.Index == RHS.Index; }
	inline friend bool operator!=(const TMovieSceneInterrogationIterator& LHS, const TMovieSceneInterrogationIterator& RHS) { return &LHS.Container != &RHS.Container || LHS.Index != RHS.Index; }

	inline friend TMovieSceneInterrogationIterator<DataType> begin(const TMovieSceneInterrogationIterator<DataType>& In) { return In; }
	inline friend TMovieSceneInterrogationIterator<DataType> end(  const TMovieSceneInterrogationIterator<DataType>& In) { TMovieSceneInterrogationIterator<DataType> New = In; New.Index = -1; return New; }

	void NextElement()
	{
		while (Container.TokenData.IsValidIndex(++Index))
		{
			if (!PredicateKey.IsSet() || Container.TokenData[Index]->Key == PredicateKey)
			{
				break;
			}
		}

		if (!Container.TokenData.IsValidIndex(Index))
		{
			Index = -1;
		}
	}

private:
	const FMovieSceneInterrogationData& Container;
	TOptional<FMovieSceneInterrogationKey> PredicateKey;
	int32 Index;
};


