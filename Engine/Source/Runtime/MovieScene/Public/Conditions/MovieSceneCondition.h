// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneSequenceID.h"
#include "MovieSceneSignedObject.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Containers/Array.h"
#include "MovieSceneBindingProxy.h"
#include "MovieSceneCondition.generated.h"

namespace UE
{
	namespace MovieScene
	{
		struct FSharedPlaybackState;
	}
}

/*
* Defines the scope of a particular condition type.
* By default, the condition scope will determine whether conditions need to be re-evaluated for different bindings or entities in the Sequence.
*/
UENUM(BlueprintType)
enum class EMovieSceneConditionScope : uint8
{
	/* Condition has the same result regardless of the binding or entity.*/
	Global,
	/* Condition may have different results for different object bindings. */
	Binding,
	/* Condition may have different results for each different outer object owner (i.e. track, section) in the Sequence.*/
	OwnerObject,
};

/* Defines how often a condition needs to be checked. 
*  Most conditions should return 'Once', but if the condition result can change during playback, 'OnTick' can be chosen to have the condition re-evaluated each tick.
*/
UENUM(BlueprintType)
enum class EMovieSceneConditionCheckFrequency : uint8
{
	/* Condition result will not change during sequence playback and only needs to get checked once. */
	Once,
	/* Condition result may change during sequence playback and should be checked per tick. */
	OnTick,
};

/*
* Blueprint-friendly struct containing any context needed to evaluate conditions. 
*/
USTRUCT(BlueprintType)
struct FMovieSceneConditionContext
{
GENERATED_BODY()
public:

	/* The world context*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Default")
	TObjectPtr<UObject> WorldContext;
	
	/* Binding for the bound object currently evaluating this condition if applicable (BindingId will be invalid for conditions on global tracks/sections). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Default")
	FMovieSceneBindingProxy Binding;

	/* Array of objects bound to the binding currently evaluating this condition if applicable (will be empty for conditions on global tracks/sections)*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Default")
	TArray<TObjectPtr<UObject>> BoundObjects;
};

/*
* Container struct for instanced UMovieSceneConditions, existing only to allow property type customization for choosing conditions.
*/
USTRUCT(BlueprintType)
struct FMovieSceneConditionContainer
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Instanced, Category="Sequencer|Condition", meta = (EditInline, AllowEditInlineCustomization))
	TObjectPtr<UMovieSceneCondition> Condition = nullptr;
};

/**
 * Abstract condition class. Conditions can be applied to sections, tracks, and track rows to determine whether or not they are evaluated at runtime.
 * This allows developers to create Sequences with dynamic behavior based on gameplay state, local player state, player hardware, etc.
 */
UCLASS(abstract, Blueprintable, DefaultToInstanced, EditInlineNew, meta = (ShowWorldContextPin), CollapseCategories, MinimalAPI)
class UMovieSceneCondition
	: public UMovieSceneSignedObject
{
	GENERATED_BODY()

public:

	/* Called by Sequencer code to evaluate this condition, passing relevant context. Note that BindingGuid will be invalid for conditions on global sections/tracks. */
	MOVIESCENE_API bool EvaluateCondition(FGuid BindingGuid, FMovieSceneSequenceID SequenceID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const;

	/* Called by Sequencer to compute a cache key for this condition given the passed in context.
	*  By default, this key will be computed based on the Binding Scope, and if relevant, the binding and entity owner.
	* If a condition returns the same cache key given the same or different contexts, it will not be rechecked, and a cached value may be used.
	*/
	MOVIESCENE_API virtual uint32 ComputeCacheKey(FGuid BindingGuid, FMovieSceneSequenceID SequenceID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, UObject* EntityOwner) const;

	MOVIESCENE_API virtual bool CanCacheResult(TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const;

	MOVIESCENE_API EMovieSceneConditionScope GetConditionScope() const;

#if WITH_EDITORONLY_DATA
	/* If true, will skip evaluating the condition and always return true. Useful for authoring or debugging. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Default")
	bool bEditorForceTrue = false;

	friend class FMovieSceneConditionCustomization;
#endif

protected:

	/* Override to implement your condition.*/
	UFUNCTION(BlueprintImplementableEvent, CallInEditor, meta=(DisplayName = "On Evaluate Condition"))
	MOVIESCENE_API bool BP_EvaluateCondition(const FMovieSceneConditionContext& ConditionContext) const;

	/* Override in native code to implement your condition. Note that BindingGuid will be invalid for conditions on global sections/tracks.*/
	MOVIESCENE_API virtual bool EvaluateConditionInternal(FGuid BindingGuid, FMovieSceneSequenceID SequenceID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const;
	
	/* Returns the scope of the condition, which determines whether the condition needs to be re-evaluated for different bindings or entities in the Sequence. */
	UFUNCTION(BlueprintNativeEvent, CallInEditor, meta=(DisplayName = "Get Scope"))
	MOVIESCENE_API EMovieSceneConditionScope BP_GetScope() const;

	/* Returns the scope of the condition, which determines whether the condition needs to be re-evaluated for different bindings or entities in the Sequence. */
	MOVIESCENE_API virtual EMovieSceneConditionScope GetScopeInternal() const;

	/* Returns the check frequency of the condition, which determines whether the condition result can change during playback and needs to get re-evaluated. */
	UFUNCTION(BlueprintNativeEvent, CallInEditor, meta=(DisplayName = "Get Check Frequency"))
	MOVIESCENE_API EMovieSceneConditionCheckFrequency BP_GetCheckFrequency() const;

	/* Returns the check frequency of the condition, which determines whether the condition result can change during playback and needs to get re-evaluated. */
	MOVIESCENE_API virtual EMovieSceneConditionCheckFrequency GetCheckFrequencyInternal() const;

protected:
	/* If true, inverts the result of the condition check.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Default")
	bool bInvert = false;

};
