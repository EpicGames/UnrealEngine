// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Conditions/MovieSceneCondition.h"
#include "MovieSceneSequenceID.h"
#include "MovieSceneSignedObject.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Containers/Array.h"
#include "MovieSceneGroupCondition.generated.h"

#define UE_API MOVIESCENE_API

namespace UE
{
	namespace MovieScene
	{
		struct FSharedPlaybackState;
	}
}

/* Which operator to use in evaluating a MovieSceneGroupCondition*/
UENUM(BlueprintType)
enum class EMovieSceneGroupConditionOperator : uint8
{
	And,
	Or,
	Xor
};

/**
 * Condition class that allows the grouping of other conditions using 'and', 'or', or 'xor'.
 */ 
UCLASS(MinimalAPI, BlueprintType, DefaultToInstanced, EditInlineNew, Meta=(DisplayName="Group Condition"))
class UMovieSceneGroupCondition
	: public UMovieSceneCondition
{
	GENERATED_BODY()

public:
	
	/* Which operator to use in evaluating the group condition */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sequencer|Condition")
	EMovieSceneGroupConditionOperator Operator = EMovieSceneGroupConditionOperator::And;

	/* List of sub-conditions to evaluate as part of this condition. Condition results will be combined together using ConditionOperator */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sequencer|Condition")
	TArray<FMovieSceneConditionContainer> SubConditions;

	/* Cache Key overridden to combine cache keys of sub conditions*/
	UE_API uint32 ComputeCacheKey(FGuid BindingGuid, FMovieSceneSequenceID SequenceID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, UObject* EntityOwner) const override;

protected:

	/* UMovieSceneCondition overrides */
	UE_API bool EvaluateConditionInternal(FGuid BindingGuid, FMovieSceneSequenceID SequenceID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const override;
	
	UE_API virtual bool CanCacheResult(TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const override;
};

#undef UE_API
