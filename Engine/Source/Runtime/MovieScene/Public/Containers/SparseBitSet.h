// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/UnrealTemplate.h"
#include "Containers/Array.h"

namespace UE::MovieScene
{

template<typename T, int32 InlineSize>
struct TDynamicSparseBitSetBucketStorage;

template<typename T>
struct TFixedSparseBitSetBucketStorage;

template<typename HashType, typename BucketStorage = TDynamicSparseBitSetBucketStorage<uint8, 4>>
struct TFixedSparseBitSet;

template<typename HashType, typename BucketStorage = TDynamicSparseBitSetBucketStorage<uint8, 4>>
struct TDynamicSparseBitSet;

enum class ESparseBitSetBitResult
{
	NewlySet,
	AlreadySet,
};

namespace Private
{
	static uint32 CountTrailingZeros(uint8 In)
	{
		const uint32 X = 0xFFFFFF00 | uint32(In);
		return FMath::CountTrailingZeros(X);
	}
	static uint32 CountTrailingZeros(uint16 In)
	{
		const uint32 X = 0xFFFF0000 | uint32(In);
		return FMath::CountTrailingZeros(X);
	}
	static uint32 CountTrailingZeros(uint32 In)
	{
		return FMath::CountTrailingZeros(In);
	}
	static uint32 CountTrailingZeros(uint64 In)
	{
		return static_cast<uint32>(FMath::CountTrailingZeros64(In));
	}

	template <UE::CIntegral T>
	static int32 CountBits(T In)
	{
		if constexpr (sizeof(T) > 4)
		{
			return FMath::CountBits64(In);
		}
		else
		{
			return FMath::CountBits(In);
		}
	}

	/** Return a bitmask of all the bits less-than BitOffset */
	template<typename T, typename U>
	static T BitOffsetToLowBitMask(U BitOffset)
	{
		constexpr T One(1);
		const T Index = static_cast<T>(BitOffset);
		return (One << Index)-One;
	}

	/** Return a bitmask of all the bits greater-than-or-equal to BitOffset */
	template<typename T, typename U>
	static T BitOffsetToHighBitMask(U BitOffset)
	{
		return ~BitOffsetToLowBitMask<T>(BitOffset);
	}
}

/**
 * NOTE: This class is currently considered internal only, and should only be used by engine code.
 * A sparse bitset comprising a hash of integer indexes with set bits, and a sparse array of unsigned integers (referred to as buckets) whose width is defined by the storage.
 *
 * The maximum size bitfield that is representible by this bitset is defined as sizeof(HashType)*sizeof(BucketStorage::BucketType). For example, a 64 bit hash with 32 bit buckets
 * can represent a bitfield of upto 2048 bits.
 *
 * The hash allows for empty buckets to be completely omitted from memory, and affords very fast comparison for buckets that have no set bits.
 * This container is specialized for relatively large bitfields that have small numbers of set bits (ie, needles in haystacks) as they will provide the best memory vs CPU tradeoffs.
 */
template<typename HashType, typename BucketStorage>
struct TFixedSparseBitSet
{
	using BucketType = typename BucketStorage::BucketType;
	static constexpr uint32 HashSize    = sizeof(HashType)*8;
	static constexpr uint32 BucketSize  = sizeof(typename BucketStorage::BucketType)*8;
	static constexpr uint32 MaxNumBits  = HashSize * BucketSize;

	explicit TFixedSparseBitSet()
		: BucketHash(0)
	{}

	template<typename ...StorageArgs>
	explicit TFixedSparseBitSet(StorageArgs&& ...Storage)
		: Buckets(Forward<StorageArgs>(Storage)...)
		, BucketHash(0)
	{}

	TFixedSparseBitSet(const TFixedSparseBitSet&) = default;
	TFixedSparseBitSet& operator=(const TFixedSparseBitSet&) = default;

	TFixedSparseBitSet(TFixedSparseBitSet&&) = default;
	TFixedSparseBitSet& operator=(TFixedSparseBitSet&&) = default;

	template<typename OtherHashType, typename OtherStorageType>
	void CopyTo(TFixedSparseBitSet<OtherHashType, OtherStorageType>& Other) const
	{
		static_assert(TFixedSparseBitSet<OtherHashType, OtherStorageType>::BucketSize == BucketSize, "Cannot copy sparse bitsets of different bucket sizes");

		// Copy the buckets
		const uint32 NumBuckets = Private::CountBits(Other.BucketHash);
		Other.Buckets.SetNum(NumBuckets);
		CopyToUnsafe(Other, NumBuckets);
	}

	/** Copy this bitset to another without resizing the destination's bucket storage. Bucket storage must be >= this size. */
	template<typename OtherHashType, typename OtherStorageType>
	void CopyToUnsafe(TFixedSparseBitSet<OtherHashType, OtherStorageType>& Other, uint32 OtherBucketCapacity) const
	{
		static_assert(TFixedSparseBitSet<OtherHashType, OtherStorageType>::BucketSize == BucketSize, "Cannot copy sparse bitsets of different bucket sizes");

		const uint32 ThisNumBuckets = this->NumBuckets();
		checkf(OtherBucketCapacity >= ThisNumBuckets, TEXT("Attempting to copy a sparse bitset without enough capacity in the destination (%d, required %d)"), OtherBucketCapacity, ThisNumBuckets);

		// Copy the hash
		Other.BucketHash = this->BucketHash;

		// Copy the buckets
		FMemory::Memcpy(Other.Buckets.GetData(), this->Buckets.Storage.GetData(), sizeof(typename BucketStorage::BucketType)*ThisNumBuckets);
	}

	TFixedSparseBitSet& operator|=(const TFixedSparseBitSet& Other)
	{
		using namespace Private;

		HashType One(1u);
		HashType NewHash   = Other.BucketHash | BucketHash;
		HashType OtherHash = Other.BucketHash;

		uint32 OtherBucketIndex    = 0;
		uint32 OtherBucketBitIndex = Private::CountTrailingZeros(OtherHash);

		while (OtherBucketBitIndex < HashSize)
		{
			const HashType HashBit = HashType(1) << OtherBucketBitIndex;
			const uint32 ThisBucketIndex = Private::CountBits(BucketHash & (HashBit-1));

			if ((BucketHash & HashBit) == 0)
			{
				Buckets.Insert(Other.Buckets.Get(OtherBucketIndex), ThisBucketIndex);
			}
			else
			{
				Buckets.Get(ThisBucketIndex) |= Other.Buckets.Get(OtherBucketIndex);
			}
			
			BucketHash |= HashBit;

			++OtherBucketIndex;

			// Mask out this bit and find the index of the next one
			OtherHash &= ~(One << OtherBucketBitIndex);
			OtherBucketBitIndex = Private::CountTrailingZeros(OtherHash);
		}

		return *this;
	}

	/**
	 * Count the number of buckets in this bitset
	 */
	uint32 NumBuckets() const
	{
		return Private::CountBits(this->BucketHash);
	}

	uint32 CountSetBits() const
	{
		uint32 Total = 0;
		const uint32 TotalNumBuckets = NumBuckets();
		for (uint32 Index = 0; Index < TotalNumBuckets; ++Index)
		{
			Total += Private::CountBits(Buckets.Get(Index));
		}
		return Total;
	}

	uint32 GetMaxNumBits() const
	{
		return MaxNumBits;
	}

	bool IsEmpty() const
	{
		return this->BucketHash == 0;
	}

	/**
	 * Set the bit at the specified index.
	 * Any bits between Num and BitIndex will be considered 0.
	 *
	 * @return true if the bit was previously considered 0 and is now set, false if it was already set.
	 */
	ESparseBitSetBitResult SetBit(uint32 BitIndex)
	{
		CheckIndex(BitIndex);

		FBitOffsets Offsets(BucketHash, BitIndex);

		// Do we need to add a new bucket?
		if ( (BucketHash & Offsets.HashBit) == 0)
		{
			BucketHash |= Offsets.HashBit;
			Buckets.Insert(Offsets.BitMaskWithinBucket, Offsets.BucketIndex);
			return ESparseBitSetBitResult::NewlySet;
		}
		else if ((Buckets.Get(Offsets.BucketIndex) & Offsets.BitMaskWithinBucket) == 0)
		{
			Buckets.Get(Offsets.BucketIndex) |= Offsets.BitMaskWithinBucket;
			return ESparseBitSetBitResult::NewlySet;
		}

		return ESparseBitSetBitResult::AlreadySet;
	}

	/**
	 * Check whether the specified bit index is set
	 */
	bool IsBitSet(uint32 BitIndex) const
	{
		CheckIndex(BitIndex);

		const uint32   Hash    = BitIndex / BucketSize;
		const HashType HashBit = (HashType(1) << Hash);
		if (BucketHash & HashBit)
		{
			const uint32     BucketIndex  = Private::CountBits(BucketHash & (HashBit-1));
			const uint32     ThisBitIndex = (BitIndex-BucketSize*Hash);
			const BucketType ThisBitMask  = BucketType(1u) << ThisBitIndex;

			return Buckets.Get(BucketIndex) & ThisBitMask;
		}
		return false;
	}

	/**
	 * Get the sparse bucket index of the specified bit
	 */
	int32 GetSparseBucketIndex(uint32 BitIndex) const
	{
		CheckIndex(BitIndex);

		const uint32   Hash    = BitIndex / BucketSize;
		const HashType HashBit = (HashType(1) << Hash);
		if (BucketHash & HashBit)
		{
			uint32 BucketIndex = Private::CountBits(BucketHash & (HashBit-1));

			const uint32 ThisBitIndex = (BitIndex-BucketSize*Hash);
			const BucketType ThisBitMask = static_cast<BucketType>(BucketType(1u) << ThisBitIndex);

			BucketType ThisBucket = Buckets.Get(BucketIndex);
			if (ThisBucket & ThisBitMask)
			{
				// Compute the offset
				int32 SparseIndex = Private::CountBits(ThisBucket & (ThisBitMask-1));

				// Count all the preceding buckets to find the final sparse index
				while (BucketIndex > 0)
				{
					--BucketIndex;
					SparseIndex += Private::CountBits(Buckets.Get(BucketIndex));
				}
				return SparseIndex;
			}
		}
		return INDEX_NONE;
	}

	struct FIterator
	{
		FIterator()
			: BitSet(nullptr)
			, BucketBitIndex(0)
			, IndexWithinBucket(0)
			, CurrentBucket(0)
		{
		}

		static FIterator Begin(const TFixedSparseBitSet<HashType, BucketStorage>* InBitSet)
		{
			FIterator It;
			It.BitSet = InBitSet;
			It.CurrentBucket = 0;

			if (InBitSet->BucketHash != 0)
			{
				It.BucketBitIndex = Private::CountTrailingZeros(InBitSet->BucketHash);
				It.CurrentBucket = InBitSet->Buckets.Get(0);
				It.IndexWithinBucket = Private::CountTrailingZeros(It.CurrentBucket);
			}
			else
			{
				It.BucketBitIndex = HashSize;
				It.IndexWithinBucket = 0;
			}
			return It;
		}
		static FIterator End(const TFixedSparseBitSet<HashType, BucketStorage>* InBitSet)
		{
			FIterator It;
			It.BitSet = InBitSet;
			It.CurrentBucket = 0;
			It.BucketBitIndex = HashSize;
			It.IndexWithinBucket = 0;
			return It;
		}

		void operator++()
		{
			using namespace Private;

			// Clear the lowest 1 bit
			CurrentBucket = CurrentBucket & (CurrentBucket - 1);

			if (CurrentBucket != 0)
			{
				IndexWithinBucket = CountTrailingZeros(CurrentBucket);
			}
			else
			{
				// If this was the last bit, reset the iterator to end()
				if (BucketBitIndex == HashSize-1)
				{
					IndexWithinBucket = 0;
					BucketBitIndex = HashSize;
					return;
				}

				HashType UnvisitedBucketBitMask = BitOffsetToHighBitMask<HashType>(BucketBitIndex+1);
				BucketBitIndex = CountTrailingZeros(HashType(BitSet->BucketHash & UnvisitedBucketBitMask));

				// Check whether we're at the end
				if (BucketBitIndex == HashSize)
				{
					IndexWithinBucket = 0;
				}
				else
				{
					const uint8 NextBucketIndex = Private::CountBits(BitSet->BucketHash & BitOffsetToLowBitMask<HashType>(BucketBitIndex));
					CurrentBucket = BitSet->Buckets.Get(NextBucketIndex);
					IndexWithinBucket = CountTrailingZeros(CurrentBucket);
				}
			}
		}

		int32 operator*() const
		{
			return BucketSize*BucketBitIndex + IndexWithinBucket;
		}

		explicit operator bool() const
		{
			return BucketBitIndex < HashSize;
		}

		friend bool operator==(const FIterator& A, const FIterator& B)
		{
			return A.BitSet == B.BitSet && A.BucketBitIndex == B.BucketBitIndex && A.IndexWithinBucket == B.IndexWithinBucket;
		}
		friend bool operator!=(const FIterator& A, const FIterator& B)
		{
			return !(A == B);
		}
private:

		const TFixedSparseBitSet<HashType, BucketStorage>* BitSet;
		uint8 BucketBitIndex;
		uint8 IndexWithinBucket;

		BucketType CurrentBucket;
	};

	friend FIterator begin(const TFixedSparseBitSet<HashType, BucketStorage>& In) { return FIterator::Begin(&In); }
	friend FIterator end(const TFixedSparseBitSet<HashType, BucketStorage>& In)   { return FIterator::End(&In); }

private:

	template<typename, typename>
	friend struct TFixedSparseBitSet;

	inline void CheckIndex(uint32 BitIndex) const
	{
		checkfSlow(BitIndex < MaxNumBits, TEXT("Invalid index (%d) specified for a sparse bitset of maximum size (%d)"), BitIndex, MaxNumBits);
	}

	struct FBitOffsets
	{
		HashType HashBit;
		BucketType BitMaskWithinBucket;
		int32  BucketIndex;
		FBitOffsets(HashType InBucketHash, uint32 BitIndex)
		{
			const HashType Hash(BitIndex / BucketSize);
			HashBit = HashType(1) << Hash;

			BucketIndex = Private::CountBits(InBucketHash & (HashBit-1u));

			const uint32 ThisBitIndex = (BitIndex-BucketSize*Hash);
			BitMaskWithinBucket = BucketType(1u) << ThisBitIndex;
		}
	};

	BucketStorage Buckets;
	HashType      BucketHash;
};



template<typename T, int32 InlineSize = 8>
struct TDynamicSparseBitSetBucketStorage
{
	using BucketType = T;

	TArray<BucketType, TInlineAllocator<InlineSize>> Storage;

	void Insert(BucketType InitialValue, int32 Index)
	{
		Storage.Insert(InitialValue, Index);
	}

	BucketType* GetData()              { return Storage.GetData(); }

	BucketType& Get(int32 Index)       { return Storage[Index]; }
	BucketType  Get(int32 Index) const { return Storage[Index]; }
};

template<typename T>
struct TDynamicSparseBitSetBucketStorage<T, 0>
{
	using BucketType = T;

	TArray<BucketType> Storage;

	void Insert(BucketType InitialValue, int32 Index)
	{
		Storage.Insert(InitialValue, Index);
	}

	BucketType* GetData()              { return Storage.GetData(); }

	BucketType& Get(int32 Index)       { return Storage[Index]; }
	BucketType  Get(int32 Index) const { return Storage[Index]; }
};

template<typename T>
struct TFixedSparseBitSetBucketStorage
{
	using BucketType = T;

	explicit TFixedSparseBitSetBucketStorage()
		: Storage(nullptr)
	{}

	explicit TFixedSparseBitSetBucketStorage(BucketType* StoragePtr)
		: Storage(StoragePtr)
	{}

	TFixedSparseBitSetBucketStorage(const TFixedSparseBitSetBucketStorage&) = delete;
	void operator=(const TFixedSparseBitSetBucketStorage&) = delete;

	TFixedSparseBitSetBucketStorage(TFixedSparseBitSetBucketStorage&&) = delete;
	void operator=(TFixedSparseBitSetBucketStorage&&) = delete;

	BucketType* Storage;

	BucketType* GetData()              { return Storage; }

	BucketType& Get(int32 Index)       { return Storage[Index]; }
	BucketType  Get(int32 Index) const { return Storage[Index]; }
};

/**
 * NOTE: This class is currently considered internal only, and should only be used by engine code.
 * A dynamically sized sparse bitset comprising multiple TFixedSparseBitSets.
 * 
 * In theory this class supports the full integer range, it is optimized for small numbers of set bits within a large range, ideally when they occupy the same adjacent space.
 */
template<typename HashType, typename BucketStorage>
struct TDynamicSparseBitSet
{
	/**
	 * Get the maximum number of bits that this bitset supports
	 */
	uint32 GetMaxNumBits() const
	{
		return MAX_uint32;
	}


	/**
	 * Set the bit at the specified index.
	 * Any bits between Num and BitIndex will be considered 0.
	 *
	 * @return true if the bit was previously considered 0 and is now set, false if it was already set.
	 */
	ESparseBitSetBitResult SetBit(uint32 Bit)
	{
		const uint32 Bucket = Bit / NumBitsInBucket;

		Bit -= Bucket*NumBitsInBucket;

		FEntry* EntrPtr = Entries.GetData();

		const int32 Num = Entries.Num();
		for (int32 EntryIndex = 0; EntryIndex < Num; ++EntryIndex)
		{
			if (EntrPtr[EntryIndex].Offset == Bucket)
			{
				return EntrPtr[EntryIndex].Bits.SetBit(Bit);
			}
			else if (EntrPtr[EntryIndex].Offset > Bucket)
			{
				Entries.InsertUninitialized(EntryIndex);
				new(&Entries[EntryIndex]) FEntry(Bucket, Bit);
				return ESparseBitSetBitResult::NewlySet;
			}
		}

		Entries.Emplace(Bucket, Bit);
		return ESparseBitSetBitResult::NewlySet;
	}


	/**
	 * Check whether this container has any bits set
	 */
	bool IsEmpty() const
	{
		return Entries.Num() == 0;
	}


	/**
	 * Check whether the specified bit index is set
	 */
	bool IsBitSet(uint32 Bit) const
	{
		const uint32 Bucket = Bit / NumBitsInBucket;

		const FEntry* EntrPtr = Entries.GetData();

		const int32 Num = Entries.Num();
		for (int32 EntryIndex = 0; EntryIndex < Num; ++EntryIndex)
		{
			if (EntrPtr[EntryIndex].Offset == Bucket)
			{
				return EntrPtr[EntryIndex].Bits.IsBitSet(Bit);
			}
			if (EntrPtr[EntryIndex].Offset > Bucket)
			{
				return false;
			}
		}

		return false;
	}

	/**
	 * Count the total number of set bits in this container
	 */
	uint32 CountSetBits() const
	{
		uint32 SetBits = 0;
		for (const FEntry& Entry : Entries)
		{
			SetBits += Entry.Bits.CountSetBits();
		}
		return SetBits;
	}


	TDynamicSparseBitSet<HashType, BucketStorage>& operator|=(const TDynamicSparseBitSet<HashType, BucketStorage>& Other)
	{
		if (Other.Entries.Num() == 0)
		{
			return *this;
		}

		if (Entries.Num() == 0)
		{
			*this = Other;
			return *this;
		}


		int32 ThisIndex = 0;
		int32 OtherIndex = 0;

		while (OtherIndex < Other.Entries.Num() && Other.Entries[OtherIndex].Offset < Entries[0].Offset)
		{
			++OtherIndex;
		}

		if (OtherIndex > 0)
		{
			Entries.Insert(Other.Entries.GetData(), OtherIndex, 0);
		}

		while (OtherIndex < Other.Entries.Num() && ThisIndex < Entries.Num())
		{
			if (Other.Entries[OtherIndex].Offset < Entries[ThisIndex].Offset)
			{
				Entries.Insert(Other.Entries[OtherIndex], ThisIndex);
				++OtherIndex;
			}
			else if (Other.Entries[OtherIndex].Offset == Entries[ThisIndex].Offset)
			{
				Entries[ThisIndex].Bits |= Other.Entries[OtherIndex].Bits;
				++OtherIndex;
				++ThisIndex;
			}
			else
			{
				++ThisIndex;
			}
		}

		return *this;
	}


private:

	using FBucketBitSet = TFixedSparseBitSet<HashType, BucketStorage>;

	static constexpr uint32 NumBitsInBucket = FBucketBitSet::MaxNumBits;

	struct FEntry
	{
		FEntry(uint32 InOffset)
			: Offset(InOffset)
		{
		}
		FEntry(uint32 InOffset, uint32 InBit)
			: Offset(InOffset)
		{
			checkSlow(InBit < FBucketBitSet::MaxNumBits);
			Bits.SetBit(InBit);
		}

		FBucketBitSet Bits;
		uint32 Offset;
	};

public:

	struct FIterator
	{
		static FIterator Begin(const TDynamicSparseBitSet<HashType, BucketStorage>* InBitSet)
		{
			FIterator It;
			It.Entries = InBitSet->Entries.GetData();
			It.NumEntries = InBitSet->Entries.Num();
			It.EntryIndex = 0;
			It.CurrentOffsetInBits = 0;

			if (It.NumEntries != 0)
			{
				It.CurrentOffsetInBits = It.Entries[0].Offset * NumBitsInBucket;
				It.BucketIt = BucketIterator::Begin(&It.Entries[0].Bits);
			}
			return It;
		}
		static FIterator End(const TDynamicSparseBitSet<HashType, BucketStorage>* InBitSet)
		{
			FIterator It;
			It.Entries = InBitSet->Entries.GetData();
			It.NumEntries = InBitSet->Entries.Num();
			It.EntryIndex = It.NumEntries;
			It.CurrentOffsetInBits = 0;
			return It;
		}

		void operator++()
		{
			using namespace Private;

			++BucketIt;
			if (!BucketIt)
			{
				++EntryIndex;
				if (EntryIndex < NumEntries)
				{
					CurrentOffsetInBits = Entries[EntryIndex].Offset * NumBitsInBucket;
					BucketIt = BucketIterator::Begin(&Entries[EntryIndex].Bits);
				}
				else
				{
					CurrentOffsetInBits = 0;
					BucketIt = BucketIterator();
				}
			}
		}

		int32 operator*() const
		{
			return CurrentOffsetInBits + *BucketIt;
		}

		explicit operator bool() const
		{
			return EntryIndex < NumEntries;
		}

		friend bool operator==(const FIterator& A, const FIterator& B)
		{
			return A.Entries == B.Entries && A.EntryIndex == B.EntryIndex && A.BucketIt == B.BucketIt;
		}
		friend bool operator!=(const FIterator& A, const FIterator& B)
		{
			return !(A == B);
		}
private:
		FIterator() = default;

		using BucketIterator = typename TFixedSparseBitSet<HashType, BucketStorage>::FIterator;

		const typename TDynamicSparseBitSet<HashType, BucketStorage>::FEntry* Entries;
		BucketIterator BucketIt;
		int32 NumEntries;
		int32 EntryIndex;
		int32 CurrentOffsetInBits;
	};

	friend FIterator begin(const TDynamicSparseBitSet<HashType, BucketStorage>& In) { return FIterator::Begin(&In); }
	friend FIterator end(const TDynamicSparseBitSet<HashType, BucketStorage>& In)   { return FIterator::End(&In); }

	TArray<FEntry> Entries;
};

} // namespace UE::MovieScene
