// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/Interface.h"
#include "MovieSceneSection.h"
#include "IMovieSceneChannelDecoration.generated.h"

struct FMovieSceneChannelProxyData;

UINTERFACE(MinimalAPI)
class UMovieSceneChannelDecoration : public UInterface
{
public:
	GENERATED_BODY()
};


/**
 * Optional decoration that can be added to sections to add channels
 */
class IMovieSceneChannelDecoration
{
public:
	GENERATED_BODY()


	/**
	 * Called to add channels to the channel proxy
	 */
	virtual EMovieSceneChannelProxyType PopulateChannelProxy(FMovieSceneChannelProxyData& OutProxyData)
	{
		return EMovieSceneChannelProxyType::Static;
	}

	// Mark this decoration's channel proxy as needing reconstruction.
	// Call when internal state changes which channels PopulateChannelProxy returns.
	void InvalidateDecorationChannelProxy() { bDecorationChannelProxyDirty = true; }

	// Returns true and clears the flag if the channel proxy needs rebuilding.
	bool ConsumeDecorationChannelProxyDirty()
	{
		bool bWasDirty = bDecorationChannelProxyDirty;
		bDecorationChannelProxyDirty = false;
		return bWasDirty;
	}

private:
	bool bDecorationChannelProxyDirty = false;
};

