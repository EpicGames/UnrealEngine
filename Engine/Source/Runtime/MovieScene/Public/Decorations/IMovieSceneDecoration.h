// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/Interface.h"
#include "IMovieSceneDecoration.generated.h"

class UMovieScene;

UINTERFACE(MinimalAPI)
class UMovieSceneDecoration : public UInterface
{
public:
	GENERATED_BODY()
};


/** 
 * Optional interface that can be added to any UObject that is used as a decoration on UMovieScene objects
 *        to receive decoration and compilation events.
 * 
 * @note: This interface has no effect to objects used as UMovieSceneTrack or UMovieSceneSection decorations:
 *        use IMovieSceneTrackDecoration or IMovieSceneSectionDecoration for these cases.

 * @see UMovieScene::GetOrCreateDecoration, UMovieScene::AddDecoration
 */
class IMovieSceneDecoration
{
public:
	GENERATED_BODY()

	/**
	 * Called when this decoration is first added directly to a UMovieScene
	 */
	virtual void OnDecorationAdded(UMovieScene* MovieScene)
	{
	}


	/**
	 * Called when this decoration is removed from a UMovieScene
	 */
	virtual void OnDecorationRemoved()
	{
	}


	/**
	 * Called before the movie scene this decoration exists on is compiled
	 */
	virtual void OnPreDecorationCompiled()
	{
	}


	/**
	 * Called after the movie scene this decoration exists on is compiled
	 */
	virtual void OnPostDecorationCompiled()
	{
	}
};

