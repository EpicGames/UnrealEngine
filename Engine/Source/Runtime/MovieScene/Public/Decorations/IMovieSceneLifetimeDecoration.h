// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/Interface.h"
#include "IMovieSceneLifetimeDecoration.generated.h"

class UMovieScene;

UINTERFACE(MinimalAPI)
class UMovieSceneLifetimeDecoration : public UInterface
{
public:
	GENERATED_BODY()
};


/** 
 * Optional interface that can be added to any decoration to provide 'construct' / 'destroy' semantics for when a decoration
 *        is added to or removed from a well-formed UMovieScene hierarchy.
 */
class IMovieSceneLifetimeDecoration
{
public:
	GENERATED_BODY()


	/**
	 * Called to reconstruct this decoration for the specified MovieScene.
	 * @note: this may get called multiple times on the same instance
	 */
	virtual void OnReconstruct(UMovieScene* MovieScene)
	{
	}


	/**
	 * Called when this decoration is removed from a UMovieScene hierarchy
	 */
	virtual void OnDestroy(UMovieScene* MovieScene)
	{
	}
};

