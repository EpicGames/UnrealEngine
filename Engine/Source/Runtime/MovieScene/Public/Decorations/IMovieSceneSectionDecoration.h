// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/Interface.h"
#include "IMovieSceneSectionDecoration.generated.h"

class UMovieSceneSection;

UINTERFACE(MinimalAPI)
class UMovieSceneSectionDecoration : public UInterface
{
public:
	GENERATED_BODY()
};


/** 
 * Optional interface that can be added to any UObject that is used as a decoration on UMovieSceneSection objects
 *        to receive decoration events.

 * @see UMovieSceneTrack::GetOrCreateDecoration, UMovieSceneTrack::AddDecoration
 */
class IMovieSceneSectionDecoration
{
public:
	GENERATED_BODY()


	/**
	 * Called when this decoration is first added directly to a UMovieSceneSection
	 */
	virtual void OnDecorationAdded(UMovieSceneSection* Section)
	{
	}


	/**
	 * Called when this decoration is removed from a UMovieSceneSection
	 */
	virtual void OnDecorationRemoved()
	{
	}
};

