// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SubclassOf.h"
#include "UObject/Interface.h"
#include "IMovieSceneSectionProviderDecoration.generated.h"

class UMovieSceneSection;

UINTERFACE(MinimalAPI)
class UMovieSceneSectionProviderDecoration : public UInterface
{
public:
	GENERATED_BODY()
};

/**
 * Optional interface for movie scene decorations that contain their own sections.
 * During compilation, the compiled data manager automatically adds sections from
 * decorations implementing this interface to the entity field for evaluation.
 */
class IMovieSceneSectionProviderDecoration
{
public:
	GENERATED_BODY()

	/** Returns the sections provided by this decoration */
	virtual TArrayView<TObjectPtr<UMovieSceneSection>> GetSections() = 0;

	virtual bool SupportsMultipleSections() { return false; }

	/** The section class hosted by this decoration, or null if this decoration is not a paste-routable host. */
	virtual TSubclassOf<UMovieSceneSection> GetHostedSectionClass() const { return nullptr; }

	virtual UMovieSceneSection* CreateNewSection() { return nullptr; }

	virtual void AddSection(UMovieSceneSection* InSection) {}

	virtual void RemoveSection(UMovieSceneSection& SectionToRemove) {}

#if WITH_EDITOR
	void MarkStructureChanged() { bStructureDirty = true; }

	bool ConsumeStructureChanged ()
	{
		const bool bWasDirty = bStructureDirty;
		bStructureDirty = false;
		return bWasDirty;
	}

private:
	// Have sections been added or removed, and not reflected in the UI?
	bool bStructureDirty = false;
#endif
};
