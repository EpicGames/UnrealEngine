// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/Interface.h"
#include "IMovieSceneTrackDecoration.generated.h"

class UMovieSceneTrack;

UINTERFACE(MinimalAPI)
class UMovieSceneTrackDecoration : public UInterface
{
public:
	GENERATED_BODY()
};


/** 
 * Optional interface that can be added to any UObject that is used as a decoration on UMovieSceneTrack objects
 *        to receive decoration events.

 * @see UMovieSceneTrack::GetOrCreateDecoration, UMovieSceneTrack::AddDecoration
 */
class IMovieSceneTrackDecoration
{
public:
	GENERATED_BODY()


	/**
	 * Called when this decoration is first added directly to a UMovieSceneTrack
	 */
	virtual void OnDecorationAdded(UMovieSceneTrack* Track)
	{
	}


	/**
	 * Called when this decoration is removed from a UMovieSceneTrack
	 */
	virtual void OnDecorationRemoved()
	{
	}
};

