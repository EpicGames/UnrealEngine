// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/Interface.h"
#include "MovieSceneSignedObject.h"
#include "Templates/FunctionFwd.h"
#include "MovieSceneDecorationContainer.generated.h"

template <typename T> class TSubclassOf;

USTRUCT()
struct FMovieSceneDecorationContainer
{
public:

	GENERATED_BODY()

	/**
	 * Find meta-data of a particular type for this level sequence instance.
	 * @param InClass - Class that you wish to find the metadata object for.
	 * @return An instance of this class if it already exists as metadata on this Level Sequence, otherwise null.
	 */
	MOVIESCENE_API UObject* FindDecoration(const TSubclassOf<UObject>& InClass) const;

	/**
	 * Find meta-data of a particular type for this level sequence instance, adding it if it doesn't already exist.
	 * @param InClass - Class that you wish to find or create the metadata object for.
	 * @return An instance of this class as metadata on this Level Sequence.
	 */
	MOVIESCENE_API void AddDecoration(UObject* InDecoration, UObject* Outer, TFunctionRef<void(UObject*)> Event);

	/**
	 * Find meta-data of a particular type for this level sequence instance, adding it if it doesn't already exist.
	 * @param InClass - Class that you wish to find or create the metadata object for.
	 * @return An instance of this class as metadata on this Level Sequence.
	 */
	MOVIESCENE_API UObject* GetOrCreateDecoration(const TSubclassOf<UObject>& InClass, UObject* Outer, TFunctionRef<void(UObject*)> Event);

	/**
	* Remove meta-data of a particular type for this level sequence instance, if it exists
	* @param InClass - The class type that you wish to remove the metadata for
	*/
	MOVIESCENE_API void RemoveDecoration(const TSubclassOf<UObject>& InClass, TFunctionRef<void(UObject*)> Event);

	/**
	* Retrieve all modular Decorations
	*/
	MOVIESCENE_API TArrayView<const TObjectPtr<UObject>> GetDecorations() const;

	/**
	 * Find meta-data of a particular type for this level sequence instance
	 */
	template<typename DecorationType>
	DecorationType* FindDecoration() const
	{
		UObject* Found = nullptr;

		if constexpr (TIsIInterface<DecorationType>::Value)
		{
			Found = FindDecoration(DecorationType::UClassType::StaticClass());
		}
		else
		{
			Found = FindDecoration(DecorationType::StaticClass());
		}

		return CastChecked<DecorationType>(Found, ECastCheckedType::NullAllowed);
	}

	/**
	 * Find meta-data of a particular type for this level sequence instance, adding one if it was not found.
	 * Meta-data may implement the ILevelSequenceMetaData interface in order to hook into default ULevelSequence functionality.
	 */
	template<typename DecorationType>
	DecorationType* GetOrCreateDecoration(UObject* Outer, TFunctionRef<void(UObject*)> Event)
	{
		UObject* Found = GetOrCreateDecoration(DecorationType::StaticClass(), Event);
		return CastChecked<DecorationType>(Found);
	}

	/**
	 * Remove meta-data of a particular type for this level sequence instance, if it exists
	 */
	template<typename DecorationType>
	void RemoveDecoration()
	{
		RemoveDecoration(DecorationType::StaticClass());
	}

	/**
	 * Remove any null decoration ptrs
	 */
	bool RemoveNulls()
	{
		return Decorations.Remove(nullptr) > 0;
	}

protected:

	UPROPERTY()
	TArray<TObjectPtr<UObject>> Decorations;
};


UCLASS(DefaultToInstanced, MinimalAPI)
class UMovieSceneDecorationContainerObject
	: public UMovieSceneSignedObject
{
	GENERATED_BODY()

public:

	/**
	 * Find meta-data of a particular type for this level sequence instance.
	 * @param InClass - Class that you wish to find the metadata object for.
	 * @return An instance of this class if it already exists as metadata on this Level Sequence, otherwise null.
	 */
	UObject* FindDecoration(const TSubclassOf<UObject>& InClass) const
	{
		return Decorations.FindDecoration(InClass);
	}

	MOVIESCENE_API void AddDecoration(UObject* InDecoration);

	/**
	 * Find meta-data of a particular type for this level sequence instance, adding it if it doesn't already exist.
	 * @param InClass - Class that you wish to find or create the metadata object for.
	 * @return An instance of this class as metadata on this Level Sequence.
	 */
	MOVIESCENE_API UObject* GetOrCreateDecoration(const TSubclassOf<UObject>& InClass);

	/**
	* Remove meta-data of a particular type for this level sequence instance, if it exists
	* @param InClass - The class type that you wish to remove the metadata for
	*/
	MOVIESCENE_API void RemoveDecoration(const TSubclassOf<UObject>& InClass);


	/**
	* Retrieve all modular Decorations
	*/
	TArrayView<const TObjectPtr<UObject>> GetDecorations() const
	{
		return Decorations.GetDecorations();
	}

	/**
	 * Find meta-data of a particular type for this level sequence instance
	 */
	template<typename DecorationType>
	DecorationType* FindDecoration() const
	{
		return Decorations.FindDecoration<DecorationType>();
	}

	/**
	 * Find meta-data of a particular type for this level sequence instance, adding one if it was not found.
	 * Meta-data may implement the ILevelSequenceMetaData interface in order to hook into default ULevelSequence functionality.
	 */
	template<typename DecorationType>
	DecorationType* GetOrCreateDecoration()
	{
		return CastChecked<DecorationType>(GetOrCreateDecoration(DecorationType::StaticClass()));
	}

	/**
	 * Remove meta-data of a particular type for this level sequence instance, if it exists
	 */
	template<typename DecorationType>
	void RemoveDecoration()
	{
		return RemoveDecoration(DecorationType::StaticClass());
	}

	/**
	 * Retrieve the list of compatible decorations for this object
	 */
	void GetCompatibleUserDecorations(TSet<UClass*>& OutClasses) const
	{
		GetCompatibleUserDecorationsImpl(OutClasses);
	}

	MOVIESCENE_API bool IsDecorationMuted(const UObject* Decoration) const;
	MOVIESCENE_API void SetDecorationMuted(UObject* Decoration, bool bMuted);

	MOVIESCENE_API bool IsDecorationDisabled(const UObject* Decoration) const;
	MOVIESCENE_API void SetDecorationDisabled(UObject* Decoration, bool bDisabled);

	// Returns true if the decoration is neither muted nor disabled
	MOVIESCENE_API bool IsDecorationActive(const UObject* Decoration) const;

	// Returns only decorations that are neither muted nor disabled
	MOVIESCENE_API TArray<UObject*> GetActiveDecorations() const;

protected:

	virtual void OnDecorationAdded(UObject* Decoration)
	{}

	virtual void OnDecorationRemoved(UObject* Decoration)
	{}

	virtual void GetCompatibleUserDecorationsImpl(TSet<UClass*>& OutClasses) const
	{}

protected:

	MOVIESCENE_API virtual void Serialize(FArchive& Ar) override;

protected:

	/** Array of decorations for this movie scene */
	UPROPERTY(EditAnywhere, Category = "Modifiers")
	FMovieSceneDecorationContainer Decorations;

	UPROPERTY(Transient)
	TSet<TObjectPtr<UObject>> MutedDecorations;

	UPROPERTY()
	TSet<TObjectPtr<UObject>> DisabledDecorations;
};
