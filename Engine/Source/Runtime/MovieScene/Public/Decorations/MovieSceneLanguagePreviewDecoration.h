// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/Interface.h"
#include "MovieSceneLanguagePreviewDecoration.generated.h"

class UMovieSceneSection;

UCLASS(MinimalAPI)
class UMovieSceneLanguagePreviewDecoration
	: public UObject
{
public:

	GENERATED_BODY()

	/** Currently assigned preview language */
	UPROPERTY()
	FString PreviewLanguage;

	/**
	 * Find the localized asset for this preview language based on the specified reference asset
	 */
	MOVIESCENE_API UObject* FindLocalizedAsset(UObject* InUnlocalizedAsset) const;

	/**
	 * Find the localized asset for a preview language based on the specified reference asset and section.
	 * @note This function uses the UMovieSceneLanguagePreviewDecoration on the specified section's MovieScene, if present
	 */
	MOVIESCENE_API static UObject* FindLocalizedAsset(UObject* InUnlocalizedAsset, const UMovieSceneSection* Section);

	/**
	 * Find the localized asset for a preview language based on the specified reference asset and section.
	 * @note This function uses the UMovieSceneLanguagePreviewDecoration on the specified section's MovieScene, if present
	 */
	template<typename AssetClass>
	static AssetClass* FindLocalizedAsset(TObjectPtr<AssetClass> InUnlocalizedAsset, const UMovieSceneSection* Section)
	{
		UObject* UntypedResult = FindLocalizedAsset(static_cast<UObject*>(InUnlocalizedAsset), Section);
		return CastChecked<AssetClass>(UntypedResult, ECastCheckedType::NullAllowed);
	}

	/**
	 * Find the localized asset for a preview language based on the specified reference asset and section.
	 * @note This function uses the UMovieSceneLanguagePreviewDecoration on the specified section's MovieScene, if present
	 */
	template<typename AssetClass>
	static AssetClass* FindLocalizedAsset(AssetClass* InUnlocalizedAsset, const UMovieSceneSection* Section)
	{
		UObject* UntypedResult = FindLocalizedAsset(static_cast<UObject*>(InUnlocalizedAsset), Section);
		return CastChecked<AssetClass>(UntypedResult, ECastCheckedType::NullAllowed);
	}

	/**
	 * Find the localized asset for this preview language based on the specified reference asset
	 */
	template<typename AssetClass>
	AssetClass* FindLocalizedAsset(TObjectPtr<AssetClass> InUnlocalizedAsset) const
	{
		UObject* UntypedResult = FindLocalizedAsset(static_cast<UObject*>(InUnlocalizedAsset));
		return CastChecked<AssetClass>(UntypedResult, ECastCheckedType::NullAllowed);
	}
};