// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Decorations/IMovieSceneDecoration.h"
#include "MovieSceneMuteSoloDecoration.generated.h"

#define UE_API MOVIESCENE_API

struct FMovieSceneMuteSoloData
{
	bool bMuted = false;
	bool bSoloed = false;
};

UCLASS(MinimalAPI, Transient)
class UMovieSceneMuteSoloDecoration
	: public UObject
	, public IMovieSceneDecoration
{
public:

	UMovieSceneMuteSoloDecoration();

	GENERATED_BODY()

	void SetMuted(bool bMuted) { MuteSoloData.bMuted = bMuted; }
	bool IsMuted() const { return MuteSoloData.bMuted; }

	void SetSoloed(bool bSoloed) { MuteSoloData.bSoloed = bSoloed; }
	bool IsSoloed() const { return MuteSoloData.bSoloed; }

private:
    FMovieSceneMuteSoloData MuteSoloData;
};

#undef UE_API
