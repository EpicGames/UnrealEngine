// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneTimeWarpChannel.h"
#include "Channels/MovieScenePiecewiseCurve.h"
#include "Decorations/IMovieSceneDecoration.h"
#include "Decorations/MovieSceneTimeWarpDecoration.h"
#include "Variants/MovieScenePlayRateCurve.h"
#include "MovieSceneScalingAnchors.generated.h"



/**
 * Structure that defines a single anchor with an optional duration
 * Anchor durations do not get scaled by other anchors, but positions will be offset based on
 * previous anchor stretching or movement
 */
USTRUCT()
struct FMovieSceneScalingAnchor
{
	GENERATED_BODY()

	/** Pefines the position of this anchor in ticks */
	UPROPERTY()
	FFrameNumber Position;

	/** Defines the duration of this anchor in ticks */
	UPROPERTY()
	int32 Duration = 0;
};




UINTERFACE(MinimalAPI)
class UMovieSceneScalingDriver : public UInterface
{
public:
	GENERATED_BODY()
};


/**
 * Interface that can be added to any object within a MovieScene in order to supply scaling anchors to the
 *      UMovieSceneScalingAnchors decoration that acts as a registry for all anchors
 */
class IMovieSceneScalingDriver
{
public:
	GENERATED_BODY()

	/**
	 * Populate the 'unscaled' map of anchors to use as a basis for scaling the seqence.
	 * Anchors are represented by a GUID that uniquely identifies them within the sequence.
	 */
	virtual void PopulateInitialAnchors(TMap<FGuid, FMovieSceneScalingAnchor>& OutAnchors)
	{
		PopulateAnchors(OutAnchors);
	}

	/**
	 * Populate the scaled map of anchors from which scaling factors will be computed
	 */
	virtual void PopulateAnchors(TMap<FGuid, FMovieSceneScalingAnchor>& OutAnchors) = 0;
};



/**
 * Structure that defines a grouping of sections to be scaled as one
 */
USTRUCT()
struct FMovieSceneAnchorsScalingGroup
{
	GENERATED_BODY()

	/** Set of all the sections that are contained within this group */
	UPROPERTY()
	TSet<TObjectPtr<UMovieSceneSection>> Sections;
};


/**
 * Decoration that is added to a UMovieScene in order to define dynamic scaling anchors to a the sequence.
 * Anchors are defined by scaling 'drivers' which control both the initial (unscaled) position of their anchors,
 * and their scaled position.
 */
UCLASS(MinimalAPI, DisplayName="Anchors", meta=(Hidden))
class UMovieSceneScalingAnchors
	: public UMovieScenePlayRateCurve
	, public IMovieSceneDecoration
	, public IMovieSceneTimeWarpSource
{
public:

	GENERATED_BODY()

	UMovieSceneScalingAnchors();

#if 0
	static uint32 DefaultTimeWarpEntityID;
#endif

public:

	/**
	 * Add a scaling driver to this anchor registry
	 * 
	 * @param InScalingDriver     The scaling driver to add to this registry. Must not be null.
	 */
	MOVIESCENE_API void AddScalingDriver(TScriptInterface<IMovieSceneScalingDriver> InScalingDriver);


	/**
	 * Remove a  scaling driver to this anchor registry
	 */
	MOVIESCENE_API void RemoveScalingDriver(TScriptInterface<IMovieSceneScalingDriver> InScalingDriver);

	MOVIESCENE_API bool HasAnyScalingDrivers() const;

public:

	/**
	 * Retrieve a scaling group of the specified identifier, creating it if necessary
	 */
	MOVIESCENE_API FMovieSceneAnchorsScalingGroup& GetOrCreateScalingGroup(const FGuid& Guid);

	/**
	 * Retrieve a scaling group by its ID
	 */
	MOVIESCENE_API FMovieSceneAnchorsScalingGroup* FindScalingGroup(const FGuid& Guid);

	/**
	 * Destroy a scaling group by its ID
	 */
	MOVIESCENE_API void RemoveScalingGroup(const FGuid& Guid);

	/**
	 * Retrieve all scaling groups
	 */
	MOVIESCENE_API const TMap<FGuid, FMovieSceneAnchorsScalingGroup>& GetScalingGroups() const;

public:

	/**
	 * Retrieve the initial anchors stored in this registry that act as the unscaled basis for the scaling
	 */
	MOVIESCENE_API const TMap<FGuid, FMovieSceneScalingAnchor>& GetInitialAnchors() const;


	/**
	 * Retrieve the current, scaled anchors stored in this registry
	 */
	MOVIESCENE_API const TMap<FGuid, FMovieSceneScalingAnchor>& GetCurrentAnchors() const;

public:

	/*~ Begin UObject Implementation */
	virtual void Serialize(FArchive& Ar) override;
	/*~ End UObject Implementation */

	/*~ Begin UMovieSceneTimeWarpGetter Implementation */
	virtual FFrameTime RemapTime(FFrameTime In) const override;
	virtual TOptional<FFrameTime> InverseRemapTimeCycled(FFrameTime InValue, FFrameTime InTimeHint, const UE::MovieScene::FInverseTransformTimeParams& Params) const override;
	virtual TRange<FFrameTime> ComputeTraversedHull(const TRange<FFrameTime>& Range) const override;
	virtual bool InverseRemapTimeWithinRange(FFrameTime InTime, FFrameTime RangeStart, FFrameTime RangeEnd, const TFunctionRef<bool(FFrameTime)>& VisitorCallback) const override;
	virtual EMovieSceneChannelProxyType PopulateChannelProxy(FMovieSceneChannelProxyData& OutProxyData, EAllowTopLevelChannels AllowTopLevel) override;
	virtual bool DeleteChannel(FMovieSceneTimeWarpVariant& OutVariant, FName ChannelName) override;
	virtual void ScaleBy(double UnwarpedScaleFactor) override;
	/* End UMovieSceneTimeWarpGetter Implementation */

	/*~ Begin IMovieSceneDecoration Implementation */
	virtual void OnDecorationAdded(UMovieScene* MovieScene) override;
	virtual void OnDecorationRemoved() override;
	virtual void OnPreDecorationCompiled() override;
	/*~ End IMovieSceneDecoration Implementation */

	/*~ Begin IMovieSceneTimeWarpSource Implementation */
	virtual FMovieSceneNestedSequenceTransform GenerateTimeWarpTransform() override;
	virtual bool IsTimeWarpActive() const override;
	virtual void SetIsTimeWarpActive(bool bInActive) override;
	virtual int32 GetTimeWarpSortOrder() const override;
	/*~ End IMovieSceneTimeWarpSource Implementation */

private:

	UMovieScenePlayRateCurve* Initialize(TSharedPtr<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const;

	void UpdateCurve(UMovieScenePlayRateCurve* Curve) const;

	void UpdateFromSource() const;

	void ResetScaling();

private:

	UPROPERTY()
	TArray<TScriptInterface<IMovieSceneScalingDriver>> ScalingDrivers;

	UPROPERTY()
	TMap<FGuid, FMovieSceneScalingAnchor> InitialAnchors;

	UPROPERTY()
	TMap<FGuid, FMovieSceneAnchorsScalingGroup> ScalingGroups;

	mutable TMap<FGuid, FMovieSceneScalingAnchor> CurrentAnchors;

	mutable FFrameTime NewDuration;
	mutable bool bPlayRateCurveIsUpToDate = false;
};