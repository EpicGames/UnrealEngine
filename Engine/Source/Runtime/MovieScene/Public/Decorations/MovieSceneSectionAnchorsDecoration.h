// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Decorations/MovieSceneScalingAnchors.h"
#include "Decorations/IMovieSceneSectionDecoration.h"
#include "Decorations/IMovieSceneLifetimeDecoration.h"
#include "UObject/Interface.h"
#include "MovieSceneSectionAnchorsDecoration.generated.h"


UCLASS(MinimalAPI)
class UMovieSceneSectionAnchorsDecoration
	: public UObject
	, public IMovieSceneSectionDecoration
	, public IMovieSceneLifetimeDecoration
	, public IMovieSceneScalingDriver
{
public:

	GENERATED_BODY()

	UPROPERTY()
	FGuid StartAnchor;

private:

	virtual void OnReconstruct(UMovieScene* MovieScene) override;
	virtual void OnDestroy(UMovieScene* MovieScene) override;

	virtual void PopulateInitialAnchors(TMap<FGuid, FMovieSceneScalingAnchor>& OutAnchors) override;
	virtual void PopulateAnchors(TMap<FGuid, FMovieSceneScalingAnchor>& OutAnchors) override;

	virtual void PostEditImport() override;
};