// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/Interface.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieSceneTimeWarpDecoration.generated.h"


struct FMovieSceneSequenceTransform;
struct FMovieSceneNestedSequenceTransform;


UINTERFACE(MinimalAPI)
class UMovieSceneTimeWarpSource : public UInterface
{
public:
	GENERATED_BODY()
};

class IMovieSceneTimeWarpSource
{
public:
	GENERATED_BODY()

	virtual FMovieSceneNestedSequenceTransform GenerateTimeWarpTransform() = 0;

	virtual bool IsTimeWarpActive() const = 0;

	virtual void SetIsTimeWarpActive(bool bInActive) = 0;

	virtual int32 GetTimeWarpSortOrder() const = 0;
};


UCLASS(MinimalAPI)
class UMovieSceneTimeWarpDecoration
	: public UObject
	, public IMovieSceneEntityProvider
{
public:

	GENERATED_BODY()

	MOVIESCENE_API FMovieSceneSequenceTransform GenerateTransform() const;

	MOVIESCENE_API void OnCompiled();

	MOVIESCENE_API void AddTimeWarpSource(TScriptInterface<IMovieSceneTimeWarpSource> InSource);

	MOVIESCENE_API void RemoveTimeWarpSource(TScriptInterface<IMovieSceneTimeWarpSource> InSource);

	bool HasAnySources() const { return !Sources.IsEmpty(); }

private:

	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
	virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;

	virtual void Serialize(FArchive& AR) override;
private:


	UPROPERTY()
	TArray<TScriptInterface<IMovieSceneTimeWarpSource>> Sources;
};
