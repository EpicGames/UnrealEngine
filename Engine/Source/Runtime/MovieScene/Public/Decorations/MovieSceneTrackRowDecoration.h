// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Decorations/IMovieSceneDecoration.h"
#include "Decorations/MovieSceneMuteSoloDecoration.h"
#include "MovieSceneTrackRowDecoration.generated.h"

#define UE_API MOVIESCENE_API

UCLASS(MinimalAPI, Transient)
class UMovieSceneTrackRowDecoration
	: public UObject
	, public IMovieSceneDecoration
{
public:

	GENERATED_BODY()

	UMovieSceneTrackRowDecoration();

	UE_API void SetMuted(int32 InRowIndex, bool bMuted);
	UE_API bool IsMuted(int32 InRowIndex) const;

	UE_API void SetSoloed(int32 InRowIndex, bool bSoloed);
	UE_API bool IsSoloed(int32 InRowIndex) const;

	UE_API void OnRowIndicesChanged(const TMap<int32, int32>& NewToOldRowIndices);

private:
    TMap<int32, FMovieSceneMuteSoloData> MuteSoloData;
};

#undef UE_API