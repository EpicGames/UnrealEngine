// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSequenceNavigationDefs.generated.h"

#define UE_API MOVIESCENE_API

UENUM()
enum class ENavigationToolItemFlags : uint8
{
	None = 0,
	/** Whether item should get the underlying UObject, ignoring if it's pending kill */
	IgnorePendingKill = 1 << 0,
	/** Item pending removal */
	PendingRemoval = 1 << 1,
	/** Whether the item is in expanded state to show its child items */
	Expanded = 1 << 2,
};
ENUM_CLASS_FLAGS(ENavigationToolItemFlags);

USTRUCT()
struct FNavigationToolSerializedItem
{
	GENERATED_BODY()

	FNavigationToolSerializedItem() = default;

	FNavigationToolSerializedItem(const FString& InId)
	{
		Id = InId;
	}

	bool IsValid() const
	{
		return !Id.IsEmpty();
	}

	friend uint32 GetTypeHash(const FNavigationToolSerializedItem& InItem)
	{
		return GetTypeHash(InItem.Id);
	}

	bool operator==(const FNavigationToolSerializedItem& InOther) const
	{
		return Id == InOther.Id;
	}

private:
	UPROPERTY()
	FString Id;
};

USTRUCT()
struct FNavigationToolSerializedTreeNode
{
	GENERATED_BODY()

	friend FNavigationToolSerializedTree;

	int32 GetLocalIndex() const { return LocalIndex; }
	int32 GetGlobalIndex() const { return GlobalIndex; }
	int32 GetParentIndex() const { return ParentIndex; }

	TConstArrayView<int32> GetChildrenIndices() const { return ChildrenIndices; }

	UE_API const FNavigationToolSerializedTreeNode* GetParentTreeNode() const;

	UE_API int32 CalculateHeight() const;

	UE_API TArray<const FNavigationToolSerializedTreeNode*> FindPath(const TArray<const FNavigationToolSerializedTreeNode*>& InItems) const;

	UE_API void Reset();

private:
	/** Index of this tree node relative to the parent node children items. Can be used as means of ordering. */
	UPROPERTY()
	int32 LocalIndex = INDEX_NONE;

	/** Index of this tree node in the owning tree */
	UPROPERTY()
	int32 GlobalIndex = INDEX_NONE;

	/** Absolute Index of the Parent Node in the owning tree. If INDEX_NONE, it means Parent is Root. */
	UPROPERTY()
	int32 ParentIndex = INDEX_NONE;

	/** Absolute indices of the children in the owning tree */
	UPROPERTY()
	TArray<int32> ChildrenIndices;

	/** Pointer to the tree that owns this node */
	FNavigationToolSerializedTree* OwningTree = nullptr;
};

USTRUCT()
struct FNavigationToolSerializedTree
{
	GENERATED_BODY()

	UE_API FNavigationToolSerializedTree();

	UE_API void PostSerialize(const FArchive& Ar);

	FNavigationToolSerializedTreeNode& GetRootNode() { return RootNode; }
	const FNavigationToolSerializedTreeNode& GetRootNode() const { return RootNode; }

	UE_API FNavigationToolSerializedTreeNode* FindTreeNode(const FNavigationToolSerializedItem& InItem);
	UE_API const FNavigationToolSerializedTreeNode* FindTreeNode(const FNavigationToolSerializedItem& InItem) const;

	UE_API const FNavigationToolSerializedItem* GetItemAtIndex(int32 InIndex) const;

	UE_API FNavigationToolSerializedTreeNode& GetOrAddTreeNode(const FNavigationToolSerializedItem& InItem, const FNavigationToolSerializedItem& InParentItem);

	static UE_API const FNavigationToolSerializedTreeNode* FindLowestCommonAncestor(const TArray<const FNavigationToolSerializedTreeNode*>& InItems);

	static UE_API bool CompareTreeItemOrder(const FNavigationToolSerializedTreeNode* InA, const FNavigationToolSerializedTreeNode* InB);

	UE_API void Reset();

private:
	UE_API void UpdateTreeNodes();

	UPROPERTY()
	FNavigationToolSerializedTreeNode RootNode;

	UPROPERTY()
	TArray<FNavigationToolSerializedItem> SceneItems;

	UPROPERTY()
	TMap<FNavigationToolSerializedItem, FNavigationToolSerializedTreeNode> ItemTreeMap;
};

USTRUCT()
struct FNavigationToolViewColumnSaveState
{
	GENERATED_BODY()

	UPROPERTY()
	bool bVisible = false;

	UPROPERTY()
	float Size = 1.f;
};

USTRUCT()
struct FNavigationToolViewSaveState
{
	GENERATED_BODY()

	/** Items specific to this tool instance, rather than being shared across all tool instances (e.g. Expansion flags) */
	UPROPERTY()
	TMap<FString, ENavigationToolItemFlags> ViewItemFlags;

	/** Map of the column Ids to their overriden (i.e. saved) visibility */
	UPROPERTY()
	TMap<FName, FNavigationToolViewColumnSaveState> ColumnsState;

	/** Active list of item filters */
	UPROPERTY()
	TSet<FName> ActiveItemFilters;
};

USTRUCT()
struct FNavigationToolSaveState
{
	GENERATED_BODY()

	UPROPERTY(Transient)
	FNavigationToolSerializedTree SerializedTree;

	UPROPERTY()
	TMap<FString, FColor> ItemColorMap;

	UPROPERTY()
	TArray<FNavigationToolViewSaveState> ToolViewSaveStates;

	UPROPERTY()
	FString ContextPath;
};

template<>
struct TStructOpsTypeTraits<FNavigationToolSerializedTree> : TStructOpsTypeTraitsBase2<FNavigationToolSerializedTree>
{
	enum 
	{
		WithPostSerialize = true,
	};
};

#undef UE_API
