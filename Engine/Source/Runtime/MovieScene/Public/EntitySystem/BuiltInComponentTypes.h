// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "EntitySystem/MovieSceneBlenderSystemTypes.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/MovieSceneInitialValueCache.h"
#include "EntitySystem/MovieScenePropertyRegistry.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "EntitySystem/MovieSceneVariantPropertyTypeIndex.h"
#include "Evaluation/Blending/MovieSceneBlendType.h"
#include "Evaluation/IMovieSceneEvaluationHook.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "UObject/GCObjectScopeGuard.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ScriptInterface.h"

#include "BuiltInComponentTypes.generated.h"

class IMovieSceneEvaluationHook;

namespace UE::MovieScene
{
	struct FCustomPropertyIndex;
	struct FInitialValueIndex;
	struct FInstanceHandle;
	struct FInterrogationKey;
	struct FInterrogationInstance;
	struct FRootInstanceHandle;
	namespace Interpolation
	{
		struct FCachedInterpolation;
	}
}

struct FFrameTime;
struct FMovieSceneBlendChannelID;
struct FMovieSceneSequenceID;

enum class EMovieSceneBlendType : uint8;
class FTrackInstancePropertyBindings;
class UMovieSceneBlenderSystem;
class UMovieSceneSection;
class UMovieSceneTrackInstance;
struct FMovieSceneBoolChannel;
struct FMovieSceneByteChannel;
struct FMovieSceneDoubleChannel;
struct FMovieSceneFloatChannel;
struct FMovieSceneIntegerChannel;
struct FMovieSceneObjectPathChannel;
struct FMovieSceneStringChannel;
struct FMovieSceneTextChannel;
struct FMovieScenePropertyBinding;


/**
 * Easing component data.
 */
USTRUCT()
struct FEasingComponentData
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UMovieSceneSection> Section = nullptr;
};


/**
 * A component that defines a type for a track instance
 */
USTRUCT()
struct FMovieSceneTrackInstanceComponent
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UMovieSceneSection> Owner = nullptr;

	UPROPERTY()
	TSubclassOf<UMovieSceneTrackInstance> TrackInstanceClass;
};


/**
 * A component that defines a hook for direct evaluation
 */
USTRUCT()
struct FMovieSceneEvaluationHookComponent
{
	GENERATED_BODY()

	UPROPERTY()
	TScriptInterface<IMovieSceneEvaluationHook> Interface;

	FGuid ObjectBindingID;
};


USTRUCT()
struct FTrackInstanceInputComponent
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UMovieSceneSection> Section = nullptr;

	UPROPERTY()
	int32 OutputIndex = INDEX_NONE;
};


namespace UE
{
namespace MovieScene
{

/**
 * The component data for evaluating a bool channel
 */
struct FSourceBoolChannel
{
	FSourceBoolChannel()
		: Source(nullptr)
	{}

	FSourceBoolChannel(const FMovieSceneBoolChannel* InSource)
		: Source(InSource)
	{}

	const FMovieSceneBoolChannel* Source;
};

/**
 * The component data for evaluating a byte channel
 */
struct FSourceByteChannel
{
	FSourceByteChannel()
		: Source(nullptr)
	{}

	FSourceByteChannel(const FMovieSceneByteChannel* InSource)
		: Source(InSource)
	{}

	const FMovieSceneByteChannel* Source;
};

/**
 * The component data for evaluating an integer channel
 */
struct FSourceIntegerChannel
{
	FSourceIntegerChannel()
		: Source(nullptr)
	{}

	FSourceIntegerChannel(const FMovieSceneIntegerChannel* InSource)
		: Source(InSource)
	{}

	const FMovieSceneIntegerChannel* Source;
};

/**
 * The component data for evaluating a float channel
 */
struct FSourceFloatChannel
{
	FSourceFloatChannel()
		: Source(nullptr)
	{}

	FSourceFloatChannel(const FMovieSceneFloatChannel* InSource)
		: Source(InSource)
	{}

	const FMovieSceneFloatChannel* Source;
};

/**
 * The component data for evaluating a double channel
 */
struct FSourceDoubleChannel
{
	FSourceDoubleChannel()
		: Source(nullptr)
	{}

	FSourceDoubleChannel(const FMovieSceneDoubleChannel* InSource)
		: Source(InSource)
	{}

	const FMovieSceneDoubleChannel* Source;
};

/**
 * The component data for evaluating a string channel
 */
struct FSourceStringChannel
{
	FSourceStringChannel()
		: Source(nullptr)
	{}

	FSourceStringChannel(const FMovieSceneStringChannel* InSource)
		: Source(InSource)
	{}

	const FMovieSceneStringChannel* Source;
};

/**
 * The component data for evaluating a text channel
 */
struct FSourceTextChannel
{
	FSourceTextChannel()
		: Source(nullptr)
	{}

	FSourceTextChannel(const FMovieSceneTextChannel* InSource)
		: Source(InSource)
	{}

	const FMovieSceneTextChannel* Source;
};

struct FEvaluationHookFlags
{
	bool bHasBegun = false;
};

struct FSourceObjectPathChannel
{
	FSourceObjectPathChannel()
		: Source(nullptr)
	{}

	FSourceObjectPathChannel(const FMovieSceneObjectPathChannel* InSource)
		: Source(InSource)
	{}

	const FMovieSceneObjectPathChannel* Source;
};

/** A component that represents a UObject* either as a strong or weak reference */
struct FObjectComponent
{
	FObjectComponent()
		: ObjectPtr(nullptr)
	{}

	/** Construct a new null component */
	static FObjectComponent Null()
	{
		return FObjectComponent{ nullptr, FObjectKey() };
	}

	/** Construct a new strongly-referenced component from an object ptr */
	static FObjectComponent Strong(UObject* InObject)
	{
		return FObjectComponent{ InObject, FObjectKey() };
	}

	/** Construct a new weakly-referenced component from an object ptr */
	static FObjectComponent Weak(UObject* InObject)
	{
		return FObjectComponent{ InObject, InObject };
	}

	/** Check whether this object component is valid */
	explicit operator bool() const
	{
		return ObjectPtr != nullptr;
	}

	/** Compare this component with another object ptr */
	bool operator==(UObject* InObject) const
	{
		return ObjectPtr == InObject;
	}

	/** Compare this component with another object ptr */
	bool operator!=(UObject* InObject) const
	{
		return ObjectPtr != InObject;
	}

	/** Equality operator */
	friend bool operator==(const FObjectComponent& A, const FObjectComponent& B)
	{
		if (A.ObjectKey != FObjectKey() || B.ObjectKey != FObjectKey())
		{
			return A.ObjectKey == B.ObjectKey;
		}

		return A.ObjectPtr == B.ObjectPtr;
	}

	/** Generate a type has from this component */
	friend uint32 GetTypeHash(const FObjectComponent& In)
	{
		if (In.ObjectKey != FObjectKey())
		{
			return GetTypeHash(In.ObjectKey);
		}

		return GetTypeHash(In.ObjectPtr);
	}

	UObject* operator->() const
	{
		UObject* Object = GetObject();
		checkSlow(Object);
		return Object;
	}

	MOVIESCENE_API UObject* GetObject() const;

	/** Conditionally add a reference for the specified component data based on whether it is strongly referenced or not */
	MOVIESCENE_API friend void AddReferencedObjectForComponent(FReferenceCollector* ReferenceCollector, FObjectComponent* ComponentData);

private:

	FObjectComponent(UObject* InObject, const FObjectKey& InObjectKey)
		: ObjectPtr(InObject)
		, ObjectKey(InObjectKey)
	{}

	bool IsStrongReference() const;

	/** Raw pointer to the object. This is explicitly assigned for both strong and weak ptrs, but is only added to the reference graph when FObjectKey() is default constructed.  */
	TObjectPtr<UObject> ObjectPtr;

	/** Default constructed for strong pointers. Assigned on construction for weak ptrs. */
	FObjectKey ObjectKey;
};


enum class EMovieSceneBindingLifetimeState : uint8
{
	/** Object Binding State is Active. Tracks on this binding will be active. */
	Active,
	/** Object Binding is Inactive. Tracks on this binding will be inactive and states will be restored/kept depending on settings. */
	Inactive
};

struct FMovieSceneBindingLifetimeComponentData
{
	EMovieSceneBindingLifetimeState BindingLifetimeState = EMovieSceneBindingLifetimeState::Active;
};

using FBoundObjectResolver = UObject* (*)(UObject*);

/**
 * Specifies a unique, sorted path of hbiases that contribute to a blended output
 * Supports up to 8 unique HBiases in its path
 */
struct FHierarchicalBlendTarget
{
	/** Default Constructor */
	MOVIESCENE_API FHierarchicalBlendTarget();

	MOVIESCENE_API FHierarchicalBlendTarget(const FHierarchicalBlendTarget& RHS);
	MOVIESCENE_API FHierarchicalBlendTarget& operator=(const FHierarchicalBlendTarget& RHS);

	MOVIESCENE_API FHierarchicalBlendTarget(FHierarchicalBlendTarget&& RHS);
	MOVIESCENE_API FHierarchicalBlendTarget& operator=(FHierarchicalBlendTarget&& RHS);

	MOVIESCENE_API ~FHierarchicalBlendTarget();

	/**
	 * Add the specified HBias to this blend target.
	 * Duplicates are not supported - if the HBias already exists in this chain, this function does nothing.
	 */
	MOVIESCENE_API void Add(int16 HBias);

	/**
	 * Return the number of hbiases that contribute to this blend target
	 */
	MOVIESCENE_API int32 Num() const;

	/**
	 * Return the HBias at the specified index. Out of bounds indices will fail an assertion
	 */
	MOVIESCENE_API int16 operator[](int32 Index) const;

	/**
	 * Convert this blend target into an array view
	 */
	MOVIESCENE_API TArrayView<const int16> AsArray() const;

	/**
	 * Retrieve the current capacity of this container - only to be used for debugging and testing purposes
	 */
	uint16 GetCapacity() const
	{
		return Capacity;
	}

public:

	friend uint32 GetTypeHash(const FHierarchicalBlendTarget& In)
	{
		// Use 32 bit ints for hashing speed
		const void* Memory = In.GetMemory();
		const uint32* Data32 = static_cast<const uint32*>(Memory);

		if (In.Capacity == InlineCapacity)
		{
			static_assert(sizeof(FHierarchicalBlendTarget) == sizeof(uint32)*4);

			// Use 32 bit ints for hashing speed
			// this actually ends up hashing the capacity as well since we have 7 int16s + 1 uint16
			return HashCombine(Data32[0], Data32[1]) ^ HashCombine(Data32[2], Data32[3]);
		}
		else
		{
			static_assert(GrowAmount%4 == 0, "GrowAmount is not a multiple of 2 which is required for this loop to work");
			check(In.Capacity%4 == 0);
			uint32 Hash = 0;
			for (int32 Index = 0; Index < In.Capacity/4; Index += 4)
			{
				Hash ^= HashCombine(Data32[Index+0], Data32[Index+1]) ^ HashCombine(Data32[Index+2], Data32[Index+3]);
			}
			return Hash;
		}
	}
	friend bool operator<(const FHierarchicalBlendTarget& A, const FHierarchicalBlendTarget& B)
	{
		if (A.Capacity != B.Capacity)
		{
			return A.Capacity < B.Capacity;
		}
		return FMemory::Memcmp(A.GetMemory(), B.GetMemory(), sizeof(int16)*A.Capacity) < 0;
	}
	friend bool operator==(const FHierarchicalBlendTarget& A, const FHierarchicalBlendTarget& B)
	{
		if (A.Capacity != B.Capacity)
		{
			return false;
		}
		return FMemory::Memcmp(A.GetMemory(), B.GetMemory(), sizeof(int16)*A.Capacity) == 0;
	}
	friend bool operator!=(const FHierarchicalBlendTarget& A, const FHierarchicalBlendTarget& B)
	{
		if (A.Capacity != B.Capacity)
		{
			return true;
		}
		return FMemory::Memcmp(A.GetMemory(), B.GetMemory(), sizeof(int16)*A.Capacity) != 0;
	}

private:

	void FreeAllocation();

	MOVIESCENE_API int16* GetMemory();
	MOVIESCENE_API const int16* GetMemory() const;

	TArrayView<int16> GetAllEntries();
	TArrayView<const int16> GetAllEntries() const;

	void Grow(uint16 NewCapacity);

	/**
	 * 14 bytes of hbias chain.
	 * Type is either:
	 *		int16[7] where Capacity == InlineCapacity
	 *		int16*   (heap allocated) where Capacity > InlineCapacity
	 * Supports inline up to a maximum of 7 sub-sequences, or any number on the heap allocation.
	 * Implemented using type-erasure since a union would be padded up to 16 bytes, and we want to keep sizeof this type == 16 bytes
	 */
	alignas(8) uint8 Data[14];

	/** 2 bytes - maximum number of hbiases in this chain. */
	uint16 Capacity;

	static constexpr int32 InlineCapacity = sizeof(Data) / sizeof(int16);
	static constexpr uint16 GrowAmount = 16u;
};

/**
 * The key to a grouping policy registered on the grouping system (see UMovieSceneEntityGroupingSystem)
 */
struct FEntityGroupingPolicyKey
{
	static const FEntityGroupingPolicyKey Invalid() { return FEntityGroupingPolicyKey(); }

	FEntityGroupingPolicyKey() {}
	FEntityGroupingPolicyKey(int32 FromIndex) : Index(FromIndex) {}

	bool IsValid() const
	{
		return Index != INDEX_NONE;
	}

	friend uint32 GetTypeHash(const FEntityGroupingPolicyKey& Key)
	{
		return GetTypeHash(Key.Index);
	}

	friend bool operator==(const FEntityGroupingPolicyKey& A, const FEntityGroupingPolicyKey& B)
	{
		return A.Index == B.Index;
	}

	int32 Index = INDEX_NONE;
};

/**
 * Flags for FEntityGroupID
 */
enum class EEntityGroupFlags : uint8
{
	None = 0,
	RemovedFromGroup = 1 << 0
};

ENUM_CLASS_FLAGS(EEntityGroupFlags);

/**
 * The component data for describing what group an entity belongs to (see UMovieSceneEntityGroupingSystem)
 */
struct FEntityGroupID
{
	static const FEntityGroupID Invalid() { return FEntityGroupID(); }

	FEntityGroupingPolicyKey PolicyKey;
	int32 GroupIndex = INDEX_NONE;
	EEntityGroupFlags Flags = EEntityGroupFlags::None;

	FEntityGroupID() {}
	FEntityGroupID(const FEntityGroupingPolicyKey InPolicyKey, int32 InGroupIndex) : PolicyKey(InPolicyKey), GroupIndex(InGroupIndex) {}

	/**
	 * Returns whether this component points to a valid grouping policy, and has a valid group.
	 */
	bool IsValid() const
	{
		return PolicyKey.IsValid() && GroupIndex != INDEX_NONE;
	}

	/**
	 * Returns whether this component is valid (see IsValid) and if it still belongs to a group.
	 * If the entity was removed from its group, the GroupIndex would still be valid, but we would
	 * have set Flags to RemovedFromGroup. This lets downstream systems still know what group the
	 * entity *used* to be part of, for their own book-keeping.
	 */
	bool HasGroup() const
	{
		return PolicyKey.IsValid() && GroupIndex != INDEX_NONE && !EnumHasAnyFlags(Flags, EEntityGroupFlags::RemovedFromGroup);
	}

	friend uint32 GetTypeHash(const FEntityGroupID& GroupID)
	{
		return HashCombine(GetTypeHash(GroupID.PolicyKey.Index), GetTypeHash(GroupID.GroupIndex));
	}

	friend bool operator==(const FEntityGroupID& A, const FEntityGroupID& B)
	{
		return A.PolicyKey == B.PolicyKey && A.GroupIndex == B.GroupIndex;
	}
};

/**
 * Pre-defined built in component types
 */
struct FBuiltInComponentTypes
{
	MOVIESCENE_API ~FBuiltInComponentTypes();

public:

	FPropertyRegistry PropertyRegistry;

public:

	/**
	 * Component mask where set bits denot component types that should trigger instantiation when present.
	 * After instantiation, these components will be removed from any entities to prevent instantiation being run constantly
	 */
	FComponentMask RequiresInstantiationMask;

	TComponentTypeID<FMovieSceneEntityID> ParentEntity;

	/**
	 * A bound object ptr component that defines the object being animated. This ptr is explicitly hidden from the reference graph and cleaned up
	 * after a garbage collection pass if it becomes invalid by checking the BoundObjectKey component that must exist alongside it
	 */
	TComponentTypeID<UObject*>            BoundObject;
	TComponentTypeID<FObjectKey>          BoundObjectKey;

	TComponentTypeID<FInstanceHandle>     InstanceHandle;

	TComponentTypeID<FRootInstanceHandle> RootInstanceHandle;

	TComponentTypeID<FMovieSceneSequenceID> SequenceID;

	TComponentTypeID<FFrameTime>          EvalTime;

	TComponentTypeID<double>              EvalSeconds;

	TComponentTypeID<FEntityGroupID>	  Group;

public:

	TComponentTypeID<FMovieSceneBlendChannelID> BlendChannelInput;

	TComponentTypeID<FMovieSceneBlendChannelID> BlendChannelOutput;

	TComponentTypeID<int16>               HierarchicalBias;

	TComponentTypeID<FInitialValueIndex>  InitialValueIndex;
public:

	// An FMovieScenePropertyBinding structure
	TComponentTypeID<FMovieScenePropertyBinding> PropertyBinding;

	// An FGuid relating to a direct object binding in a sequence
	TComponentTypeID<FGuid> GenericObjectBinding;

	// A custom bound object resolver that defines a function to resolve a bound object (ie, from AActor -> RootComponent where a track must operate on a component)
	TComponentTypeID<FBoundObjectResolver> BoundObjectResolver;

	// An FGuid that is always resolved as a USceneComponent either directly or through the AActor that the GUID relates to
	UE_DEPRECATED(5.5, "Please use GenericObjectBinding and BoundObjectResolver")
	TComponentTypeID<FGuid> SceneComponentBinding;

	// An FGuid relating to a spawnable binding in a sequence
	TComponentTypeID<FGuid> SpawnableBinding;

	// Data relating to the lifetime of bindings
	TComponentTypeID<FMovieSceneBindingLifetimeComponentData> BindingLifetime;

public:

	// A boolean repesenting the output of a bool property track or channel
	TComponentTypeID<bool> BoolResult;

	// An FMovieSceneByteChannel
	TComponentTypeID<FSourceByteChannel> ByteChannel;

	// A byte representing the output of a byte or enum track or channel
	TComponentTypeID<uint8> ByteResult;

	// A byte representing the base value for the byte channel for the purposes of "additive from base" blending.
	TComponentTypeID<uint8> BaseByte;

	// An FMovieSceneIntegerChannel
	TComponentTypeID<FSourceIntegerChannel> IntegerChannel;

	// An integer representing the output of an integer track or channel
	TComponentTypeID<int64> IntegerResult;

	// An integer representing the base value for the integer channel for the purposes of "additive from base" blending.
	TComponentTypeID<int64> BaseInteger;

	// An FMovieSceneBoolChannel
	TComponentTypeID<FSourceBoolChannel> BoolChannel;

	// An FMovieSceneFloatChannel considered to be at index N within the source structure (ie 0 = Location.X, Vector.X, Color.R; 1 = Location.Y, Vector.Y, Color.G)
	TComponentTypeID<FSourceFloatChannel> FloatChannel[9];

	// An FMovieSceneDoubleChannel considered to be at index N within the source structure (ie 0 = Location.X, Vector.X; 1 = Location.Y, Vector.Y)
	TComponentTypeID<FSourceDoubleChannel> DoubleChannel[9];

	// An FMovieSceneStringChannel
	TComponentTypeID<FSourceStringChannel> StringChannel;

	// An FMovieSceneTextChannel
	TComponentTypeID<FSourceTextChannel> TextChannel;

	// A cached interpolation structure relating to either float channels or double channels
	TComponentTypeID<Interpolation::FCachedInterpolation> CachedInterpolation[9];

	// An FMovieSceneFloatChannel that represents an arbitrary weight
	TComponentTypeID<FSourceFloatChannel> WeightChannel;
	TComponentTypeID<Interpolation::FCachedInterpolation> CachedWeightChannelInterpolation;

	// FMovieSceneObjectPathChannel that represents a changing object path over time
	TComponentTypeID<FSourceObjectPathChannel> ObjectPathChannel;

	// A double considered to be at index N within the source structure (ie 0 = Location.X, Vector.X; 1 = Location.Y, Vector.Y)
	TComponentTypeID<double> DoubleResult[9];

	// A double representing the base value for the double channel at index N, for the purposes of "additive from base" blending.
	TComponentTypeID<double> BaseDouble[9];

	// The time (in frames or in seconds) at which to evaluate a base value, such as BaseFloat[] or BaseDouble[].
	TComponentTypeID<FFrameTime> BaseValueEvalTime;
	TComponentTypeID<double> BaseValueEvalSeconds;

	// A float representing the evaluated output of a weight channel
	TComponentTypeID<double> WeightResult;
	TComponentTypeID<double> EasingResult;

	// The result of an evaluated FMovieSceneStringChannel
	TComponentTypeID<FString> StringResult;

	// Result of an evaluated FMovieSceneTextChannel
	TComponentTypeID<FText> TextResult;

	// The result of an evaluated FMovieSceneObjectPathChannel
	TComponentTypeID<FObjectComponent> ObjectResult;

	// Additional timewarp to apply to Eval Times on the same entity
	TComponentTypeID<TObjectPtr<UMovieSceneTimeWarpGetter>> TimeWarp;

public:

	// An FEasingComponentData for computing easing curves
	TComponentTypeID<FEasingComponentData> Easing;

	// An index associated to hierarchical easing for the owning sub-sequence
	TComponentTypeID<uint16> HierarchicalEasingChannel;

	// The sub-sequence ID that should receive ease in/out as a whole
	TComponentTypeID<FMovieSceneSequenceID> HierarchicalEasingProvider;

	// Defines an HBias level that is the highest blend target for a given set of components that need to blend together
	TComponentTypeID<FHierarchicalBlendTarget> HierarchicalBlendTarget;

	// A float representing the evaluated easing weight
	TComponentTypeID<double> WeightAndEasingResult;

	/** A blender type that should be used for blending this entity */
	TComponentTypeID<TSubclassOf<UMovieSceneBlenderSystem>> BlenderType;

	/** Value that is used to sort blending order */
	TComponentTypeID<int32> BlendingOrder;

	// An FMovieSceneTrackInstanceComponent that defines the track instance to use
	TComponentTypeID<FMovieSceneTrackInstanceComponent> TrackInstance;

	// An FTrackInstanceInputComponent that defines an input for a track instance
	TComponentTypeID<FTrackInstanceInputComponent> TrackInstanceInput;

	// An FMovieSceneEvaluationHookComponent that defines a stateless hook interface that doesn't need any overlap handling (track instances should be preferred there)
	TComponentTypeID<FMovieSceneEvaluationHookComponent> EvaluationHook;

	TComponentTypeID<FEvaluationHookFlags> EvaluationHookFlags;

	TComponentTypeID<TObjectPtr<const UMovieSceneCondition>> Condition;

public:

	// 
	TComponentTypeID<FCustomPropertyIndex> CustomPropertyIndex;

	// A property offset from a UObject* that points to the memory for a given property - care should be taken to ensure that this is only ever accessed in conjunction with a property tag
	TComponentTypeID<uint16> FastPropertyOffset;

	// A property binding that supports setters and notifications
	TComponentTypeID<TSharedPtr<FTrackInstancePropertyBindings>> SlowProperty;

	/**
	 * Specifies an index that uniquely identifies the variant type of a property component by its position in
	 *     either the compile-time list of variant types, or the dynamic list of variants registered at runtime.
	 * 
	 * For example: A TDynamicVariantPropertyTraits<FMyIntermediateType, double, float, int, int64> with 2 runtime-registered
	 *              variants (FFloat16, and FFloat8) will represent the variants with the following indices:
	 *              double: 0, float: 1, int: 2, int64: 3, FFloat16: 4, and; FFloat8: 5
	 */
	TComponentTypeID<FVariantPropertyTypeIndex> VariantPropertyTypeIndex;

	struct
	{
		// A tag specifying that an entity wants to restore state on completioon
		FComponentTypeID RestoreState;

		// A tag that specifies this entity should always contribute to the output, regardless of hbias
		FComponentTypeID IgnoreHierarchicalBias;
		FComponentTypeID BlendHierarchicalBias;

		FComponentTypeID AbsoluteBlend;
		FComponentTypeID RelativeBlend;
		FComponentTypeID AdditiveBlend;
		FComponentTypeID AdditiveFromBaseBlend;
		FComponentTypeID OverrideBlend;

		FComponentTypeID AdditiveAnimation;

		FComponentTypeID NeedsLink;
		FComponentTypeID NeedsUnlink;

		/** Tag that is added to imported entities with a GenericObjectBinding whose binding did not resolve */
		FComponentTypeID HasUnresolvedBinding;

		FComponentTypeID HasAssignedInitialValue;

		FComponentTypeID ImportedEntity;

		FComponentTypeID SubInstance;

		FComponentTypeID Root;

		FComponentTypeID FixedTime;

		FComponentTypeID SectionPreRoll;
		FComponentTypeID PreRoll;

		FComponentTypeID Finished;

		FComponentTypeID Ignored;

		FComponentTypeID ExternalBlending;

		FComponentTypeID AlwaysCacheInitialValue;

		FComponentTypeID DontOptimizeConstants;

		UE_DEPRECATED(5.5, "This tag is no longer used. Blend targets are entirely managed by UMovieSceneHierarchicalBiasSystem now.")
		FComponentTypeID RemoveHierarchicalBlendTarget;

		FComponentTypeID OldStyleSpawnable;

	} Tags;

	struct
	{
		TComponentTypeID<FInterrogationKey> InputKey;
		TComponentTypeID<FInterrogationInstance> Instance;
		TComponentTypeID<FInterrogationKey> OutputKey;
	} Interrogation;

	struct
	{
		FComponentTypeID CreatesEntities;
	} SymbolicTags;

	FComponentMask FinishedMask;

public:

	static MOVIESCENE_API void Destroy();

	static MOVIESCENE_API FBuiltInComponentTypes* Get();

	inline static bool IsBoundObjectGarbage(UObject* InObject)
	{
		return InObject == nullptr || !IsValidChecked(InObject) || InObject->IsUnreachable();
	}

	MOVIESCENE_API FComponentTypeID GetBaseValueComponentType(const FComponentTypeID& InResultComponentType);

private:

	MOVIESCENE_API FBuiltInComponentTypes();

	TMap<FComponentTypeID, FComponentTypeID> ResultToBase;
};

} // namespace MovieScene
} // namespace UE
