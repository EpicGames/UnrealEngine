// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"

namespace UE
{
namespace MovieScene
{

class FEntityManager;
struct FEntityAllocation;
struct FEntityAllocationIterator;

using FEntityAllocationIteratorItem = FEntityAllocationProxy;

/**
 * Object that iterates all entity allocations that match a specific filter
 */
struct FEntityAllocationIterator
{
	/**
	 * End iterator constructor
	 */
	MOVIESCENE_API explicit FEntityAllocationIterator(const FEntityManager* InManager);

	/**
	 * Construction from the entity manager to iterate, and a filter
	 *
	 * @param InManager        The entity manager to iterate
	 * @param InFilter         Filter that defines the components to match. Copied into this object.
	 */
	MOVIESCENE_API explicit FEntityAllocationIterator(const FEntityManager* InManager, const FEntityComponentFilter* InFilter);

	MOVIESCENE_API FEntityAllocationIterator(FEntityAllocationIterator&&);
	MOVIESCENE_API FEntityAllocationIterator& operator=(FEntityAllocationIterator&&);

	/**
	 * Destructor
	 */
	MOVIESCENE_API ~FEntityAllocationIterator();


	/**
	 * Retrieve the entity allocation that this iterator represents. Only valid if this iterator != end()
	 */
	MOVIESCENE_API FEntityAllocationIteratorItem operator*() const;


	/**
	 * Increment this iterator to the next matching allocation. Only valid if this iterator != end()
	 */
	MOVIESCENE_API FEntityAllocationIterator& operator++();


	/**
	 * Test whether this iterator is valid (ie not at the end of the iteration)
	 */
	MOVIESCENE_API bool operator!=(const FEntityAllocationIterator& Other) const;

private:

	/**
	 * Find the index of the next allocation that matches our filter
	 */
	int32 FindMatchingAllocationStartingAt(int32 Index) const;

	/** Filter to match entity allocations against */
	const FEntityComponentFilter* Filter;

	/** Entity manager being iterated */
	const FEntityManager* Manager;

	/** Current allocation index or Manager->EntityAllocationMasks.GetMaxIndex() when finished */
	int32 AllocationIndex;
};


/**
 * Iterator proxy type returned from FEntityManager::Iterate that supports ranged-for iteration of all entity allocations matching the specified filter
 */
struct FEntityAllocationIteratorProxy
{
	FEntityAllocationIteratorProxy(const FEntityManager* InManager, const FEntityComponentFilter* InFilter)
		: Manager(InManager), Filter(InFilter)
	{}

	/**~ Do not use directly - for stl iteration */
	FEntityAllocationIterator begin() const { return FEntityAllocationIterator(Manager, Filter); }
	FEntityAllocationIterator end()  const  { return FEntityAllocationIterator(Manager);         }

private:

	const FEntityManager* Manager;
	const FEntityComponentFilter* Filter;
};


} // namespace MovieScene
} // namespace UE
