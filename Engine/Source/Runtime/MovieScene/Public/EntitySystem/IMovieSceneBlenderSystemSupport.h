// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "Containers/Array.h"
#include "Templates/SubclassOf.h"

#include "IMovieSceneBlenderSystemSupport.generated.h"

class UMovieSceneBlenderSystem;

UINTERFACE(MinimalAPI)
class UMovieSceneBlenderSystemSupport : public UInterface
{
public:
	GENERATED_BODY()
};


/**
 * Interface that can be added to UMovieSceneTracks to enable user-selection of blender systems
 */
class IMovieSceneBlenderSystemSupport
{
public:
	GENERATED_BODY()

	virtual TSubclassOf<UMovieSceneBlenderSystem> GetBlenderSystem() const = 0;
	virtual void SetBlenderSystem(TSubclassOf<UMovieSceneBlenderSystem> BlenderSystemClass) = 0;
	virtual void GetSupportedBlenderSystems(TArray<TSubclassOf<UMovieSceneBlenderSystem>>& OutSystemClasses) const = 0;
};
