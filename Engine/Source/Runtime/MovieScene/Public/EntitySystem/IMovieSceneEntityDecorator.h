// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "IMovieSceneEntityDecorator.generated.h"


UINTERFACE(MinimalAPI)
class UMovieSceneEntityDecorator : public UInterface
{
public:
	GENERATED_BODY()
};


/** 
 * Optional interface that can be added to any UMovieSceneSection decoration in order to decorate the ECS entities it produces at runtime
 */
class IMovieSceneEntityDecorator
{
public:

	using FEntityImportParams        = UE::MovieScene::FEntityImportParams;
	using FImportedEntity            = UE::MovieScene::FImportedEntity;

	GENERATED_BODY()

	void ExtendEntity(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
	{
		ExtendEntityImpl(EntityLinker, Params, OutImportedEntity);
	}

private:

	virtual void ExtendEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) = 0;
};
