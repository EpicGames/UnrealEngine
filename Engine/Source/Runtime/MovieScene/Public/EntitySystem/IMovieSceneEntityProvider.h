// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Evaluation/MovieSceneSectionParameters.h"
#include "EntitySystem/MovieSceneEntityBuilder.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "Evaluation/MovieSceneCompletionMode.h"
#include "HAL/Platform.h"
#include "Math/Range.h"
#include "Misc/Guid.h"
#include "Misc/InlineValue.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "IMovieSceneEntityProvider.generated.h"

class UClass;
class UMovieSceneEntitySystemLinker;
class UMovieSceneSection;
class UObject;
struct FFrameNumber;
struct FMovieSceneEntityComponentFieldBuilder;
struct FMovieSceneEvaluationFieldEntityMetaData;
struct FMovieSceneEvaluationFieldSharedEntityMetaData;
struct FMovieSceneTimeTransform;
template <typename ElementType> class TRange;


namespace UE
{
namespace MovieScene
{

class FEntityManager;
struct FEntityImportParams;
struct IEntityBuilder;


struct FImportedEntity
{
	bool IsEmpty() const
	{
		return Builders.Num() == 0;
	}

	template<typename BuilderType>
	void AddBuilder(BuilderType&& InBuilder)
	{
		Builders.Add(Forward<BuilderType>(InBuilder));
	}

	MOVIESCENE_API FMovieSceneEntityID Manufacture(const FEntityImportParams& Params, FEntityManager* EntityManager);

private:

	TArray<TInlineValue<IEntityBuilder>, TInlineAllocator<1>> Builders;
};

struct FEntityImportSequenceParams
{
	FEntityImportSequenceParams()
		: HierarchicalBias(0)
		, SequenceID(MovieSceneSequenceID::Root)
		, DefaultCompletionMode(EMovieSceneCompletionMode::KeepState)
		, SubSectionFlags(EMovieSceneSubSectionFlags::None)
		, bPreRoll(false)
		, bPostRoll(false)
		, bDynamicWeighting(false)
	{}

	int32 HierarchicalBias;

	FMovieSceneSequenceID SequenceID;
	FInstanceHandle InstanceHandle;
	FRootInstanceHandle RootInstanceHandle;

	EMovieSceneCompletionMode DefaultCompletionMode;
	EMovieSceneSubSectionFlags SubSectionFlags;

	bool bPreRoll : 1;
	bool bPostRoll : 1;
	bool bDynamicWeighting : 1;
};

struct FEntityImportParams
{
	const FMovieSceneEvaluationFieldEntityMetaData* EntityMetaData = nullptr;
	const FMovieSceneEvaluationFieldSharedEntityMetaData* SharedMetaData = nullptr;

	uint32 EntityID = 0;

	FInterrogationKey InterrogationKey;
	FInterrogationInstance InterrogationInstance;

	FEntityImportSequenceParams Sequence;

	MOVIESCENE_API FGuid GetObjectBindingID() const;
};

} // namespace MovieScene
} // namespace UE


UINTERFACE(MinimalAPI)
class UMovieSceneEntityProvider : public UInterface
{
public:
	GENERATED_BODY()
};

/**
 * Interface to be added to UMovieSceneSection types when they contain entity data
 */
class IMovieSceneEntityProvider
{
public:

	using FEntityImportParams        = UE::MovieScene::FEntityImportParams;
	using FImportedEntity            = UE::MovieScene::FImportedEntity;


	GENERATED_BODY()


	/**
	 * Populate an evaluation field with this provider's entities
	 */
	bool PopulateEvaluationField(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
	{
		return PopulateEvaluationFieldImpl(EffectiveRange, InMetaData, OutFieldBuilder);
	}


	MOVIESCENE_API void ImportEntity(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity);
	MOVIESCENE_API void InterrogateEntity(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity);

private:

	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) = 0;
	virtual void InterrogateEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) { ImportEntityImpl(EntityLinker, Params, OutImportedEntity); }

	/** Optional user-implementation function for populating an evaluation entity field */
	virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) { return false; }
};
