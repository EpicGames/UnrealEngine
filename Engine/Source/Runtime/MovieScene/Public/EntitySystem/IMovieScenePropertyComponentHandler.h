// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneVariantPropertyTypeIndex.h"
#include "EntitySystem/MovieSceneIntermediatePropertyValue.h"


class UMovieSceneSection;
class UMovieSceneBlenderSystem;
class UMovieSceneEntitySystemLinker;

namespace UE
{
namespace MovieScene
{

struct FPropertyStats;
struct FEntityAllocation;
struct FInitialValueCache;
struct FPropertyDefinition;
struct IPreAnimatedStorage;
struct FEntityComponentFilter;
struct FUnpackedChannelValues;
struct FSystemSubsequentTasks;
struct FSystemTaskPrerequisites;
struct FVariantPropertyTypeIndex;
struct FValueDecompositionParams;
struct FPreAnimatedStateExtension;
struct FPropertyCompositeDefinition;

class IEntitySystemScheduler;

/** Type-erased view of a component. Used for passing typed data through the IPropertyComponentHandler interface */
struct FPropertyComponentView
{
	/** Construction from a specific piece of data. Specified data must outlive this view */
	template<typename T>
	FPropertyComponentView(T& InData) : Data(&InData), DataSizeof(sizeof(T)) {}

	/** Construction from a pointer to a piece of data, and its type's size. Specified data must outlive this view */
	FPropertyComponentView(void* InData, int32 InDataSizeof) : Data(InData), DataSizeof(InDataSizeof) {}

	/**
	 * Retrieve the size of this component
	 */
	int32 Sizeof() const { return DataSizeof; }

	/**
	 * Cast this type-erased view to a known data type. Only crude size checking is performned - user is responsible for ensuring that the cast is valid.
	 */
	template<typename T>
	T& ReinterpretCast() const { check(sizeof(T) <= DataSizeof); return *static_cast<T*>(Data); }

private:
	void* Data;
	int32 DataSizeof;
};

/** Type-erased view of a constant component. Used for passing typed data through the IPropertyComponentHandler interface */
struct FConstPropertyComponentView
{
	/** Construction from a specific piece of data. Specified data must outlive this view */
	template<typename T>
	FConstPropertyComponentView(const T& InData) : Data(&InData), DataSizeof(sizeof(T)) {}

	/** Construction from a pointer to a piece of data, and its type's size. Specified data must outlive this view */
	FConstPropertyComponentView(const void* InData, int32 InDataSizeof) : Data(InData), DataSizeof(InDataSizeof) {}

	/**
	 * Retrieve the size of this component
	 */
	int32 Sizeof() const { return DataSizeof; }

	/**
	 * Cast this type-erased view to a known data type. Only crude size checking is performned - user is responsible for ensuring that the cast is valid.
	 */
	template<typename T>
	const T& ReinterpretCast() const { check(sizeof(T) <= DataSizeof); return *static_cast<const T*>(Data); }

private:
	const void* Data;
	int32 DataSizeof;
};


/** Type-erased view of an array of components. Used for passing typed arrays of data through the IPropertyComponentHandler interface */
struct FPropertyComponentArrayView
{
	/** Construction from an array */
	template<typename T, typename Allocator>
	FPropertyComponentArrayView(TArray<T, Allocator>& InRange)
		: Data(InRange.GetData())
		, DataSizeof(sizeof(T))
		, ArrayNum(InRange.Num())
	{}

	/** Access the number of items in the array */
	int32 Num() const
	{
		return ArrayNum;
	}

	/** Access the sizeof a single item in the array view, in bytes */
	int32 Sizeof() const
	{
		return DataSizeof;
	}

	/** Cast this view to a typed array view. Only crude size checking is performed - the user is responsible for ensuring the cast is valid */
	template<typename T>
	TArrayView<T> ReinterpretCast() const
	{
		check(sizeof(T) == DataSizeof);
		return MakeArrayView(static_cast<T*>(Data), ArrayNum);
	}

	/** Access an element in the array */
	FPropertyComponentView operator[](int32 Index)
	{
		check(Index < ArrayNum);
		return FPropertyComponentView(static_cast<uint8*>(Data) + DataSizeof*Index, DataSizeof);
	}

	/** Access an element in the array */
	FConstPropertyComponentView operator[](int32 Index) const
	{
		check(Index < ArrayNum);
		return FConstPropertyComponentView(static_cast<uint8*>(Data) + DataSizeof*Index, DataSizeof);
	}

private:
	void* Data;
	int32 DataSizeof;
	int32 ArrayNum;
};

/** Interface required for initializing initial values on entities */
struct IInitialValueProcessor
{
	virtual ~IInitialValueProcessor()= default;

	/** Initialize this processor before any allocations are visited */
	virtual void Initialize(UMovieSceneEntitySystemLinker* Linker, FInitialValueCache* InitialValueCache) = 0;
	/** Populate a filter that controls whether this processor will run on a given allocation or not */
	virtual void PopulateFilter(FEntityComponentFilter& OutFilter) const = 0;
	/** Process all initial values for the specified allocation */
	virtual void Process(const FEntityAllocation* Allocation, const FComponentMask& AllocationType) = 0;
	/** Finish processing */
	virtual void Finalize() = 0;
};

/** Interface for a property type handler that is able to interact with properties in sequencer */
struct IPropertyComponentHandler
{
	virtual ~IPropertyComponentHandler()= default;

	virtual bool SupportsProperty(const FPropertyDefinition& Definition, const FProperty& InProperty) const = 0;
	virtual FIntermediatePropertyValue CoercePropertyValue(const FPropertyDefinition& Definition, const FProperty& InProperty, const FSourcePropertyValue& InPropertyValue) const = 0;
	virtual void UnpackChannels(const FPropertyDefinition& Definition, const FProperty& Property, const FIntermediatePropertyValueConstRef& Value, FUnpackedChannelValues& OutUnpackedValues) const = 0;

	/**
	 * Dispatch tasks that apply any entity that matches this property type to their final values
	 *
	 * @param Definition       The property definition this handler was registered for
	 * @param Composites       The composite channels that this property type comprises
	 * @param Stats            Stats pertaining to the entities that currently exist in the entity manager
	 * @param InPrerequisites  Task prerequisites for any entity system tasks that are dispatched
	 * @param Subsequents      Subsequents to add any dispatched tasks to
	 * @param Linker           The linker that owns the entity manager to dispatch tasks for
	 */
	virtual void ScheduleSetterTasks(const FPropertyDefinition& Definition, TArrayView<const FPropertyCompositeDefinition> Composites, const FPropertyStats& Stats, IEntitySystemScheduler* TaskScheduler, UMovieSceneEntitySystemLinker* Linker) = 0;
	virtual void DispatchSetterTasks(const FPropertyDefinition& Definition, TArrayView<const FPropertyCompositeDefinition> Composites, const FPropertyStats& Stats, FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents, UMovieSceneEntitySystemLinker* Linker) = 0;

	/**
	 * Dispatch tasks that cache a pre-animated value for any entities that have the CachePreAnimatedState tag
	 *
	 * @param Definition       The property definition this handler was registered for
	 * @param InPrerequisites  Task prerequisites for any entity system tasks that are dispatched
	 * @param Subsequents      Subsequents to add any dispatched tasks to
	 * @param Linker           The linker that owns the entity manager to dispatch tasks for
	 * @param MetaDataProvider Interface that is able to locate properties for entity IDs
	 */
	virtual void DispatchInitializePropertyMetaDataTasks(const FPropertyDefinition& Definition, FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents, UMovieSceneEntitySystemLinker* Linker) {}

	/**
	 * Retrieve the pre-animated storage for the property that this handler represents
	 *
	 * @param Definition       The property definition this handler was registered for
	 * @param Container        The Pre-Animated state container extension that owns all pre-anim state for this evaluation
	 */
	virtual TSharedPtr<IPreAnimatedStorage> GetPreAnimatedStateStorage(const FPropertyDefinition& Definition, FPreAnimatedStateExtension* Container) { return nullptr; }

	/**
	 * Run a recomposition using the specified params and values. The current value and result views must be of type StorageType
	 *
	 * @param Definition       The property definition this handler was registered for
	 * @param Composites       The composite channels that this property type comprises
	 * @param Params           The decomposition parameters
	 * @param Blender          The blender system to recompose from
	 * @param InCurrentValue   The current value (of type StorageType) to recompose using. For instance, if a property comprises 3 additive values (a:1, b:2, c:3), and we recompose 'a' with an InCurrentValue of 10, the result for 'a' would be 5.
	 * @param OutResult        The result to receieve recomposed values, one for every entitiy in Params.Query.Entities. Must be of type StorageType.
	 */
	virtual void RecomposeBlendOperational(const FPropertyDefinition& Definition, TArrayView<const FPropertyCompositeDefinition> Composites, const FValueDecompositionParams& Params, UMovieSceneBlenderSystem* Blender, const FIntermediatePropertyValueConstRef& InCurrentValue, TArrayView<FIntermediatePropertyValue> OutResult) = 0;

	/**
	 * Run a recomposition using the specified params and values.
	 *
	 * @param Definition       The property definition this handler was registered for
	 * @param Composite        The composite channel of the property type that we want to decompose
	 * @param Params           The decomposition parameters
	 * @param Blender          The blender system to recompose from
	 * @param InCurrentValue   The current value (of type StorageType) to recompose using. For instance, if a property comprises 3 additive values (a:1, b:2, c:3), and we recompose 'a' with an InCurrentValue of 10, the result for 'a' would be 5.
	 * @param OutResults       The result to receieve recomposed values, one for every entitiy in Params.Query.Entities.
	 */
	virtual void RecomposeBlendChannel(const FPropertyDefinition& Definition, TArrayView<const FPropertyCompositeDefinition> Composites, int32 CompositeIndex, const FValueDecompositionParams& Params, UMovieSceneBlenderSystem* Blender, double InCurrentValue, TArrayView<double> OutResults) = 0;

	/**
	 * Rebuild operational values from the given entities. These entities are expected to store the value type's composite values.
	 *
	 * @param Definition		The property definition this handler was registered for
	 * @param Composites		The composite channels that this property type comproses
	 * @param EntityIDs			The entities on which the composite values will be found
	 * @param Linker			The linker that owns the entity manager where the entities live
	 * @param OutResult			The result to receieve rebuilt values, one for every entitiy in EntityIDs. Must be of type StorageType.
	 */
	virtual void RebuildOperational(const FPropertyDefinition& Definition, TArrayView<const FPropertyCompositeDefinition> Composites, const TArrayView<FMovieSceneEntityID>& EntityIDs, UMovieSceneEntitySystemLinker* Linker, FPropertyComponentArrayView OutResult) = 0;

	/**
	 * Retrieve an initial value processor interface for this property type
	 */
	virtual TSharedPtr<IInitialValueProcessor> MakeInitialValueProcessor(const FPropertyDefinition& Definition) = 0;
};


} // namespace MovieScene
} // namespace UE


