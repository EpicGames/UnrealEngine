// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stats/Stats.h"
#include "Misc/TVariant.h"
#include "EntitySystem/RelativePtr.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "EntitySystem/EntityAllocationIterator.h"

namespace UE::MovieScene
{

/**
 * Typedef for a pointer that is relative to another. For most component data this is relative either to the FEntityAllocation itself, or FEntityAllocation::ComponentData
 * Using a relative ptr for these allows us to store the same information with half of the memory (or 1/4 if we were able to use uint16 - I've used uint32 for safety here)
 */
using FPreLockedDataPtr = TRelativePtr<void, uint32>;

struct FTaskID
{
	int32 Index;

	explicit FTaskID()
		: Index(INDEX_NONE)
	{}

	explicit FTaskID(int32 InIndex)
		: Index(InIndex)
	{}

	static FTaskID None()
	{
		return FTaskID(INDEX_NONE);
	}

	explicit operator bool() const
	{
		return Index != INDEX_NONE;
	}
};

struct FTaskParams
{
	explicit FTaskParams(const TStatId& InStatId)
		:
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		DebugName(nullptr),
#endif
		StatId(InStatId)
	{
		bForceGameThread = false;
		bSerialTasks = false;
		bForcePropagateDownstream = false;
		bForceConsumeUpstream = false;
		bForcePrePostTask = false;
	}

	explicit FTaskParams(const TCHAR* InDebugName, const TStatId& InStatId = TStatId())
		: 
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		DebugName(InDebugName), 
#endif
		StatId(InStatId)
	{
		bForceGameThread = false;
		bSerialTasks = false;
		bForcePropagateDownstream = false;
		bForceConsumeUpstream = false;
		bForcePrePostTask = false;
	}

	/**
	 * Set a custom stat ID for this task
	 */
	FTaskParams& Stat(const TStatId& InStatId)
	{
		StatId = InStatId;
		return *this;
	}

	/**
	 * Force this task to run on the game thread
	 */
	FTaskParams& ForceGameThread()
	{
		bForceGameThread = true;
		return *this;
	}

	/**
	 * Force this task to run Pre/Post callbacks even if there is no meaningful work to be done in the body
	 */
	FTaskParams& ForcePrePostTask()
	{
		bForcePrePostTask = true;
		return *this;
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	const TCHAR* DebugName;
#endif
	TStatId StatId;
	uint8 bForceGameThread : 1;
	uint8 bSerialTasks : 1;
	uint8 bForcePrePostTask : 1;
	uint8 bForcePropagateDownstream : 1;
	uint8 bForceConsumeUpstream : 1;
};


struct ITaskContext
{
	virtual ~ITaskContext() = default;
};

template<typename TaskType>
struct TAnonTaskWrapper : ITaskContext
{
	TaskType Task;

	template<typename ...ArgTypes>
	explicit TAnonTaskWrapper(ArgTypes&&... InArgs)
		: Task{ Forward<ArgTypes>(InArgs)... }
	{}

	static void Execute(const ITaskContext* Context, FEntityAllocationWriteContext WriteContext)
	{
		static_cast<const TAnonTaskWrapper<TaskType>*>(Context)->Task.Run(WriteContext);
	}
};

template<typename ClassType>
struct TMemberFunctionTaskWrapper : ITaskContext
{
	using MemberFunctionPtr = void (ClassType::*)();

	ClassType* ClassPtr;
	MemberFunctionPtr FunctionPtr;

	explicit TMemberFunctionTaskWrapper(ClassType* InClassPtr, MemberFunctionPtr InFunctionPtr)
		: ClassPtr(InClassPtr)
		, FunctionPtr(InFunctionPtr)
	{}

	static void Execute(const ITaskContext* Context, FEntityAllocationWriteContext WriteContext)
	{
		const TMemberFunctionTaskWrapper<ClassType>* This = static_cast<const TMemberFunctionTaskWrapper<ClassType>*>(Context);
		(This->ClassPtr->*This->FunctionPtr)();
	}
};


using UnboundTaskFunctionPtr             = void (*)(const ITaskContext* TaskContext, FEntityAllocationWriteContext WriteContext);
using AllocationFunctionPtr              = void (*)(const FEntityAllocation* Allocation, const ITaskContext* TaskContext, FEntityAllocationWriteContext WriteContext);
using AllocationItemFunctionPtr          = void (*)(FEntityAllocationIteratorItem Item, const ITaskContext* TaskContext, FEntityAllocationWriteContext WriteContext);
using PreLockedAllocationItemFunctionPtr = void (*)(FEntityAllocationIteratorItem Item, TArrayView<const FPreLockedDataPtr> PreLockedData, const ITaskContext* TaskContext, FEntityAllocationWriteContext WriteContext);

using TaskFunctionPtr = TVariant<UnboundTaskFunctionPtr, AllocationFunctionPtr, AllocationItemFunctionPtr, PreLockedAllocationItemFunctionPtr>;

class IEntitySystemScheduler
{
public:

	/**
	 * Add a new task of the specified type for the currently open node ID
	 *
	 * Example usage:
	 * TaskScheduler->AddTask<FMyTaskType>(FTaskParams(GET_STAT_ID(StatId)));
	 * TaskScheduler->AddTask<FMyTaskType2>(FTaskParams(GET_STAT_ID(StatId)), ConstructorArg1, ConstructorArg2);
	 */
	template<typename TaskType, typename ...TaskArgTypes>
	FTaskID AddTask(const FTaskParams& InParams, TaskArgTypes&&... Args)
	{
		TaskFunctionPtr Function(TInPlaceType<UnboundTaskFunctionPtr>(), TAnonTaskWrapper<TaskType>::Execute);
		return AddTask(InParams, MakeShared<TAnonTaskWrapper<TaskType>>(Forward<TaskArgTypes>(Args)...), Function);
	}

	/**
	 * Add a new task that calls a member function of the type void (*)()
	 *
	 * Example usage:
	 * TaskScheduler->AddTask(FTaskParams(GET_STAT_ID(StatId)), this, &UMyClass::ResetWeights);
	 */
	template<typename TaskType>
	FTaskID AddMemberFunctionTask(const FTaskParams& InParams, TaskType* Instance, typename TMemberFunctionTaskWrapper<TaskType>::MemberFunctionPtr FunctionPtr)
	{
		TaskFunctionPtr Function(TInPlaceType<UnboundTaskFunctionPtr>(), TMemberFunctionTaskWrapper<TaskType>::Execute);
		return AddTask(InParams, MakeShared<TMemberFunctionTaskWrapper<TaskType>>(Instance, FunctionPtr), Function);
	}

	/**
	 * Add a 'null' task that can be used to join many tasks into a single dependency
	 */
	MOVIESCENE_API FTaskID AddNullTask();

	/**
	 * Add an anonymous unbound task for doing non-ecs work
	 */
	MOVIESCENE_API FTaskID AddTask(const FTaskParams& InParams, TSharedPtr<ITaskContext> InTaskContext, TaskFunctionPtr InTaskFunction);

	/**
	 * Create one task for each of the entity allocations that match the specified filter
	 */
	MOVIESCENE_API FTaskID CreateForkedAllocationTask(const FTaskParams& InParams, TSharedPtr<ITaskContext> InTaskContext, TaskFunctionPtr InTaskFunction, TFunctionRef<void(FEntityAllocationIteratorItem,TArray<FPreLockedDataPtr>&)> InPreLockFunc, const FEntityComponentFilter& Filter, const FComponentMask& ReadDeps, const FComponentMask& WriteDeps);

	/**
	 * Define a prerequisite for the given task 
	 */
	MOVIESCENE_API void AddPrerequisite(FTaskID Prerequisite, FTaskID Subsequent);

	/**
	 * Add a child to the front of a previously created 'forked' task. Used for defining 'PreTask' work
	 */
	MOVIESCENE_API void AddChildBack(FTaskID Parent, FTaskID Child);

	/**
	 * Add a child to the back of a previously created 'forked' task. Used for defining 'PostTask' work
	 */
	MOVIESCENE_API void AddChildFront(FTaskID Parent, FTaskID Child);

private:
	friend class FEntitySystemScheduler;
	IEntitySystemScheduler() = default;
};


} // namespace UE::MovieScene
