// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieSceneCachedEntityFilterResult.h"
#include "MovieSceneBindingLifetimeSystem.generated.h"

class UObject;
struct FMovieSceneAnimTypeID;

/**
 * Systems that (optionally) controls the lifetime of bindings by communicating with the MovieSceneObjectCache.
 */

UCLASS(MinimalAPI)
class UMovieSceneBindingLifetimeSystem : public UMovieSceneEntitySystem
{
public:
	GENERATED_BODY()

	MOVIESCENE_API UMovieSceneBindingLifetimeSystem(const FObjectInitializer& ObjInit);

private:

	MOVIESCENE_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
};