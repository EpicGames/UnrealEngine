// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/BitArray.h"
#include "EntitySystem/MovieSceneBlenderSystemTypes.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "Math/NumericLimits.h"
#include "Misc/AssertionMacros.h"
#include "Templates/SubclassOf.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneBlenderSystem.generated.h"

class UMovieSceneEntitySystemLinker;
class UObject;

/**
 * Base class for all systems that blend data from multiple entities/components into a single entity
 *
 * This system has direct coupling to TCompositePropertySystemManager and forms the basis for built-in
 * blend modes in Sequencer (Absolute, Relative and Additive). Blend 'channels' are allocated in this
 * system which define a many to one relationship by corresponding blend input(many)/output(one) components
 * added to the relevant entities. Blend input and output channel components are uint16.
 *
 * The class supports a maximum of 65535 blend channels.
 *
 * Additionally, blender systems tag their inputs using the built in Absolute, Relative and Additive tags
 * for efficient computation of each type of blending (this allows each blend type to be computed without branching).
 *
 * A simple example of 3 blended floats is as follows:
 *   Entity Data:
 *       Inputs:
 *         float [float component], uint16 [blend channel input], [Absolute Tag] => [ { 100.f, 0 }, { 200.f, 1 } ]
 *         float [float component], uint16 [blend channel input], [Additive Tag] => [ { 50.f, 1} ]
 *       Outputs:
 *         float [float component], uint16 [blend channel output]                => [ { 0.f, 0}, { 0.f, 1} ]
 * To perform blending for this data, accumulation buffers are allocated per-blend-type, and each blend accumulates
 * into the index of its blend channel input component. A final combination pass walks over blend channel outputs
 * a writes the results into the result component from the accumulation buffer.
 */
UCLASS(Abstract, MinimalAPI)
class UMovieSceneBlenderSystem : public UMovieSceneEntitySystem
{
public:

	GENERATED_BODY()

	/** Default blender system priority. Higher priorities win, negative priorities are supported. */
	static constexpr int32 DefaultPriority = 0;

	/**
	 * Get a blender system class, given a blender system ID.
	 */
	static MOVIESCENE_API TSubclassOf<UMovieSceneBlenderSystem> GetBlenderSystemClass(FMovieSceneBlenderSystemID InSystemID);

	/**
	 * Get the unique ID for a given blender system type.
	 */
	template<typename BlenderSystemClass>
	static FMovieSceneBlenderSystemID GetBlenderSystemID()
	{
		UMovieSceneBlenderSystem* DefaultObject = BlenderSystemClass::StaticClass()->template GetDefaultObject<UMovieSceneBlenderSystem>();
		check(DefaultObject);
		return DefaultObject->SystemID;
	}

	/**
	 * Get the unique ID for this blender system's underlying type.
	 */
	MOVIESCENE_API FMovieSceneBlenderSystemID GetBlenderSystemID() const;

	/**
	 * Allocate a new blend channel.
	 * @note Must be released when it is no longer needed in order to prevent leaking channels.
	 */
	MOVIESCENE_API FMovieSceneBlendChannelID AllocateBlendChannel();

	/**
	 * Release a previously allocated blend channel.
	 */
	MOVIESCENE_API void ReleaseBlendChannel(FMovieSceneBlendChannelID BlendChannelID);

	/**
	 * Retrieve the selection priority to use when choosing a blender system. Highest wins.
	 */
	int32 GetSelectionPriority() const
	{
		return SelectionPriority;
	}

	/**
	 * Retrieve the tag that should accompany any blend input and output components to be used by this blender
	 */
	UE::MovieScene::FComponentTypeID GetBlenderTypeTag() const
	{
		return BlenderTypeTag;
	}

protected:

	MOVIESCENE_API UMovieSceneBlenderSystem(const FObjectInitializer& ObjInit);

	MOVIESCENE_API virtual bool IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const override;

	MOVIESCENE_API void CompactBlendChannels();

	/** Bit array specifying currently allocated blend channels */
	TBitArray<> AllocatedBlendChannels;

	/** Priority to use when selecting a blender system for a given set of entities. Highest priority will be used. */
	int32 SelectionPriority;

private:

	/** Cached blender system ID */
	FMovieSceneBlenderSystemID SystemID;

	/** Type that is unique to this blender system type that distinquishes its BlendInput and Output components */
	UE::MovieScene::FComponentTypeID BlenderTypeTag;
};
