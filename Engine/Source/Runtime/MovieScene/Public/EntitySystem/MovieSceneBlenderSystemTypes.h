// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Math/NumericLimits.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Templates/TypeHash.h"
#include "EntitySystem/MovieSceneComponentDebug.h"

class UMovieSceneBlenderSystem;
class UMovieSceneEntitySystemLinker;

/**
 * Identifier for a type of blender system (e.g. float blender, integer blender, etc.)
 */
struct FMovieSceneBlenderSystemID
{
	static FMovieSceneBlenderSystemID Invalid;

	FMovieSceneBlenderSystemID() : Value(TNumericLimits<uint8>::Max()) {}
	explicit FMovieSceneBlenderSystemID(uint8 InValue) : Value(InValue) {}

	bool IsValid() const { return Value != Invalid.Value; }

	friend uint32 GetTypeHash(FMovieSceneBlenderSystemID SystemID)
	{
		return GetTypeHash(SystemID.Value);
	}

	friend bool operator==(FMovieSceneBlenderSystemID A, FMovieSceneBlenderSystemID B)
	{
		return A.Value == B.Value;
	}

	friend bool operator<(FMovieSceneBlenderSystemID A, FMovieSceneBlenderSystemID B)
	{
		return A.Value < B.Value;
	}

private:
	uint8 Value;
};

/**
 * A blend channel ID
 */
struct FMovieSceneBlendChannelID
{
	/** Invalid blend channel ID */
	static constexpr uint16 INVALID_BLEND_CHANNEL = uint16(-1);

	/** Builds a default, invalid channel ID */
	FMovieSceneBlendChannelID() : ChannelID(INVALID_BLEND_CHANNEL) {}
	/** Builds a channel ID */
	FMovieSceneBlendChannelID(FMovieSceneBlenderSystemID InSystemID, uint16 InChannelID) : SystemID(InSystemID), ChannelID(InChannelID) {}

	/** Returns whether this ID is valid */
	bool IsValid() const { return ChannelID != INVALID_BLEND_CHANNEL; }

	/** Returns the blender system type */
	MOVIESCENE_API TSubclassOf<UMovieSceneBlenderSystem> GetSystemClass() const;

	/** Returns the blender system instance found in a given linker */
	MOVIESCENE_API UMovieSceneBlenderSystem* FindSystem(const UMovieSceneEntitySystemLinker* Linker) const;

	/** Returns the blender system instance found in a given linker */
	template<typename BlenderSystemClass>
	BlenderSystemClass* FindSystem(const UMovieSceneEntitySystemLinker* Linker) const
	{
		return CastChecked<BlenderSystemClass>(FindSystem(Linker));
	}

	friend bool operator==(const FMovieSceneBlendChannelID A, const FMovieSceneBlendChannelID B)
	{
		return (A.SystemID == B.SystemID && A.ChannelID == B.ChannelID);
	}

	friend uint32 GetTypeHash(const FMovieSceneBlendChannelID A)
	{
		return HashCombine(GetTypeHash(A.SystemID), GetTypeHash(A.ChannelID));
	}

	/** The blender system associated with the blend channel */
	FMovieSceneBlenderSystemID SystemID;

	/** The blend channel ID for the given blender system */
	uint16 ChannelID;
};
