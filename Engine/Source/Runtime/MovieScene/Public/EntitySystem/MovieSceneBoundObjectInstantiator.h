// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntityInstantiatorSystem.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneBoundObjectInstantiator.generated.h"

class UObject;


UCLASS(MinimalAPI)
class UMovieSceneGenericBoundObjectInstantiator : public UMovieSceneEntityInstantiatorSystem
{
	GENERATED_BODY()

	UMovieSceneGenericBoundObjectInstantiator(const FObjectInitializer& ObjInit);

private:

	MOVIESCENE_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override final;
};

