// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "HAL/Platform.h"

namespace UE
{
namespace MovieScene
{

class FEntityManager;
struct FEntityAllocation;

/** Signifies whether any caches held for the current cached result are still up to date, or whether they should be updated */
enum class ECachedEntityManagerState
{
	/** The structure of the entity manager has not changed since it was last cached */
	UpToDate,

	/** The structure of the entity manager has changed since it was last cached and any caches must be re-generated */
	Stale,
};


/**
 * Small utility struct that caches the entity manager's structural serial number to generated cached data
 * NOTE: Must only be used with a single entity manager
 */
struct FCachedEntityManagerState
{
	/**
	 * Check our system version number against the current structural number in the entity manager.
	 * @return UpToDate if the structure has not changed, Stale if it has
	 */
	MOVIESCENE_API ECachedEntityManagerState Update(const FEntityManager& InEntityManager);

	/**
	 * Forcibly invalidate this state
	 */
	void Invalidate()
	{
		LastSystemVersion = 0;
	}

	/**
	 * Retrieve the serial number for this cache
	 */
	inline uint64 GetSerial() const
	{
		return LastSystemVersion;
	}

private:

	/** The value of FEntityManager::GetSystemSerial when this filter was last cached */
	uint64 LastSystemVersion = 0;
};


/**
 * Simple cached filter results that stores whether its filter passes or not
 */
struct FCachedEntityFilterResult_Match
{
	FEntityComponentFilter Filter;

	MOVIESCENE_API bool Matches(const FEntityManager& InEntityManager);

	MOVIESCENE_API void Invalidate();

private:

	FCachedEntityManagerState Cache;

	/** The cached result of the filter */
	bool bContainsMatch = false;
};


/**
 * Cached filter result that caches pointers to allocations matching the specified filter
 */
struct FCachedEntityFilterResult_Allocations
{
	FEntityComponentFilter Filter;

	MOVIESCENE_API TArrayView<FEntityAllocation* const> GetMatchingAllocations(const FEntityManager& InEntityManager);

	MOVIESCENE_API void Invalidate();

private:

	FCachedEntityManagerState Cache;

	/** Set bits indicate matching allocations */
	TArray<FEntityAllocation*> MatchedEntityAllocations;
};


} // namespace MovieScene
} // namespace UE

