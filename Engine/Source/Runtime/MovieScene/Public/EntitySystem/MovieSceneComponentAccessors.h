// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "EntitySystem/EntityAllocationIterator.h"
#include "EntitySystem/MovieSceneComponentPtr.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "EntitySystem/MovieSceneSystemTaskDependencies.h"
#include "EntitySystem/RelativePtr.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "MovieSceneEntityIDs.h"
#include "Templates/Tuple.h"


#include <initializer_list>

namespace UE
{
namespace MovieScene
{

class FEntityManager;
using FPreLockedDataPtr = TRelativePtr<void, uint32>;

struct FComponentAccess
{
	FComponentTypeID ComponentType;

	inline void PreLockComponentData(const FEntityAllocation* Allocation, FPreLockedDataPtr* PrelockedComponentData) const
	{
		// Always relative to the component data which is a separate allocation
		const FComponentHeader& Header = Allocation->GetComponentHeaderChecked(ComponentType);
		PrelockedComponentData->Reset(Allocation->GetComponentDataAddress(), Header.Components);
	}
};
struct FReadAccess : FComponentAccess
{
	FReadAccess(FComponentTypeID InComponentType) : FComponentAccess{ InComponentType } {}
};
struct FWriteAccess : FComponentAccess
{
	FWriteAccess(FComponentTypeID InComponentType) : FComponentAccess{ InComponentType } {}
};


struct FOptionalComponentAccess
{
	FComponentTypeID ComponentType;
	FComponentTypeIDFilter Condition;

	inline void PreLockComponentData(FEntityAllocationIteratorItem Item, FPreLockedDataPtr* PrelockedComponentData) const
	{
		if (!Condition.Passes(Item.GetAllocationType()))
		{
			return;
		}

		const FEntityAllocation* Allocation = Item.GetAllocation();

		const FComponentHeader* Header = Allocation->FindComponentHeader(ComponentType);
		if (Header)
		{
			// Always relative to the component data which is a separate allocation
			PrelockedComponentData->Reset(Allocation->GetComponentDataAddress(), Header->Components);
		}
	}
};
struct FOptionalReadAccess : FOptionalComponentAccess
{
	FOptionalReadAccess(FComponentTypeID InComponentType, FComponentTypeIDFilter InConditionType = FComponentTypeIDFilter()) : FOptionalComponentAccess{ InComponentType, InConditionType } {}
};
struct FOptionalWriteAccess : FOptionalComponentAccess
{
	FOptionalWriteAccess(FComponentTypeID InComponentType, FComponentTypeIDFilter InConditionType = FComponentTypeIDFilter()) : FOptionalComponentAccess{ InComponentType, InConditionType } {}
};


struct FEntityIDAccess
{
	using AccessType = const FMovieSceneEntityID;
	static constexpr int32 PreLockedDataNum = 1;


	inline void PreLockComponentData(const FEntityAllocation* Allocation, FPreLockedDataPtr* PrelockedComponentData) const
	{
		// Must be relative to the base allocation ptr
		PrelockedComponentData->Reset(Allocation, Allocation->GetRawEntityIDs());
	}
	inline TRead<FMovieSceneEntityID> ResolvePreLockedComponentData(const FEntityAllocation* Allocation, const FPreLockedDataPtr* Ptr, FEntityAllocationWriteContext WriteContext) const
	{
		// Resolve using the same base ptr as PreLockComponentData
		return TRead<FMovieSceneEntityID>(Ptr->Resolve<FMovieSceneEntityID>(Allocation));
	}

	inline TRead<FMovieSceneEntityID> LockComponentData(const FEntityAllocation* Allocation, FEntityAllocationWriteContext WriteContext) const
	{
		return TRead<FMovieSceneEntityID>(Allocation->GetRawEntityIDs());
	}
};

struct FFilterMatchPassthrough
{
	using AccessType = bool;
	//static constexpr int32 PreLockedDataNum = 1;

	inline void PreLockComponentData(FEntityAllocationIteratorItem Item, FPreLockedDataPtr* PrelockedComponentData) const
	{
		// Must be relative to the base allocation ptr
		const bool bFilterMatch = Filter.Match(Item.GetAllocationType());

		// We abuse the pointer here by using either the allocation ptr itself as 'true' and a nullptr as 'false'
		if (bFilterMatch)
		{
			PrelockedComponentData->Reset(nullptr);
		}
		else
		{
			const FEntityAllocation* Allocation = Item.GetAllocation();
			PrelockedComponentData->Reset(Allocation, Allocation);
		}
	}
	inline bool ResolvePreLockedComponentData(const FEntityAllocation* Allocation, const FPreLockedDataPtr* Ptr, FEntityAllocationWriteContext WriteContext) const
	{
		return Ptr->Resolve<FEntityAllocation>(Allocation) != nullptr;
	}

	inline bool LockComponentData(FEntityAllocationIteratorItem Item, FEntityAllocationWriteContext WriteContext) const
	{
		return Filter.Match(Item.GetAllocationType());
	}

	FEntityComponentFilter Filter;
};

template<typename T>
struct TReadAccess : FReadAccess
{
	using AccessType = const T;
	static constexpr int32 PreLockedDataNum = 1;

	TReadAccess(FComponentTypeID InComponentTypeID)
		: FReadAccess{ InComponentTypeID }
	{}

	inline TRead<T> ResolvePreLockedComponentData(const FEntityAllocation* Allocation, const FPreLockedDataPtr* Ptr, FEntityAllocationWriteContext WriteContext) const
	{
		// Resolve using the same base ptr as PreLockComponentData
		return TRead<T>(Ptr->Resolve<const T>(Allocation->GetComponentDataAddress()));
	}
	inline TComponentLock<TRead<T>> LockComponentData(const FEntityAllocation* Allocation, FEntityAllocationWriteContext WriteContext) const
	{
		return Allocation->ReadComponents(ComponentType.ReinterpretCast<T>());
	}
};


struct FErasedReadAccess : FReadAccess
{
	static constexpr int32 PreLockedDataNum = 1;

	FErasedReadAccess(FComponentTypeID InComponentTypeID)
		: FReadAccess{ InComponentTypeID }
	{}

	inline void PreLockComponentData(const FEntityAllocation* Allocation, FPreLockedDataPtr* PrelockedComponentData) const
	{
		// Erased reads always pass the header since they need the size information
		const FComponentHeader& Header = Allocation->GetComponentHeaderChecked(ComponentType);
		PrelockedComponentData->Reset(Allocation, &Header);
	}
	inline FReadErased ResolvePreLockedComponentData(const FEntityAllocation* Allocation, const FPreLockedDataPtr* Ptr, FEntityAllocationWriteContext WriteContext) const
	{
		// Resolve using the same base ptr as PreLockComponentData
		return FReadErased(Ptr->Resolve<const FComponentHeader>(Allocation));
	}
	inline FComponentReader LockComponentData(const FEntityAllocation* Allocation, FEntityAllocationWriteContext WriteContext) const
	{
		return Allocation->ReadComponentsErased(ComponentType);
	}
};
struct FErasedOptionalReadAccess : FReadAccess
{
	static constexpr int32 PreLockedDataNum = 1;

	FComponentTypeIDFilter Condition;

	FErasedOptionalReadAccess(FComponentTypeID InComponentTypeID, FComponentTypeIDFilter InCondition)
		: FReadAccess{ InComponentTypeID }
		, Condition(InCondition)
	{}

	inline void PreLockComponentData(FEntityAllocationIteratorItem Item, FPreLockedDataPtr* PrelockedComponentData) const
	{
		if (!Condition.Passes(Item.GetAllocationType()))
		{
			return;
		}

		const FEntityAllocation* Allocation = Item.GetAllocation();
		const FComponentHeader* Header = Allocation->FindComponentHeader(ComponentType);
		if (Header)
		{
			// Erased reads always pass the header since they need the size information
			PrelockedComponentData->Reset(Allocation, Header);
		}
	}
	inline FReadErasedOptional ResolvePreLockedComponentData(const FEntityAllocation* Allocation, const FPreLockedDataPtr* Ptr, FEntityAllocationWriteContext WriteContext) const
	{
		// Resolve using the same base ptr as PreLockComponentData
		return FReadErasedOptional(Ptr->Resolve<const FComponentHeader>(Allocation));
	}
	inline FOptionalComponentReader LockComponentData(FEntityAllocationIteratorItem Item, FEntityAllocationWriteContext WriteContext) const
	{
		if (!Condition.Passes(Item.GetAllocationType()))
		{
			return FOptionalComponentReader();
		}

		return Item.GetAllocation()->TryReadComponentsErased(ComponentType);
	}
};

struct FErasedWriteAccess : FWriteAccess
{
	static constexpr int32 PreLockedDataNum = 1;

	FErasedWriteAccess(FComponentTypeID InComponentTypeID)
		: FWriteAccess{ InComponentTypeID }
	{}

	inline void PreLockComponentData(const FEntityAllocation* Allocation, FPreLockedDataPtr* PrelockedComponentData) const
	{
		const FComponentHeader& Header = Allocation->GetComponentHeaderChecked(ComponentType);
		// Erased writes always pass the header since they need the size information
		PrelockedComponentData->Reset(Allocation, &Header);
	}
	inline FWriteErased ResolvePreLockedComponentData(const FEntityAllocation* Allocation, const FPreLockedDataPtr* Ptr, FEntityAllocationWriteContext WriteContext) const
	{
		// Resolve using the same base ptr as PreLockComponentData
		return FWriteErased(Ptr->Resolve<FComponentHeader>(Allocation));
	}
	inline FComponentWriter LockComponentData(const FEntityAllocation* Allocation, FEntityAllocationWriteContext WriteContext) const
	{
		return Allocation->WriteComponentsErased(ComponentType, WriteContext);
	}
};




template<typename T>
struct TWriteAccess : FWriteAccess
{
	using AccessType = T;
	static constexpr int32 PreLockedDataNum = 1;

	TWriteAccess(FComponentTypeID InComponentTypeID)
		: FWriteAccess{ InComponentTypeID }
	{}

	inline TWrite<T> ResolvePreLockedComponentData(const FEntityAllocation* Allocation, const FPreLockedDataPtr* Ptr, FEntityAllocationWriteContext WriteContext) const
	{
		// Resolve using the same base ptr as PreLockComponentData
		return TWrite<T>(Ptr->Resolve<T>(Allocation->GetComponentDataAddress()));
	}
	inline TComponentLock<TWrite<T>> LockComponentData(const FEntityAllocation* Allocation, FEntityAllocationWriteContext WriteContext) const
	{
		return Allocation->WriteComponents(ComponentType.ReinterpretCast<T>(), WriteContext);
	}
};




template<typename T>
struct TOptionalReadAccess : FOptionalReadAccess
{
	using AccessType = const T;
	static constexpr int32 PreLockedDataNum = 1;

	TOptionalReadAccess(FComponentTypeID InComponentTypeID, FComponentTypeIDFilter InConditionType = FComponentTypeIDFilter())
		: FOptionalReadAccess{InComponentTypeID, InConditionType}
	{}

	inline TReadOptional<T> ResolvePreLockedComponentData(const FEntityAllocation* Allocation, const FPreLockedDataPtr* Ptr, FEntityAllocationWriteContext WriteContext) const
	{
		// Resolve using the same base ptr as PreLockComponentData
		return TReadOptional<T>(Ptr->Resolve<T>(Allocation->GetComponentDataAddress()));
	}
	inline TComponentLock<TReadOptional<T>> LockComponentData(const FEntityAllocation* Allocation, FEntityAllocationWriteContext WriteContext) const
	{
		return Allocation->TryReadComponents(ComponentType.ReinterpretCast<T>());
	}
};




template<typename T>
struct TOptionalWriteAccess : FOptionalWriteAccess
{
	using AccessType = T;
	static constexpr int32 PreLockedDataNum = 1;

	TOptionalWriteAccess(FComponentTypeID InComponentTypeID, FComponentTypeIDFilter InConditionType = FComponentTypeIDFilter())
		: FOptionalWriteAccess{ InComponentTypeID, InConditionType }
	{}

	inline TWriteOptional<T> ResolvePreLockedComponentData(const FEntityAllocation* Allocation, const FPreLockedDataPtr* Ptr, FEntityAllocationWriteContext WriteContext) const
	{
		// Resolve using the same base ptr as PreLockComponentData
		return TWriteOptional<T>(Ptr->Resolve<T>(Allocation->GetComponentDataAddress()));
	}
	inline TComponentLock<TWriteOptional<T>> LockComponentData(const FEntityAllocation* Allocation, FEntityAllocationWriteContext WriteContext) const
	{
		return Allocation->TryWriteComponents(ComponentType.ReinterpretCast<T>(), WriteContext);
	}
};


template<typename...>
struct TUnpackMultiComponentData;

template<typename ...T, int ...Indices>
struct TUnpackMultiComponentData<TIntegerSequence<int, Indices...>, T...>
{
	static TMultiComponentData<TReadOptional<T>...> ResolvePreLockedComponentData(const FEntityAllocation* Allocation, const FOptionalReadAccess* ComponentTypes, const FPreLockedDataPtr* PrelockedComponentData, FEntityAllocationWriteContext WriteContext)
	{
		return TMultiComponentData<TReadOptional<T>...>(
			TOptionalReadAccess<T>(ComponentTypes[Indices].ComponentType).ResolvePreLockedComponentData(Allocation, &PrelockedComponentData[Indices], WriteContext)...
			);
	}
	static TMultiComponentLock<TReadOptional<T>...> LockComponentData(const FEntityAllocation* Allocation, const FOptionalReadAccess* ComponentTypes, FEntityAllocationWriteContext WriteContext)
	{
		return TMultiComponentLock<TReadOptional<T>...>(
			TOptionalReadAccess<T>(ComponentTypes[Indices].ComponentType).LockComponentData(Allocation, WriteContext)...
			);
	}
};

template<typename... T>
struct TReadOneOfAccessor
{
	using AccessType = TMultiComponentLock<TReadOptional<T>...>;
	static constexpr int32 PreLockedDataNum = sizeof...(T);

	TReadOneOfAccessor(TComponentTypeID<T>... InComponentTypeIDs)
		: ComponentTypes{ InComponentTypeIDs... }
	{}

	void PreLockComponentData(FEntityAllocationIteratorItem Item, FPreLockedDataPtr* PrelockedComponentData) const
	{
		for (int32 Index = 0; Index < sizeof...(T); ++Index)
		{
			ComponentTypes[Index].PreLockComponentData(Item, &PrelockedComponentData[Index]);
		}
	}

	TMultiComponentData<TReadOptional<T>...> ResolvePreLockedComponentData(const FEntityAllocation* Allocation, const FPreLockedDataPtr* PrelockedComponentData, FEntityAllocationWriteContext WriteContext) const
	{
		return TUnpackMultiComponentData<TMakeIntegerSequence<int, sizeof...(T)>, T...>
			::ResolvePreLockedComponentData(Allocation, ComponentTypes, PrelockedComponentData, WriteContext);
	}

	TMultiComponentLock<TReadOptional<T>...> LockComponentData(const FEntityAllocation* Allocation, FEntityAllocationWriteContext WriteContext) const
	{
		return TUnpackMultiComponentData<TMakeIntegerSequence<int, sizeof...(T)>, T...>
			::LockComponentData(Allocation, ComponentTypes, WriteContext);
	}

	FOptionalReadAccess ComponentTypes[sizeof...(T)];
};




template<typename... T>
struct TReadOneOrMoreOfAccessor
{
	using AccessType = TMultiComponentLock<TReadOptional<T>...>;
	static constexpr int32 PreLockedDataNum = sizeof...(T);

	TReadOneOrMoreOfAccessor(TComponentTypeID<T>... InComponentTypeIDs)
		: ComponentTypes{ InComponentTypeIDs... }
	{}

	void PreLockComponentData(FEntityAllocationIteratorItem Item, FPreLockedDataPtr* PrelockedComponentData) const
	{
		for (int32 Index = 0; Index < sizeof...(T); ++Index)
		{
			ComponentTypes[Index].PreLockComponentData(Item, &PrelockedComponentData[Index]);
		}
	}

	TMultiComponentData<TReadOptional<T>...> ResolvePreLockedComponentData(const FEntityAllocation* Allocation, const FPreLockedDataPtr* PrelockedComponentData, FEntityAllocationWriteContext WriteContext) const
	{
		return TUnpackMultiComponentData<TMakeIntegerSequence<int, sizeof...(T)>, T...>
			::ResolvePreLockedComponentData(Allocation, ComponentTypes, PrelockedComponentData, WriteContext);
	}

	TMultiComponentLock<TReadOptional<T>...> LockComponentData(const FEntityAllocation* Allocation, FEntityAllocationWriteContext WriteContext) const
	{
		return TUnpackMultiComponentData<TMakeIntegerSequence<int, sizeof...(T)>, T...>
			::LockComponentData(Allocation, ComponentTypes, WriteContext);
	}

	FOptionalReadAccess ComponentTypes[sizeof...(T)];
};


inline void AddAccessorToFilter(const FEntityIDAccess*, FEntityComponentFilter* OutFilter)
{
}
inline void AddAccessorToFilter(const FFilterMatchPassthrough*, FEntityComponentFilter* OutFilter)
{
}
inline void AddAccessorToFilter(const FComponentAccess* In, FEntityComponentFilter* OutFilter)
{
	check(In->ComponentType);
	OutFilter->All({ In->ComponentType });
}
inline void AddAccessorToFilter(const FOptionalComponentAccess* In, FEntityComponentFilter* OutFilter)
{
}
template<typename... T>
void AddAccessorToFilter(const TReadOneOfAccessor<T...>* In, FEntityComponentFilter* OutFilter)
{
	FComponentMask Mask;
	for (int32 Index = 0; Index < sizeof...(T); ++Index)
	{
		FComponentTypeID Component = In->ComponentTypes[Index].ComponentType;
		if (Component)
		{
			Mask.Set(Component);
		}
	}

	check(Mask.NumComponents() != 0);
	OutFilter->Complex(Mask, EComplexFilterMode::OneOf);
}
template<typename... T>
void AddAccessorToFilter(const TReadOneOrMoreOfAccessor<T...>* In, FEntityComponentFilter* OutFilter)
{
	FComponentMask Mask;
	for (int32 Index = 0; Index < sizeof...(T); ++Index)
	{
		FComponentTypeID Component = In->ComponentTypes[Index].ComponentType;
		if (Component)
		{
			Mask.Set(Component);
		}
	}

	check(Mask.NumComponents() != 0);
	OutFilter->Complex(Mask, EComplexFilterMode::OneOrMoreOf);
}


inline void PopulatePrerequisites(const FEntityIDAccess*, const FSystemTaskPrerequisites& InPrerequisites, FGraphEventArray* OutGatheredPrereqs)
{
}
inline void PopulatePrerequisites(const FComponentAccess* In, const FSystemTaskPrerequisites& InPrerequisites, FGraphEventArray* OutGatheredPrereqs)
{
	check(In->ComponentType);
	InPrerequisites.FilterByComponent(*OutGatheredPrereqs, In->ComponentType);
}
inline void PopulatePrerequisites(const FOptionalComponentAccess* In, const FSystemTaskPrerequisites& InPrerequisites, FGraphEventArray* OutGatheredPrereqs)
{
	if (In->ComponentType)
	{
		check(In->ComponentType);
		InPrerequisites.FilterByComponent(*OutGatheredPrereqs, In->ComponentType);
	}
}
template<typename... T>
void PopulatePrerequisites(const TReadOneOfAccessor<T...>* In, const FSystemTaskPrerequisites& InPrerequisites, FGraphEventArray* OutGatheredPrereqs)
{
	for (int32 Index = 0; Index < sizeof...(T); ++Index)
	{
		PopulatePrerequisites(&In->ComponentTypes[Index], InPrerequisites, OutGatheredPrereqs);
	}
}
template<typename... T>
void PopulatePrerequisites(const TReadOneOrMoreOfAccessor<T...>* In, const FSystemTaskPrerequisites& InPrerequisites, FGraphEventArray* OutGatheredPrereqs)
{
	for (int32 Index = 0; Index < sizeof...(T); ++Index)
	{
		PopulatePrerequisites(&In->ComponentTypes[Index], InPrerequisites, OutGatheredPrereqs);
	}
}



inline void PopulateSubsequents(const FWriteAccess* In, const FGraphEventRef& InEvent, FSystemSubsequentTasks& OutSubsequents)
{
	check(In->ComponentType);
	OutSubsequents.AddComponentTask(In->ComponentType, InEvent);
}
inline void PopulateSubsequents(const FOptionalWriteAccess* In, const FGraphEventRef& InEvent, FSystemSubsequentTasks& OutSubsequents)
{
	if (In->ComponentType)
	{
		OutSubsequents.AddComponentTask(In->ComponentType, InEvent);
	}
}
inline void PopulateSubsequents(const void* In, const FGraphEventRef& InEvent, FSystemSubsequentTasks& OutSubsequents)
{
}



inline void PopulateReadWriteDependencies(const FEntityIDAccess*, FComponentMask& OutReadDependencies, FComponentMask& OutWriteDependencies)
{
}
inline void PopulateReadWriteDependencies(const FReadAccess* In, FComponentMask& OutReadDependencies, FComponentMask& OutWriteDependencies)
{
	checkSlow(In->ComponentType);
	OutReadDependencies.Set(In->ComponentType);
}
inline void PopulateReadWriteDependencies(const FOptionalReadAccess* In, FComponentMask& OutReadDependencies, FComponentMask& OutWriteDependencies)
{
	if (In->ComponentType)
	{
		OutReadDependencies.Set(In->ComponentType);
	}
}
inline void PopulateReadWriteDependencies(const FWriteAccess* In, FComponentMask& OutReadDependencies, FComponentMask& OutWriteDependencies)
{
	checkSlow(In->ComponentType);
	OutWriteDependencies.Set(In->ComponentType);
}
inline void PopulateReadWriteDependencies(const FOptionalWriteAccess* In, FComponentMask& OutReadDependencies, FComponentMask& OutWriteDependencies)
{
	if (In->ComponentType)
	{
		OutWriteDependencies.Set(In->ComponentType);
	}
}
template<typename... T>
void PopulateReadWriteDependencies(const TReadOneOfAccessor<T...>* In, FComponentMask& OutReadDependencies, FComponentMask& OutWriteDependencies)
{
	for (int32 Index = 0; Index < sizeof...(T); ++Index)
	{
		PopulateReadWriteDependencies(&In->ComponentTypes[Index], OutReadDependencies, OutWriteDependencies);
	}
}
template<typename... T>
void PopulateReadWriteDependencies(const TReadOneOrMoreOfAccessor<T...>* In, FComponentMask& OutReadDependencies, FComponentMask& OutWriteDependencies)
{
	for (int32 Index = 0; Index < sizeof...(T); ++Index)
	{
		PopulateReadWriteDependencies(&In->ComponentTypes[Index], OutReadDependencies, OutWriteDependencies);
	}
}



inline bool HasBeenWrittenToSince(const FEntityIDAccess* In, FEntityAllocation* Allocation, uint64 InSystemSerial)
{
	return Allocation->HasStructureChangedSince(InSystemSerial);
}
inline bool HasBeenWrittenToSince(const FComponentAccess* In, FEntityAllocation* Allocation, uint64 InSystemSerial)
{
	return Allocation->GetComponentHeaderChecked(In->ComponentType).HasBeenWrittenToSince(InSystemSerial);
}
inline bool HasBeenWrittenToSince(const FOptionalComponentAccess* In, FEntityAllocation* Allocation, uint64 InSystemSerial)
{
	if (FComponentHeader* Header = Allocation->FindComponentHeader(In->ComponentType))
	{
		return Header->HasBeenWrittenToSince(InSystemSerial);
	}
	return false;
}
template<typename... T>
bool HasBeenWrittenToSince(const TReadOneOfAccessor<T...>* In, FEntityAllocation* Allocation, uint64 InSystemSerial)
{
	bool bAnyWrittenTo = false;
	for (int32 Index = 0; Index < sizeof...(T); ++Index)
	{
		bAnyWrittenTo |= HasBeenWrittenToSince(&In->ComponentTypes[Index], Allocation, InSystemSerial);
	}
	return bAnyWrittenTo;
}
template<typename... T>
bool HasBeenWrittenToSince(const TReadOneOrMoreOfAccessor<T...>* In, FEntityAllocation* Allocation, uint64 InSystemSerial)
{
	bool bAnyWrittenTo = false;
	for (int32 Index = 0; Index < sizeof...(T); ++Index)
	{
		bAnyWrittenTo |= HasBeenWrittenToSince(&In->ComponentTypes[Index], Allocation, InSystemSerial);
	}
	return bAnyWrittenTo;
}

template<typename T>
auto GetComponentAtIndex(T* InAccessor, int32 Index)
	-> decltype(DeclVal<T>().ComponentAtIndex(0))
{
	return InAccessor->ComponentAtIndex(Index);
}

inline bool GetComponentAtIndex(const bool* BoolPassthrough, int32 Index)
{
	return *BoolPassthrough;
}

inline bool IsAccessorValid(const FEntityIDAccess*)
{
	return true;
}
inline bool IsAccessorValid(const FFilterMatchPassthrough*)
{
	return true;
}
inline bool IsAccessorValid(const FComponentAccess* In)
{
	return In->ComponentType != FComponentTypeID::Invalid();
}
inline bool IsAccessorValid(const FOptionalComponentAccess* In)
{
	return true;
}
template<typename... T>
inline bool IsAccessorValid(const TReadOneOfAccessor<T...>* In)
{
	bool bValid = false;
	for (int32 Index = 0; Index < sizeof...(T); ++Index)
	{
		bValid |= IsAccessorValid(&In->ComponentTypes[Index]);
	}
	return bValid;
}
template<typename... T>
inline bool IsAccessorValid(const TReadOneOrMoreOfAccessor<T...>* In)
{
	bool bValid = false;
	for (int32 Index = 0; Index < sizeof...(T); ++Index)
	{
		bValid |= IsAccessorValid(&In->ComponentTypes[Index]);
	}
	return bValid;
}



inline bool HasAccessorWork(const FEntityManager*, const FEntityIDAccess*)
{
	return true;
}
inline bool HasAccessorWork(const FEntityManager* EntityManager, const FComponentAccess* In)
{
	return EntityManager->ContainsComponent(In->ComponentType);
}
inline bool HasAccessorWork(const FEntityManager* EntityManager, const FOptionalComponentAccess* In)
{
	return true;
}
template<typename... T>
inline bool HasAccessorWork(const FEntityManager* EntityManager, const TReadOneOfAccessor<T...>* In)
{
	bool bAnyWork = false;
	for (int32 Index = 0; Index < sizeof...(T); ++Index)
	{
		bAnyWork |= HasAccessorWork(EntityManager, &In->ComponentTypes[Index]);
	}
	return bAnyWork;
}
template<typename... T>
inline bool HasAccessorWork(const FEntityManager* EntityManager, const TReadOneOrMoreOfAccessor<T...>* In)
{
	bool bAnyWork = false;
	for (int32 Index = 0; Index < sizeof...(T); ++Index)
	{
		bAnyWork |= HasAccessorWork(EntityManager, &In->ComponentTypes[Index]);
	}
	return bAnyWork;
}

#if UE_MOVIESCENE_ENTITY_DEBUG

	MOVIESCENE_API void AccessorToString(const FReadAccess* In, FEntityManager* EntityManager, FString& OutString);
	MOVIESCENE_API void AccessorToString(const FWriteAccess* In, FEntityManager* EntityManager, FString& OutString);
	MOVIESCENE_API void AccessorToString(const FOptionalReadAccess* In, FEntityManager* EntityManager, FString& OutString);
	MOVIESCENE_API void AccessorToString(const FOptionalWriteAccess* In, FEntityManager* EntityManager, FString& OutString);
	MOVIESCENE_API void AccessorToString(const FEntityIDAccess*, FEntityManager* EntityManager, FString& OutString);
	MOVIESCENE_API void OneOfAccessorToString(const FOptionalReadAccess* In, FEntityManager* EntityManager, FString& OutString);

	template<typename... T>
	void AccessorToString(const TReadOneOfAccessor<T...>* In, FEntityManager* EntityManager, FString& OutString)
	{
		TArray<FString> Strings;

		for (int32 Index = 0; Index < sizeof...(T); ++Index)
		{
			OneOfAccessorToString(&In->ComponentTypes[Index], EntityManager, Strings.Emplace_GetRef());
		}

		OutString += FString::Printf(TEXT("\n\tRead One Of: [ %s ]"), *FString::Join(Strings, TEXT(",")));
	}

	template<typename... T>
	void AccessorToString(const TReadOneOrMoreOfAccessor<T...>* In, FEntityManager* EntityManager, FString& OutString)
	{
		TArray<FString> Strings;

		for (int32 Index = 0; Index < sizeof...(T); ++Index)
		{
			OneOfAccessorToString(&In->ComponentTypes[Index], EntityManager, Strings.Emplace_GetRef());
		}

		OutString += FString::Printf(TEXT("\n\tRead One Or More Of: [ %s ]"), *FString::Join(Strings, TEXT(",")));
	}

#endif // UE_MOVIESCENE_ENTITY_DEBUG


} // namespace MovieScene
} // namespace UE
