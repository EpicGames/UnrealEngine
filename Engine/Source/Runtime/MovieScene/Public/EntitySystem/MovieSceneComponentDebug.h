// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneFwd.h"

#if UE_MOVIESCENE_ENTITY_DEBUG

#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "Math/Vector4.h"
#include "Misc/FrameTime.h"
#include "UObject/ObjectKey.h"
#include "Misc/InlineValue.h"

class FName;

namespace UE::MovieScene
{

MOVIESCENE_API extern bool GRichComponentDebugging;

struct IComponentDebuggingTypedPtr
{
	virtual ~IComponentDebuggingTypedPtr() = default;
	void* Ptr;
};
template<typename T>
struct TComponentDebuggingTypedPtr : IComponentDebuggingTypedPtr
{
	TComponentDebuggingTypedPtr()
	{
		static_assert(sizeof(TComponentDebuggingTypedPtr) == sizeof(IComponentDebuggingTypedPtr), "Size must match");
	}
};

template<typename T>
struct TComponentHeader : FComponentHeader
{
	TComponentHeader()
	{
		static_assert(sizeof(TComponentHeader<T>) == sizeof(FComponentHeader), "Size must match!");
	}
};

/**
 * Debug information for a component type
 */
struct FComponentTypeDebugInfo
{
	FString DebugName;
	const TCHAR* DebugTypeName = nullptr;

	virtual ~FComponentTypeDebugInfo() = default;
	virtual void InitializeComponentHeader(void* Ptr) const
	{
		new (Ptr) FComponentHeader();
	}
	virtual void InitializeDebugComponentData(FComponentHeader& Header, uint8 Capacity) const
	{
	}
};

template<typename T>
struct TComponentTypeDebugInfo : FComponentTypeDebugInfo
{
	void InitializeComponentHeader(void* Ptr) const override
	{
		new (Ptr) TComponentHeader<T>();
	}

	virtual void InitializeDebugComponentData(FComponentHeader& Header, uint8 Capacity) const
	{
		TComponentDebuggingTypedPtr<T>* TypedComponents = new TComponentDebuggingTypedPtr<T>[Capacity];
		
		for (int32 Index = 0; Index < Capacity; ++Index)
		{
			TypedComponents[Index].Ptr = static_cast<T*>(Header.GetValuePtr(Index));
		}

		Header.DebugComponents = TypedComponents;
	}
};

} // namespace UE::MovieScene


#endif // UE_MOVIESCENE_ENTITY_DEBUG
