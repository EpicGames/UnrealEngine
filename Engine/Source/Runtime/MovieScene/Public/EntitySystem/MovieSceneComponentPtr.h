// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include <type_traits>

namespace UE
{
namespace MovieScene
{


/** Facade for any component data type */
template<typename T>
struct TComponentPtr
{
	/** Value type is either a T or const T& for read-only types, or T& for writeable types */
	using ValueType = std::conditional_t<std::is_const_v<T>, typename TCallTraits<T>::ParamType, T&>;

	TComponentPtr()
		: ComponentPtr(nullptr)
	{
	}

	explicit TComponentPtr(T* InComponentPtr)
		: ComponentPtr(InComponentPtr)
	{
	}

	TComponentPtr(const TComponentPtr& RHS) = default;
	TComponentPtr& operator=(const TComponentPtr& RHS) = default;

	TComponentPtr(TComponentPtr&& RHS)
		: ComponentPtr(RHS.ComponentPtr)
	{
		RHS.ComponentPtr = nullptr;
	}

	TComponentPtr& operator=(TComponentPtr&& RHS)
	{
		ComponentPtr = RHS.ComponentPtr;
		RHS.ComponentPtr = nullptr;
		return *this;
	}

	inline explicit operator bool() const
	{
		return IsValid();
	}

	inline bool IsValid() const
	{
		return ComponentPtr != nullptr;
	}

	/** Explicitly convert this component data to its underlying pointer type */
	inline T* AsPtr() const
	{
		return ComponentPtr;
	}

	/** Retrieve this component data as an array view of the specified size (usually the size will be FEntityAllocation::Num()) */
	inline TArrayView<T> AsArray(int32 Num) const
	{
		return MakeArrayView(ComponentPtr, Num);
	}

	/** Retrieve a slice of this component data as an array view of the specified size and offset (usually the size will be FEntityAllocation::Num()) */
	inline TArrayView<T> Slice(int32 StartIndex, int32 Num) const
	{
		return MakeArrayView(ComponentPtr + StartIndex, Num);
	}

	inline T* operator->() const
	{
		return ComponentPtr;
	}

	inline ValueType operator*() const
	{
		return *ComponentPtr;
	}

	inline ValueType operator[](int32 Index) const
	{
		return ComponentPtr[Index];
	}

	ValueType ComponentAtIndex(int32 Index) const
	{
		return (*this)[Index];
	}

	inline operator T*() const
	{
		return ComponentPtr;
	}

protected:

	T* ComponentPtr;
};


/**
 * Typed write-lock for component data
 */
template<typename Accessor> struct TComponentLock;
template<typename LockType> struct TComponentLockMixin;

template<>
struct TComponentLockMixin<FScopedHeaderReadLock>
{
	TComponentLockMixin() = default;
	explicit TComponentLockMixin(const FComponentHeader* InHeader, EComponentHeaderLockMode InLockMode)
		: Lock(InHeader, InLockMode)
	{}

private:
	FScopedHeaderReadLock Lock;
};

template<>
struct TComponentLockMixin<FScopedHeaderWriteLock>
{
	TComponentLockMixin() = default;
	explicit TComponentLockMixin(const FComponentHeader* InHeader, EComponentHeaderLockMode InLockMode, FEntityAllocationWriteContext InWriteContext)
		: Lock(InHeader, InLockMode, InWriteContext)
	{}

private:
	FScopedHeaderWriteLock Lock;
};

struct FReadErased
{
	FReadErased(const FComponentHeader* InHeader, int32 ComponentOffset = 0)
		: ComponentPtr(InHeader->Components + ComponentOffset*InHeader->Sizeof)
		, Sizeof(InHeader->Sizeof)
	{}

	inline const void* AsPtr() const
	{
		return ComponentPtr;
	}
	inline const void* operator[](int32 Index) const
	{
		return ComponentPtr + Sizeof*Index;
	}
	inline const void* ComponentAtIndex(int32 Index) const
	{
		return ComponentPtr + Sizeof*Index;
	}
	inline bool IsValid() const
	{
		return ComponentPtr != nullptr;
	}

protected:
	FReadErased()
		: ComponentPtr(nullptr)
		, Sizeof(0)
	{}

	const uint8* ComponentPtr;
	int32 Sizeof;
};

struct FReadErasedOptional : FReadErased
{
	FReadErasedOptional() = default;
	explicit FReadErasedOptional(const FComponentHeader* InHeader, int32 ComponentOffset = 0)
	{
		if (InHeader)
		{
			ComponentPtr = InHeader->Components + ComponentOffset*InHeader->Sizeof;
			Sizeof = InHeader->Sizeof;
		}
		else
		{
			ComponentPtr = nullptr;
			Sizeof = 0;
		}
	}

	const void* ComponentAtIndex(int32 Index) const
	{
		return this->ComponentPtr ? FReadErased::ComponentAtIndex(Index) : nullptr;
	}
};

template<typename T>
struct TRead : TComponentPtr<const T>
{
	TRead() = default;
	explicit TRead(const T* ComponentPtr, int32 ComponentOffset = 0)
		: TComponentPtr<const T>(ComponentPtr + ComponentOffset)
	{}
	explicit TRead(const FComponentHeader* InHeader, int32 ComponentOffset = 0)
		: TComponentPtr<const T>(reinterpret_cast<const T*>(InHeader->Components) + ComponentOffset)
	{}
};
using FReadEntityIDs = TRead<FMovieSceneEntityID>;

template<typename T>
struct TReadOptional : TComponentPtr<const T>
{
	TReadOptional() = default;
	explicit TReadOptional(const T* InData, int32 ComponentOffset = 0)
	{
		if (InData)
		{
			this->ComponentPtr = InData + ComponentOffset;
		}
	}
	explicit TReadOptional(const FComponentHeader* InHeader, int32 ComponentOffset = 0)
	{
		if (InHeader)
		{
			this->ComponentPtr = reinterpret_cast<const T*>(InHeader->Components) + ComponentOffset;
		}
	}

	const T* ComponentAtIndex(int32 Index) const
	{
		return this->ComponentPtr ? &this->ComponentPtr[Index] : nullptr;
	}
};


struct FWriteErased
{
	FWriteErased()
		: ComponentPtr(nullptr)
		, Sizeof(0)
	{}
	FWriteErased(const FComponentHeader* InHeader, int32 ComponentOffset = 0)
		: ComponentPtr(InHeader->Components + ComponentOffset*InHeader->Sizeof)
		, Sizeof(InHeader->Sizeof)
	{}
	inline void* AsPtr() const
	{
		return ComponentPtr;
	}
	inline void* operator[](int32 Index) const
	{
		return ComponentPtr + Sizeof*Index;
	}
	inline bool IsValid() const
	{
		return ComponentPtr != nullptr;
	}
	void* ComponentAtIndex(int32 Index) const
	{
		return (*this)[Index];
	}

protected:
	uint8* ComponentPtr;
	int32 Sizeof;
};

struct FWriteErasedOptional : FWriteErased
{
	FWriteErasedOptional() = default;
	FWriteErasedOptional(const FComponentHeader* InHeader, int32 ComponentOffset = 0)
	{
		if (InHeader)
		{
			ComponentPtr = InHeader->Components + ComponentOffset*InHeader->Sizeof;
			Sizeof = InHeader->Sizeof;
		}
		else
		{
			ComponentPtr = nullptr;
			Sizeof = 0;
		}
	}
	void* ComponentAtIndex(int32 Index) const
	{
		return ComponentPtr ? (*this)[Index] : nullptr;
	}
};

template<typename T>
struct TWrite : TComponentPtr<T>
{
	TWrite() = default;
	explicit TWrite(T* InData, int32 ComponentOffset = 0)
		: TComponentPtr<T>(InData + ComponentOffset)
	{}
	explicit TWrite(const FComponentHeader* InHeader, int32 ComponentOffset = 0)
		: TComponentPtr<T>(reinterpret_cast<T*>(InHeader->Components) + ComponentOffset)
	{}
};

template<typename T>
struct TWriteOptional : TComponentPtr<T>
{
	TWriteOptional() = default;
	explicit TWriteOptional(T* InData, int32 ComponentOffset = 0)
	{
		if (InData)
		{
			this->ComponentPtr = InData + ComponentOffset;
		}
	}
	explicit TWriteOptional(const FComponentHeader* InHeader, int32 ComponentOffset = 0)
	{
		if (InHeader)
		{
			this->ComponentPtr = reinterpret_cast<T*>(InHeader->Components) + ComponentOffset;
		}
	}

	T* ComponentAtIndex(int32 Index) const
	{
		return this->ComponentPtr ? &this->ComponentPtr[Index] : nullptr;
	}
};



template<typename AccessorType> struct TComponentLock;

template<>
struct TComponentLock<FReadErased> : TComponentLockMixin<FScopedHeaderReadLock>, FReadErased
{
	explicit TComponentLock(const FComponentHeader* InHeader, EComponentHeaderLockMode InLockMode, int32 ComponentOffset = 0)
		: TComponentLockMixin<FScopedHeaderReadLock>(InHeader, InLockMode)
		, FReadErased(InHeader, ComponentOffset)
	{}

	const void* ComponentAtIndex(int32 Index) const
	{
		return (*this)[Index];
	}
};
template<>
struct TComponentLock<FReadErasedOptional> : TComponentLockMixin<FScopedHeaderReadLock>, FReadErasedOptional
{
	TComponentLock() = default;
	explicit TComponentLock(const FComponentHeader* InHeader, EComponentHeaderLockMode InLockMode, int32 ComponentOffset = 0)
	{
		if (InHeader)
		{
			*static_cast<TComponentLockMixin<FScopedHeaderReadLock>*>(this) = TComponentLockMixin<FScopedHeaderReadLock>(InHeader, InLockMode);
			*static_cast<FReadErasedOptional*>(this) = FReadErasedOptional(InHeader, ComponentOffset);
		}
	}

	explicit operator bool() const
	{
		return this->ComponentPtr != nullptr;
	}
};
template<>
struct TComponentLock<FWriteErased> : TComponentLockMixin<FScopedHeaderWriteLock>, FWriteErased
{
	explicit TComponentLock(const FComponentHeader* InHeader, EComponentHeaderLockMode InLockMode, FEntityAllocationWriteContext InWriteContext, int32 ComponentOffset = 0)
		: TComponentLockMixin<FScopedHeaderWriteLock>(InHeader, InLockMode, InWriteContext)
		, FWriteErased(InHeader, ComponentOffset)
	{}
};
template<>
struct TComponentLock<FWriteErasedOptional> : TComponentLockMixin<FScopedHeaderWriteLock>, FWriteErasedOptional
{
	TComponentLock() = default;
	explicit TComponentLock(const FComponentHeader* InHeader, EComponentHeaderLockMode InLockMode, FEntityAllocationWriteContext InWriteContext, int32 ComponentOffset = 0)
	{
		if (InHeader)
		{
			*static_cast<TComponentLockMixin<FScopedHeaderWriteLock>*>(this) = TComponentLockMixin<FScopedHeaderWriteLock>(InHeader, InLockMode, InWriteContext);
			*static_cast<FWriteErasedOptional*>(this) = FWriteErasedOptional(InHeader, ComponentOffset);
		}
	}
};
template<typename T>
struct TComponentLock<TRead<T>> : TComponentLockMixin<FScopedHeaderReadLock>, TRead<T>
{
	explicit TComponentLock(const FComponentHeader* InHeader, EComponentHeaderLockMode InLockMode , int32 ComponentOffset = 0)
		: TComponentLockMixin<FScopedHeaderReadLock>(InHeader, InLockMode)
		, TRead<T>(InHeader, ComponentOffset)
	{}
};

template<typename T>
struct TComponentLock<TReadOptional<T>> : TComponentLockMixin<FScopedHeaderReadLock>, TReadOptional<T>
{
	TComponentLock() = default;
	explicit TComponentLock(const FComponentHeader* InHeader, EComponentHeaderLockMode InLockMode, int32 ComponentOffset = 0)
	{
		if (InHeader)
		{
			*static_cast<TComponentLockMixin<FScopedHeaderReadLock>*>(this) = TComponentLockMixin<FScopedHeaderReadLock>(InHeader, InLockMode);
			*static_cast<TReadOptional<T>*>(this) = TReadOptional<T>(InHeader, ComponentOffset);
		}
	}
};
template<typename T>
struct TComponentLock<TWrite<T>> : TComponentLockMixin<FScopedHeaderWriteLock>, TWrite<T>
{
	explicit TComponentLock(const FComponentHeader* InHeader, EComponentHeaderLockMode InLockMode, FEntityAllocationWriteContext InWriteContext, int32 ComponentOffset = 0)
		: TComponentLockMixin<FScopedHeaderWriteLock>(InHeader, InLockMode, InWriteContext)
		, TWrite<T>(InHeader, ComponentOffset)
	{}
};

template<typename T>
struct TComponentLock<TWriteOptional<T>> : TComponentLockMixin<FScopedHeaderWriteLock>, TWriteOptional<T>
{
	TComponentLock() = default;
	explicit TComponentLock(const FComponentHeader* InHeader, EComponentHeaderLockMode InLockMode, FEntityAllocationWriteContext InWriteContext, int32 ComponentOffset = 0)
	{
		if (InHeader)
		{
			*static_cast<TComponentLockMixin<FScopedHeaderWriteLock>*>(this) = TComponentLockMixin<FScopedHeaderWriteLock>(InHeader, InLockMode, InWriteContext);
			*static_cast<TWriteOptional<T>*>(this) = TWriteOptional<T>(InHeader, ComponentOffset);
		}
	}
};


using FComponentReader = TComponentLock<FReadErased>;
using FOptionalComponentReader = TComponentLock<FReadErasedOptional>;
using FComponentWriter = TComponentLock<FWriteErased>;
using FOptionalComponentWriter = TComponentLock<FWriteErasedOptional>;
template<typename T> using TComponentReader = TComponentLock<TRead<T>>;
template<typename T> using TOptionalComponentReader = TComponentLock<TReadOptional<T>>;
template<typename T> using TComponentWriter = TComponentLock<TWrite<T>>;
template<typename T> using TOptionalComponentWriter = TComponentLock<TWriteOptional<T>>;


template<typename... T> using TMultiComponentLock = TTuple<TComponentLock<T>...>;

template<typename... T>
struct TMultiComponentData
{
	TMultiComponentData(T... InData)
		: Data(InData...)
	{}

	TMultiComponentData(const TMultiComponentLock<T...>& InAggregateLock)
	{
		auto Init = [](const auto& InLock, auto& OutData)
		{
			if (InLock)
			{
				OutData = InLock;
			}
		};
		VisitTupleElements(Init, InAggregateLock, this->Data);
	}

	template<int Index>
	auto* Get() const
	{
		return Data.template Get<Index>().AsPtr();
	}

	template<int Index>
	auto GetAsArray(int32 ArraySize) const
	{
		auto* ComponentPtr = Data.template Get<Index>().AsPtr();
		return MakeArrayView(ComponentPtr, ComponentPtr ? ArraySize : 0);
	}

private:

	TTuple<T...> Data;
};

template<typename... T> using TMultiReadOptional = TMultiComponentData<TReadOptional<T>...>;
template<typename... T> using TReadOneOrMoreOf = TMultiComponentData<TReadOptional<T>...>;
template<typename... T> using TReadOneOf = TMultiComponentData<TReadOptional<T>...>;

} // namespace UE
} // namespace MovieScene
