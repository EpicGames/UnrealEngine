// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "EntitySystem/MovieSceneComponentTypeInfo.h"
#include "EntitySystem/MovieSceneEntityFactory.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "HAL/Platform.h"
#include "Misc/GeneratedTypeName.h"

class UScriptStruct;

namespace UE
{
namespace MovieScene
{

template<typename> struct TPropertyComponents;

struct FNewComponentTypeParams
{
	FNewComponentTypeParams()
		: ReferenceCollectionCallback(nullptr)
		, Flags(EComponentTypeFlags::None)
	{}

	/** Implicit construction from type flags to support legacy API */
	FNewComponentTypeParams(EComponentTypeFlags InFlags)
		: ReferenceCollectionCallback(nullptr)
		, Flags(InFlags)
	{}

	explicit FNewComponentTypeParams(FComponentReferenceCollectionPtr RefCollectionPtr, EComponentTypeFlags InFlags)
		: ReferenceCollectionCallback(RefCollectionPtr)
		, Flags(InFlags)
	{}

	FComponentReferenceCollectionPtr ReferenceCollectionCallback;
	EComponentTypeFlags Flags;
};

struct FComponentRegistry
{
public:
	FEntityFactories Factories;

	FComponentRegistry() = default;

	FComponentRegistry(const FComponentRegistry&) = delete;
	FComponentRegistry& operator=(const FComponentRegistry&) = delete;

	FComponentRegistry(FComponentRegistry&&) = delete;
	FComponentRegistry& operator=(FComponentRegistry&&) = delete;

public:

	/**
	 * Define a new tag type using the specified information. Tags have 0 memory overhead.
	 * @note Transitory tag types must be unregistered when no longer required by calling DestroyComponentTypeSafe or Unsafe to prevent leaking component type IDs
	 *
	 * @param DebugName      A developer friendly name that accompanies this component type for debugging purposes
	 * @param Flags          Flags relating to the new component type
	 * @return A new component type identifier for the tag
	 */
	MOVIESCENE_API FComponentTypeID NewTag(const TCHAR* const DebugName, EComponentTypeFlags Flags = EComponentTypeFlags::None);


	/**
	 * Define a new transient tag type using the specified information. Tags have 0 memory overhead.
	 * @note Transitory tag types must be unregistered when no longer required by calling DestroyComponentTypeSafe or Unsafe to prevent leaking component type IDs
	 *
	 * @param DebugName      A developer friendly name that accompanies this component type for debugging purposes
	 * @param Params         (Optional) Parameters for the type including component flags
	 * @return A new component type identifier for the tag
	 */
	template<typename T>
	TComponentTypeID<T> NewComponentType(const TCHAR* const DebugName, const FNewComponentTypeParams& Params = FNewComponentTypeParams());

	/**
	 * Same as NewComponentType but specifically does not expose the component type to the reference graph. Use with caution!
	 */
	template<typename T>
	TComponentTypeID<T> NewComponentTypeNoAddReferencedObjects(const TCHAR* const DebugName, const FNewComponentTypeParams& Params = FNewComponentTypeParams());

	template<typename T>
	void NewComponentType(TComponentTypeID<T>* Ref, const TCHAR* const DebugName, const FNewComponentTypeParams& Params = FNewComponentTypeParams())
	{
		*Ref = NewComponentType<T>(DebugName, Params);
	}

	template<typename T>
	void NewComponentTypeNoAddReferencedObjects(TComponentTypeID<T>* Ref, const TCHAR* const DebugName, const FNewComponentTypeParams& Params = FNewComponentTypeParams())
	{
		*Ref = NewComponentTypeNoAddReferencedObjects<T>(DebugName, Params);
	}

	template<typename PropertyTraits>
	void NewPropertyType(TPropertyComponents<PropertyTraits>& OutComponents, const TCHAR* DebugName)
	{
#if UE_MOVIESCENE_ENTITY_DEBUG
		FString InitialValueDebugName = FString("Initial ") + DebugName;

		OutComponents.PropertyTag = NewTag(DebugName, EComponentTypeFlags::CopyToChildren | EComponentTypeFlags::CopyToOutput);
		NewComponentType(&OutComponents.InitialValue, *InitialValueDebugName, EComponentTypeFlags::Preserved | EComponentTypeFlags::MigrateToOutput);
#else
		OutComponents.PropertyTag = NewTag(nullptr, EComponentTypeFlags::CopyToChildren | EComponentTypeFlags::CopyToOutput);
		NewComponentType(&OutComponents.InitialValue, nullptr, EComponentTypeFlags::Preserved | EComponentTypeFlags::MigrateToOutput);
#endif
	}

	MOVIESCENE_API const FComponentTypeInfo& GetComponentTypeChecked(FComponentTypeID ComponentTypeID) const;

public:


	/**
	 * Destroy a component type by first removing it from all existing entities
	 * @note Will not invalidate any cached FComponentTypeID or TComponentTypeID structures
	 *
	 * @param ComponentTypeID The component type to destroy
	 */
	MOVIESCENE_API void DestroyComponentTypeSafe(FComponentTypeID ComponentTypeID);


	/**
	 * Destroy a component type that definitely does not exist on any entities or is cached elsewhere
	 * @note Will not invalidate any cached FComponentTypeID or TComponentTypeID structures
	 *
	 * @param ComponentTypeID The component type to destroy
	 */
	MOVIESCENE_API void DestroyComponentUnsafeFast(FComponentTypeID ComponentTypeID);

public:

	/**
	 * Retrieve a mask of all data component types (ie all components that are not tags).
	 */
	const FComponentMask& GetDataComponentTypes() const
	{
		return NonTagComponentMask;
	}

	/**
	 * Retrieve a mask of all components that are to be preserved
	 */
	const FComponentMask& GetPreservationMask() const
	{
		return PreservationMask;
	}

	/**
	 * Retrieve a mask of all components that are to be migrated to outputs if there are multiple entities animating the same thing
	 */
	const FComponentMask& GetMigrationMask() const
	{
		return MigrationMask;
	}

	/**
	 * Retrive a mask of all components to be copied or migrated to outputs
	 */
	const FComponentMask& GetCopyAndMigrationMask() const
	{
		return CopyAndMigrationMask;
	}

private:

	MOVIESCENE_API FComponentTypeID NewComponentTypeInternal(FComponentTypeInfo&& TypeInfo);

	template<typename T>
	FComponentTypeInfo MakeComponentTypeInfoWithoutComponentOps(const TCHAR* const DebugName, const FNewComponentTypeParams& Params);

private:

	TSparseArray<FComponentTypeInfo> ComponentTypes;
	TSparseArray<UScriptStruct*>     ComponentStructs;

	/** A component mask for all component types that are NOT tags, cached and updated when ComponentTypes is modified. */
	FComponentMask NonTagComponentMask;

	/** Mask containing all components that have the flag EComponentTypeFlags::Preserved */
	FComponentMask PreservationMask;

	/** Mask containing all components that have the flag EComponentTypeFlags::MigrateToOutput */
	FComponentMask MigrationMask;

	/** Mask containing all components that have the EComponentTypeFlags::CopyToOutput and EComponentTypeFlags::MigrateToOutput */
	FComponentMask CopyAndMigrationMask;
};


} // namespace MovieScene
} // namespace UE
