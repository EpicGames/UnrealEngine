// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

struct FGuid;
class FArchive;
class UScriptStruct;

namespace UE
{
namespace MovieScene
{

struct FComponentTypeID;
class FEntityManager;

/**
 * Interface for defining new registered component types within an FEntityManager
 */
struct IComponentTypeHandler
{
	virtual ~IComponentTypeHandler()= default;


	/**
	 * Called when this component type does not yet exist in an entity manager, and needs to be defined.
	 * Most implementations will call some variation of EntityManager->NewComponentType
	 *
	 * @param ComponentGuid The guid that this type handler was registered with
	 * @param EntityManager The entity manager to create the component type within
	 */
	virtual FComponentTypeID InitializeComponentType(FEntityManager* EntityManager) = 0;

	virtual UScriptStruct* GetScriptStruct() const { return nullptr; }
};


} // namespace MovieScene
} // namespace UE
