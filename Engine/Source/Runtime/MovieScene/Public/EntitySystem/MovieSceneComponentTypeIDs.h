// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntityIDs.h"
#include "Delegates/IntegerSequence.h"
#include "Containers/ArrayView.h"

namespace UE::MovieScene
{

namespace Private
{
	template<typename...>
	struct TComponentTypeIDsImpl;

	template <typename T, int Index>
	struct TComponentGetterElement{};

	template <int Index>
	struct TComponentGetter
	{
		template<typename Deduced>
		static FORCEINLINE TComponentTypeID<Deduced> GetImpl(const TComponentGetterElement<Deduced, Index>&, TArrayView<const FComponentTypeID> Values)
		{
			return Values[Index].template ReinterpretCast<Deduced>();
		}

		template<typename DerivedType>
		static FORCEINLINE auto Get(const DerivedType& In)
		{
			return GetImpl(In, In.GetTypes());
		}
	};

	template<>
	struct TComponentTypeIDsImpl<TIntegerSequence<int>>
	{
		void Initialize()
		{
		}
		template<int Index>
		FORCEINLINE void GetType() const
		{
		}
		FORCEINLINE static TArrayView<const FComponentTypeID> GetTypes()
		{
			return TArrayView<const FComponentTypeID>();
		}
	};


	template<typename ...T, int... Indices>
	struct TComponentTypeIDsImpl<TIntegerSequence<int, Indices...>, T...> : Private::TComponentGetterElement<T, Indices>...
	{
		template<int Index>
		FORCEINLINE auto GetType() const
		{
			static_assert(Index < sizeof...(Indices), "Unable to retrieve a component type for an invalid index");
			return Private::TComponentGetter<Index>::Get(*this);
		}

		FORCEINLINE FComponentTypeID GetType(int Index) const
		{
			return Values[Index];
		}
		TArrayView<const FComponentTypeID> GetTypes() const
		{
			return MakeArrayView(Values);
		}
		void Initialize(TComponentTypeID<T>... InTypes)
		{
			(..., (Values[Indices] = InTypes));
		}

	protected:

		static constexpr int32 SIZE = sizeof...(T);
		FComponentTypeID Values[SIZE];
	};

} // namespace Private

template<typename ...T>
using TComponentTypeIDs = Private::TComponentTypeIDsImpl<TMakeIntegerSequence<int, sizeof...(T)>, T...>;


} // namespace UE::MovieScene


