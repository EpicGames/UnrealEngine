// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/UniquePtr.h"
#include "Templates/MemoryOps.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "EntitySystem/MovieSceneComponentDebug.h"
#include "EntitySystem/MovieSceneEntityIDs.h"

struct FObjectKey;
template <typename> class TObjectKey;
template <typename> struct TObjectPtr;

namespace UE
{
namespace MovieScene
{

struct FNotImplemented;
struct FComponentTypeDebugInfo;

using FComponentReferenceCollectionPtr = void (*)(FReferenceCollector&, void*, int32);

/**
 * Stub for components that do not need reference collection. Overload void AddReferencedObjectForComponent(FReferenceCollector* ReferenceCollector, T* Component) for your own implementation
 * NOTE: The return value is _very_ important - overloaded AddReferencedObjectForComponent functions should return void otherwise they will not get called by the entity manager.
 */
inline FNotImplemented* AddReferencedObjectForComponent(...) { return nullptr; }

/** Instantiation for raw pointers */
template<typename T>
typename TEnableIf<TPointerIsConvertibleFromTo<T, UObject>::Value>::Type AddReferencedObjectForComponent(FReferenceCollector* ReferenceCollector, T** Component)
{
	// Ideally this would be a compile-time check buyt 
	constexpr bool bTypeDependentFalse = !std::is_same_v<T, T>;
	static_assert(bTypeDependentFalse, "Raw object pointers are no longer supported. Please use TObjectPtr<T> instead.");
}

template<typename T>
void AddReferencedObjectForComponent(FReferenceCollector* ReferenceCollector, TObjectPtr<T>* Component)
{
	ReferenceCollector->AddReferencedObject(*Component);
}

// Hack to enable garbage collection path for object keys
inline void AddReferencedObjectForComponent(FReferenceCollector* ReferenceCollector, FObjectKey* Component)
{}

template<typename T>
void AddReferencedObjectForComponent(FReferenceCollector* ReferenceCollector, TObjectKey<T>* Component)
{}

template<typename T, typename U = decltype(T::StaticStruct())>
void AddReferencedObjectForComponent(FReferenceCollector* ReferenceCollector, T* Component)
{
	for (TPropertyValueIterator<const FObjectProperty> It(T::StaticStruct(), Component); It; ++It)
	{
		ReferenceCollector->AddReferencedObject(It.Key()->GetObjectPtrPropertyValueRef(It.Value()));
	}
}

template<typename T>
struct THasAddReferencedObjectForComponent
{
	static constexpr bool Value = !std::is_same_v< FNotImplemented*, decltype( AddReferencedObjectForComponent((FReferenceCollector*)0, (T*)0) ) >;
};

/**
 * Class that is instantiated to specify custom behavior for interacting with component data
 */
struct IComplexComponentOps
{
	virtual ~IComplexComponentOps() = default;
	virtual void ConstructItems(void* Dest, int32 Num) const = 0;
	virtual void DestructItems(void* Dest, int32 Num) const = 0;
	virtual void CopyItems(void* Dest, const void* Src, int32 Num) const = 0;
	virtual void RelocateConstructItems(void* Dest, void* Src, int32 Num) const = 0;
	virtual void AddReferencedObjects(FReferenceCollector& ReferenceCollector, void* ComponentStart, int32 Num) = 0;
};


/**
 * Complete type information for a component within an FEntityManager
 * This structure defines how to interact with the component data for operations such as
 * copying, relocating, initializing, destructing and reference collection.
 */
struct FComponentTypeInfo
{
	FComponentTypeInfo()
		: bIsZeroConstructType(0)
		, bIsTriviallyDestructable(0)
		, bIsTriviallyCopyAssignable(0)
		, bIsPreserved(0)
		, bIsCopiedToOutput(0)
		, bIsMigratedToOutput(0)
		, bHasReferencedObjects(0)
	{}

	/** 16 bytes - Custom native definition for non-POD types */
	mutable TUniquePtr<IComplexComponentOps> ComplexComponentOps;

#if UE_MOVIESCENE_ENTITY_DEBUG
	/** 8 Bytes - Debugging information primarily for natviz support*/
	TUniquePtr<FComponentTypeDebugInfo> DebugInfo;
#endif

	/** 1 byte - The size of the component in bytes */
	uint8 Sizeof;
	/** 1 byte - The required alignment of the component in bytes */
	uint8 Alignment;
	/** 1 byte - Type flags for the component type  */
	uint8 bIsZeroConstructType : 1;       // Whether TIsZeroConstructType<T> is true
	uint8 bIsTriviallyDestructable : 1;   // Whether std::is_trivially_destructible_v<T> is true
	uint8 bIsTriviallyCopyAssignable : 1; // Whether TIsTriviallyCopyAssignable<T> is true
	uint8 bIsPreserved : 1;               // Whether this component should be preserved when an entity containing it is replaced or overwritten
	uint8 bIsCopiedToOutput : 1;          // Whether this component should be copied to an output entity if there are now multiple contributors to the same property/state
	uint8 bIsMigratedToOutput : 1;        // Whether this component should be migrated to an output entity if there are now multiple contributors to the same property/state
	uint8 bHasReferencedObjects : 1;      // Whether this component contains any data that should be reference collected

	/**
	 * Whether this component type describes a tag, i.e. a component with no data.
	 */
	bool IsTag() const { return Sizeof == 0; }

	/**
	 * Construct a contiguous array of components
	 */
	void ConstructItems(void* Components, int32 Num) const
	{
		if (bIsZeroConstructType)
		{
			FMemory::Memset(Components, 0, Sizeof * Num);
		}
		else
		{
			checkSlow(ComplexComponentOps.IsValid());
			ComplexComponentOps->ConstructItems(Components, Num);
		}
	}

	/**
	 * Destruct a contiguous array of components
	 */
	void DestructItems(void* Components, int32 Num) const
	{
		if (!bIsTriviallyDestructable)
		{
			checkSlow(ComplexComponentOps.IsValid());
			ComplexComponentOps->DestructItems(Components, Num);
		}
	}

	/**
	 * Copy a contiguous array of components
	 */
	void CopyItems(void* Dest, const void* Source, int32 Num) const
	{
		if (bIsTriviallyCopyAssignable)
		{
			FMemory::Memcpy(Dest, Source, Sizeof * Num);
		}
		else
		{
			checkSlow(ComplexComponentOps.IsValid());
			ComplexComponentOps->CopyItems(Dest, Source, Num);
		}
	}

	/**
	 * Copy a contiguous array of components
	 */
	void RelocateConstructItems(void* Dest, void* Source, int32 Num) const
	{
		if (bIsTriviallyCopyAssignable && bIsTriviallyDestructable)
		{
			FMemory::Memmove(Dest, Source, Sizeof * Num);
		}
		else
		{
			checkSlow(ComplexComponentOps.IsValid());
			ComplexComponentOps->RelocateConstructItems(Dest, Source, Num);
		}
	}

	/**
	 * Reference collect a contiguous array of components
	 */
	void AddReferencedObjects(FReferenceCollector& ReferenceCollector, void* ComponentStart, int32 Num) const
	{
		if (ComplexComponentOps.IsValid())
		{
			ComplexComponentOps->AddReferencedObjects(ReferenceCollector, ComponentStart, Num);
		}
	}

	/**
	 * Define complex component operations for this type of component
	 */
	template<typename T>
	void MakeComplexComponentOps()
	{
		ComplexComponentOps = MakeUnique<TComplexComponentOps<T>>();
	}

	/**
	 * Define complex component operations for this type of component without implementing AddReferencedObjects - use with caution!
	 */
	template<typename T>
	void MakeComplexComponentOpsNoAddReferencedObjects()
	{
		ComplexComponentOps = MakeUnique<TComplexComponentOpsBase<T>>();
	}

	/**
	 * Define complex component with a specific reference collection callback
	 */
	template<typename T>
	void MakeComplexComponentOps(FComponentReferenceCollectionPtr RefCollectionPtr)
	{
		ComplexComponentOps = MakeUnique<TComplexComponentOpsCustomRefCollection<T>>(RefCollectionPtr);
	}

private:

	template<typename T>
	struct TComplexComponentOpsBase : IComplexComponentOps
	{
		virtual void ConstructItems(void* Components, int32 Num) const
		{
			T* TypedComponents = static_cast<T*>(Components);
			::DefaultConstructItems<T>(TypedComponents, Num);
		}
		virtual void DestructItems(void* Components, int32 Num) const
		{
			T* TypedComponents = static_cast<T*>(Components);
			::DestructItems<T>(TypedComponents, Num);
		}
		virtual void CopyItems(void* Dest, const void* Source, int32 Num) const
		{
			T* TypedDst = static_cast<T*>(Dest);
			const T* TypedSrc = static_cast<const T*>(Source);

			while (Num-- > 0)
			{
				(*TypedDst++) = (*TypedSrc++);
			}
		}
		virtual void RelocateConstructItems(void* Dest, void* Source, int32 Num) const
		{
			typedef T RelocateItemsElementTypeTypedef;

			T* TypedDst = static_cast<T*>(Dest);
			T* TypedSrc = static_cast<T*>(Source);

			while (Num-- > 0)
			{
				new (TypedDst) T(MoveTemp(*TypedSrc));
				TypedSrc->RelocateItemsElementTypeTypedef::~RelocateItemsElementTypeTypedef();

				++TypedDst;
				++TypedSrc;
			}
		}
		virtual void AddReferencedObjects(FReferenceCollector& ReferenceCollector, void* ComponentStart, int32 Num)
		{}
	};
	template<typename T>
	struct TComplexComponentOps : TComplexComponentOpsBase<T>
	{
		virtual void AddReferencedObjects(FReferenceCollector& ReferenceCollector, void* ComponentStart, int32 Num)
		{
			T* ComponentData = static_cast<T*>(ComponentStart);
			while (Num-- > 0)
			{
				AddReferencedObjectForComponent(&ReferenceCollector, ComponentData);

				++ComponentData;
			}
		}
	};
	template<typename T>
	struct TComplexComponentOpsCustomRefCollection : TComplexComponentOpsBase<T>
	{
		FComponentReferenceCollectionPtr RefCollectionPtr;
		TComplexComponentOpsCustomRefCollection(FComponentReferenceCollectionPtr InRefCollectionPtr)
			: RefCollectionPtr(InRefCollectionPtr)
		{}

		virtual void AddReferencedObjects(FReferenceCollector& ReferenceCollector, void* ComponentStart, int32 Num)
		{
			(*RefCollectionPtr)(ReferenceCollector, ComponentStart, Num);
		}
	};
};



} // namespace MovieScene
} // namespace UE
