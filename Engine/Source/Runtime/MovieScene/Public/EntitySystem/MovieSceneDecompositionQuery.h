// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneIntermediatePropertyValue.h"
#include "HAL/Platform.h"
#include "Templates/Tuple.h"
#include "Templates/TypeCompatibleBytes.h"
#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneDecompositionQuery.generated.h"

class UObject;

namespace UE
{
namespace MovieScene
{

	/**
	 * Structure used to decompose the blended result of multiple components.
	 *
	 * Defines an object to query, and the entities that should have their pre-blended component values extracted
	 */
	struct FDecompositionQuery
	{
		/** Defines all the entities that should have their pre-component values extracted for recomposition */
		TArrayView<const FMovieSceneEntityID> Entities;

		/** Whether the entities above are source entities or runtime entities */
		bool bConvertFromSourceEntityIDs = true;

		/** The object that is being decomposed */
		UObject* Object = nullptr;
	};

	/** Holds a weighted value */
	struct FWeightedValue
	{
		/** The raw value */
		double Value = 0.f;
		/** The weight of the value */
		float Weight = 0.f;
		/** The base value, which should only be set when the value has additive-from-base blend type */
		double BaseValue = 0.f;
		/** Optional Blending Order*/
		int32 BlendingOrder = INDEX_NONE;
		/** if BlendingOrder set, whether additive or override*/
		bool bIsAdditive = true;
		/** Multiplies the value by the weight */
		double Get() const
		{
			return (Value - BaseValue) * Weight;
		}
		bool operator<(const FWeightedValue& RHS) const
		{
			return BlendingOrder < RHS.BlendingOrder;
		}
	};

	/** Holds an accumulation of weighted values */
	struct FAccumulatedWeightedValue
	{
		/** Accumulated value */
		double Total = 0.f;
		/** The total weight of all accumulated values */
		float TotalWeight = 0.f;

		/** Normalizes the total value by the total weight */
		FORCENOINLINE double Normalize() const
		{
			return TotalWeight != 0.f ? Total / TotalWeight : Total;
		}

		/** Accumulates more values into this value */
		FORCENOINLINE FAccumulatedWeightedValue& AccumulateThis(const FWeightedValue& Other)
		{
			Total += Other.Get();
			TotalWeight += Other.Weight;
			return *this;
		}
		/** Accumulates more values into this value */
		FORCENOINLINE FAccumulatedWeightedValue& AccumulateThis(const FAccumulatedWeightedValue& Other)
		{
			Total += Other.Total;
			TotalWeight += Other.TotalWeight;
			return *this;
		}
		/** Accumulates values */
		FORCENOINLINE FAccumulatedWeightedValue Accumulate(const FWeightedValue& Other) const
		{
			return FAccumulatedWeightedValue{ Total + Other.Get(), TotalWeight + Other.Weight };
		}
		/** Accumulates values */
		FORCENOINLINE FAccumulatedWeightedValue Accumulate(const FAccumulatedWeightedValue& Other) const
		{
			return FAccumulatedWeightedValue{ Total + Other.Total, TotalWeight + Other.TotalWeight };
		}
	};

	/**
	 * A structure for holding all the values of channels contributing to an animated property.
	 *
	 * Some of these channels are "decomposed", i.e. their values are set aside and matched to the channel entity that
	 * produced them. The other channels are mixed together in a non-decomposed result.
	 */
	struct FDecomposedValue
	{
		struct FResult
		{
			/** Accumulated absolute values */
			FAccumulatedWeightedValue Absolute;

			/** Accumulated additive values (already weighted) */
			double Additive = 0.f;
		};

		/** The absolute and additive values for all the non-decomposed channels */
		FResult Result;

		/** values for non-decomposed overides, we leave it decomposed */
		mutable TArray <FWeightedValue> AllDecomposedOverrides;
		/**
		 * Decomposed values for channels we're interested in. Note that decomposed additives-from-base are lumped with
		 * decomposed additives
		 */
		TArray<TTuple<FMovieSceneEntityID, FWeightedValue>> DecomposedAbsolutes;
		TArray<TTuple<FMovieSceneEntityID, FWeightedValue>> DecomposedAdditives;
		TArray<TTuple<FMovieSceneEntityID, FWeightedValue>> DecomposedOverrides;


		/**
		 * Get the value that the channel behind the given entity should have in order for the combined blended values
		 * of all known channels to produce the desired current value.
		 *
		 * @param EntityID The entity matching the channel we want to determine the value of
		 * @param CurrentValue The desired current value for the overall property animated by the channel (among other
		 *					   channels)
		 * @param InitialValue An optional pointer to the initial value of the property
		 * @return The value that the entity's channel should have to produce the desired current value
		 */
		MOVIESCENE_API double Recompose(FMovieSceneEntityID EntityID, double CurrentValue, const double* InitialValue) const;

	private:
		enum class EDecomposedValueBlendType
		{
			Absolute,
			Additive,
			AdditiveFromBase,
			Override,
		};

		/**
		 * The current structure keeps track of more than one channel/entity we're interested in. This method will
		 * reduce it to just one particular channel/entity, throwing the other ones in the per-blend-type outputs.
		 */
		void Decompose(
				FMovieSceneEntityID EntityID, 
				FWeightedValue& ThisValue, EDecomposedValueBlendType& OutBlendType, 
				FAccumulatedWeightedValue& Absolutes, 
				FAccumulatedWeightedValue& Additives) const;
	};

	// Align results to cache lines so there's no contention between cores
	struct MS_ALIGN(PLATFORM_CACHE_LINE_SIZE) FAlignedDecomposedValue
	{
		FDecomposedValue Value;
	} GCC_ALIGN(PLATFORM_CACHE_LINE_SIZE);

	struct FValueDecompositionParams
	{
		FDecompositionQuery Query;
		uint16 DecomposeBlendChannel;
		FMovieSceneEntityID PropertyEntityID;
		FComponentTypeID ResultComponentType;
		FComponentTypeID PropertyTag;
	};

	template<typename PropertyType>
	struct TRecompositionResult
	{
		TRecompositionResult(const PropertyType& InCurrentValue, int32 Num)
		{
			while (--Num >= 0)
			{
				Values.Add(InCurrentValue);
			}
		}

		TArray<PropertyType, TInlineAllocator<1>> Values;
	};

	struct FRecompositionResult
	{
		TArray<FIntermediatePropertyValue, TInlineAllocator<1>> Values;
	};

} // namespace MovieScene
} // namespace UE


UINTERFACE(MinimalAPI)
class UMovieSceneValueDecomposer : public UInterface
{
public:
	GENERATED_BODY()
};

class IMovieSceneValueDecomposer
{
public:
	GENERATED_BODY()

	virtual FGraphEventRef DispatchDecomposeTask(const UE::MovieScene::FValueDecompositionParams& Params, UE::MovieScene::FAlignedDecomposedValue* Output) = 0;
};
