// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/MovieSceneComponentRegistry.h"
#include "EntitySystem/MovieSceneEntityFactory.h"
#include "EntitySystem/MovieSceneMutualComponentInclusivity.h"
#include "Delegates/IntegerSequence.h"

#include <initializer_list>


namespace UE
{
namespace MovieScene
{

struct FAdd;
struct FAddMany;
struct FAddConditional;

template<typename T> struct TAdd;
template<typename T> struct TAddConditional;

template <typename... T> struct TEntityBuilder;
template <typename... T> struct TEntityBuilderImpl;

/**
 * Specifies a mask of components to add and remove from an entity
 */
struct FTypelessMutation
{
	FComponentMask AddMask;
	FComponentMask RemoveMask;

	bool bRemoveAll = false;

	/**
	 * Direct this mutation to add the specified component types to an entity
	 */
	FTypelessMutation& Add(std::initializer_list<FComponentTypeID> TypeIDs)
	{
		AddMask.SetAll(TypeIDs);
		return *this;
	}

	/**
	 * Direct this mutation to remove the specified component types to an entity
	 */
	FTypelessMutation& Remove(std::initializer_list<FComponentTypeID> TypeIDs)
	{
		RemoveMask.SetAll(TypeIDs);
		return *this;
	}

	/**
	 * Direct this mutation to remove all components from the entity
	 */
	FTypelessMutation& RemoveAll()
	{
		bRemoveAll = true;
		return *this;
	}

	/**
	 * Combine our masks into the specified pre-existing mask
	 */
	FComponentMask MutateType(const FComponentMask& Current) const;
};


struct IEntityBuilder
{
	virtual ~IEntityBuilder() = default;

	virtual FMovieSceneEntityID Create(FEntityManager* EntityManager) = 0;
	virtual void GenerateType(FEntityManager* EntityManager, FComponentMask& OutMask, bool& OutAddMutualComponents) = 0;
	virtual void Initialize(FEntityManager* EntityManager, const FEntityInfo& EntityInfo) = 0;
};


/**
 * TEntityBuilder is a utility class that can be used to marshal type-safe component data into entites, either on construction or mutation of an existing entity.
 * It is a general purpose utility that should not be used for high-performance code, but is useful for one-off changes to entity component structures
 *
 * In general, this type is intended to be used declaratively, and will forward its state to the final function call (to prevent a copy of the payload data)
 *
 * Example usage:
 *
 * TComponentTypeID<float> FloatComponent1 = ...;
 * TComponentTypeID<float> FloatComponent2 = ...;
 * TComponentTypeID<FMovieSceneFloatChannel> FloatChannel = ...;
 *
 * FMovieSceneFloatChannel NewChannelType;
 * NewChannelType.SetDefault(1.f);
 *
 * auto EntityBuilder = FEntityBuilder()
 *     .AddDefaulted(FloatComponent1)
 *     .AddDefaulted(FloatComponent2)
 *     .Add(FloatChannel, MoveTemp(NewChannelType));
 *
 * // Create a new entity with 2 defaulted floats, and a float channel with a default of 1.f
 * FMovieSceneEntityID NewEntity = CopyTemp(EntityBuilder).CreateEntity(EntityManager);
 *
 * // Add the data to an existing entity - will invalidate the entity builder
 * EntityBuilder.MutateExisting(EntityManager, Existing);
 */
template<typename... T>
struct TEntityBuilder : TEntityBuilderImpl<TMakeIntegerSequence<int, sizeof...(T)>, T...>
{
	using Super = TEntityBuilderImpl<TMakeIntegerSequence<int, sizeof...(T)>, T...>;

	TEntityBuilder()
	{
		static_assert(sizeof...(T) == 0, "Default construction is only supported for TEntityBuilder<>");
	}

	template<typename... Args>
	TEntityBuilder(Args&&... InTypes)
		: Super(Forward<Args>(InTypes)...)
	{}
};


template<>
struct TEntityBuilderImpl<TIntegerSequence<int>>
{
	/**
	 * Add the specified default-constructed component type to the entity
	 * 
	 * @param ComponentType  A valid component type or tag ID to add. Must be valid.
	 * @return A new builder that includes the new component
	 */
	TEntityBuilder< FAdd > AddDefaulted(FComponentTypeID ComponentType);

	/**
	 * Add all the specified default-constructed component type to the entity
	 * 
	 * @param InComponentsToAdd  A (possibly empty) component mask that defines all the components to add
	 * @return A new builder that includes the new component
	 */
	TEntityBuilder< FAddMany > AddMany(const FComponentMask& InComponentsToAdd);

	/**
	 * Add the specified tag to the entity. Equivalent to AddDefaulted.
	 * 
	 * @param Tag  A valid component tag ID to add. Must be valid.
	 * @return A new builder that includes the new component
	 */
	TEntityBuilder< FAdd > AddTag(FComponentTypeID TagType);

	/**
	 * Add a component to the entity with a specific value
	 * 
	 * @param ComponentType  The component type ID to add to the entity. Must be valid.
	 * @param InPayload      User-specified data to forward into the component
	 * @return A new builder that includes the new component and payload
	 */
	template<typename U, typename PayloadType>
	TEntityBuilder< TAdd<U> > Add(TComponentTypeID<U> ComponentType, PayloadType&& InPayload);

	/**
	 * Conditionally add a component to the entity with a specific value
	 * 
	 * @param ComponentType  The component type ID to add to the entity. Must be valid if bCondition is true.
	 * @param InPayload      User-specified data to forward into the component
	 * @param bCondition     Condition specifying whether this component should be added or not
	 * @return A new builder that includes the new component and payload
	 */
	template<typename U, typename PayloadType>
	TEntityBuilder< TAddConditional<U> > AddConditional(TComponentTypeID<U> ComponentType, PayloadType&& InPayload, bool bCondition);

	/**
	 * Add the specified default-constructed component to the entity if a condition is met
	 * 
	 * @param ComponentType  A valid component type ID to add. Must be valid.
	 * @param bCondition     Condition specifying whether this tag should be added or not
	 * @return A new builder that includes the new component
	 */
	TEntityBuilder< FAddConditional > AddDefaultedConditional(FComponentTypeID ComponentType, bool bCondition);

	/**
	 * Add the specified tag to the entity if a condition is met
	 * 
	 * @param Tag         A valid component tag ID to add. Must be valid.
	 * @param bCondition  Condition specifying whether this tag should be added or not
	 * @return A new builder that includes the new component
	 */
	TEntityBuilder< FAddConditional > AddTagConditional(FComponentTypeID TagType, bool bCondition);

	/**
	 * Append another component type to this builder
	 *
	 * @return A new builder that includes the new component
	 */
	template<typename U>
	TEntityBuilder< U > Append(U&& InOther);
};

struct FEntityBuilderImplBase : public IEntityBuilder
{
	/**
	 * Create a new entity using this builder's definition by moving the payload components into the new entity.
	 * @note Will invalidate this instance of TEntityBuilder so its payload cannot be used again.
	 *
	 * @param EntityManager The entity manager to create the entity within. All component types *must* relate to this class.
	 * @param NewType       (Optional) An additional base type to use for the new entity. Any component types not stored by this builder will be default-constructed.
	 * @return The created entity's ID
	 */
	FMovieSceneEntityID CreateEntity(FEntityManager* EntityManager, FComponentMask NewType = FComponentMask())
	{
		bool bLocalAddMutualComponents = false;
		GenerateType(EntityManager, NewType, bLocalAddMutualComponents);

		FMutualComponentInitializers MutualInitializers;
		FEntityAllocationWriteContext WriteContext(*EntityManager);
		EMutuallyInclusiveComponentType MutualTypes = bLocalAddMutualComponents ? EMutuallyInclusiveComponentType::All : EMutuallyInclusiveComponentType::Mandatory;

		EntityManager->GetComponents()->Factories.ComputeMutuallyInclusiveComponents(MutualTypes, NewType, MutualInitializers);

		FEntityInfo Entry = EntityManager->AllocateEntity(NewType);
		Initialize(EntityManager, Entry);

		// Run mutual initializers after the builder has actually constructed the entity
		// otherwise mutual components would be reading garbage
		MutualInitializers.Execute(Entry.Data.AsRange(), WriteContext);

		return Entry.EntityID;
	}
};

template <typename... T, int... Indices>
struct TEntityBuilderImpl<TIntegerSequence<int, Indices...>, T...> : FEntityBuilderImplBase
{
	virtual FMovieSceneEntityID Create(FEntityManager* EntityManager) override final
	{
		return CreateEntity(EntityManager);
	}

	virtual void GenerateType(FEntityManager* EntityManager, FComponentMask& OutMask, bool& OutAddMutualComponents) override final
	{
		VisitTupleElements([&OutMask](auto& In){ In.AccumulateMask(OutMask); }, this->Payload);

		if (bAddMutualComponents)
		{
			OutAddMutualComponents = true;
		}
	}

	virtual void Initialize(FEntityManager* EntityManager, const FEntityInfo& Entity) override final
	{
		if (Entity.Data.Allocation != nullptr)
		{
			VisitTupleElements([Entity](auto& In){ In.Apply(Entity.Data.Allocation, Entity.Data.ComponentOffset); }, this->Payload);
		}
	}

	using FEntityBuilderImplBase::CreateEntity;

	/**
	 * Replace the components of an entity with this builder's definition.
	 * @note Will invalidate this instance of TEntityBuilder so its payload cannot be used again.
	 *
	 * @param EntityManager The entity manager that houses EntityID. All component types *must* relate to this class.
	 * @param EntityID      The entity ID to replace
	 * @param NewType       (Optional) An additional base type to use for the new entity. Any component types not stored by this builder will be default-constructed.
	 */
	void ReplaceEntity(FEntityManager* EntityManager, FMovieSceneEntityID& InOutEntityID, FComponentMask NewType = FComponentMask())
	{
		FMovieSceneEntityID NewEntityID = CreateEntity(EntityManager, NewType);
		EntityManager->ReplaceEntityID(InOutEntityID, NewEntityID);
	}


	/**
	 * Mutate an existing entity using this instance's payload and an additional mask of components.
	 * @note Will invalidate this instance of TEntityBuilder so its payload cannot be used again.
	 *
	 * @param EntityManager The entity manager that houses EntityID. All component types *must* relate to this class.
	 * @param EntityID      The entity to mutate
	 * @param Base          (Optional) An additional base mutation to apply while modifying this entity
	 */
	void MutateExisting(FEntityManager* EntityManager, FMovieSceneEntityID EntityID, const FTypelessMutation& Base = FTypelessMutation())
	{
		FComponentMask OldMask = EntityManager->GetEntityType(EntityID);
		FComponentMask NewMask = Base.MutateType(OldMask);

		VisitTupleElements([&NewMask](auto& In){ In.AccumulateMask(NewMask); }, this->Payload);

		FMutualComponentInitializers MutualInitializers;
		FEntityAllocationWriteContext WriteContext(*EntityManager);

		EMutuallyInclusiveComponentType MutualTypes = bAddMutualComponents ? EMutuallyInclusiveComponentType::All : EMutuallyInclusiveComponentType::Mandatory;
		EntityManager->GetComponents()->Factories.ComputeMutuallyInclusiveComponents(MutualTypes, NewMask, MutualInitializers);

		if (!NewMask.CompareSetBits(OldMask))
		{
			EntityManager->ChangeEntityType(EntityID, NewMask);
		}

		FEntityInfo Entry = EntityManager->GetEntity(EntityID);

		if (Entry.Data.Allocation != nullptr)
		{
			VisitTupleElements([Entry](auto& In){ In.Apply(Entry.Data.Allocation, Entry.Data.ComponentOffset); }, this->Payload);

			// Run mutual initializers after the builder has actually constructed the entity
			// otherwise mutual components would be reading garbage
			MutualInitializers.Execute(Entry.Data.AsRange(), WriteContext);
		}
	}

	/**
	 * Mutate an existing entity using this instance's payload and an additional mask of components.
	 * @note Will invalidate this instance of TEntityBuilder so its payload cannot be used again.
	 *
	 * @param EntityManager The entity manager that houses EntityID. All component types *must* relate to this class.
	 * @param EntityID      The entity to mutate
	 * @param Base          (Optional) An additional base mutation to apply while modifying this entity
	 */
	void CreateOrUpdate(FEntityManager* EntityManager, FMovieSceneEntityID& InOutEntityID, const FTypelessMutation& Base = FTypelessMutation().RemoveAll())
	{
		if (InOutEntityID)
		{
			MutateExisting(EntityManager, InOutEntityID, Base);
		}
		else
		{
			InOutEntityID = CreateEntity(EntityManager, Base.AddMask);
		}
	}

	/**
	 * Add the specified default-constructed component type to the entity
	 * 
	 * @param ComponentType  A valid component type or tag ID to add. Must be valid.
	 * @return A new builder that includes the new component
	 */
	TEntityBuilder<T..., FAdd > AddDefaulted(FComponentTypeID ComponentType);

	/**
	 * Add all the specified default-constructed component type to the entity
	 * 
	 * @param InComponentsToAdd  A (possibly empty) component mask that defines all the components to add
	 * @return A new builder that includes the new component
	 */
	TEntityBuilder<T..., FAddMany > AddMany(const FComponentMask& InComponentsToAdd);

	/**
	 * Add the specified tag to the entity. Equivalent to AddDefaulted.
	 * 
	 * @param Tag  A valid component tag ID to add. Must be valid.
	 * @return A new builder that includes the new component
	 */
	TEntityBuilder<T..., FAdd > AddTag(FComponentTypeID TagType);

	/**
	 * Add a component to the entity with a specific value
	 * 
	 * @param ComponentType  The component type ID to add to the entity. Must be valid.
	 * @param InPayload      User-specified data to forward into the component
	 * @return A new builder that includes the new component and payload
	 */
	template<typename U, typename PayloadType>
	TEntityBuilder<T..., TAdd<U> > Add(TComponentTypeID<U> ComponentType, PayloadType&& InPayload);

	/**
	 * Conditionally add a component to the entity with a specific value
	 * 
	 * @param ComponentType  The component type ID to add to the entity. Must be valid if bCondition is true.
	 * @param InPayload      User-specified data to forward into the component
	 * @param bCondition     Condition specifying whether this component should be added or not
	 * @return A new builder that includes the new component and payload
	 */
	template<typename U, typename PayloadType>
	TEntityBuilder<T..., TAddConditional<U> > AddConditional(TComponentTypeID<U> ComponentType, PayloadType&& InPayload, bool bCondition);

	/**
	 * Add the specified default-constructed component to the entity if a condition is met
	 * 
	 * @param ComponentType  A valid component type ID to add. Must be valid.
	 * @param bCondition     Condition specifying whether this tag should be added or not
	 * @return A new builder that includes the new component
	 */
	TEntityBuilder< T..., FAddConditional > AddDefaultedConditional(FComponentTypeID ComponentType, bool bCondition);

	/**
	 * Add the specified tag to the entity. Equivalent to AddDefaulted.
	 * 
	 * @param Tag         A valid component tag ID to add. Must be valid.
	 * @param bCondition  Condition specifying whether this tag should be added or not
	 * @return A new builder that includes the new component
	 */
	TEntityBuilder<T..., FAddConditional > AddTagConditional(FComponentTypeID TagType, bool bCondition);


	/**
	 * Append another component type to this builder
	 *
	 * @return A new builder that includes the new component
	 */
	template<typename U>
	TEntityBuilder<T..., U > Append(U&& InOther);


	/**
	 * Add any mutual components defined by the entity factory
	 */
	TEntityBuilder<T...> AddMutualComponents();

protected:

	TEntityBuilderImpl(T&&... InArgs, bool bInAddMutualComponents)
		: Payload(MoveTemp(InArgs)...)
		, bAddMutualComponents(bInAddMutualComponents)
	{}

	/** Payload data */
	TTuple< T... > Payload;

	bool bAddMutualComponents;
};


using FEntityBuilder = TEntityBuilder<>;


/** Implemtntation of an untyped add payload */
struct FAddMany
{
	FComponentMask BaseComponentMask;
	explicit FAddMany(const FComponentMask& InBaseComponentMask)
		: BaseComponentMask(InBaseComponentMask)
	{}

	void AccumulateMask(FComponentMask& OutMask) const
	{
		OutMask.CombineWithBitwiseOR(BaseComponentMask, EBitwiseOperatorFlags::MaxSize);
	}

	void Apply(FEntityAllocation* Allocation, int32 ComponentOffset)
	{
	}
};


/** Implemtntation of an untyped add payload */
struct FAdd
{
	FComponentTypeID ComponentTypeID;
	explicit FAdd(FComponentTypeID InComponentTypeID)
		: ComponentTypeID(InComponentTypeID)
	{}

	void AccumulateMask(FComponentMask& OutMask) const
	{
		OutMask.Set(ComponentTypeID);
	}

	void Apply(FEntityAllocation* Allocation, int32 ComponentOffset)
	{
	}
};

/** Implemtntation of a contitional untyped add payload */
struct FAddConditional
{
	FComponentTypeID ComponentTypeID;
	bool bCondition;

	explicit FAddConditional(FComponentTypeID InComponentTypeID, bool bInCondition)
		: ComponentTypeID(InComponentTypeID)
		, bCondition(bInCondition)
	{}

	void AccumulateMask(FComponentMask& OutMask) const
	{
		if (bCondition)
		{
			OutMask.Set(ComponentTypeID);
		}
	}

	void Apply(FEntityAllocation* Allocation, int32 ComponentOffset)
	{
	}
};

/** Implemtntation of a typed add payload */
template<typename T>
struct TAdd : FAdd
{
	TOptional<T> Payload;

	template<typename PayloadType>
	TAdd(TComponentTypeID<T> InComponentTypeID, PayloadType&& InPayload)
		: FAdd(InComponentTypeID)
		, Payload(Forward<PayloadType>(InPayload))
	{}

	void Apply(FEntityAllocation* Allocation, int32 ComponentOffset)
	{
		const FComponentHeader& Header = Allocation->GetComponentHeaderChecked(ComponentTypeID);
		check(!Header.IsTag());

		FScopedHeaderWriteLock WriteLock(&Header, Allocation->GetCurrentLockMode(), FEntityAllocationWriteContext::NewAllocation());

		T* ComponentPtr = static_cast<T*>(Header.GetValuePtr(ComponentOffset));
		*ComponentPtr = MoveTemp(Payload.GetValue());
		Payload.Reset();
	}
};

/** Implemtntation of a conditional typed add payload */
template<typename T>
struct TAddConditional : FAddConditional
{
	TOptional<T> Payload;

	template<typename PayloadType>
	TAddConditional(TComponentTypeID<T> ComponentTypeID, PayloadType&& InPayload, bool bInCondition)
		: FAddConditional(ComponentTypeID, bInCondition)
		, Payload(Forward<PayloadType>(InPayload))
	{}

	void Apply(FEntityAllocation* Allocation, int32 ComponentOffset)
	{
		if (bCondition)
		{
			const FComponentHeader& Header = Allocation->GetComponentHeaderChecked(ComponentTypeID);
			check(!Header.IsTag());

			FScopedHeaderWriteLock WriteLock(&Header, Allocation->GetCurrentLockMode(), FEntityAllocationWriteContext::NewAllocation());

			T* ComponentPtr = static_cast<T*>(Header.GetValuePtr(ComponentOffset));
			*ComponentPtr = MoveTemp(Payload.GetValue());
			Payload.Reset();
		}
	}
};

inline FComponentMask FTypelessMutation::MutateType(const FComponentMask& Current) const
{
	FComponentMask NewMask;
	if (!bRemoveAll)
	{
		NewMask = Current;
	}

	if (RemoveMask.Num())
	{
		FComponentMask MaskOut = RemoveMask;
		MaskOut.BitwiseNOT();

		NewMask.CombineWithBitwiseAND(MaskOut, EBitwiseOperatorFlags::MaintainSize | EBitwiseOperatorFlags::OneFillMissingBits);
	}

	if (AddMask.Num())
	{
		NewMask.CombineWithBitwiseOR(AddMask, EBitwiseOperatorFlags::MaxSize);
	}
	return NewMask;
}

inline TEntityBuilder< FAdd > TEntityBuilderImpl<TIntegerSequence<int>>::AddDefaulted(FComponentTypeID ComponentType)
{
	return TEntityBuilder< FAdd >( FAdd(ComponentType), false );
}

inline TEntityBuilder< FAddMany > TEntityBuilderImpl<TIntegerSequence<int>>::AddMany(const FComponentMask& InComponentsToAdd)
{
	return TEntityBuilder< FAddMany >( FAddMany(InComponentsToAdd), false );
}

inline TEntityBuilder< FAdd > TEntityBuilderImpl<TIntegerSequence<int>>::AddTag(FComponentTypeID TagType)
{
	return TEntityBuilder< FAdd >( FAdd(TagType), false );
}

inline TEntityBuilder< FAddConditional > TEntityBuilderImpl<TIntegerSequence<int>>::AddDefaultedConditional(FComponentTypeID TagType, bool bCondition)
{
	return TEntityBuilder< FAddConditional >( FAddConditional(TagType, bCondition), false );
}

inline TEntityBuilder< FAddConditional > TEntityBuilderImpl<TIntegerSequence<int>>::AddTagConditional(FComponentTypeID TagType, bool bCondition)
{
	return TEntityBuilder< FAddConditional >( FAddConditional(TagType, bCondition), false );
}

template<typename U, typename PayloadType>
inline TEntityBuilder< TAdd<U> > TEntityBuilderImpl<TIntegerSequence<int>>::Add(TComponentTypeID<U> ComponentType, PayloadType&& InPayload)
{
	return TEntityBuilder< TAdd<U> >( TAdd<U>(ComponentType, Forward<PayloadType>(InPayload)), false );
}

template<typename U, typename PayloadType>
inline TEntityBuilder< TAddConditional<U> > TEntityBuilderImpl<TIntegerSequence<int>>::AddConditional(TComponentTypeID<U> ComponentType, PayloadType&& InPayload, bool bCondition)
{
	return TEntityBuilder< TAddConditional<U> >( TAddConditional<U>(ComponentType, Forward<PayloadType>(InPayload), bCondition), false );
}

template <typename U>
TEntityBuilder< U > TEntityBuilderImpl<TIntegerSequence<int>>::Append(U&& InOther)
{
	return TEntityBuilder< U >( MoveTemp(InOther), false );
}



template <typename... T, int... Indices>
TEntityBuilder<T..., FAdd > TEntityBuilderImpl<TIntegerSequence<int, Indices...>, T...>::AddDefaulted(FComponentTypeID ComponentType)
{
	return TEntityBuilder<T..., FAdd >(MoveTemp(Payload.template Get<Indices>())..., FAdd(ComponentType), bAddMutualComponents );
}

template <typename... T, int... Indices>
TEntityBuilder<T..., FAddMany > TEntityBuilderImpl<TIntegerSequence<int, Indices...>, T...>::AddMany(const FComponentMask& InComponentsToAdd)
{
	return TEntityBuilder<T..., FAddMany >( FAddMany(InComponentsToAdd), bAddMutualComponents );
}

template <typename... T, int... Indices>
TEntityBuilder<T..., FAdd > TEntityBuilderImpl<TIntegerSequence<int, Indices...>, T...>::AddTag(FComponentTypeID TagType)
{
	return TEntityBuilder<T..., FAdd >(MoveTemp(Payload.template Get<Indices>())..., FAdd(TagType), bAddMutualComponents );
}

template <typename... T, int... Indices>
template<typename U, typename PayloadType>
TEntityBuilder<T..., TAdd<U> > TEntityBuilderImpl<TIntegerSequence<int, Indices...>, T...>::Add(TComponentTypeID<U> ComponentType, PayloadType&& InPayload)
{
	return TEntityBuilder<T..., TAdd<U> >(MoveTemp(Payload.template Get<Indices>())..., TAdd<U>(ComponentType, Forward<PayloadType>(InPayload)), bAddMutualComponents );
}

template <typename... T, int... Indices>
template<typename U, typename PayloadType>
TEntityBuilder<T..., TAddConditional<U> > TEntityBuilderImpl<TIntegerSequence<int, Indices...>, T...>::AddConditional(TComponentTypeID<U> ComponentType, PayloadType&& InPayload, bool bCondition)
{
	return TEntityBuilder<T..., TAddConditional<U> >(MoveTemp(Payload.template Get<Indices>())..., TAddConditional<U>(ComponentType, Forward<PayloadType>(InPayload), bCondition), bAddMutualComponents );
}

template <typename... T, int... Indices>
TEntityBuilder<T..., FAddConditional > TEntityBuilderImpl<TIntegerSequence<int, Indices...>, T...>::AddDefaultedConditional(FComponentTypeID TagType, bool bCondition)
{
	return TEntityBuilder<T..., FAddConditional >(MoveTemp(Payload.template Get<Indices>())..., FAddConditional(TagType, bCondition), bAddMutualComponents );
}

template <typename... T, int... Indices>
TEntityBuilder<T..., FAddConditional > TEntityBuilderImpl<TIntegerSequence<int, Indices...>, T...>::AddTagConditional(FComponentTypeID TagType, bool bCondition)
{
	return TEntityBuilder<T..., FAddConditional >(MoveTemp(Payload.template Get<Indices>())..., FAddConditional(TagType, bCondition), bAddMutualComponents );
}

template <typename... T, int... Indices>
template<typename U>
TEntityBuilder< T..., U > TEntityBuilderImpl<TIntegerSequence<int, Indices...>, T...>::Append(U&& InOther)
{
	return TEntityBuilder< T..., U >(MoveTemp(Payload.template Get<Indices>())..., MoveTemp(InOther), bAddMutualComponents );
}

template <typename... T, int... Indices>
TEntityBuilder<T...> TEntityBuilderImpl<TIntegerSequence<int, Indices...>, T...>::AddMutualComponents()
{
	return TEntityBuilder<T... >(MoveTemp(Payload.template Get<Indices>())..., true );
}

} // namespace MovieScene
} // namespace UE
