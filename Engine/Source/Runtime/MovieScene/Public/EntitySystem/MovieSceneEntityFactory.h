// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Containers/SortedMap.h"
#include "CoreTypes.h"
#include "EntitySystem/MovieSceneEntityFactoryTypes.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntitySystemDirectedGraph.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "EntitySystem/MovieSceneMutualComponentInclusivity.h"
#include "Misc/AssertionMacros.h"
#include "Misc/InlineValue.h"

#include <initializer_list>


namespace UE
{
namespace MovieScene
{

struct IMutualComponentInitializer;

struct FEntityRange;
struct FEntityAllocation;
struct FMutualComponentInitializers;

/**
 * A class that contains all the component factory relationships.
 *
 * A source component (imported from an entity provider) can trigger the creation of other components on
 * the same entity or on children entities of its entity.
 */
struct FEntityFactories
{
	/**
	 * Defines a component as something that should always be created on every child entity.
	 */
	void DefineChildComponent(FComponentTypeID InChildComponent)
	{
		ParentToChildComponentTypes.AddUnique(FComponentTypeID::Invalid(), InChildComponent);
	}

	/**
	 * Specifies that if a component is present on a parent entity, the given child component should
	 * be created on any child entity.
	 */
	void DefineChildComponent(FComponentTypeID InParentComponent, FComponentTypeID InChildComponent)
	{
		ParentToChildComponentTypes.AddUnique(InParentComponent, InChildComponent);
	}

	/**
	 * Makes the given component automatically copied from a parent entity to all its children entities.
	 * @note: include "EntitySystem/MovieSceneEntityFactoryTemplates.h" for definition
	 */
	template<typename ComponentType>
	void DuplicateChildComponent(TComponentTypeID<ComponentType> InComponent);

	/**
	 * Makes the given component automatically copied from a parent entity to all its children entities,
	 * but only if the parent entity passes the given InParentComponentMask.
	 * @note: include "EntitySystem/MovieSceneEntityFactoryTemplates.h" for definition
	 */
	template<typename ComponentType>
	void ConditionallyDuplicateChildComponent(TComponentTypeID<ComponentType> InComponent, FComponentMask InParentComponentMask);

	/**
	 * Specifies that if a component is present on a parent entity, the given child component should
	 * be created on any child entity, and initialized with the given initializer.
	 * @note: include "EntitySystem/MovieSceneEntityFactoryTemplates.h" for definition
	 */
	template<typename ParentComponent, typename ChildComponent, typename InitializerCallback>
	void DefineChildComponent(TComponentTypeID<ParentComponent> InParentType, TComponentTypeID<ChildComponent> InChildType, InitializerCallback&& InInitializer);

	/**
	 * Adds the definition for a child component. The helper methods above are easier and preferrable.
	 */
	MOVIESCENE_API void DefineChildComponent(TInlineValue<FChildEntityInitializer>&& InInitializer);

	/**
	 * Indicates that if the first component exists on an entity, the second component should be created on
	 * that entity too.
	 *
	 * @note: the inverse is not implied (ie B can still exist without A)
	 */
	void DefineMutuallyInclusiveComponent(FComponentTypeID InComponentA, FComponentTypeID InComponentB)
	{
		DefineMutuallyInclusiveComponents(InComponentA, { InComponentB });
	}

	/**
	 * Indicates that if the first component exists on an entity, the specified components should be created on
	 * that entity too.
	 *
	 * @note: the inverse is not implied (ie B can still exist without A)
	 */
	MOVIESCENE_API void DefineMutuallyInclusiveComponents(FComponentTypeID InComponentA, std::initializer_list<FComponentTypeID> InMutualComponents);

	/**
	 * Specifies a mutual inclusivity relationship along with a custom initializer for initializing the mutual component(s)
	 */
	MOVIESCENE_API void DefineMutuallyInclusiveComponents(FComponentTypeID InComponentA, std::initializer_list<FComponentTypeID> InMutualComponents, FMutuallyInclusiveComponentParams&& Params);

	/**
	 * Specifies that if an entity matches the given filter, the specified component should be created on it.
	 */
	MOVIESCENE_API void DefineComplexInclusiveComponents(const FComplexInclusivityFilter& InFilter, FComponentTypeID InComponent);

	/**
	 * Specifies that if an entity matches the given filter, the specified components should be created on it.
	 */
	MOVIESCENE_API void DefineComplexInclusiveComponents(const FComplexInclusivityFilter& InFilter, std::initializer_list<FComponentTypeID> InComponents, FMutuallyInclusiveComponentParams&& Params);

	/**
	 * Given a set of components on a parent entity, compute what components should exist on a child entity.
	 *
	 * This resolves all the parent-to-child relationships.
	 */
	MOVIESCENE_API int32 ComputeChildComponents(const FComponentMask& ParentComponentMask, FComponentMask& ChildComponentMask);

	/**
	 * Given a set of components on an entity, computes what other components should also exist on this entity.
	 *
	 * This resolves all the mutual and complex inclusivity relationships.
	 */
	MOVIESCENE_API int32 ComputeMutuallyInclusiveComponents(EMutuallyInclusiveComponentType MutualTypes, FComponentMask& ComponentMask, FMutualComponentInitializers& OutInitializers);

	MOVIESCENE_API void RunInitializers(const FComponentMask& ParentType, const FComponentMask& ChildType, const FEntityAllocation* ParentAllocation, TArrayView<const int32> ParentAllocationOffsets, const FEntityRange& InChildEntityRange);

private:

	TArray<TInlineValue<FChildEntityInitializer>> ChildInitializers;
	TMultiMap<FComponentTypeID, FComponentTypeID> ParentToChildComponentTypes;
	UE::MovieScene::FMutualInclusivityGraph MutualInclusivityGraph;
};


}	// using namespace MovieScene
}	// using namespace UE
