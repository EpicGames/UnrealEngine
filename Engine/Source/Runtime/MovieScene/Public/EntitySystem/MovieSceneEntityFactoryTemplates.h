// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/SortedMap.h"

#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntityRange.h"
#include "EntitySystem/MovieSceneEntityFactory.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneComponentRegistry.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "EntitySystem/MovieSceneComponentAccessors.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "EntitySystem/MovieSceneEntitySystemDirectedGraph.h"

#include <type_traits>


namespace UE
{
namespace MovieScene
{

struct FInstanceRegistry;
using FBoundObjectResolver = UObject* (*)(UObject*);

template<typename ParentComponentType, typename ChildComponentType>
struct TChildEntityInitializer : FChildEntityInitializer
{
	explicit TChildEntityInitializer(TComponentTypeID<ParentComponentType> InParentComponent, TComponentTypeID<ChildComponentType> InChildComponent)
		: FChildEntityInitializer(InParentComponent, InChildComponent)
	{}

	TComponentTypeID<ParentComponentType> GetParentComponent() const
	{
		return ParentComponent.ReinterpretCast<ParentComponentType>();
	}

	TComponentTypeID<ChildComponentType> GetChildComponent() const
	{
		return ChildComponent.ReinterpretCast<ChildComponentType>();
	}

	TComponentLock<TRead<ParentComponentType>> GetParentComponents(const FEntityAllocation* Allocation) const
	{
		return Allocation->ReadComponents(GetParentComponent());
	}

	TComponentLock<TWrite<ChildComponentType>> GetChildComponents(const FEntityAllocation* Allocation) const
	{
		return Allocation->WriteComponents(GetChildComponent(), FEntityAllocationWriteContext::NewAllocation());
	}
};

// Callback must be compatible with form void(const ParentComponentType, ChildComponentType&);
template<typename ParentComponentType, typename ChildComponentType, typename InitializerType>
struct TStaticChildEntityInitializer : TChildEntityInitializer<ParentComponentType, ChildComponentType>
{
	InitializerType Callback;

	explicit TStaticChildEntityInitializer(TComponentTypeID<ParentComponentType> InParentComponent, TComponentTypeID<ChildComponentType> InChildComponent, InitializerType InCallback)
		: TChildEntityInitializer<ParentComponentType, ChildComponentType>(InParentComponent, InChildComponent)
		, Callback(InCallback)
	{}

	virtual void Run(const FEntityRange& ChildRange, const FEntityAllocation* ParentAllocation, TArrayView<const int32> ParentAllocationOffsets)
	{
		TComponentLock<TRead<ParentComponentType>> ParentComponents = ParentAllocation->ReadComponents(this->GetParentComponent());
		TComponentLock<TWrite<ChildComponentType>> ChildComponents  = ChildRange.Allocation->WriteComponents(this->GetChildComponent(), FEntityAllocationWriteContext::NewAllocation());

		for (int32 Index = 0; Index < ChildRange.Num; ++Index)
		{
			const int32 ParentIndex = ParentAllocationOffsets[Index];
			const int32 ChildIndex  = ChildRange.ComponentStartOffset + Index;

			Callback(ParentComponents[ParentIndex], ChildComponents[ChildIndex]);
		}
	}
};

template<typename ComponentType>
struct TDuplicateChildEntityInitializer : FChildEntityInitializer
{
	explicit TDuplicateChildEntityInitializer(TComponentTypeID<ComponentType> InComponent)
		: FChildEntityInitializer(InComponent, InComponent)
	{}

	TComponentTypeID<ComponentType> GetComponent() const
	{
		return ParentComponent.ReinterpretCast<ComponentType>();
	}

	virtual void Run(const FEntityRange& ChildRange, const FEntityAllocation* ParentAllocation, TArrayView<const int32> ParentAllocationOffsets)
	{
		TComponentLock<TRead<ComponentType>>  ParentComponents = ParentAllocation->ReadComponents(GetComponent());
		TComponentLock<TWrite<ComponentType>> ChildComponents  = ChildRange.Allocation->WriteComponents(GetComponent(), FEntityAllocationWriteContext::NewAllocation());

		TArrayView<ComponentType> ChildComponentSlice = ChildComponents.Slice(ChildRange.ComponentStartOffset, ChildRange.Num);
		for (int32 Index = 0; Index < ParentAllocationOffsets.Num(); ++Index)
		{
			ChildComponentSlice[Index] = ParentComponents[ParentAllocationOffsets[Index]];
		}
	}
};

/* Duplicates child components, but only if the parent entity passes the given component mask*/
template<typename ComponentType>
struct TConditionalDuplicateChildEntityInitializer : TDuplicateChildEntityInitializer<ComponentType>
{
	explicit TConditionalDuplicateChildEntityInitializer(TComponentTypeID<ComponentType> InComponent, FComponentMask InParentComponentMask)
		: TDuplicateChildEntityInitializer<ComponentType>(InComponent)
		, ParentComponentMask(InParentComponentMask)
	{}

	virtual bool IsRelevant(const FComponentMask& InParentType, const FComponentMask& InChildType) const override
	{
		return TDuplicateChildEntityInitializer<ComponentType>::IsRelevant(InParentType, InChildType) && InParentType.ContainsAll(ParentComponentMask);
	}

	FComponentMask ParentComponentMask;
};

struct FObjectFactoryBatch : FChildEntityFactory
{
	void Add(int32 EntityIndex, UObject* BoundObject);

	virtual void GenerateDerivedType(FComponentMask& OutNewEntityType) override;

	virtual void InitializeAllocation(UMovieSceneEntitySystemLinker* Linker, const FComponentMask& ParentType, const FComponentMask& ChildType, const FEntityAllocation* ParentAllocation, TArrayView<const int32> ParentAllocationOffsets, const FEntityRange& InChildEntityRange) override;

	virtual void PostInitialize(UMovieSceneEntitySystemLinker* InLinker) override;

	TMap<TTuple<UObject*, FMovieSceneEntityID>, FMovieSceneEntityID>* StaleEntitiesToPreserve;

private:
	TSortedMap<FMovieSceneEntityID, FMovieSceneEntityID> PreservedEntities;
	TArray<UObject*> ObjectsToAssign;
};

struct FBoundObjectTask
{
	FBoundObjectTask(UMovieSceneEntitySystemLinker* InLinker);

	void Apply();

	void ForEachAllocation(FEntityAllocationProxy AllocationProxy, FReadEntityIDs EntityIDs, TRead<FInstanceHandle> Instances, TRead<FGuid> ObjectBindings, TReadOptional<FBoundObjectResolver> Resolvers);

	void PostTask();

private:

	struct FEntityMutationData
	{
		FMovieSceneEntityID EntityID;
		FComponentTypeID ComponentTypeID;
		bool bAddComponent;
	};

	TMap<TTuple<UObject*, FMovieSceneEntityID>, FMovieSceneEntityID> StaleEntitiesToPreserve;
	TMap<FEntityAllocationProxy, FObjectFactoryBatch> Batches;
	TArray<FMovieSceneEntityID> EntitiesToDiscard;
	TArray<FEntityMutationData> EntityMutations;

protected:

	UMovieSceneEntitySystemLinker* Linker;
};


template<typename ComponentType>
inline void FEntityFactories::DuplicateChildComponent(TComponentTypeID<ComponentType> InComponent)
{
	DefineChildComponent(TDuplicateChildEntityInitializer<ComponentType>(InComponent));
}

template<typename ComponentType>
inline void FEntityFactories::ConditionallyDuplicateChildComponent(TComponentTypeID<ComponentType> InComponent, FComponentMask InParentComponentMask)
{
	DefineChildComponent(TConditionalDuplicateChildEntityInitializer<ComponentType>(InComponent, InParentComponentMask));
}

template<typename ParentComponent, typename ChildComponent, typename InitializerCallback>
inline void FEntityFactories::DefineChildComponent(TComponentTypeID<ParentComponent> InParentType, TComponentTypeID<ChildComponent> InChildType, InitializerCallback&& InInitializer)
{
	using FInitializer = TStaticChildEntityInitializer<ParentComponent, ChildComponent, InitializerCallback>;

	DefineChildComponent(InParentType, InChildType);
	ChildInitializers.Add(FInitializer(InParentType, InChildType, Forward<InitializerCallback>(InInitializer)));
}

template<typename T>
FComponentTypeInfo FComponentRegistry::MakeComponentTypeInfoWithoutComponentOps(const TCHAR* const DebugName, const FNewComponentTypeParams& Params)
{
	static const uint32 ComponentTypeSize = sizeof(T);
	static_assert(ComponentTypeSize < TNumericLimits<decltype(FComponentTypeInfo::Sizeof)>::Max(), "Type too large to be used as component data");

	static const uint32 Alignment = alignof(T);
	static_assert(Alignment < TNumericLimits<decltype(FComponentTypeInfo::Alignment)>::Max(), "Type alignment too large to be used as component data");

	FComponentTypeInfo NewTypeInfo;

	NewTypeInfo.Sizeof = ComponentTypeSize;
	NewTypeInfo.Alignment = Alignment;
	NewTypeInfo.bIsZeroConstructType = TIsZeroConstructType<T>::Value;
	NewTypeInfo.bIsTriviallyDestructable = std::is_trivially_destructible_v<T>;
	NewTypeInfo.bIsTriviallyCopyAssignable = TIsTriviallyCopyAssignable<T>::Value;
	NewTypeInfo.bIsPreserved = EnumHasAnyFlags(Params.Flags, EComponentTypeFlags::Preserved);
	NewTypeInfo.bIsCopiedToOutput = EnumHasAnyFlags(Params.Flags, EComponentTypeFlags::CopyToOutput);
	NewTypeInfo.bIsMigratedToOutput = EnumHasAnyFlags(Params.Flags, EComponentTypeFlags::MigrateToOutput);
	NewTypeInfo.bHasReferencedObjects = false;

#if UE_MOVIESCENE_ENTITY_DEBUG
	NewTypeInfo.DebugInfo = MakeUnique<TComponentTypeDebugInfo<T>>();
	NewTypeInfo.DebugInfo->DebugName = DebugName;
	NewTypeInfo.DebugInfo->DebugTypeName = GetGeneratedTypeName<T>();
#endif

	return NewTypeInfo;
}

template<typename T>
TComponentTypeID<T> FComponentRegistry::NewComponentType(const TCHAR* const DebugName, const FNewComponentTypeParams& Params)
{
	FComponentTypeInfo NewTypeInfo = FComponentRegistry::MakeComponentTypeInfoWithoutComponentOps<T>(DebugName, Params);

	NewTypeInfo.bHasReferencedObjects = Params.ReferenceCollectionCallback != nullptr || THasAddReferencedObjectForComponent<T>::Value;

	if (Params.ReferenceCollectionCallback)
	{
		NewTypeInfo.MakeComplexComponentOps<T>(Params.ReferenceCollectionCallback);
	}
	else if (!NewTypeInfo.bIsZeroConstructType || !NewTypeInfo.bIsTriviallyDestructable || !NewTypeInfo.bIsTriviallyCopyAssignable || NewTypeInfo.bHasReferencedObjects)
	{
		NewTypeInfo.MakeComplexComponentOps<T>();
	}

	FComponentTypeID    NewTypeID = NewComponentTypeInternal(MoveTemp(NewTypeInfo));
	TComponentTypeID<T> TypedTypeID = NewTypeID.ReinterpretCast<T>();

	if (EnumHasAnyFlags(Params.Flags, EComponentTypeFlags::CopyToChildren))
	{
		Factories.DefineChildComponent(TDuplicateChildEntityInitializer<T>(TypedTypeID));
	}

	return TypedTypeID;
}

template<typename T>
TComponentTypeID<T> FComponentRegistry::NewComponentTypeNoAddReferencedObjects(const TCHAR* const DebugName, const FNewComponentTypeParams& Params)
{
	FComponentTypeInfo NewTypeInfo = FComponentRegistry::MakeComponentTypeInfoWithoutComponentOps<T>(DebugName, Params);

	NewTypeInfo.bHasReferencedObjects = false;
	if (!NewTypeInfo.bIsZeroConstructType || !NewTypeInfo.bIsTriviallyDestructable || !NewTypeInfo.bIsTriviallyCopyAssignable)
	{
		NewTypeInfo.MakeComplexComponentOpsNoAddReferencedObjects<T>();
	}

	FComponentTypeID    NewTypeID = NewComponentTypeInternal(MoveTemp(NewTypeInfo));
	TComponentTypeID<T> TypedTypeID = NewTypeID.ReinterpretCast<T>();

	if (EnumHasAnyFlags(Params.Flags, EComponentTypeFlags::CopyToChildren))
	{
		Factories.DefineChildComponent(TDuplicateChildEntityInitializer<T>(TypedTypeID));
	}

	return TypedTypeID;
}

}	// using namespace MovieScene
}	// using namespace UE
