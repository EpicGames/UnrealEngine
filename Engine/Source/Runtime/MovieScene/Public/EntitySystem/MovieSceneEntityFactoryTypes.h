// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"

#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"

class UMovieSceneEntitySystemLinker;


namespace UE
{
namespace MovieScene
{

struct FEntityRange;

struct FChildEntityFactory
{
	virtual ~FChildEntityFactory()= default;

	int32 Num() const;

	int32 GetCurrentIndex() const;

	void Apply(UMovieSceneEntitySystemLinker* Linker, FEntityAllocationProxy ParentAllocationProxy);
	
	void Add(int32 EntityIndex)
	{
		ParentEntityOffsets.Add(EntityIndex);
	}

	virtual void GenerateDerivedType(FComponentMask& OutNewEntityType)
	{}

	virtual void PostInitialize(UMovieSceneEntitySystemLinker* Linker)
	{}

	virtual void InitializeAllocation(UMovieSceneEntitySystemLinker* Linker, const FComponentMask& ParentType, const FComponentMask& ChildType, const FEntityAllocation* ParentAllocation, TArrayView<const int32> ParentAllocationOffsets, const FEntityRange& InChildEntityRange)
	{}

protected:
	TArrayView<const int32> CurrentEntityOffsets;
	TArray<int32> ParentEntityOffsets;
};



struct FChildEntityInitializer
{
	virtual ~FChildEntityInitializer()= default;

	virtual bool IsRelevant(const FComponentMask& InParentType, const FComponentMask& InChildType) const
	{
		// Entity initializers with no parent component are always valid
		const bool bHasParentComponent = !ParentComponent || InParentType.Contains(ParentComponent);
		const bool bHasChildComponent  =  ChildComponent  && InChildType.Contains(ChildComponent);

		return bHasParentComponent && bHasChildComponent;
	}

	FComponentTypeID GetParentComponent() const
	{
		return ParentComponent;
	}

	FComponentTypeID GetChildComponent() const
	{
		return ChildComponent;
	}

	virtual void Run(const FEntityRange&, const FEntityAllocation*, TArrayView<const int32>) = 0;

protected:

	FComponentTypeID ParentComponent, ChildComponent;

	explicit FChildEntityInitializer(FComponentTypeID InParentComponent, FComponentTypeID InChildComponent)
		: ParentComponent(InParentComponent)
		, ChildComponent(InChildComponent)
	{}
};



}	// using namespace MovieScene
}	// using namespace UE
