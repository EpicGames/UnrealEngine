// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/SparseArray.h"
#include "CoreTypes.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneComponentTypeInfo.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "Templates/PointerIsConvertibleFromTo.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneEntityGroupingSystem.generated.h"

#define UE_API MOVIESCENE_API

class UMovieSceneEntityGroupingSystem;

namespace UE::MovieScene
{

struct FAddGroupMutation;
struct FUpdateGroupsTask;

/** Concept that checks whether a grouping policy supports batch operations based on the presence of a InitializeGroupKeys function */
struct CBatchGroupingPolicy
{
	template <typename T>
	auto Requires(T& In) -> decltype(&T::InitializeGroupKeys);
};

/**
 * Utility class used by the grouping system's policies (see below) to manage groups.
 */
struct FEntityGroupBuilder
{
	UE_API FEntityGroupBuilder(UMovieSceneEntityGroupingSystem* InOwner, FEntityGroupingPolicyKey InPolicyKey);

	/** Make a full group ID from an existing group index */
	FEntityGroupID MakeGroupID(int32 GroupIndex) const { return FEntityGroupID(PolicyKey, GroupIndex); }
	/** Make an invalid group ID that is associated with the policy key */
	FEntityGroupID MakeInvalidGroupID() const { return FEntityGroupID(PolicyKey, INDEX_NONE); }
	/** Add the entity to the given group. The entity must already have the group ID component. */
	UE_API void AddEntityToGroup(const FMovieSceneEntityID& InEntity, const FEntityGroupID& InNewGroupID);
	/** Remove the entity from the given group. The entity must already have the group ID component. */
	UE_API void RemoveEntityFromGroup(const FMovieSceneEntityID& InEntity, const FEntityGroupID& InPreviousGroupID);

	UE_API int32 AllocateGroupIndex();

	UE_API void ReportUsedGroupIndex(int32 GroupIndex);

private:
	UMovieSceneEntityGroupingSystem* Owner;
	FEntityGroupingPolicyKey PolicyKey;
};

/**
 * Base class for grouping handlers, used by the grouping system (see below).
 */
struct IEntityGroupingHandler
{
	virtual ~IEntityGroupingHandler() = default;
	virtual void ProcessAllocation(FEntityAllocationIteratorItem Item, FReadEntityIDs EntityIDs, TWrite<FEntityGroupID> GroupIDs, FEntityGroupBuilder* Builder) = 0;
	virtual void OnGroupIndexFreed(int32 InGroupIndex) {}

#if WITH_EDITOR
	virtual void OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap) = 0;
#endif
};

/**
 * Strongly-typed grouping handler class, which knows about the exact components to look for, and how
 * to use them to group entities.
 */
template<typename GroupingPolicy, typename ComponentIndices, typename ...ComponentTypes>
struct TEntityGroupingHandlerImpl;

template<typename GroupingPolicy, typename ...ComponentTypes>
struct TEntityGroupingHandler 
	: TEntityGroupingHandlerImpl<GroupingPolicy, TMakeIntegerSequence<int, sizeof...(ComponentTypes)>, ComponentTypes...>
{
	TEntityGroupingHandler(GroupingPolicy&& InPolicy, TComponentTypeID<ComponentTypes>... InComponents)
		: TEntityGroupingHandlerImpl<GroupingPolicy, TMakeIntegerSequence<int, sizeof...(ComponentTypes)>, ComponentTypes...>(
				MoveTemp(InPolicy), InComponents...)
	{
	}
};



template<typename GroupingPolicy>
struct TEntityGroupingHandlerBase
{
	using GroupKeyType = typename GroupingPolicy::GroupKeyType;

	int32 GetOrAllocateGroupIndex(typename TCallTraits<GroupKeyType>::ParamType InGroupKey, FEntityGroupBuilder* Builder)
	{
		int32& GroupIndex = GroupKeyToIndex.FindOrAdd(InGroupKey, INDEX_NONE);
		if (GroupIndex == INDEX_NONE)
		{
			// This group key isn't known to us... let's allocate a new group index for it.
			// Try to find an available index first. Otherwise use a new high index.
			GroupIndex = Builder->AllocateGroupIndex();
			IndexToGroupKey.FindOrAdd(GroupIndex, InGroupKey);
		}
		else
		{
			// We know this group key, so we'll return the group index we already have
			// associated with it. We just need to "revive" it in case it was scheduled
			// for being freed.
			Builder->ReportUsedGroupIndex(GroupIndex);
		}
		return GroupIndex;
	}

protected:

	/** The group keys that we know about, mapped to their corresponding group index */
	TMap<GroupKeyType, int32> GroupKeyToIndex;
	TMap<int32, GroupKeyType> IndexToGroupKey;
};




template<typename GroupingPolicy, int ...ComponentIndices, typename ...ComponentTypes>
struct TEntityGroupingHandlerImpl<GroupingPolicy, TIntegerSequence<int, ComponentIndices...>, ComponentTypes...> : TEntityGroupingHandlerBase<GroupingPolicy>, IEntityGroupingHandler
{
	using GroupKeyType = typename GroupingPolicy::GroupKeyType;

	/** The grouping policy */
	GroupingPolicy Policy;

	/** The components that are required for making up a group key */
	TTuple<TComponentTypeID<ComponentTypes>...> Components;

	TEntityGroupingHandlerImpl(GroupingPolicy&& InPolicy, TComponentTypeID<ComponentTypes>... InComponents)
		: Policy(MoveTemp(InPolicy))
		, Components(InComponents...)
	{
	}

	/** Process an allocation and group the entities found therein */
	virtual void ProcessAllocation(FEntityAllocationIteratorItem Item,  FReadEntityIDs EntityIDs, TWrite<FEntityGroupID> GroupIDs, FEntityGroupBuilder* Builder) override
	{
		const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
		const FComponentMask& AllocationType = Item.GetAllocationType();
		const bool bNeedsLink = AllocationType.Contains(BuiltInComponents->Tags.NeedsLink);
		const bool bNeedsUnlink = AllocationType.Contains(BuiltInComponents->Tags.NeedsUnlink);
		ensure(bNeedsLink || bNeedsUnlink);

		if (bNeedsLink)
		{
			VisitLinkedEntities(Item, EntityIDs, GroupIDs, Builder);
		}
		else if (bNeedsUnlink)
		{
			VisitUnlinkedEntities(Item, EntityIDs, GroupIDs, Builder);
		}
	}
	
	void VisitLinkedEntities(FEntityAllocationIteratorItem Item, FReadEntityIDs EntityIDs, TWrite<FEntityGroupID> GroupIDs, FEntityGroupBuilder* Builder)
	{
		if constexpr (TModels_V<CBatchGroupingPolicy, GroupingPolicy>)
		{
			const FEntityAllocation* Allocation = Item.GetAllocation();
			Policy.InitializeGroupKeys(*this, Builder, Item, EntityIDs, GroupIDs, Allocation->ReadComponents(Components.template Get<ComponentIndices>())...);
		}
		else
		{
			const FEntityAllocation* Allocation = Item.GetAllocation();
			const int32 Num = Allocation->Num();

			const FEntityGroupID InvalidGroupID = Builder->MakeInvalidGroupID();

			TTuple<TComponentReader<ComponentTypes>...> ComponentReaders(
					Allocation->ReadComponents(Components.template Get<ComponentIndices>())...);

			for (int32 Index = 0; Index < Num; ++Index)
			{
				GroupKeyType GroupKey;
				const bool bValidGroupKey = Policy.GetGroupKey(ComponentReaders.template Get<ComponentIndices>()[Index]..., GroupKey);

				const FMovieSceneEntityID EntityID(EntityIDs[Index]);
				FEntityGroupID& GroupID(GroupIDs[Index]);

				if (bValidGroupKey)
				{
					// Find or create the appropriate group and put the entity in it.
					int32 NewGroupIndex = this->GetOrAllocateGroupIndex(GroupKey, Builder);
					FEntityGroupID NewGroupID = Builder->MakeGroupID(NewGroupIndex);
					Builder->AddEntityToGroup(EntityID, NewGroupID);
					GroupID = NewGroupID;
				}
				else
				{
					// This entity doesn't belong to any group.
					// Let's assign an invalid group ID that nonetheless has a valid policy key
					// pointing to this grouping.
					GroupID = InvalidGroupID;
				}
			}
		}
	}

	void VisitUnlinkedEntities(FEntityAllocationIteratorItem Item, FReadEntityIDs EntityIDs, TWrite<FEntityGroupID> GroupIDs, FEntityGroupBuilder* Builder)
	{
		const FEntityAllocation* Allocation = Item.GetAllocation();
		const int32 Num = Allocation->Num();

		for (int32 Index = 0; Index < Num; ++Index)
		{
			const FMovieSceneEntityID EntityID(EntityIDs[Index]);
			FEntityGroupID& GroupID(GroupIDs[Index]);

			if (GroupID.HasGroup())
			{
				Builder->RemoveEntityFromGroup(EntityID, GroupID);
				// Leave the GroupID on the entity so that downstream systems can use it to track
				// that this entity is leaving its group, but flag it so we don't re-free it.
				ensure(!EnumHasAllFlags(GroupID.Flags , EEntityGroupFlags::RemovedFromGroup));
				GroupID.Flags |= EEntityGroupFlags::RemovedFromGroup;
			}
		}
	}

	virtual void OnGroupIndexFreed(int32 GroupIndex) override
	{
		GroupKeyType Key = this->IndexToGroupKey.FindChecked(GroupIndex);
		ensure(this->IndexToGroupKey.Remove(GroupIndex) == 1);
		ensure(this->GroupKeyToIndex.Remove(Key) == 1);
	}

#if WITH_EDITOR
	virtual void OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap) override
	{
		// Get a list of keys that contain replaced objects.
		TMap<GroupKeyType, GroupKeyType> ReplacedKeys;
		for (const TPair<GroupKeyType, int32>& Pair : this->GroupKeyToIndex)
		{
			GroupKeyType NewKey = Pair.Key;
			if (Policy.OnObjectsReplaced(NewKey, ReplacementMap))
			{
				ReplacedKeys.Add(Pair.Key, NewKey);
			}
		}
		// Replace the keys but keep the group indices.
		for (const TPair<GroupKeyType, GroupKeyType>& Pair : ReplacedKeys)
		{
			int32 GroupIndex;
			const bool bRemoved = this->GroupKeyToIndex.RemoveAndCopyValue(Pair.Key, GroupIndex);
			if (ensure(bRemoved))
			{
				this->GroupKeyToIndex.Add(Pair.Value, GroupIndex);
				this->IndexToGroupKey.Add(GroupIndex, Pair.Value);
			}
		}
	}
#endif
};

namespace Private
{
	template<typename T>
	bool ReplaceGroupKeyObjectElement(T&& InElem, const TMap<UObject*, UObject*>& ReplacementMap)
	{
		return false;
	}

	template<typename T>
	typename TEnableIf<TPointerIsConvertibleFromTo<T, UObject>::Value, bool>::Type ReplaceGroupKeyObjectElement(T& InOutElem, const TMap<UObject*, UObject*>& ReplacementMap)
	{
		UObject* CurrentObject = InOutElem;
		if (UObject* const* NewObject = ReplacementMap.Find(CurrentObject))
		{
			InOutElem = NewObject;
			return true;
		}
		return false;
	}
}

/**
 * A simple grouping policy that uses a tuple of component values as the group key.
 */
template<typename... ComponentTypes>
struct TTupleGroupingPolicy
{
	using GroupKeyType = TTuple<ComponentTypes...>;

	bool GetGroupKey(ComponentTypes... InComponents, GroupKeyType& OutGroupKey)
	{
		OutGroupKey = MakeTuple(InComponents...);
		return true;
	}

	bool OnObjectsReplaced(GroupKeyType& InOutKey, const TMap<UObject*, UObject*>& ReplacementMap)
	{
		bool bReplaced = false;
		VisitTupleElements([&ReplacementMap, &bReplaced](auto& InElem)
			{
				bReplaced |= Private::ReplaceGroupKeyObjectElement(InElem, ReplacementMap);
			},
			InOutKey);
		return bReplaced;
	}
};

} // namespace UE::MovieScene


UCLASS(MinimalAPI)
class UMovieSceneEntityGroupingSystem : public UMovieSceneEntitySystem
{
public:

	using FEntityGroupID = UE::MovieScene::FEntityGroupID;
	using FEntityGroupingPolicyKey = UE::MovieScene::FEntityGroupingPolicyKey;
	using IEntityGroupingHandler = UE::MovieScene::IEntityGroupingHandler;
	using FMovieSceneEntityID = UE::MovieScene::FMovieSceneEntityID;

	GENERATED_BODY()

	UE_API UMovieSceneEntityGroupingSystem(const FObjectInitializer& ObjInit);

	/**
	 * Add a new grouping policy that will use the given components to make up a group key.
	 *
	 * Grouping policies must be simple structs that can be copied and owned by the grouping system, and
	 * that implement the following members:
	 *
	 * - GroupKeyType [mandatory]
	 *		A typedef or alias to the group key type to use to group entities together.
	 *
	 * - GetGroupKey(ComponentTypes... InComponents, GroupKeyType& OutGroupKey) [mandatory]
	 *		A function that creates a group key used to group entities together.
	 *		Returns true to indicate the key is valid, false otherwise. If false, the entity
	 *		corresponding to the given components will not be grouped.
	 *
	 *	- OnObjectsReplaced(GroupKeyType& InOut, const TMap<UObject*, UObject*>&) [mandatory, sadly]
	 *		Potentially changes a key if it contains a pointer to a replaced object. Should
	 *		return true if any replacement occured.
	 *
	 *	- PreTask() [optional]
	 *		A function called before any grouping is done during an instantiation phase.
	 *
	 *	- PostTask() [optional]
	 *		A function called after any grouping is done dunring an instantiation phase.
	 *
	 *	- PreProcessGroups() [optional]
	 *
	 *	- PostProcessGroups() [optional]
	 */
	template<typename GroupingPolicy, typename ...ComponentTypes>
	FEntityGroupingPolicyKey AddGrouping(GroupingPolicy&& InPolicy, TComponentTypeID<ComponentTypes>... InComponents)
	{
		return AddGrouping(Forward<GroupingPolicy>(InPolicy), UE::MovieScene::FEntityComponentFilter(), InComponents...);
	}

	template<typename GroupingPolicy, typename ...ComponentTypes>
	FEntityGroupingPolicyKey AddGrouping(GroupingPolicy&& InPolicy, UE::MovieScene::FEntityComponentFilter&& InComponentFilter, TComponentTypeID<ComponentTypes>... InComponents)
	{
		using namespace UE::MovieScene;

		using NewGroupHandlerType = TEntityGroupingHandler<GroupingPolicy, ComponentTypes...>;

		static_assert(sizeof(NewGroupHandlerType) <= 256, "Handler type too big! Please increase the TInlineValue size.");

		const int32 HandlerIndex = GroupHandlers.Emplace();
		FEntityGroupingHandlerInfo& HandlerInfo = GroupHandlers[HandlerIndex];
		HandlerInfo.Handler.Emplace<NewGroupHandlerType>(MoveTemp(InPolicy), InComponents...);
		HandlerInfo.ComponentFilter = MoveTemp(InComponentFilter);
		HandlerInfo.ComponentFilter.All({ InComponents... });

		FEntityGroupingPolicyKey NewPolicyKey{ HandlerIndex };
		return NewPolicyKey;
	}

	/**
	 * Add a new grouping policy that will make a key that is a tuple of the given components' values.
	 */
	template<typename ...ComponentTypes>
	FEntityGroupingPolicyKey AddGrouping(TComponentTypeID<ComponentTypes>... InComponents)
	{
		UE::MovieScene::TTupleGroupingPolicy<ComponentTypes...> TuplePolicy;
		return AddGrouping(MoveTemp(TuplePolicy), InComponents...);
	}

	/**
	 * Remove a previously added grouping policy.
	 */
	UE_API void RemoveGrouping(FEntityGroupingPolicyKey InPolicyKey);

	/**
	 * Allocate a new group index used to uniquely identify a collection of entities that animate the same target.
	 * Group indices are globally unique within this system, regardless of the 'type' of the target.
	 */
	UE_API int32 AllocateGroupIndex(FEntityGroupingPolicyKey InPolicy);


	/**
	 * Return the maximum number of groups currently allocated
	 */
	int32 NumGroups() const
	{
		return AllocatedGroupIndices.GetMaxIndex();
	}

	void FreeEmptyGroups();

private:

	virtual bool IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
	virtual void OnLink() override;
	virtual void OnUnlink() override;
	virtual void OnCleanTaggedGarbage() override;

	void ProcessModifiedGroups();

#if WITH_EDITOR
	void OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap);
#endif

private:


	/** The list of group indices in use */
	TSparseArray<FEntityGroupingPolicyKey> AllocatedGroupIndices;

	struct FEntityGroupInfo
	{
		int32 NumEntities = 0;
	};

	TMap<FEntityGroupID, FEntityGroupInfo> Groups;

	TSparseArray<FEntityGroupID> EntityIDToGroup;

	/** The transient list of groups freed this frame */
	TBitArray<> EmptyGroupIndices;

	struct FEntityGroupingHandlerInfo
	{
		TInlineValue<IEntityGroupingHandler, 256> Handler;
		UE::MovieScene::FEntityComponentFilter ComponentFilter;
	};

	TSparseArray<FEntityGroupingHandlerInfo> GroupHandlers;

	friend struct UE::MovieScene::FAddGroupMutation;
	friend struct UE::MovieScene::FUpdateGroupsTask;
	friend struct UE::MovieScene::FEntityGroupBuilder;
};


#undef UE_API 
