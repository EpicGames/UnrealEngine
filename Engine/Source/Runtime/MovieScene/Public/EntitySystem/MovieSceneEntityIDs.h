// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/BitArray.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Guid.h"
#include "Templates/TypeHash.h"

#include <initializer_list>

namespace UE
{
namespace MovieScene
{

class FEntityManager;
struct FComponentTypeID;
template<typename T> struct TComponentTypeID;

/**
 * Defines the absolute maximum number of component *types* that can exist at any one time in an FEntityManager
 * In general component types should be kept to a minimum, but if large numbers of blocking sequences are required (which each require their own linker and instance tags),
 * or large numbers of components are required for whatever reason, this limit can be increased.
 *
 * Component masks are stored as fixed size inline bit arrays, so considerations should be made before increasing this number.
 */
static constexpr int32 MaximumNumComponentsSupported = 384;

/** Defines the number of DWORDs that we need to store for component masks */
static constexpr int32 ComponentMaskDWORDSize = (MaximumNumComponentsSupported + 31) / 32;

using FComponentMaskAllocator = TFixedAllocator<ComponentMaskDWORDSize>;
using FComponentMaskIterator = TConstSetBitIterator<FComponentMaskAllocator>;

// Done as a concrete type so that we can make a debug visualizer for the type.
struct FComponentMask
{
	FComponentMask(){}

	FComponentMask(bool bValue, int32 InNum)
		: Bits(bValue, InNum)
	{}

	FComponentMask(std::initializer_list<FComponentTypeID> InComponentTypes)
	{
		SetAll(InComponentTypes);
	}

	FBitReference operator[](FComponentTypeID ComponentType);

	FConstBitReference operator[](FComponentTypeID ComponentType) const;

	FComponentMaskIterator Iterate() const
	{
		return FComponentMaskIterator(Bits);
	}

	friend uint32 GetTypeHash(const FComponentMask& Mask)
	{
		return GetTypeHash(Mask.Bits);
	}

	friend bool operator==(const FComponentMask& A, const FComponentMask& B)
	{
		return A.Bits == B.Bits;
	}

	bool CompareSetBits(const FComponentMask& Other) const
	{
		return Bits.CompareSetBits(Other.Bits, false);
	}

	inline FComponentMask& CombineWithBitwiseOR(const FComponentMask& Other, EBitwiseOperatorFlags InFlags)
	{
		Bits.CombineWithBitwiseOR(Other.Bits, InFlags);
		return *this;
	}

	inline FComponentMask& CombineWithBitwiseXOR(const FComponentMask& Other, EBitwiseOperatorFlags InFlags)
	{
		Bits.CombineWithBitwiseXOR(Other.Bits, InFlags);
		return *this;
	}
	
	inline FComponentMask& CombineWithBitwiseAND(const FComponentMask& Other, EBitwiseOperatorFlags InFlags)
	{
		Bits.CombineWithBitwiseAND(Other.Bits, InFlags);
		return *this;
	}

	inline static FComponentMask BitwiseAND(const FComponentMask& A, const FComponentMask& B, EBitwiseOperatorFlags InFlags)
	{
		FComponentMask Result;
		Result.Bits = TBitArray<FComponentMaskAllocator>::BitwiseAND(A.Bits, B.Bits, InFlags);
		return Result;
	}

	inline static FComponentMask BitwiseOR(const FComponentMask& A, const FComponentMask& B, EBitwiseOperatorFlags InFlags)
	{
		FComponentMask Result;
		Result.Bits = TBitArray<FComponentMaskAllocator>::BitwiseOR(A.Bits, B.Bits, InFlags);
		return Result;
	}

	inline static FComponentMask BitwiseXOR(const FComponentMask& A, const FComponentMask& B, EBitwiseOperatorFlags InFlags)
	{
		FComponentMask Result;
		Result.Bits = TBitArray<FComponentMaskAllocator>::BitwiseXOR(A.Bits, B.Bits, InFlags);
		return Result;
	}

	inline static FComponentMask BitwiseNOT(const FComponentMask& A)
	{
		FComponentMask Result = A;
		Result.Bits.BitwiseNOT();
		return Result;
	}

	inline void BitwiseNOT()
	{
		Bits.BitwiseNOT();
	}

	bool Contains(FComponentTypeID InComponentType) const;
	bool ContainsAll(const FComponentMask& InComponentMask) const;
	bool ContainsAny(const FComponentMask& InComponentMask) const;

	void Set(FComponentTypeID InComponentType);
	void SetAll(std::initializer_list<FComponentTypeID> InComponentTypes);
	void SetAllLenient(std::initializer_list<FComponentTypeID> InComponentTypes);

	void Remove(FComponentTypeID InComponentType);

	/**
	 * Find the first component type ID in this mask, or Invalid if the mask is empty.
	 */
	FComponentTypeID First() const;

	inline void  Reset()
	{
		Bits.Reset();
	}

	inline int32 Num() const
	{
		return Bits.Num();
	}

	inline int32 Find(bool bValue) const
	{
		return Bits.Find(bValue);
	}

	inline int32 NumComponents() const
	{
		return Bits.CountSetBits();
	}

	inline void PadToNum(int32 Num, bool bPadValue)
	{
		Bits.PadToNum(Num, bPadValue);
	}

private:

	TBitArray<FComponentMaskAllocator> Bits;

};

struct FComponentTypeID
{
	FComponentTypeID()
		: Value(INVALID)
	{}

	static FComponentTypeID Invalid()
	{
		return FComponentTypeID(INVALID);
	}

	static FComponentTypeID FromBitIndex(int32 BitIndex)
	{
		check( (BitIndex & 0xFFFF0000) == 0 );
		return FComponentTypeID((uint16)BitIndex);
	}

	template<typename T>
	TComponentTypeID<T> ReinterpretCast() const;

	explicit operator bool() const
	{
		return Value != INVALID;
	}

	friend bool operator<(FComponentTypeID A, FComponentTypeID B)
	{
		return A.Value < B.Value;
	}

	friend bool operator==(FComponentTypeID A, FComponentTypeID B)
	{
		return A.Value == B.Value;
	}

	friend bool operator!=(FComponentTypeID A, FComponentTypeID B)
	{
		return A.Value != B.Value;
	}

	friend uint32 GetTypeHash(FComponentTypeID In)
	{
		return ::GetTypeHash(In.Value);
	}

	int32 BitIndex() const
	{
		return Value;
	}

	friend FComponentMask operator|(FComponentTypeID A, FComponentTypeID B)
	{
		check(A.Value != INVALID && B.Value != INVALID);

		FComponentMask Mask;
		Mask.Set(A);
		Mask.Set(B);
		return Mask;
	}

	friend FComponentMask& operator|(FComponentMask& A, FComponentTypeID B)
	{
		check(B.Value != INVALID);
		A.Set(B);
		return A;
	}

protected:

	explicit FComponentTypeID(uint16 InValue)
		: Value(InValue)
	{}

	static const uint16 INVALID = uint16(-1);
	uint16 Value;
};

struct FComponentTypeIDFilter
{
	FComponentTypeIDFilter()
		: Value(INVALID)
		, PassFlag(false)
	{}

	FComponentTypeIDFilter(FComponentTypeID InComponentType, bool bInPassFlag)
		: Value(InComponentType ? uint16(InComponentType.BitIndex()) : INVALID)
		, PassFlag(bInPassFlag)
	{
	}

	MOVIESCENE_API bool Passes(const FComponentMask& Type) const;

	FComponentTypeID GetComponentType() const
	{
		if (Value == INVALID)
		{
			return FComponentTypeID();
		}
		return FComponentTypeID::FromBitIndex(Value);
	}

private:
	static const uint16 INVALID = uint16(-1) >> 1u;

	uint16 Value:15;
	uint16 PassFlag:1;
};

template<typename T>
struct TComponentTypeID : public FComponentTypeID
{
	typedef T Type;

	TComponentTypeID()
	{}

	static TComponentTypeID FromBitIndex(int32 BitIndex)
	{
		check( (BitIndex & 0xFFFF0000) == 0 );
		return TComponentTypeID(static_cast<uint16>(BitIndex));
	}
private:

	explicit TComponentTypeID(uint16 InValue)
		: FComponentTypeID(InValue)
	{}
};

struct FMovieSceneEntityID
{
	FMovieSceneEntityID()
		: Value(INVALID)
	{}

	explicit operator bool() const
	{
		return IsValid();
	}

	friend bool operator==(FMovieSceneEntityID A, FMovieSceneEntityID B)
	{
		return A.Value == B.Value;
	}

	friend bool operator!=(FMovieSceneEntityID A, FMovieSceneEntityID B)
	{
		return A.Value != B.Value;
	}

	friend bool operator<(FMovieSceneEntityID A, FMovieSceneEntityID B)
	{
		return A.Value < B.Value;
	}

	friend uint32 GetTypeHash(FMovieSceneEntityID In)
	{
		return ::GetTypeHash(In.Value);
	}

	static FMovieSceneEntityID Max()
	{
		return FMovieSceneEntityID((uint16)-2);
	}

	static FMovieSceneEntityID FromIndex(int32 Index)
	{
		check( (Index & 0xFFFF0000) == 0 );
		return FMovieSceneEntityID((uint16)Index);
	}

	static FMovieSceneEntityID Invalid()
	{
		return FMovieSceneEntityID();
	}

	int32 AsIndex() const
	{
		return Value;
	}

	bool IsValid() const
	{
		return Value != INVALID;
	}

private:

	explicit FMovieSceneEntityID(uint16 SpecifiedValue)
		: Value(SpecifiedValue)
	{}

	static const uint16 INVALID = uint16(-1);

	uint16 Value;
};


struct FEntityHandle
{
	FEntityHandle()
		: HandleGeneration(0)
	{}

	bool IsEmpty() const
	{
		return ID.IsValid() == false;
	}

	MOVIESCENE_API bool IsValid(const FEntityManager&) const;

	friend bool operator<(FEntityHandle A, FEntityHandle B)
	{
		if (A.ID == B.ID)
		{
			return A.HandleGeneration < B.HandleGeneration;
		}
		return A.ID < B.ID;
	}

	friend bool operator==(FEntityHandle A, FEntityHandle B)
	{
		return A.ID == B.ID && A.HandleGeneration == B.HandleGeneration;
	}

	friend uint32 GetTypeHash(FEntityHandle In)
	{
		return GetTypeHash(In.ID);
	}

	FMovieSceneEntityID GetEntityID() const
	{
		return ID;
	}

private:

	explicit FEntityHandle(FMovieSceneEntityID InEntityID, uint32 InHandleGeneration)
		: HandleGeneration(InHandleGeneration), ID(InEntityID)
	{}

	friend FEntityManager;
	uint32 HandleGeneration;
	FMovieSceneEntityID ID;
};


inline FBitReference FComponentMask::operator[](FComponentTypeID ComponentType)
{
	return Bits[ComponentType.BitIndex()];
}

inline FConstBitReference FComponentMask::operator[](FComponentTypeID ComponentType) const
{
	return Bits[ComponentType.BitIndex()];
}

inline bool FComponentMask::Contains(FComponentTypeID InComponentType) const
{
	return InComponentType && Bits.IsValidIndex(InComponentType.BitIndex()) && Bits[InComponentType.BitIndex()] == true;
}

inline bool FComponentMask::ContainsAll(const FComponentMask& InComponentMask) const
{
	return FComponentMask::BitwiseAND(*this, InComponentMask, EBitwiseOperatorFlags::MinSize).CompareSetBits(InComponentMask);
}

inline bool FComponentMask::ContainsAny(const FComponentMask& InComponentMask) const
{
	return FComponentMask::BitwiseAND(*this, InComponentMask, EBitwiseOperatorFlags::MinSize).Find(true) != INDEX_NONE;
}

inline void FComponentMask::Set(FComponentTypeID InComponentType)
{
	checkSlow(InComponentType);

	const int32 BitIndex = InComponentType.BitIndex();

	Bits.PadToNum(BitIndex + 1, false);
	Bits[BitIndex] = true;
}

inline void FComponentMask::SetAll(std::initializer_list<FComponentTypeID> InComponentTypes)
{
	for (FComponentTypeID ComponentType : InComponentTypes)
	{
		checkSlow(ComponentType);

		const int32 BitIndex = ComponentType.BitIndex();

		Bits.PadToNum(BitIndex + 1, false);
		Bits[BitIndex] = true;
	}
}

inline void FComponentMask::SetAllLenient(std::initializer_list<FComponentTypeID> InComponentTypes)
{
	for (FComponentTypeID ComponentType : InComponentTypes)
	{
		if (ComponentType)
		{
			const int32 BitIndex = ComponentType.BitIndex();

			Bits.PadToNum(BitIndex + 1, false);
			Bits[BitIndex] = true;
		}
	}
}

inline void FComponentMask::Remove(FComponentTypeID InComponentType)
{
	checkSlow(InComponentType);
	if (Bits.Num() > InComponentType.BitIndex())
	{
		Bits[InComponentType.BitIndex()] = false;
	}
}

inline FComponentTypeID FComponentMask::First() const
{
	const int32 FirstBit = Bits.Find(true);
	if (FirstBit != INDEX_NONE)
	{
		return FComponentTypeID::FromBitIndex(FirstBit);
	}
	return FComponentTypeID::Invalid();
}

template<typename T>
inline TComponentTypeID<T> FComponentTypeID::ReinterpretCast() const
{
	return TComponentTypeID<T>::FromBitIndex(BitIndex());
}

} // namespace MovieScene
} // namespace UE
