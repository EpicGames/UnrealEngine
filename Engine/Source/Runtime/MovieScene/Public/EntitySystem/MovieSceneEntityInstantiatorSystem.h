// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "MovieSceneInstanceRegistry.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneEntityInstantiatorSystem.generated.h"

class UMovieSceneEntitySystem;
class UMovieSceneEntitySystemLinker;
class UObject;
namespace UE { namespace MovieScene { template <typename T> struct TComponentTypeID; } }
struct FGuid;
struct FMovieSceneObjectBindingID;

UCLASS(Abstract, MinimalAPI)
class UMovieSceneEntityInstantiatorSystem : public UMovieSceneEntitySystem
{
public:
	GENERATED_BODY()

	MOVIESCENE_API UMovieSceneEntityInstantiatorSystem(const FObjectInitializer& ObjInit);

	MOVIESCENE_API void UnlinkStaleObjectBindings(UE::MovieScene::TComponentTypeID<FGuid> BindingType);
	MOVIESCENE_API void UnlinkStaleObjectBindings(UE::MovieScene::TComponentTypeID<FMovieSceneObjectBindingID> BindingType);
};
