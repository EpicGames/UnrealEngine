// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "CoreTypes.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "Evaluation/MovieSceneEvaluationField.h"
#include "UObject/WeakObjectPtr.h"

class IMovieScenePlayer;
class UMovieSceneEntitySystem;
class UMovieSceneEntitySystemLinker;
struct FMovieSceneEntityComponentField;

namespace UE
{
namespace MovieScene
{
struct FEntityImportSequenceParams;
struct IMovieScenePerEntityMutation;

enum class EUnlinkEverythingMode
{
	Normal,
	CleanGarbage,
};

/**
 * An entity ledger is responsible for tracking the entites that have been imported for the currently relevant frame of a sequence instance.
 * It is responsible for linking and unlinking new and expired entities in the linker's entity manager based on the current set of entities required for evaluation.
 */
struct FEntityLedger
{
	UE_DEPRECATED(5.5, "Please call the version that takes OutPerTickConditionalEntities and ConditionResultCache")
	MOVIESCENE_API void UpdateEntities(UMovieSceneEntitySystemLinker* Linker, const FEntityImportSequenceParams& ImportParams, const FMovieSceneEntityComponentField* EntityField, const FMovieSceneEvaluationFieldEntitySet& NewEntities);
	
	/**
	 * To be called any time this ledger's instance is to be evaluated with a different set of entities - updates the set of entities that are required for the current evaluation environment
	 *
	 * @param Linker					The linker that owns this ledger
	 * @param ImportParams				Basis for import parameters
	 * @param EntityField				Possibly null if NewEntities is empty- an entity field containing structural information about the sequence
	 * @param NewEntities				A set specifying all the entities required for the next evaluation. Specifying an empty set will unlink all existing entities.
	 * @param OutConditionalEntities    Output set returning conditional entities that require re-evaluating in between full updates.
	 * @param ConditionResultCache      Cache of previous condition results, to be used and potentially modified during any condition checking in updating entities.
	 */
	MOVIESCENE_API void UpdateEntities(UMovieSceneEntitySystemLinker* Linker, const FEntityImportSequenceParams& ImportParams, const FMovieSceneEntityComponentField* EntityField, const FMovieSceneEvaluationFieldEntitySet& NewEntities, FMovieSceneEvaluationFieldEntitySet& OutPerTickConditionalEntities, TMap<uint32, bool>& ConditionResultCache);

	
	UE_DEPRECATED(5.5, "Please call the version that takes ConditionResultCache")
	MOVIESCENE_API void UpdateOneShotEntities(UMovieSceneEntitySystemLinker* Linker, const FEntityImportSequenceParams& ImportParams, const FMovieSceneEntityComponentField* EntityField, const FMovieSceneEvaluationFieldEntitySet& NewEntities);
	
	/**
	 * Update any one-shot entities for the current frame
	 *
	 * @param Linker         The linker that owns this ledger
	 * @param ImportParams   Basis for import parameters
	 * @param EntityField    Possibly null if NewEntities is empty- an entity field containing structural information about the sequence
	 * @param NewEntities    A set specifying all the entities required for the next evaluation. Specifying an empty set will unlink all existing entities.
	 * @param ConditionResultCache      Cache of previous condition results, to be used and potentially modified during any condition checking in updating entities.
	 */
	MOVIESCENE_API void UpdateOneShotEntities(UMovieSceneEntitySystemLinker* Linker, const FEntityImportSequenceParams& ImportParams, const FMovieSceneEntityComponentField* EntityField, const FMovieSceneEvaluationFieldEntitySet& NewEntities, TMap<uint32, bool>& ConditionResultCache);

	/**
	 * Update any conditional entities for the current frame
	 *
	 * @param Linker				The linker that owns this ledger
	 * @param ImportParams			Basis for import parameters
	 * @param EntityField			Possibly null if NewEntities is empty- an entity field containing structural information about the sequence
	 * @param ConditionalEntities   A set specifying any conditional entities that need to be checked again and possibly imported or removed.
	 */
	MOVIESCENE_API void UpdateConditionalEntities(UMovieSceneEntitySystemLinker* Linker, const FEntityImportSequenceParams& ImportParams, const FMovieSceneEntityComponentField* EntityField, const FMovieSceneEvaluationFieldEntitySet& ConditionalEntities);

	/**
	 * Invalidate any and all entities that are currently being tracked, causing new linker entities to be created on the next evaluation, and ones to become unlinked (preserving any components with the preserve flag)
	 */
	MOVIESCENE_API void Invalidate();

	/*
	* Check whether the ledger has been invalidated.
	*/
	bool IsInvalidated() const { return bInvalidated;}

public:

	/**
	 * Check whether this ledger contains any information at all (ie is tracking any global entities, even if it has not created any linker entities for them)
	 */
	MOVIESCENE_API bool IsEmpty() const;

	/**
	 * Check whether the specified entity is being tracked by this ledger at all
	 */
	MOVIESCENE_API bool HasImportedEntity(const FMovieSceneEvaluationFieldEntityKey& EntityKey) const;

	/**
	 * Find an imported entity
	 */
	MOVIESCENE_API FMovieSceneEntityID FindImportedEntity(const FMovieSceneEvaluationFieldEntityKey& EntityKey) const;

	/**
	 * Find imported entities
	 */
	MOVIESCENE_API void FindImportedEntities(TWeakObjectPtr<UObject> EntityOwner, TArray<FMovieSceneEntityID>& OutEntityIDs) const;

	/*
	* Checks to see whether we can currently import the requested entity, evaluating any conditions that may exist first.
	*/
	MOVIESCENE_API bool CanImportEntity(UMovieSceneEntitySystemLinker* Linker, const FEntityImportSequenceParams& ImportParams, const FMovieSceneEntityComponentField* EntityField, const FMovieSceneEvaluationFieldEntityQuery& Query, FMovieSceneEvaluationFieldEntitySet& OutPerTickConditionalEntities, TMap<uint32, bool>& ConditionResultCache, bool bUpdatingPerTickEntities=false);

	
	UE_DEPRECATED(5.5, "Please call the version that takes OutPerTickConditionalEntities and ConditionResultCache")
	MOVIESCENE_API void ImportEntity(UMovieSceneEntitySystemLinker* Linker, const FEntityImportSequenceParams& ImportParams, const FMovieSceneEntityComponentField* EntityField, const FMovieSceneEvaluationFieldEntityQuery& Query);

	/**
	 * Indicate that the specified field entity is currently being evaluated
	 *
	 * @param Linker         The linker To import into
	 * @param InstanceHandle A handle to the sequence instance that the entity relates to (relating to Linker->GetInstanceRegistry())
	 * @param Entity         The field entity that is being imported
	 * @param OutConditionalEntities    Output set returning conditional entities that require re-evaluating in between full updates.
	 * @param ConditionResultCache      Cache of previous condition results, to be used and potentially modified during any condition checking in updating entities.
	 */
	MOVIESCENE_API void ImportEntity(UMovieSceneEntitySystemLinker* Linker, const FEntityImportSequenceParams& ImportParams, const FMovieSceneEntityComponentField* EntityField, const FMovieSceneEvaluationFieldEntityQuery& Query, FMovieSceneEvaluationFieldEntitySet& OutPerTickConditionalEntities, TMap<uint32, bool>& ConditionResultCache);

	/**
	 * Unlink all imported linker entities and their children, whilst maintaining the map of imported entities
	 *
	 * @param Linker The linker that owns this ledger
	 */
	MOVIESCENE_API void UnlinkEverything(UMovieSceneEntitySystemLinker* Linker, EUnlinkEverythingMode Garbage = EUnlinkEverythingMode::Normal);

	/**
	 * Unlink all imported one-shot linker entities and their children and clear the list of one shots
	 *
	 * @param Linker The linker that owns this ledger
	 * @return A mask representing the changes made to the environment
	 */
	MOVIESCENE_API void UnlinkOneShots(UMovieSceneEntitySystemLinker* Linker);


	/**
	 * Check whether any of the entities in this ledger or their children match the specified filter
	 */
	MOVIESCENE_API bool Contains(UMovieSceneEntitySystemLinker* Linker, const FEntityComponentFilter& Filter) const;

	/**
	 * Mutate all the entities within this ledger by using the specified filter
	 */
	MOVIESCENE_API void MutateAll(UMovieSceneEntitySystemLinker* Linker, const FEntityComponentFilter& Filter, const IMovieScenePerEntityMutation& Mutation) const;

public:

	/**
	 * Called in order to tag garbage as NeedsUnlink
	 *
	 * @param Linker         The linker that owns this ledger
	 */
	MOVIESCENE_API void TagGarbage(UMovieSceneEntitySystemLinker* Linker);

	/**
	 * Remove linker entity IDs that exist in the specified set since they are no longer valid
	 *
	 * @param LinkerEntities Set of entity IDs that have been destroyed
	 */
	MOVIESCENE_API void CleanupLinkerEntities(const TSet<FMovieSceneEntityID>& LinkerEntities);

private:

	struct FImportedEntityData
	{
		int32 MetaDataIndex;
		FMovieSceneEntityID EntityID;
	};

	/** Map of source entities that were swept this frame */
	TArray<FMovieSceneEntityID> OneShotEntities;

	/** Map of source field entity key -> imported linker entities */
	TMap<FMovieSceneEvaluationFieldEntityKey, FImportedEntityData> ImportedEntities;

	/** Whether we have been invalidated, and need to re-instantiate everything */
	bool bInvalidated;
};


} // namespace MovieScene
} // namespace UE
