// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/AllOf.h"
#include "Algo/AnyOf.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "EntitySystem/MovieSceneComponentTypeHandler.h"
#include "EntitySystem/MovieSceneComponentTypeInfo.h"
#include "EntitySystem/MovieSceneEntityFactoryTypes.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "EntitySystem/MovieSceneMaybeAtomic.h"
#include "Evaluation/MovieScenePlayback.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformCrt.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/InlineValue.h"
#include "MovieSceneSequenceID.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/StrongObjectPtr.h"
#include "UObject/UObjectArray.h"

#include <initializer_list>

class FArchive;
class FReferenceCollector;
class UObjectBase;
struct FGuid;
struct FMovieSceneExportedEntity;

namespace UE
{
namespace MovieScene
{

struct FChildEntityInitializer;
struct FComponentRegistry;
struct FEntityAllocationIteratorProxy;
struct FFreeEntityOperation;
struct FMutualEntityInitializer;
struct IComponentTypeHandler;
struct IMovieSceneEntityMutation;
template <typename T> struct TReadOptional;
template <typename T> struct TWriteOptional;
struct IMovieSceneConditionalEntityMutation;

enum class EMutuallyInclusiveComponentType : uint8;

enum class EEntityRecursion : uint8
{
	/** Perform the action on this entity (ie, the entity passed in) */
	This      = 1 << 0,
	/** Perform the action on this entity's children recursively */
	Children  = 1 << 1,
	/** Perform the action on this entity and all its children recursively */
	Full      = This | Children,
};
ENUM_CLASS_FLAGS(EEntityRecursion);

/**
 * Top-level manager class that is responsible for all entity data and interaction/
 *
 * An entity is a stable index into the manager's EntityLocations, which defines the location for that entity's data. This allows the manager to relocate entity data at will without invalidating client held entity IDs.
 * An entity may contain 0 or more components. Components are concrete pieces of data with a unique type identifier. Additionally, tags can be added to entities or whole batches of entities with zero memory overhead.
 *
 * Entity Component data is stored in allocations organized by each unique combination of components. Each component type is stored as a contiguous array within each allocation, with each entity's component being a fixed
 * offset from the start of each component array. This enables efficient read/write access into specific component arrays and makes issuing parallel tasks that require component data trivial. See FEntityAllocation for
 * a more detailed explanation of entity component data layout.
 */
class FEntityManager : public FUObjectArray::FUObjectDeleteListener
{
public:

	MOVIESCENE_API FEntityManager();
	MOVIESCENE_API ~FEntityManager();

	FEntityManager(const FEntityManager&) = delete;
	void operator=(const FEntityManager&) = delete;

	FEntityManager(FEntityManager&&) = delete;
	void operator=(FEntityManager&&) = delete;


	FComponentRegistry* GetComponents() const
	{
		return ComponentRegistry;
	}

	void SetComponentRegistry(FComponentRegistry* InComponents)
	{
		ComponentRegistry = InComponents;
	}

public:


	/**
	 * Destroy this entity manager and all the entities and components contained within it, resetting it back to its default state
	 */
	MOVIESCENE_API void Destroy();


	/**
	 * Allocate a new entity with no components
	 *
	 * @return A stable ID that relates to this entity. Will remain valid until the entity is freed
	 */
	MOVIESCENE_API FMovieSceneEntityID AllocateEntity();


	/**
	 * Free an entity and relinquish its entity ID
	 *
	 * @param EntityID  A valid entity ID to free
	 * @return The number of entities released
	 */
	MOVIESCENE_API void FreeEntity(FMovieSceneEntityID EntityID);


	/**
	 * Free all entities that match the specified filter
	 *
	 * @param Filter            A filter that defines the entities to free. Any entity that passes the filter will be destroyed.
	 * @param OutFreedEntities (Optional) A set to populate with all the entities that were freed (including any children)
	 * @return The number of entities released
	 */
	MOVIESCENE_API int32 FreeEntities(const FEntityComponentFilter& Filter, TSet<FMovieSceneEntityID>* OutFreedEntities = nullptr);


	/**
	 * Free all entities defined by the specified operation
	 *
	 * @param Operation        A pre-populated free entity operation definition
	 * @param OutFreedEntities (Optional) A set to populate with all the entities that were freed (including any children)
	 * @return The number of entities released
	 */
	MOVIESCENE_API int32 FreeEntities(const FFreeEntityOperation& Operation, TSet<FMovieSceneEntityID>* OutFreedEntities = nullptr);


	/**
	 * Attempt to allocate a number of entities of the same type contiguously in memory
	 * @note May allocate fewer than the desired number due to allocation capacity constraints. As such, subsequent calls may be required.
	 *
	 * @param EntityComponentMask    Defines the components that should exist on the allocated components. Set bits denote allocated components and tags.
	 * @param InOutNum               A valid pointer to the desired number to allocate. Is overwritten with the number that were actually allocated.
	 * @return A structure that points to the first entity that was allocated.
	 */
	MOVIESCENE_API FEntityDataLocation AllocateContiguousEntities(const FComponentMask& EntityComponentMask, int32* InOutNum);


	/**
	 * Allocate a single entity with the specified components
	 *
	 * @param EntityComponentMask    Defines the components that should exist on the allocated components. Set bits denote allocated components and tags.
	 * @return A structure that points to the allocated entity.
	 */
	MOVIESCENE_API FEntityInfo AllocateEntity(const FComponentMask& EntityComponentMask);


	/**
	 * Retrieve an entity's allocation and component offset from its ID
	 *
	 * @return A structure that points to the allocated entity.
	 */
	MOVIESCENE_API FEntityInfo GetEntity(FMovieSceneEntityID EntityID) const;


	/**
	 * Retrieve a handle to an entity
	 *
	 * @return A handle that points to the allocated entity.
	 */
	MOVIESCENE_API FEntityHandle GetEntityHandle(FMovieSceneEntityID EntityID);


	/**
	 * Check whether an entity is allocated.
	 * @note: does not check serial numbers - should only be used when you are certain that another entity can not have been allocated over the top of the specified entity ID
	 *
	 * @return true if the entity ID is allocated, false otherwise
	 */
	bool IsAllocated(FMovieSceneEntityID EntityID) const
	{
		return EntityID && EntityLocations.IsValidIndex(EntityID.AsIndex());
	}


	/**
	 * Check whether the specified entity handle is still valid
	 *
	 * @return true if the entity ID is allocated, false otherwise
	 */
	MOVIESCENE_API bool IsHandleValid(FEntityHandle EntityID) const;


	/**
	 * Compute and return this entity manager's threading model. Does not change the current cached threading model.
	 */
	MOVIESCENE_API EEntityThreadingModel ComputeThreadingModel() const;


	/**
	 * Compute and store the current threading model.
	 */
	MOVIESCENE_API void UpdateThreadingModel();


	/**
	 * Get this entitiy manager's current threading model based on the last time UpdateThreadingModel was called.
	 */
	MOVIESCENE_API EEntityThreadingModel GetThreadingModel() const;

public:


	/**
	 * Add the specified component value to an entity
	 *
	 * @param EntityID         The ID of the entity to add the component to
	 * @param ComponentTypeID  The ID of the component type to add to the entity
	 * @param InValue          The value of the component to add
	 */
	template<typename T, typename ValueType>
	void AddComponent(FMovieSceneEntityID EntityID, TComponentTypeID<T> ComponentTypeID, ValueType&& InValue)
	{
		AddComponent(EntityID, ComponentTypeID);

		FEntityInfo Entry = GetEntity(EntityID);
		if (Entry.Data.Allocation != nullptr)
		{
			const FComponentHeader& Header = Entry.Data.Allocation->GetComponentHeaderChecked(ComponentTypeID);

			FScopedHeaderWriteLock WriteLock(&Header, Entry.Data.Allocation->GetCurrentLockMode(), FEntityAllocationWriteContext::NewAllocation());

			T* Component = reinterpret_cast<T*>(Header.GetValuePtr(Entry.Data.ComponentOffset));
			*Component = Forward<ValueType>(InValue);
		}
	}


	/**
	 * Add the specified component type to an entity. The component value will be default-initialized.
	 *
	 * @param EntityID         The ID of the entity to add the component to
	 * @param ComponentTypeID  The ID of the component type to add to the entity
	 */
	MOVIESCENE_API void AddComponent(FMovieSceneEntityID EntityID, FComponentTypeID ComponentTypeID);
	MOVIESCENE_API void AddComponent(FMovieSceneEntityID EntityID, FComponentTypeID ComponentTypeID, EEntityRecursion Recursion);


	/**
	 * Add the specified components to an entity. Existing components will remain unchanged. New component values will be default-initialized.
	 *
	 * @param EntityID             The ID of the entity to add the components to
	 * @param EntityComponentMask  A mask constituting the components that should be added (set bits indicate components to add)
	 */
	MOVIESCENE_API void AddComponents(FMovieSceneEntityID EntityID, const FComponentMask& EntityComponentMask);
	MOVIESCENE_API void AddComponents(FMovieSceneEntityID EntityID, const FComponentMask& EntityComponentMask, EEntityRecursion Recursion);


	/**
	 * Remove the specified component type from an entity if it exists.
	 *
	 * @param EntityID         The ID of the entity to remove the component from
	 * @param ComponentTypeID  The ID of the component type to remove from the entity
	 */
	MOVIESCENE_API void RemoveComponent(FMovieSceneEntityID EntityID, FComponentTypeID ComponentTypeID);
	MOVIESCENE_API void RemoveComponent(FMovieSceneEntityID EntityID, FComponentTypeID ComponentTypeID, EEntityRecursion Recursion);


	/**
	 * Remove the specified components from an entity, if they exist.
	 *
	 * @param EntityID             The ID of the entity to remove the components from
	 * @param ComponentsToRemove   A mask constituting the components that should be removed (set bits indicate components to add)
	 */
	MOVIESCENE_API void RemoveComponents(FMovieSceneEntityID EntityID, const FComponentMask& ComponentsToRemove);
	MOVIESCENE_API void RemoveComponents(FMovieSceneEntityID EntityID, const FComponentMask& ComponentsToRemove, EEntityRecursion Recursion);


	/**
	 * Copy the specified component type from an entity if it exists.
	 *
	 * @param SrcEntityID      The ID of the entity to copy from
	 * @param DstEntityID      The ID of the entity to copy to
	 * @param ComponentTypeID  The ID of the component type to copy. Must exist on SrcEntityID
	 */
	MOVIESCENE_API bool CopyComponent(FMovieSceneEntityID SrcEntityID, FMovieSceneEntityID DstEntityID, FComponentTypeID ComponentTypeID);


	/**
	 * Copy any and all the specified component types from one entity onto another. Missing components on the source entity are handled gracefully.
	 *
	 * @param SrcEntityID      The ID of the entity to copy from
	 * @param DstEntityID      The ID of the entity to copy to
	 * @param ComponentsToCopy  A mask constituting the components that should be copied (set bits indicate components to copy)
	 */
	MOVIESCENE_API void CopyComponents(FMovieSceneEntityID SrcEntityID, FMovieSceneEntityID DstEntityID, const FComponentMask& ComponentsToCopy);


	/**
	 * Check whether the specified component has a component of the specified type
	 *
	 * @param EntityID             The ID of the entity to check
	 * @param ComponentTypeID      The type of the component that is being tested for
	 */
	MOVIESCENE_API bool HasComponent(FMovieSceneEntityID EntityID, FComponentTypeID ComponentTypeID) const;


	/**
	 * Retrieve the type mask for this component
	 *
	 * @param InEntity  The ID of the entity
	 * @return The type mask for this entity, or an empty mask if it has no components
	 */
	MOVIESCENE_API const FComponentMask& GetEntityType(FMovieSceneEntityID InEntity) const;


	/**
	 * Changes the components that exist on an entity to a new mask
	 *
	 * @param InEntity   The ID of the entity
	 * @param InNewMask  The new mask of the entity that defines which components it should have
	 */
	MOVIESCENE_API void ChangeEntityType(FMovieSceneEntityID InEntity, const FComponentMask& InNewMask);


	/**
	 * Remove all but the specified components from an entity.
	 *
	 * @param EntityID             The ID of the entity to remove the components from
	 * @param EntitiesToKeep       A mask constituting the components that should be kept on the entity
	 */
	MOVIESCENE_API void FilterComponents(FMovieSceneEntityID EntityID, const FComponentMask& EntitiesToKeep);


	/**
	 * Combine the components from one entity into another, overwriting any pre-existing component values. Does not free any entities.
	 *
	 * @param DestinationEntityID  The ID of the entity to add new components to
	 * @param SourceEntityID       The ID of the entity to copy components from
	 * @param OptionalMask         (Optional) A mask constituting the components that should be copied
	 */
	MOVIESCENE_API void CombineComponents(FMovieSceneEntityID DestinationEntityID, FMovieSceneEntityID SourceEntityID, const FComponentMask* OptionalMask = nullptr);


	/**
	 * Duplicate an entity, by creating an exact copy with a new ID
	 *
	 * @param InOther  The ID of the entity to duplicate
	 * @return An ID to a new entity that has exactly the same combination and value of components as InOther
	 */
	MOVIESCENE_API FMovieSceneEntityID DuplicateEntity(FMovieSceneEntityID InOther);


	/**
	 * Duplicate an entity over the top of an existing entity ID
	 *
	 * @param InOutEntity          The ID of the entity to overwrite. Doesn't have to be valid.
	 * @param InEntityToDuplicate  The ID of the entity to duplicate
	 */
	MOVIESCENE_API void OverwriteEntityWithDuplicate(FMovieSceneEntityID& InOutEntity, FMovieSceneEntityID InEntityToDuplicate);


	/**
	 * Defines a new child initializer that applies only to entities factoried within this entity manager
	 *
	 * @param InInitializer        The initializer to insert
	 * @return An index into the array of initializers that should be used for removal
	 */
	int32 DefineInstancedChildInitializer(TInlineValue<FChildEntityInitializer>&& InInitializer)
	{
		check(InInitializer.IsValid());
		return InstancedChildInitializers.Add(MoveTemp(InInitializer));
	}

	/**
	 * Runs all initializers for the specified parent/child allocation
	 */
	MOVIESCENE_API void InitializeChildAllocation(const FComponentMask& ParentType, const FComponentMask& ChildType, const FEntityAllocation* ParentAllocation, TArrayView<const int32> ParentAllocationOffsets, const FEntityRange& InChildEntityRange);

	/**
	 * Destroy a previously registered instanced child initializer using its index
	 */
	void DestroyInstancedChildInitializer(int32 Index)
	{
		InstancedChildInitializers.RemoveAt(Index);
	}

	/**
	 * Run through all entities in this entity manager, ensuring that all mutual components exist
	 */
	MOVIESCENE_API void AddMutualComponents();

	/**
	 * Run through all entities in this entity manager, ensuring that all mutual components exist for any component types that match the specified filter
	 */
	MOVIESCENE_API void AddMutualComponents(const FEntityComponentFilter& InFilter);

public:


	/**
	 * Attempt to read a component from an entity.
	 * @note this is a general purpose convenience utility which should not be used for high-performance runtime code. See FEntityTaskBuilder.
	 *
	 * @param Entity               The ID of the entity to read from
	 * @param ComponentTypeID      The component that is to be read
	 * @return A scoped component ptr that points directly to the entity's component, or nullptr if it does not exist
	 */
	template<typename T>
	TComponentLock<TReadOptional<T>> ReadComponent(FMovieSceneEntityID Entity, TComponentTypeID<T> ComponentTypeID) const
	{
		check(Entity);

		if (!ComponentTypeID)
		{
			return TComponentLock<TReadOptional<T>>();
		}

		FEntityLocation Location = EntityLocations[Entity.AsIndex()];
		if (!Location.IsValid() || !EntityAllocationMasks[Location.GetAllocationIndex()].Contains(ComponentTypeID))
		{
			return TComponentLock<TReadOptional<T>>();
		}

		FEntityAllocation* Allocation = EntityAllocations[Location.GetAllocationIndex()];
		const int32 ComponentOffset = Location.GetEntryIndexWithinAllocation();

		EComponentHeaderLockMode LockMode = GetThreadingModel() == EEntityThreadingModel::NoThreading
			? EComponentHeaderLockMode::LockFree
			: EComponentHeaderLockMode::Mutex;

		for (FComponentHeader& Header : Allocation->GetComponentHeaders())
		{
			if (Header.ComponentType == ComponentTypeID)
			{
				return TComponentLock<TReadOptional<T>>(&Header, LockMode, ComponentOffset);
			}
		}

		return TComponentLock<TReadOptional<T>>();
	}


	/**
	 * Read a component value from an entity that is known to exist.
	 * @note this is a general purpose convenience utility which should not be used for high-performance runtime code. See FEntityTaskBuilder.
	 *
	 * @param Entity               The ID of the entity to read from
	 * @param ComponentTypeID      The component that is to be read
	 * @return The component value
	 */
	template<typename T>
	T ReadComponentChecked(FMovieSceneEntityID Entity, TComponentTypeID<T> ComponentTypeID) const
	{
		TComponentLock<TReadOptional<T>> Value = ReadComponent(Entity, ComponentTypeID);
		check(Value);
		return *Value;
	}


	/**
	 * Attempt to write to an entity's component.
	 * @note this is a general purpose convenience utility which should not be used for high-performance runtime code. See FEntityTaskBuilder.
	 *
	 * @param Entity               The ID of the entity to read from
	 * @param ComponentTypeID      The component that is to be read
	 * @return A scoped component ptr that points directly to the entity's component, or nullptr if it does not exist
	 */
	template<typename T>
	TComponentLock<TWriteOptional<T>> WriteComponent(FMovieSceneEntityID Entity, TComponentTypeID<T> ComponentTypeID)
	{
		check(Entity);

		if (!ComponentTypeID)
		{
			return TComponentLock<TWriteOptional<T>>();
		}

		FEntityLocation Location = EntityLocations[Entity.AsIndex()];
		if (!Location.IsValid() || !EntityAllocationMasks[Location.GetAllocationIndex()].Contains(ComponentTypeID))
		{
			return TComponentLock<TWriteOptional<T>>();
		}

		FEntityAllocation* Allocation = EntityAllocations[Location.GetAllocationIndex()];
		const int32 ComponentOffset = Location.GetEntryIndexWithinAllocation();

		EComponentHeaderLockMode LockMode = GetThreadingModel() == EEntityThreadingModel::NoThreading
			? EComponentHeaderLockMode::LockFree
			: EComponentHeaderLockMode::Mutex;

		for (FComponentHeader& Header : Allocation->GetComponentHeaders())
		{
			if (Header.ComponentType == ComponentTypeID)
			{
				return TComponentLock<TWriteOptional<T>>(&Header, LockMode, FEntityAllocationWriteContext(*this), ComponentOffset);
			}
		}

		return TComponentLock<TWriteOptional<T>>();
	}


	/**
	 * Write a component value that is known to exist to an entity.
	 * @note this is a general purpose convenience utility which should not be used for high-performance runtime code. See FEntityTaskBuilder.
	 *
	 * @param Entity               The ID of the entity to read from
	 * @param ComponentTypeID      The component that is to be read
	 * @param Value                The value to write
	 */
	template<typename T, typename ValueType>
	void WriteComponentChecked(FMovieSceneEntityID Entity, TComponentTypeID<T> ComponentTypeID, ValueType&& Value)
	{
		TComponentLock<TWriteOptional<T>> ComponentPtr = WriteComponent(Entity, ComponentTypeID);
		check(ComponentPtr);
		*ComponentPtr = Forward<ValueType>(Value);
	}

public:

	void SetGatherThread(ENamedThreads::Type InGatherThread)
	{
		GatherThread = InGatherThread;
	}

	ENamedThreads::Type GetGatherThread() const
	{
		return GatherThread;
	}

	void SetDispatchThread(ENamedThreads::Type InDispatchThread)
	{
		DispatchThread = InDispatchThread;
	}

	ENamedThreads::Type GetDispatchThread() const
	{
		return DispatchThread;
	}

	/**
	 * Goes through all entity data and compacts like-for-like allocations into as few allocations as possible, resulting in the optimal data layout
	 */
	MOVIESCENE_API void Compact();


	/**
	 * Efficiently mutate all entities that match a filter. Mutations can add or remove components from batches of entity data.
	 *
	 * @param Filter      The filter to match entity allocations against. Only entities that match the filter will be mutated
	 * @param Mutation    Implementation that defines how to mutate the entities that match the filter
	 * @return The number of entities that were mutated, or 0 if none were matched
	 */
	MOVIESCENE_API int32 MutateAll(const FEntityComponentFilter& Filter, const IMovieSceneEntityMutation& Mutation, EMutuallyInclusiveComponentType MutualTypes = EMutuallyInclusiveComponentType::Mandatory);


	/**
	 * Efficiently mutate all entities that match a filter. Mutations can add or remove components from batches of entity data.
	 *
	 * @param Filter      The filter to match entity allocations against. Only entities that match the filter will be mutated
	 * @param Mutation    Implementation that defines how to mutate the entities that match the filter
	 * @return The number of entities that were mutated, or 0 if none were matched
	 */
	MOVIESCENE_API int32 MutateConditional(const FEntityComponentFilter& Filter, const IMovieSceneConditionalEntityMutation& Mutation, EMutuallyInclusiveComponentType MutualTypes = EMutuallyInclusiveComponentType::Mandatory);


	/**
	 * Touch the specified entity, cause the allocation and component serial numbers to be incremented. Will invalidate any transient caches maintained for serial such numbers.
	 * @note Allocation and component serial numbers relate to the entire allocation of entities that this entity resides within. As such, it will invalidate downstream caches
	 * for any cache that relates to the entire allocation
	 *
	 * @param EntityID    The ID of the entity to touch.
	 */
	MOVIESCENE_API void TouchEntity(FMovieSceneEntityID EntityID);


	/**
	 * Set up an entity to be a child of another. Child entities will only be cleaned up if their parents are marked Unlink, and TagOrphanedChildren is called
	 *
	 * @param ParentID   The ID of the parent
	 * @param ChildID    The ID of the child
	 */
	MOVIESCENE_API void AddChild(FMovieSceneEntityID ParentID, FMovieSceneEntityID ChildID);


	/**
	 * Retrieve the immediate children of the specified entity
	 * @note: the array will not be emptied by this function - that is the responsibility of the callee, if desired.
	 *
	 * @param ParentID    The ID of the parent
	 * @param OutChildren (out) Array to populate with child entities.
	 */
	MOVIESCENE_API void GetImmediateChildren(FMovieSceneEntityID ParentID, TArray<FMovieSceneEntityID>& OutChildren) const;


	/**
	 * Retrieve all children, grandchildren etc of the specified entity using a parent first traversal
	 * @note: the array will not be emptied by this function - that is the responsibility of the callee, if desired.
	 *
	 * @param ParentID    The ID of the parent
	 * @param OutChildren (out) Array to populate with child entities.
	 */
	MOVIESCENE_API void GetChildren_ParentFirst(FMovieSceneEntityID ParentID, TArray<FMovieSceneEntityID>& OutChildren) const;


	/**
	 * Itereate the immediate children of the specified entity
	 * @note: the array will not be emptied by this function - that is the responsibility of the callee, if desired.
	 *
	 * @param ParentID    The ID of the parent
	 * @param OutChildren (out) Array to populate with child entities.
	 */
	template<typename IteratorType>
	void IterateImmediateChildren(FMovieSceneEntityID ParentID, IteratorType&& Iterator) const
	{
		for (auto ChildIt = ParentToChild.CreateConstKeyIterator(ParentID); ChildIt; ++ChildIt)
		{
			Iterator(ChildIt.Value());
		}
	}


	/**
	 * Iterate all children, grandchildren etc of the specified entity using a parent first traversal
	 *
	 * @param ParentID    The ID of the parent
	 * @param OutChildren (out) Array to populate with child entities.
	 */
	template<typename IteratorType>
	void IterateChildren_ParentFirst(FMovieSceneEntityID ParentID, IteratorType&& Iterator) const
	{
		for (auto ChildIt = ParentToChild.CreateConstKeyIterator(ParentID); ChildIt; ++ChildIt)
		{
			Iterator(ChildIt.Value());
			IterateChildren_ParentFirst(ChildIt.Value(), Iterator);
		}
	}

public:


	/**
	 * Run a serialization routine over the specified entity to approximate the memory it is using
	 * @note Does not consider per-allocation overhead such as header sizes and other per-allocation meta-data
	 *
	 * @param Ar          The archive to save to. Ar.IsCountingMemory() must be true.
	 * @param EntityID    Identifier for the entity to replace object references within
	 */
	MOVIESCENE_API void CountMemory(FArchive& Ar, FMovieSceneEntityID EntityID);


	/**
	 * Find a component type hander from its registered GUID
	 */
	static MOVIESCENE_API IComponentTypeHandler* FindComponentTypeHandler(const FGuid& ComponentGuid);

public:

	/**
	 * Iterate through all the allocations that match the specified component filter
	 *
	 * @param InFilter (required, non-null) The filter to match allocations against. Filter is copied into the iterator.
	 * @return An iterator object that walks over all allocations matching the filter.
	 */
	MOVIESCENE_API FEntityAllocationIteratorProxy Iterate(const FEntityComponentFilter* InFilter) const;


	/**
	 * Efficiently test whether this entity manager contains any allocations that match the specified filter
	 *
	 * @param InFilter The filter to match allocations against.
	 * @return True if the entity manager contains any allocations that match the filter, false otherwise
	 */
	MOVIESCENE_API bool Contains(const FEntityComponentFilter& InFilter) const;


	/**
	 * Check whether any entity in this manager has the specified component
	 *
	 * @param ComponentTypeID      The type of the component that is being tested for
	 * @return true if the specified component exists anywhere, false otherwise
	 */
	bool ContainsComponent(FComponentTypeID ComponentTypeID) const
	{
		return GetAccumulatedMask().Contains(ComponentTypeID);
	}


	/**
	 * Check whether any entity in this manager has any of the specified components
	 *
	 * @param ComponentTypeIDs     The types of the component that are being tested for
	 * @return true if any one of the specified components exist anywhere in this manager, false otherwise
	 */
	bool ContainsAnyComponent(std::initializer_list<FComponentTypeID> ComponentTypeIDs) const
	{
		const FComponentMask& Mask = GetAccumulatedMask();
		return Algo::AnyOf(ComponentTypeIDs, [&Mask](FComponentTypeID In){ return Mask.Contains(In); });
	}

	/**
	 * Check whether any entity in this manager has any of the specified components
	 *
	 * @param ComponentTypeIDs     The types of the component that are being tested for
	 * @return true if any one of the specified components exist anywhere in this manager, false otherwise
	 */
	bool ContainsAnyComponent(const FComponentMask& ComponentTypeIDs) const
	{
		const FComponentMask& Mask = GetAccumulatedMask();
		return FComponentMask::BitwiseAND(Mask, ComponentTypeIDs, EBitwiseOperatorFlags::MinSize).First() != FComponentTypeID::Invalid();
	}

	/**
	 * Check whether all of the specified components exist anywhere in this entity manager
	 * @note: Does not check whether the components exist on the same entity, just that they are present in the manager. Use Contains for a thorough filter match.
	 *
	 * @param ComponentTypeIDs      The types of the components that are being tested for
	 * @return true if each of the specified components exist anywhere in this manager, false otherwise
	 */
	bool ContainsAllComponents(std::initializer_list<FComponentTypeID> ComponentTypeIDs) const
	{
		const FComponentMask& Mask = GetAccumulatedMask();
		return Algo::AllOf(ComponentTypeIDs, [&Mask](FComponentTypeID In){ return Mask.Contains(In); });
	}


	/**
	 * Accumulate a mask from all entity types that match the specified filter
	 *
	 * @param InFilter The filter to match allocations against. Filter is copied into the iterator.
	 * @param OutMask  The mask to receive the binary OR accumulation of all entities that pass the filter
	 */
	MOVIESCENE_API void AccumulateMask(const FEntityComponentFilter& InFilter, FComponentMask& OutMask) const;


	/**
	 * Retrieve an up-to-date accumulation of all components present on entities in this manager
	 */
	MOVIESCENE_API const FComponentMask& GetAccumulatedMask() const;


	/**
	 * Efficiently test whether this entity manager contains any allocations that match the specified filter
	 *
	 * @param InFilter The filter to match allocations against.
	 * @return True if the entity manager contains any allocations that match the filter, false otherwise
	 */
	MOVIESCENE_API void EnterIteration() const;
	MOVIESCENE_API void ExitIteration() const;

	void CheckCanChangeStructure() const
	{
		checkf(IterationCount.Load(ThreadingModel) == 0, TEXT("Mutation of entities is not permissible while entities are being iterated"));
		checkf(LockdownState == ELockdownState::Unlocked, TEXT("Structural changes to the entity manager are not permitted while it is locked down"));
	}

	bool IsLockedDown() const
	{
		return LockdownState == ELockdownState::Locked;
	}

	void LockDown()
	{
		LockdownState = ELockdownState::Locked;
	}

	void ReleaseLockDown()
	{
		LockdownState = ELockdownState::Unlocked;
	}

	/**
	 * Retrieve the entity filter that should be used for any entity iteration. Can be used to constrain all iterations to specific types
	 */
	const FEntityComponentFilter& GetGlobalIterationFilter() const
	{
		return GlobalIterationFilter;
	}

	/**
	 * Modify the entity filter that should be used for any entity iteration.
	 * @note: Care should be take to reset this back to its original state when done,
	 *        as leaving this populated during the instantiation phase can result in 
	 *        undefined results.
	 */
	FEntityComponentFilter& ModifyGlobalIterationFilter()
	{
		ensureMsgf(!IsLockedDown() && IterationCount.Load(ThreadingModel) == 0, TEXT("Manipulating the global iteration filter while locked down or iterating is not recommended"));
		return GlobalIterationFilter;
	}

	/**
	 * Increment the current serial number for systems observing this manager. Should be called after any system is run
	 */
	void IncrementSystemSerial(uint64 IncAmount = 1)
	{
		SystemSerialNumber += IncAmount;
	}

	/**
	 * Get the current serial number for any system ovserving this entity manager. This serial number acts as a timestamp for 
	 * the current state of this entity manager - any serial number more recent than this dictates more recent logic or state.
	 */
	uint64 GetSystemSerial() const
	{
		return SystemSerialNumber;
	}

	/**
	 * Check whether the structure of this entity manager has changed at all since the specified serial number
	 */
	bool HasStructureChangedSince(uint64 CachedSerial) const
	{
		return CachedSerial == 0 || StructureMutationSystemSerialNumber > CachedSerial;
	}

	/**
	 * Called in order to mimic the entity structure changing, even if it has not
	 */
	void MimicStructureChanged()
	{
		OnStructureChanged();
	}

public:

	/**
	 * Set a debug name for this entity manager
	 */
	void SetDebugName(FString&& InNewDebugName)
	{
		ManagerDebugName = MoveTemp(InNewDebugName);
	}


	/**
	 * Explicitly add referenced objects from an external source. This is not called by default for the global entity manager because each entity owner should do so to avoid cyclic dependencies.
	 */
	MOVIESCENE_API void AddReferencedObjects(FReferenceCollector& ReferenceCollector);

	
	/**
	 * Replace an entity ID with the components from another, discarding the provided entity ID in the process
	 *
	 * @param InOutEntity     The entity ID to be reassigned. If it is already valid, the existing entity will be freed
	 * @param EntityToDiscard The entity containing the components to replace InOutEntity with. This Entity ID will be invalid after this function call.
	 */
	MOVIESCENE_API void ReplaceEntityID(FMovieSceneEntityID& InOutEntity, FMovieSceneEntityID EntityToDiscard);

	uint32 GetHandleGeneration() const
	{
		if (bHandleGenerationStale)
		{
			++CurrentHandleGeneration;
		}
		bHandleGenerationStale = false;
		return CurrentHandleGeneration;
	}

private:
	friend struct FEntityInitializer; 

	enum class EMemoryType
	{
		Uninitialized,
		DefaultConstructed
	};
	MOVIESCENE_API void OnStructureChanged();

	MOVIESCENE_API FEntityAllocation* CreateEntityAllocation(const FComponentMask& EntityComponentMask, uint16 InitialCapacity, uint16 MaxCapacity, FEntityAllocation* MigrateComponentDataFrom = nullptr);
	MOVIESCENE_API int32 CreateEntityAllocationEntry(const FComponentMask& EntityComponentMask, uint16 InitialCapacity, uint16 MaxCapacity);

	MOVIESCENE_API int32 GetOrCreateAllocationWithSlack(const FComponentMask& EntityComponentMask, int32* InOutDesiredSlack = nullptr);
	MOVIESCENE_API int32 CreateAllocationWithSlack(const FComponentMask& EntityComponentMask, int32* InOutDesiredSlack = nullptr);
	MOVIESCENE_API int32 MigrateEntity(int32 DestIndex, int32 SourceIndex, int32 SourceEntryIndexWithinAllocation);

	MOVIESCENE_API void CopyComponents(int32 DestAllocationIndex, int32 DestEntityIndex, int32 SourceAllocationIndex, int32 SourceEntityIndex, const FComponentMask* OptionalMask = nullptr);

	MOVIESCENE_API int32 AddEntityToAllocation(int32 AllocationIndex, FMovieSceneEntityID ID, EMemoryType MemoryType = EMemoryType::DefaultConstructed);
	MOVIESCENE_API void RemoveEntityFromAllocation(int32 AllocationIndex, int32 SourceEntryIndexWithinAllocation);

	MOVIESCENE_API FEntityAllocation* MigrateAllocation(int32 AllocationIndex, const FComponentMask& NewComponentMask);

	MOVIESCENE_API void CombineAllocations(int32 DestinationIndex, int32 SourceIndex);

	FEntityAllocation* GetAllocation(int32 AllocationIndex) const
	{
		return EntityAllocations[AllocationIndex];
	}

	MOVIESCENE_API int32 ReserveAllocation(int32 AllocationIndex, int32 NumToReserve);
	MOVIESCENE_API FEntityAllocation* GrowAllocation(int32 AllocationIndex, int32 MinNumToGrowBy = 1);

	MOVIESCENE_API void DestroyAllocation(FEntityAllocation* Allocation, bool bDestructComponentData = true);

	MOVIESCENE_API virtual void NotifyUObjectDeleted(const UObjectBase* Object, int32 Index) override;
	MOVIESCENE_API virtual void OnUObjectArrayShutdown() override;
	MOVIESCENE_API virtual FString GetReferencerName() const;

	MOVIESCENE_API void CheckInvariants();

	virtual SIZE_T GetAllocatedSize() const override
	{
		return 0;
	}

	friend struct FEntityAllocationProxy;
	friend struct FEntityAllocationIterator;
	friend struct FEntityAllocationIteratorProxy;

	friend FFreeEntityOperation;

	struct FEntityLocation
	{
		FEntityLocation()
			: AllocationIndex(INVALID)
			, EntryIndexWithinAllocation(0)
		{}

		void Reset()
		{
			AllocationIndex = INVALID;
			EntryIndexWithinAllocation = 0;
		}

		void Set(int32 InAllocationIndex, int32 InEntryIndexWithinAllocation)
		{
			check((InAllocationIndex & 0xFFFF0000) == 0 && (InEntryIndexWithinAllocation & 0xFFFF0000) == 0);
			AllocationIndex = (uint16)InAllocationIndex;
			EntryIndexWithinAllocation = (uint16)InEntryIndexWithinAllocation;
		}

		int32 GetAllocationIndex() const
		{
			return AllocationIndex;
		}

		int32 GetEntryIndexWithinAllocation() const
		{
			return EntryIndexWithinAllocation;
		}

		bool IsValid() const
		{
			return AllocationIndex != INVALID;
		}

		void SetParentID(FMovieSceneEntityID InParentID)
		{
			ParentID = InParentID;
		}

		FMovieSceneEntityID GetParentID() const
		{
			return ParentID;
		}

	private:

		static const uint16 INVALID = ~uint16(0);
		uint16 AllocationIndex;
		uint16 EntryIndexWithinAllocation;

		FMovieSceneEntityID ParentID;
	};

	/** Sparse array of masks that define each index within EntityAllocations' component types */
	TSparseArray<FComponentMask> EntityAllocationMasks;

	/** Bit mask of allocations that currently have capacity */
	TBitArray<> AllocationsWithCapacity;

	/** Sparse array of entity allocation pointers. Allocations are heap allocated. */
	TSparseArray<FEntityAllocation*> EntityAllocations;

	/** Sparse array of entity location information for each allocated entity ID. */
	TSparseArray<FEntityLocation> EntityLocations;

	/** Map of parent -> child entities */
	TMultiMap<FMovieSceneEntityID, FMovieSceneEntityID> ParentToChild;

	TSparseArray<TInlineValue<FChildEntityInitializer>> InstancedChildInitializers;

	/** Map of entity ID to the generation its handle was created in. FEntityHandle::HandleGeneration matches the value if it is still valid */
	TMap<FMovieSceneEntityID, uint32> EntityGenerationMap;

	FComponentMask AccumulatedMask;
	FEntityComponentFilter GlobalIterationFilter;

	FComponentRegistry* ComponentRegistry;

	/** Debug name for this entity manager */
	FString ManagerDebugName;

	/** System serial number incremented for every run of any system */
	uint64 SystemSerialNumber;

	/** The value of this manager's SystemSerialNumber the last time any entities were allocated, freed, or mutated in some way (this does not include component values being written to) */
	uint64 StructureMutationSystemSerialNumber;

	/** Debugging ptr for natvis */
#if UE_MOVIESCENE_ENTITY_DEBUG
	const bool* RichComponentDebuggingPtr;
#endif

	/** Serially incrementing unique identifier that is assigned to each new entity allocation that is created */
	uint32 NextAllocationID;

	/** The current generation of entities - incremented any time an entity is destroyed */
	mutable uint32 CurrentHandleGeneration;
	mutable bool bHandleGenerationStale;
	mutable bool bAccumulatedMaskStale;

	/** Atomic counter that is incremented when an iteration begins, and decremented when it finishes */
	mutable FEntitySystemMaybeAtomicInt32 IterationCount;

	ENamedThreads::Type GatherThread;
	ENamedThreads::Type DispatchThread;
	EEntityThreadingModel ThreadingModel;

	enum class ELockdownState
	{
		Locked, Unlocked
	};

	ELockdownState LockdownState;
};


struct FFreeEntityOperation
{
	FFreeEntityOperation(FEntityManager* InEntityManager)
		: EntityManager(InEntityManager)
	{}

	void MarkAllocationForFree(int32 AllocationIndex);

	void MarkEntityForFree(FMovieSceneEntityID EntityID);

private:

	struct FAllocationMask
	{
		TBitArray<> Mask;
		bool bDestroyAllocation = false;
	};

	struct FCommitData
	{
		/** Contains specific entities to destroy from allocations that are not to be destroyed completely */
		TMap<int32, FAllocationMask> AllocationsToEntities;

		/** Additional array of entitiy IDs that need destroying despite not belonging to an allocation */
		TArray<FMovieSceneEntityID> EmptyEntities;
	};

	FCommitData Commit() const;

	friend FEntityManager;

	/** */
	TArray<int32> AllocationsToDestroy;

	/** */
	TArray<FMovieSceneEntityID> LooseEntitiesToDestroy;

	FEntityManager* EntityManager;
};

extern MOVIESCENE_API FEntityManager* GEntityManagerForDebuggingVisualizers;


} // namespace MovieScene
} // namespace UE
