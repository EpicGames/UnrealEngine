// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IntegerSequence.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"

namespace UE
{
namespace MovieScene
{
class FEntityManager;
struct FEntityAllocation;
struct FEntityDataLocation;

struct IMovieSceneEntityMutation
{
	virtual ~IMovieSceneEntityMutation() = default;
	virtual void CreateMutation(FEntityManager* EntityManager, FComponentMask* InOutEntityComponentTypes) const = 0;
	virtual void InitializeAllocation(FEntityAllocation* Allocation, const FComponentMask& AllocationType) const {}
	virtual void InitializeUnmodifiedAllocation(FEntityAllocation* Allocation, const FComponentMask& AllocationType) const {}
};

struct IMovieScenePerEntityMutation
{
	virtual ~IMovieScenePerEntityMutation() = default;

	virtual void CreateMutation(FEntityManager* EntityManager, FComponentMask* InOutEntityComponentTypes) const = 0;
	virtual void InitializeEntities(const FEntityRange& EntityRange, const FComponentMask& AllocationType) const {}
};

struct IMovieSceneConditionalEntityMutation : IMovieScenePerEntityMutation
{
	virtual void MarkAllocation(FEntityAllocation* Allocation, TBitArray<>& OutEntitiesToMutate) const = 0;
};

struct FAddSingleMutation : IMovieSceneEntityMutation
{
	FComponentTypeID ComponentToAdd;
	FAddSingleMutation(FComponentTypeID InType)
		: ComponentToAdd(InType)
	{}

	MOVIESCENE_API virtual void CreateMutation(FEntityManager* EntityManager, FComponentMask* InOutEntityComponentTypes) const override;
};

struct FRemoveSingleMutation : IMovieSceneEntityMutation
{
	FComponentTypeID ComponentToRemove;

	FRemoveSingleMutation(FComponentTypeID InType)
		: ComponentToRemove(InType)
	{}

	MOVIESCENE_API virtual void CreateMutation(FEntityManager* EntityManager, FComponentMask* InOutEntityComponentTypes) const override;
};

struct FAddMultipleMutation : IMovieSceneEntityMutation
{
	FComponentMask MaskToAdd;

	MOVIESCENE_API virtual void CreateMutation(FEntityManager* EntityManager, FComponentMask* InOutEntityComponentTypes) const override;
};

struct FRemoveMultipleMutation : IMovieSceneEntityMutation
{
	/* Mask that defines components to remove by _unset_ bits. This acts as a bitmask applied as a binary AND for each mutated allocation. */
	FComponentMask MaskToRemove;

	void RemoveComponent(FComponentTypeID InComponentType);

	MOVIESCENE_API virtual void CreateMutation(FEntityManager* EntityManager, FComponentMask* InOutEntityComponentTypes) const override;
};

} // namespace MovieScene
} // namespace UE
