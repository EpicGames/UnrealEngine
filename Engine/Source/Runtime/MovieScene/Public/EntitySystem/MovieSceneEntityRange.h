// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IntegerSequence.h"
#include "Templates/TupleFwd.h"
#include "Templates/UnrealTemplate.h"

namespace UE
{
namespace MovieScene
{


template<typename... T> struct TEntityPtr;
template<typename... T> struct TEntityRangeImpl;

/**
 * Variadic template representing a contiguous range of entities with a specific set of components
 *
 * The template parameters define each component type by index, whose constness should match the 
 * read/write semantics of the accessor. For example, an entity with a float, int and bool component,
 * accessed read-only should be represented by a TEntityRange<const float, const int const bool>.
 * If one wished to write to all the float components, we would require a TEntityRange<float, const int, const bool> etc
 */
template<typename... T>
struct TEntityRange : TEntityRangeImpl<TMakeIntegerSequence<int, sizeof...(T)>, T...>
{
	using PtrType = TEntityPtr<T...>;
	using Super   = TEntityRangeImpl<TMakeIntegerSequence<int, sizeof...(T)>, T...>;

	/**
	 * Default constructor - empty range
	 */
	TEntityRange()
		: Super(0)
	{}


	/**
	 * Constructor that initializes the size, but no component ptrs. Subsequent calls to InitializeComponentArray should be made
	 */
	explicit TEntityRange(int32 InNum)
		: Super(InNum)
	{}


	/**
	 * Constructor that initializes the size and components for this range
	 */
	explicit TEntityRange(int32 InNum, T*... InComponentArrays)
		: Super(InNum, InComponentArrays...)
	{}


	/**
	 * Access the size of this range
	 * @return The number of entities in this range
	 */
	int32 Num() const
	{
		return this->NumEntities;
	}


	/**
	 * Access a specific entity within this range
	 *
	 * @param Index    The index of the entity within this range
	 * @return The entity with all its components
	 */
	TEntityPtr<T...> operator[](int32 Index) const
	{
		check(Index >= 0 && Index < this->NumEntities);
		return TEntityPtr<T...>(Index, this);
	}


	/**
	 * Return a reference to the component array pointer at the specified index. Should only be used for initialization
	 */
	template<int ComponentTypeIndex>
	auto*& GetComponentArrayReference()
	{
		return this->ComponentArrays.template Get<ComponentTypeIndex>();
	}


	/**
	 * Retrieve the raw pointer to the (possibly nullptr) component array at the templated index.
	 */
	template<int ComponentTypeIndex>
	auto* GetRawUnchecked() const
	{
		return this->ComponentArrays.template Get<ComponentTypeIndex>();
	}


	/**
	 * Get all the components for the templated index as a TArrayView.
	 *
	 * @return A TArrayView<T> for the component array at this index. T will be const for read-only components.
	 */
	template<int32 ComponentTypeIndex>
	auto GetAll() const
	{
		auto* Ptr = this->ComponentArrays.template Get<ComponentTypeIndex>();
		return MakeArrayView(Ptr, Ptr ? this->NumEntities : 0);
	}


	/**
	 * Get the component from the templated component array index, using its index within this range.
	 * @note: will fail an assertion if this entity range does not contain this component type (eg Read/WriteOptional)
	 *
	 * @param EntityIndex The index of the entity within this range
	 * @return A reference to the component matching the constness of the access-mode.
	 */
	template<int ComponentTypeIndex>
	auto& GetComponent(int32 EntityIndex) const
	{
		auto* Ptr = this->ComponentArrays.template Get<ComponentTypeIndex>();
		checkSlow(Ptr);
		return *(Ptr + EntityIndex);
	}


	/**
	 * Get a pointer to an entity's component from its templated component array index, using the entity's index within this range.
	 *
	 * @param EntityIndex The index of the entity within this range
	 * @return A pointer to the component matching the constness of the access-mode, or nullptr if the component does not exist in this entity range.
	 */
	template<int ComponentTypeIndex>
	auto* GetComponentOptional(int32 EntityIndex) const
	{
		auto* Ptr = this->ComponentArrays.template Get<ComponentTypeIndex>();
		return Ptr ? (Ptr + EntityIndex) : nullptr;
	}


	/**
	 * Assign the value of an entity's component from its templated component array index, using the entity's index within this range.
	 *
	 * @param EntityIndex The index of the entity within this range
	 * @param InValue     The value to assign to the component
	 */
	template<int ComponentTypeIndex, typename ValueType>
	void SetComponent(int32 EntityIndex, ValueType&& InValue) const
	{
		auto* Ptr = this->ComponentArrays.template Get<ComponentTypeIndex>();
		checkSlow(Ptr);
		*(Ptr + EntityIndex) = Forward<ValueType>(InValue);
	}


	/*~ stl-like range-for iterator */
	TEntityPtr<T...> begin() const { return TEntityPtr<T...>(0, this); }
	TEntityPtr<T...> end() const   { return TEntityPtr<T...>(Num(), this); }
};


/**
 * Implementation template for a range of entities
 */
template<int... Indices, typename... T>
struct TEntityRangeImpl<TIntegerSequence<int, Indices...>, T...>
{
	/**
	 * Slice this view such that it represents a sub-range of itself
	 *
	 * @param Index Starting index of the new range
	 * @param NewNum Number of elements in the new range. Must be <= Num()-Index
	 */
	void Slice(int32 Index, int32 NewNum)
	{
		check(Index >= 0 && Index + NewNum <= NumEntities);

		int Temp[] = {
			( (ComponentArrays.template Get<Indices>() ? (ComponentArrays.template Get<Indices>() = ComponentArrays.template Get<Indices>() + Index, 0) : 0), 0)...
		};
		(void)Temp;

		NumEntities = NewNum;
	}


protected:

	/** Default null constructor that just initializes the size. Subsequent calls to InitializeComponentArray are required. */
	explicit TEntityRangeImpl(int32 InNum)
		: NumEntities(InNum)
	{}

	/** Initialize the size and component ptrs for this range. */
	explicit TEntityRangeImpl(int32 InNum, T*... InBasePtrs)
		: ComponentArrays(InBasePtrs...)
		, NumEntities(InNum)
	{}

	/** The number of entities in this range */
	TTuple<T*...> ComponentArrays;

	/** The number of entities in this range */
	int32 NumEntities;
};



/**
 * Variadic template representing a single entity with a range of entities with the same a set of typed components.
 *
 * The template parameters define each component type by index, whose constness should match the 
 * read/write semantics of the accessor. For example, an entity with a float, int and bool component,
 * accessed read-only should be represented by a TEntityPtr<const float, const int const bool>.
 * If one wished to write to all the float components, we would require a TEntityPtr<float, const int, const bool> etc
 */
template<typename... T>
struct TEntityPtr
{
	/** Construct this range from a range and index */
	TEntityPtr(int32 InEntityIndex, const TEntityRange<T...>* InOwner)
		: Owner(InOwner)
		, EntityIndex(InEntityIndex)
	{
		checkSlow(Owner != nullptr);
	}

	/**
	 * Increment this pointer
	 */
	TEntityPtr& operator++()
	{
		++EntityIndex;
		return *this;
	}


	/**
	 * Increment this pointer
	 */
	TEntityPtr& operator--()
	{
		--EntityIndex;
		return *this;
	}

	/**
	 * Dereference this pointer
	 */
	TEntityPtr& operator*()
	{
		return *this;
	}


	/**
	 * Compare this pointer with another for equality
	 */
	friend bool operator !=(const TEntityPtr& A, const TEntityPtr& B)
	{
		return A.EntityIndex != B.EntityIndex || A.Owner != B.Owner;
	}


	/**
	 * Test whether this pointer is valid
	 */
	explicit operator bool() const
	{
		return EntityIndex >= 0 && EntityIndex < Owner->Num;
	}


	/**
	 * Retrieve the component at the specified Index within this TEntityPtr's parameters. Not to be used for Read/WriteOptional variants.
	 * @note The const qualifier of the component will match the access mode for the component (const for read, non-const for write)
	 *
	 * @return A reference to the component value
	 */
	template<int ComponentTypeIndex>
	auto& Get() const
	{
		return Owner->template GetComponent<ComponentTypeIndex>(EntityIndex);
	}


	/**
	 * Optionally retrieve the component at the specified ComponentTypeIndex within this TEntityPtr's parameters.
	 * @note The const qualifier of the component will match the access mode for the component (const for read, non-const for write)
	 *
	 * @return A pointer to the component value, or nullptr if this entity does not have such a component.
	 */
	template<int ComponentTypeIndex>
	auto* GetOptional() const
	{
		return Owner->template GetComponentOptional<ComponentTypeIndex>(EntityIndex);
	}


	/**
	 * Set the value of the component at the specified index withinin this entity
	 *
	 * @param InValue The value to assign to the component
	 */
	template<int ComponentTypeIndex, typename ValueType>
	void Set(ValueType&& InValue) const
	{
		Owner->SetComponent<ComponentTypeIndex>(EntityIndex, Forward<ValueType>(InValue));
	}


private:

	/** The range that this entity ptr resides within */
	const TEntityRange<T...>* Owner;

	/** This entity's index within the range */
	int32 EntityIndex;
};

template<typename T>
struct TEntityPtr<T>
{
	/** Construct this range from a range and index */
	TEntityPtr(int32 InEntityIndex, const TEntityRange<T>* InOwner)
		: Owner(InOwner)
		, EntityIndex(InEntityIndex)
	{
		checkSlow(Owner != nullptr);
	}

	/**
	 * Increment this pointer
	 */
	TEntityPtr& operator++()
	{
		++EntityIndex;
		return *this;
	}


	/**
	 * Increment this pointer
	 */
	TEntityPtr& operator--()
	{
		--EntityIndex;
		return *this;
	}

	/**
	 * Dereference this pointer
	 */
	T& operator*()
	{
		return *this;
	}


	/**
	 * Compare this pointer with another for equality
	 */
	friend bool operator !=(const TEntityPtr& A, const TEntityPtr& B)
	{
		return A.EntityIndex != B.EntityIndex || A.Owner != B.Owner;
	}


	/**
	 * Test whether this pointer is valid
	 */
	explicit operator bool() const
	{
		return EntityIndex >= 0 && EntityIndex < Owner->Num;
	}


	/**
	 * Retrieve the component at the specified Index within this TEntityPtr's parameters. Not to be used for Read/WriteOptional variants.
	 * @note The const qualifier of the component will match the access mode for the component (const for read, non-const for write)
	 *
	 * @return A reference to the component value
	 */
	template<int ComponentTypeIndex>
	auto& Get() const
	{
		return Owner->GetComponent<ComponentTypeIndex>(EntityIndex);
	}


	/**
	 * Optionally retrieve the component at the specified ComponentTypeIndex within this TEntityPtr's parameters.
	 * @note The const qualifier of the component will match the access mode for the component (const for read, non-const for write)
	 *
	 * @return A pointer to the component value, or nullptr if this entity does not have such a component.
	 */
	template<int ComponentTypeIndex>
	auto* GetOptional() const
	{
		return Owner->GetComponentOptional<ComponentTypeIndex>(EntityIndex);
	}


	/**
	 * Set the value of the component at the specified index withinin this entity
	 *
	 * @param InValue The value to assign to the component
	 */
	template<int ComponentTypeIndex, typename ValueType>
	void Set(ValueType&& InValue) const
	{
		Owner->SetComponent<ComponentTypeIndex>(EntityIndex, Forward<ValueType>(InValue));
	}


private:

	/** The range that this entity ptr resides within */
	const TEntityRange<T>* Owner;

	/** This entity's index within the range */
	int32 EntityIndex;
};


} // namespace MovieScene
} // namespace UE