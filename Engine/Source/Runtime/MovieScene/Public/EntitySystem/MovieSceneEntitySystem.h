// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"
#include "Stats/Stats.h"
#include "Templates/SubclassOf.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneEntitySystem.generated.h"

class UMovieSceneEntitySystemLinker;

namespace UE
{
namespace MovieScene
{

	struct FSystemSubsequentTasks;
	struct FSystemTaskPrerequisites;
	class IEntitySystemScheduler;

	enum class EEntitySystemCategory : uint32
	{
		// No category
		None = 0,
		// Systems lacking any particular category
		Unspecified = 1u << 0,
		// Core systems, such as the time evaluation system
		Core = 1u << 1,
		// Systems that evaluate channel values
		ChannelEvaluators = 1u << 2,
		// Systems the blend values together
		BlenderSystems = 1u << 3,
		// Systems that set properties on objects
		PropertySystems = 1u << 4,
		// Start of custom categories
		Custom = 1u << 5,

		// Last entry, used as error condition
		Last = 1u << 31,
		// All categories
		All = ~0u
	};
	ENUM_CLASS_FLAGS(EEntitySystemCategory)

} // namespace MovieScene
} // namespace UE


UCLASS(MinimalAPI)
class UMovieSceneEntitySystem : public UObject
{
public:
	GENERATED_BODY()


	template<typename T>
	using TComponentTypeID = UE::MovieScene::TComponentTypeID<T>;
	using FComponentTypeID = UE::MovieScene::FComponentTypeID;
	using FComponentMask   = UE::MovieScene::FComponentMask;

	using FSystemTaskPrerequisites = UE::MovieScene::FSystemTaskPrerequisites;
	using FSystemSubsequentTasks   = UE::MovieScene::FSystemSubsequentTasks;

	MOVIESCENE_API UMovieSceneEntitySystem(const FObjectInitializer& ObjInit);
	MOVIESCENE_API ~UMovieSceneEntitySystem();

	/** Called after all the module loading phases are complete. */
	static void StaticInitialize();

	/**
	 * Creates a relationship between the two system types that ensures any systems of type UpstreamSystemType always execute before DownstreamSystemType if they are both present
	 *
	 * @param UpstreamSystemType     The UClass of the system that should always be a prerequisite of DownstreamSystemType (ie, runs first)
	 * @param DownstreamSystemType   The UClass of the system that should always run after UpstreamSystemType
	 */
	static MOVIESCENE_API void DefineImplicitPrerequisite(TSubclassOf<UMovieSceneEntitySystem> UpstreamSystemType, TSubclassOf<UMovieSceneEntitySystem> DownstreamSystemType);

	/**
	 * Informs the dependency graph that the specified class type produces components of the specified type.
	 * Any systems set up as consumers of this component type will always be run after
	 *
	 * @param ClassType         The UClass of the system that produces the component type
	 * @param ComponentType     The type of the component produced by the system
	 */
	static MOVIESCENE_API void DefineComponentProducer(TSubclassOf<UMovieSceneEntitySystem> ClassType, FComponentTypeID ComponentType);

	/**
	 * Informs the dependency graph that the specified class type consumes components of the specified type, and as such should always execute after any producers of that component type.
	 *
	 * @param ClassType         The UClass of the system that consumes the component type
	 * @param ComponentType     The type of the component consumed by the system
	 */
	static MOVIESCENE_API void DefineComponentConsumer(TSubclassOf<UMovieSceneEntitySystem> ClassType, FComponentTypeID ComponentType);

	/**
	 * Ensure that any systems relevant to the specified linker's entity manager are linked
	 */
	static MOVIESCENE_API void LinkRelevantSystems(UMovieSceneEntitySystemLinker* InLinker);

	/**
	 * Link all systems in a given category
	 */
	static MOVIESCENE_API void LinkCategorySystems(UMovieSceneEntitySystemLinker* InLinker, UE::MovieScene::EEntitySystemCategory InCategory);

	/**
	 * Link all systems that pass the given linker's filter
	 */
	static MOVIESCENE_API void LinkAllSystems(UMovieSceneEntitySystemLinker* InLinker);

	/**
	 * Create a new system category
	 */
	static MOVIESCENE_API UE::MovieScene::EEntitySystemCategory RegisterCustomSystemCategory();

	/**
	 * Sort the given systems by their flow order, suitable for execution
	 */
	static MOVIESCENE_API void SortByFlowOrder(TArray<uint16>& InOutGlobalNodeIDs);

	/**
	 * Get the global IDs of all subsequent systems of the given system
	 */
	static MOVIESCENE_API void GetSubsequentSystems(uint16 FromGlobalNodeID, TArray<uint16>& OutSubsequentGlobalNodeIDs);

	/**
	 * Print the global system graph in graphviz format filtered by the specified global system IDs, one filter per phase
	 * *
	 * @param GroupFilters  Array of bit filters, one per phase, defining which systems to include in the output string
	 * @param OutString     String builder to add the output graph to
	 */
	static MOVIESCENE_API void PrintFilteredFlowGraph(TArrayView<const TBitArray<>> GroupFilters, FStringBuilderBase& OutString);

	/**
	 * Prints a graphviz markup for the global system dependency graph
	 */
	static MOVIESCENE_API void DebugPrintGlobalDependencyGraph(bool bUpdateCache = true);

public:

	/** Returns system categories */
	UE::MovieScene::EEntitySystemCategory GetCategories() const
	{
		return SystemCategories;
	}

	/** Returns the phase(s) during which this system should be run */
	UE::MovieScene::ESystemPhase GetPhase() const
	{
		return Phase;
	}

	/** Returns the linker that owns this system */
	UMovieSceneEntitySystemLinker* GetLinker() const
	{
		return Linker;
	}

	/** Returns the ID of this system in the system graphs */
	uint16 GetGraphID() const
	{
		return GraphID;
	}
	/** Sets the ID of this system in the system graphs */
	void SetGraphID(uint16 InGraphID)
	{
		GraphID = InGraphID;
	}

	/** Gets the ID of this system's type in the global dependency graph */
	uint16 GetGlobalDependencyGraphID() const
	{
		return GlobalDependencyGraphID;
	}

	/** Called when the system is removed from the linker */
	MOVIESCENE_API void Unlink();

	/** Called when the linker is being destroyed */
	MOVIESCENE_API void Abandon();

	/** Called when the system is added to a linker */
	MOVIESCENE_API void Link(UMovieSceneEntitySystemLinker* InLinker);

	/** Called to schedule work */
	MOVIESCENE_API void SchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* Scheduler);

	/** Called when the system should run its logic */
	MOVIESCENE_API void Run(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents);

	/** Called to know if the system is still relevant and should be kept around */
	MOVIESCENE_API bool IsRelevant(UMovieSceneEntitySystemLinker* InLinker) const;

	MOVIESCENE_API void ConditionalLinkSystem(UMovieSceneEntitySystemLinker* InLinker) const;

	MOVIESCENE_API void TagGarbage();

	MOVIESCENE_API void CleanTaggedGarbage();

	/**
	 * Enable this system if it is not already.
	 */
	MOVIESCENE_API void Enable();

	/**
	 * Disable this system if it is not already.
	 * Disabled systems will remain in the system graph, and will stay alive as long as they are relevant, but will not be Run.
	 */
	MOVIESCENE_API void Disable();


	/**
	 * Checks whether custom task sheduling is enabled or not
	 * @see FEntitySystemScheduler
	 */
	MOVIESCENE_API static bool IsCustomSchedulingEnabled();

protected:

	MOVIESCENE_API virtual bool IsReadyForFinishDestroy() override;
	MOVIESCENE_API virtual void FinishDestroy() override;

private:

	virtual void OnLink() {}

	virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) { }

	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) { }

	virtual void OnUnlink() {}

	MOVIESCENE_API virtual bool IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const;

	MOVIESCENE_API virtual void ConditionalLinkSystemImpl(UMovieSceneEntitySystemLinker* InLinker) const;

	virtual void OnTagGarbage() {}

	virtual void OnCleanTaggedGarbage() {}

protected:

	UPROPERTY()
	TObjectPtr<UMovieSceneEntitySystemLinker> Linker;

protected:

	/** Defines a single component that makes this system automatically linked when it exists in an entity manager. Override IsRelevantImpl for more complex relevancy definitions. */
	FComponentTypeID RelevantComponent;

	UE::MovieScene::ESystemPhase Phase;

	uint16 GraphID;
	uint16 GlobalDependencyGraphID;

	UE::MovieScene::EEntitySystemCategory SystemCategories;

	/** When false, this system will not call its OnRun function, but will still be kept alive as long as IsRelevant is true */
	bool bSystemIsEnabled;

#if STATS || ENABLE_STATNAMEDEVENTS
	TStatId StatID;
#endif
};

