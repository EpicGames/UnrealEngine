// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Templates/Function.h"
#include "Containers/StringFwd.h"

#include "EntitySystem/MovieSceneEntitySystemTypes.h"

namespace UE::MovieScene
{

struct FDirectedGraphStringParameters
{
	FStringView ClusterName;
	FColor Color = FColor::Black;
};

/**
 * Directed graph represented as a bitarray for allocated nodes, and edges defined by pairs of integers (from->to).
 */
struct FDirectedGraph
{
	struct FDirectionalEdge
	{
		explicit FDirectionalEdge(uint16 InFromNode, uint16 InToNode)
			: FromNode(InFromNode)
			, ToNode(InToNode)
		{}

		friend bool operator==(const FDirectionalEdge& A, const FDirectionalEdge& B)
		{
			return A.FromNode == B.FromNode && A.ToNode == B.ToNode;
		}

		friend bool operator!=(const FDirectionalEdge& A, const FDirectionalEdge& B)
		{
			return !(A == B);
		}

		friend bool operator<(const FDirectionalEdge& A, const FDirectionalEdge& B)
		{
			if (A.FromNode == B.FromNode)
			{
				return A.ToNode < B.ToNode;
			}
			return A.FromNode < B.FromNode;
		}

		uint16 FromNode;

		uint16 ToNode;
	};

public:

	struct FDepthFirstSearch
	{
		TArray<uint16> PostNodes;

		MOVIESCENE_API explicit FDepthFirstSearch(const FDirectedGraph* InGraph);

		MOVIESCENE_API void Search(uint16 Node);

		const TBitArray<>& GetVisited() const
		{
			return Visited;
		}

	private:

		TBitArray<> Visited;
		TBitArray<> IsVisiting;

		const FDirectedGraph* Graph;
	};

	struct FBreadthFirstSearch
	{
		TArray<uint16> Nodes;

		MOVIESCENE_API explicit FBreadthFirstSearch(const FDirectedGraph* InGraph);

		MOVIESCENE_API void Search(uint16 Node);

		const TBitArray<>& GetVisited() const
		{
			return Visited;
		}

	private:

		TBitArray<> Visited;
		const FDirectedGraph* Graph;
		int32 StackIndex;
	};

	struct FDiscoverCyclicEdges
	{
		MOVIESCENE_API explicit FDiscoverCyclicEdges(const FDirectedGraph* InGraph);

		MOVIESCENE_API void Search(uint16 Node);

		bool IsCyclic(const uint16 EdgeIndex) const
		{
			return CyclicEdges.IsValidIndex(EdgeIndex) && CyclicEdges[EdgeIndex] == true;
		}

		const TBitArray<>& GetCyclicEdges() const
		{
			return CyclicEdges;
		}

		MOVIESCENE_API void Search();
		MOVIESCENE_API void SearchFrom(uint16 NodeID);

	private:

		MOVIESCENE_API void DiscoverCycles(uint16 NodeID, TBitArray<>& VisitedNodes);
		MOVIESCENE_API void TagCyclicChain(uint16 CyclicNodeID);

		TBitArray<> CyclicEdges;
		TBitArray<> VisitedEdges;
		TArray<uint16, TInlineAllocator<16>> EdgeChain;
		const FDirectedGraph* Graph;
	};

public:

	FDirectedGraph()
		: bHasDanglingEdges(false)
	{}

	MOVIESCENE_API void AllocateNode(uint16 NodeID);

	MOVIESCENE_API bool IsNodeAllocated(uint16 NodeID) const;

	MOVIESCENE_API void CleanUpDanglingEdges();

	MOVIESCENE_API void RemoveNode(uint16 NodeID);

	const TBitArray<>& GetNodeMask() const
	{
		return Nodes;
	}

	MOVIESCENE_API bool IsCyclic() const;

	MOVIESCENE_API bool MakeEdge(uint16 FromNode, uint16 ToNode);

	MOVIESCENE_API void DestroyEdge(uint16 FromNode, uint16 ToNode);

	MOVIESCENE_API void DestroyAllEdges();

	MOVIESCENE_API TBitArray<> FindEdgeUpstreamNodes() const;

	MOVIESCENE_API TArrayView<const FDirectionalEdge> GetEdges() const;

	MOVIESCENE_API TArrayView<const FDirectionalEdge> GetEdgesFrom(uint16 InNode) const;

	MOVIESCENE_API bool HasEdgeFrom(uint16 InNode) const;

	MOVIESCENE_API bool HasEdgeTo(uint16 InNode) const;

	MOVIESCENE_API FString ToString(const UE::MovieScene::FDirectedGraphStringParameters& Parameters) const;

	MOVIESCENE_API FString ToString(const UE::MovieScene::FDirectedGraphStringParameters& Parameters, TFunctionRef<void(uint16, FStringBuilderBase&)> EmitLabel) const;

private:

	MOVIESCENE_API int32 FindEdgeStart(uint16 FromNode) const;

	MOVIESCENE_API int32 FindEdgeIndex(const FDirectionalEdge& Edge) const;

	MOVIESCENE_API bool EdgeExists(const FDirectionalEdge& Edge) const;

	MOVIESCENE_API bool IsCyclicImpl(uint16 NodeID, TBitArray<>& Visiting) const;

private:

	friend FDepthFirstSearch;
	friend FBreadthFirstSearch;
	friend FDiscoverCyclicEdges;

	TBitArray<> Nodes;
	TArray<FDirectionalEdge> SortedEdges;
	bool bHasDanglingEdges;
};


} // namespace UE::MovieScene
