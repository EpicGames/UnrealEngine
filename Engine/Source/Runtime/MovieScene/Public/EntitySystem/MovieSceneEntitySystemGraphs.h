// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/BitArray.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/SortedMap.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "EntitySystem/MovieSceneEntitySystemDirectedGraph.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Math/NumericLimits.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"

#include "MovieSceneEntitySystemGraphs.generated.h"

class FReferenceCollector;
class UMovieSceneEntitySystem;
class UMovieSceneEntitySystemLinker;
template <typename FuncType> class TFunctionRef;

namespace UE
{
namespace MovieScene
{

	struct FSystemSubsequentTasks;
	struct FSystemTaskPrerequisites;

	class FEntityManager;
	class FEntitySystemScheduler;


} // namespace MovieScene
} // namespace UE


USTRUCT()
struct FMovieSceneEntitySystemGraphNode
{
	GENERATED_BODY();

	FMovieSceneEntitySystemGraphNode()
		: System(nullptr)
	{}

	explicit FMovieSceneEntitySystemGraphNode(UMovieSceneEntitySystem* InSystem)
		: System(InSystem)
	{}

	TSharedPtr<UE::MovieScene::FSystemTaskPrerequisites> Prerequisites;
	TSharedPtr<UE::MovieScene::FSystemTaskPrerequisites> SubsequentTasks;

	UPROPERTY()
	TObjectPtr<UMovieSceneEntitySystem> System;
};

USTRUCT()
struct FMovieSceneEntitySystemGraphNodes
{
	GENERATED_BODY()

	void AddStructReferencedObjects(FReferenceCollector& Collector) const;

	TSparseArray<FMovieSceneEntitySystemGraphNode> Array;
};
template<>
struct TStructOpsTypeTraits<FMovieSceneEntitySystemGraphNodes> : public TStructOpsTypeTraitsBase2<FMovieSceneEntitySystemGraphNodes>
{
	enum { WithAddStructReferencedObjects = true };
};


USTRUCT()
struct FMovieSceneEntitySystemGraph
{
	using FDirectionalEdge = UE::MovieScene::FDirectedGraph::FDirectionalEdge;

	GENERATED_BODY()

	MOVIESCENE_API FMovieSceneEntitySystemGraph();
	MOVIESCENE_API ~FMovieSceneEntitySystemGraph();

	FMovieSceneEntitySystemGraph(const FMovieSceneEntitySystemGraph&) = delete;
	void operator=(const FMovieSceneEntitySystemGraph&) = delete;

	MOVIESCENE_API FMovieSceneEntitySystemGraph(FMovieSceneEntitySystemGraph&&);
	MOVIESCENE_API FMovieSceneEntitySystemGraph& operator=(FMovieSceneEntitySystemGraph&&);

	MOVIESCENE_API void AddReference(UMovieSceneEntitySystem* FromReference, UMovieSceneEntitySystem* ToReference);

	MOVIESCENE_API void RemoveReference(UMovieSceneEntitySystem* FromReference, UMovieSceneEntitySystem* ToReference);

	/** Olog(n) time */
	template<typename Allocator>
	void GatherReferencesFrom(const UMovieSceneEntitySystem* FromReference, TArray<UMovieSceneEntitySystem*, Allocator>& OutReferences)
	{
		IterateReferences(FromReference, [&OutReferences](UMovieSceneEntitySystem* System){ OutReferences.Add(System); });
	}

	template<typename Iter>
	void IterateReferencesFrom(const UMovieSceneEntitySystem* FromReference, Iter&& Iterator)
	{
		check(GetGraphID(FromReference) != TNumericLimits<uint16>::Max());
		for (const FDirectionalEdge& Edge : ReferenceGraph.GetEdgesFrom(GetGraphID(FromReference)))
		{
			Iterator(Nodes.Array[Edge.ToNode]);
		}
	}

	/** O(n) time */
	template<typename Allocator>
	void GatherReferencesTo(const UMovieSceneEntitySystem* ToReference, TArray<UMovieSceneEntitySystem*, Allocator>& OutReferences)
	{
		IterateReferences(ToReference, [&OutReferences](UMovieSceneEntitySystem* System){ OutReferences.Add(System); });
	}
	template<typename Iter>
	void IterateReferencesTo(const UMovieSceneEntitySystem* ToReference, Iter&& Iterator)
	{
		const uint16 ToNode = GetGraphID(ToReference);
		check(ToNode != TNumericLimits<uint16>::Max());
		for (const FDirectionalEdge& Edge : ReferenceGraph.GetEdges())
		{
			if (Edge.ToNode == ToNode)
			{
				Iterator(Nodes.Array[Edge.FromNode]);
			}
		}
	}

	bool IsEmpty() const
	{
		return Nodes.Array.Num() == 0;
	}

	bool HasReferencesTo(const UMovieSceneEntitySystem* ToReference) const
	{
		return ReferenceGraph.HasEdgeTo(GetGraphID(ToReference));
	}

	bool HasReferencesFrom(const UMovieSceneEntitySystem* FromReference) const
	{
		return ReferenceGraph.HasEdgeFrom(GetGraphID(FromReference));
	}

	MOVIESCENE_API void AddSystem(UMovieSceneEntitySystem* InSystem);

	MOVIESCENE_API void RemoveSystem(UMovieSceneEntitySystem* InSystem);

	MOVIESCENE_API int32 RemoveIrrelevantSystems(UMovieSceneEntitySystemLinker* Linker);

	MOVIESCENE_API void Shutdown();

	MOVIESCENE_API int32 NumInPhase(UE::MovieScene::ESystemPhase Phase) const;

	MOVIESCENE_API void ExecutePhase(UE::MovieScene::ESystemPhase Phase, UMovieSceneEntitySystemLinker* Linker, FGraphEventArray& OutTasks);

	MOVIESCENE_API void IteratePhase(UE::MovieScene::ESystemPhase Phase, TFunctionRef<void(UMovieSceneEntitySystem*)> InIter);

	MOVIESCENE_API void ReconstructTaskSchedule(UE::MovieScene::FEntityManager* EntityManager);
	MOVIESCENE_API void ScheduleTasks(UE::MovieScene::FEntityManager* EntityManager);

	MOVIESCENE_API TArray<UMovieSceneEntitySystem*> GetSystems() const;

	template<typename SystemType>
	SystemType* FindSystemOfType() const
	{
		return CastChecked<SystemType>(FindSystemOfType(SystemType::StaticClass()), ECastCheckedType::NullAllowed);
	}

	MOVIESCENE_API UMovieSceneEntitySystem* FindSystemOfType(TSubclassOf<UMovieSceneEntitySystem> InClassType) const;

	MOVIESCENE_API void DebugPrint() const;

	MOVIESCENE_API FString ToString() const;

private:

	// Implementation function that means we don't need to #include the entity system
	static MOVIESCENE_API uint16 GetGraphID(const UMovieSceneEntitySystem* InSystem);

	MOVIESCENE_API void UpdateCache();

	template<typename ArrayType>
	void ExecutePhase(UE::MovieScene::ESystemPhase Phase, const ArrayType& RetrieveEntries, UMovieSceneEntitySystemLinker* Linker, FGraphEventArray& OutTasks);

private:
	friend UE::MovieScene::FSystemSubsequentTasks;

	TArray<uint16, TInlineAllocator<4>>  SpawnPhase;
	TArray<uint16, TInlineAllocator<8>>  InstantiationPhase;
	TArray<uint16, TInlineAllocator<16>> SchedulingPhase;
	TArray<uint16, TInlineAllocator<16>> EvaluationPhase;
	TArray<uint16, TInlineAllocator<2>>  FinalizationPhase;

	TUniquePtr<UE::MovieScene::FEntitySystemScheduler> TaskScheduler;

	UPROPERTY()
	FMovieSceneEntitySystemGraphNodes Nodes;

	TMap<uint16, uint16> GlobalToLocalNodeIDs;

	UE::MovieScene::FDirectedGraph ReferenceGraph;
	uint64 SchedulerSerialNumber = 0;

	uint32 SerialNumber = 0;
	uint32 PreviousSerialNumber = 0;
	uint32 ReentrancyGuard = 0;
};


template<>
struct TStructOpsTypeTraits<FMovieSceneEntitySystemGraph> : public TStructOpsTypeTraitsBase2<FMovieSceneEntitySystemGraph>
{
	enum
	{
		WithCopy = false
	};
};
