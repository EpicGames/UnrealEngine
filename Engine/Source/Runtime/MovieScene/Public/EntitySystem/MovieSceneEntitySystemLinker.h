// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneEntityIDs.h"
#include "MovieSceneSequenceID.h"
#include "Engine/World.h"
#include "Evaluation/MovieScenePlayback.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/MovieSceneInstanceRegistry.h"
#include "Tickable.h"
#include "UObject/ObjectKey.h"
#include "Async/TaskGraphInterfaces.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "EntitySystem/MovieSceneEntitySystemGraphs.h"
#include "EntitySystem/MovieSceneSequenceInstance.h"
#include "EntitySystem/MovieSceneEntitySystemLinkerExtension.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateExtension.h"

#include "MovieSceneEntitySystemLinker.generated.h"

class FMovieSceneEntitySystemRunner;
class UMovieSceneEntitySystem;
class UMovieSceneCompiledDataManager;

namespace UE
{
namespace MovieScene
{

	struct FComponentRegistry;
	enum class ERunnerFlushState;
	enum class EEntitySystemCategory : uint32;

	enum class EAutoLinkRelevantSystems : uint8
	{
		Enabled,
		Disable,
	};

	/** Enum that describes what a sequencer ECS linker is meant for (only used for debugging reasons) */
	enum class EEntitySystemLinkerRole : uint32
	{
		/** The linker's role is unknown */
		Unknown = 0,
		/** The linker is handling level sequences */
		LevelSequences = 1,
		/** The linker is handling camera animations */
		CameraAnimations,
		/** The linker is handling UMG animations */
		UMG,
		/** The linker is handling a standalone sequence, such as those with a blocking evaluation flag */
		Standalone,
		/** This linker is running interrogations */
		Interrogation,
		/** This value and any greater values are for other custom roles */
		Custom
	};

	/** Register a new custom linker role */
	MOVIESCENE_API EEntitySystemLinkerRole RegisterCustomEntitySystemLinkerRole();

	/** Utility class for filtering systems */
	struct FSystemFilter
	{
		/** Constructs a default filter that allows all systems */
		MOVIESCENE_API FSystemFilter();

		/** Checks whether the given system class passes all filters */
		template<typename SystemClass>
		bool CheckSystem() const
		{
			return CheckSystem(SystemClass::StaticClass());
		}

		/** Checks whether the given system class passes all filters */
		MOVIESCENE_API bool CheckSystem(TSubclassOf<UMovieSceneEntitySystem> InClass) const;
		/** Checks whether the given system passes all filters */
		MOVIESCENE_API bool CheckSystem(const UMovieSceneEntitySystem* InSystem) const;

		/** Sets system categories that are allowed */
		MOVIESCENE_API void SetAllowedCategories(EEntitySystemCategory InCategory);
		/** Add system categories to be allowed */
		MOVIESCENE_API void AllowCategory(EEntitySystemCategory InCategory);
		/** Sets system categories that are disallowed */
		MOVIESCENE_API void SetDisallowedCategories(EEntitySystemCategory InCategory);
		/** Add system categories to be disallowed */
		MOVIESCENE_API void DisallowCategory(EEntitySystemCategory InCategory);

		/** Specifically allow the given system type */
		MOVIESCENE_API void AllowSystem(TSubclassOf<UMovieSceneEntitySystem> InClass);
		/** Specifically disallow the given system type */
		MOVIESCENE_API void DisallowSystem(TSubclassOf<UMovieSceneEntitySystem> InClass);

	private:
		UE::MovieScene::EEntitySystemCategory CategoriesAllowed;
		UE::MovieScene::EEntitySystemCategory CategoriesDisallowed;
		TBitArray<> SystemsAllowed;
		TBitArray<> SystemsDisallowed;
	};

}
}

DECLARE_MULTICAST_DELEGATE_OneParam(FMovieSceneEntitySystemLinkerEvent, UMovieSceneEntitySystemLinker*);
DECLARE_MULTICAST_DELEGATE_TwoParams(FMovieSceneEntitySystemLinkerAROEvent, UMovieSceneEntitySystemLinker*, FReferenceCollector&);
DECLARE_MULTICAST_DELEGATE_TwoParams(FMovieSceneEntitySystemLinkerWorldEvent, UMovieSceneEntitySystemLinker*, UWorld*);
DECLARE_MULTICAST_DELEGATE_OneParam(FMovieSceneEntitySystemLinkerPostSpawnEvent, UMovieSceneEntitySystemLinker*);

UCLASS(MinimalAPI)
class UMovieSceneEntitySystemLinker
	: public UObject
{
public:

	template<typename T>
	using TComponentTypeID    = UE::MovieScene::TComponentTypeID<T>;
	using FEntityManager      = UE::MovieScene::FEntityManager;
	using FComponentTypeID    = UE::MovieScene::FComponentTypeID;
	using FInstanceHandle     = UE::MovieScene::FInstanceHandle;
	using FMovieSceneEntityID = UE::MovieScene::FMovieSceneEntityID;
	using FInstanceRegistry   = UE::MovieScene::FInstanceRegistry;
	using FComponentRegistry  = UE::MovieScene::FComponentRegistry;

	FEntityManager EntityManager;

	UPROPERTY()
	FMovieSceneEntitySystemGraph SystemGraph;

public:

	GENERATED_BODY()

	UE::MovieScene::FPreAnimatedStateExtension PreAnimatedState;

	/** Constructs a new linker */
	MOVIESCENE_API UMovieSceneEntitySystemLinker(const FObjectInitializer& ObjInit);

	/** Gets the global component registry */
	static MOVIESCENE_API FComponentRegistry* GetComponents();

	/** Finds or creates a named linker */
	static MOVIESCENE_API UMovieSceneEntitySystemLinker* FindOrCreateLinker(UObject* PreferredOuter, UE::MovieScene::EEntitySystemLinkerRole LinkerRole, const TCHAR* Name = TEXT("DefaultMovieSceneEntitySystemLinker"));
	/** Creates a new linker */
	static MOVIESCENE_API UMovieSceneEntitySystemLinker* CreateLinker(UObject* PreferredOuter, UE::MovieScene::EEntitySystemLinkerRole LinkerRole);

	/** Gets this linker's instance registry */
	FInstanceRegistry* GetInstanceRegistry()
	{
		check(InstanceRegistry.IsValid());
		return InstanceRegistry.Get();
	}

	/** Gets this linker's instance registry */
	const FInstanceRegistry* GetInstanceRegistry() const
	{
		check(InstanceRegistry.IsValid());
		return InstanceRegistry.Get();
	}

	template<typename SystemType>
	SystemType* LinkSystem()
	{
		return CastChecked<SystemType>(LinkSystem(SystemType::StaticClass()));
	}

	/** Links a given type of system. Returns null if the system type isn't allowed on this linker */
	template<typename SystemType>
	SystemType* LinkSystemIfAllowed()
	{
		return Cast<SystemType>(LinkSystemIfAllowed(SystemType::StaticClass()));
	}

	template<typename SystemType>
	SystemType* FindSystem() const
	{
		return CastChecked<SystemType>(FindSystem(SystemType::StaticClass()), ECastCheckedType::NullAllowed);
	}

	MOVIESCENE_API UMovieSceneEntitySystem* LinkSystem(TSubclassOf<UMovieSceneEntitySystem> InClassType);
	MOVIESCENE_API UMovieSceneEntitySystem* LinkSystemIfAllowed(TSubclassOf<UMovieSceneEntitySystem> InClassType);

	MOVIESCENE_API UMovieSceneEntitySystem* FindSystem(TSubclassOf<UMovieSceneEntitySystem> Class) const;

	static MOVIESCENE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	/**
	 * Gets this linker's system filter.
	 */
	UE::MovieScene::FSystemFilter& GetSystemFilter()
	{
		return SystemFilter;
	}

	/**
	 * Gets the role of this linker
	 */
	UE::MovieScene::EEntitySystemLinkerRole GetLinkerRole() const
	{
		return Role;
	}

	/**
	 * Gets the role of this linker
	 */
	void SetLinkerRole(UE::MovieScene::EEntitySystemLinkerRole InRole)
	{
		Role = InRole;
	}

	/**
	 * Completely reset this linker back to its default state, abandoning all systems and destroying all entities
	 */
	MOVIESCENE_API void Reset();

	/**
	 * Gets the world context for this linker
	 */
	virtual UWorld* GetWorld() const override { return WeakWorld.IsValid() ? WeakWorld.Get() : Super::GetWorld(); }

	/**
	 * Sets the world context for this linker
	 */
	void SetWorld(UWorld* InWorld) { WeakWorld = InWorld; }

public:

	/**
	 * Register a new extension type for use with any instance of a UMovieSceneEntitySystemLinker
	 */
	template<typename ExtensionType>
	static UE::MovieScene::TEntitySystemLinkerExtensionID<ExtensionType> RegisterExtension()
	{
		return UE::MovieScene::TEntitySystemLinkerExtensionID<ExtensionType>(RegisterExtension().ID);
	}

	/**
	 * Add an extension to this linker.
	 *
	 * @param InID        The unique identifier for the type of extension (retrieved from RegisterExtension)
	 * @param InExtension Pointer to the extension to register - must be kept alive externally - only a raw ptr is kept in this class
	 */
	template<typename ExtensionType>
	void AddExtension(UE::MovieScene::TEntitySystemLinkerExtensionID<ExtensionType> InID, ExtensionType* InExtension)
	{
		const int32 Index = InID.ID;
		if (!ExtensionsByID.IsValidIndex(Index))
		{
			ExtensionsByID.Insert(Index, InExtension);
		}
		else
		{
			check(ExtensionsByID[Index] == InExtension);
		}
	}

	/**
	 * Add an extension to this linker.
	 *
	 * @param InID        The unique identifier for the type of extension (retrieved from RegisterExtension)
	 * @param InExtension Pointer to the extension to register - must be kept alive externally - only a raw ptr is kept in this class
	 */
	template<typename ExtensionType>
	void AddExtension(ExtensionType* InExtension)
	{
		AddExtension(ExtensionType::GetExtensionID(), InExtension);
	}

	/**
	 * Attempt to find an extension to this linker by its ID
	 *
	 * @param InID        The unique identifier for the type of extension (retrieved from RegisterExtension)
	 * @return A pointer to the extension, or nullptr if it is not active.
	 */
	template<typename ExtensionType>
	ExtensionType* FindExtension(UE::MovieScene::TEntitySystemLinkerExtensionID<ExtensionType> InID) const
	{
		const int32 Index = InID.ID;
		if (ExtensionsByID.IsValidIndex(Index))
		{
			return static_cast<ExtensionType*>(ExtensionsByID[Index]);
		}

		return nullptr;
	}


	/**
	 * Attempt to find an extension to this linker by its ID
	 *
	 * @param InID        The unique identifier for the type of extension (retrieved from RegisterExtension)
	 * @return A pointer to the extension, or nullptr if it is not active.
	 */
	template<typename ExtensionType>
	ExtensionType* FindExtension() const
	{
		const int32 Index = ExtensionType::GetExtensionID().ID;
		if (ExtensionsByID.IsValidIndex(Index))
		{
			return static_cast<ExtensionType*>(ExtensionsByID[Index]);
		}

		return nullptr;
	}


	/**
	 * Remove an extension, if it exists
	 *
	 * @param InID        The unique identifier for the type of extension (retrieved from RegisterExtension)
	 */
	void RemoveExtension(UE::MovieScene::FEntitySystemLinkerExtensionID ExtensionID)
	{
		const int32 Index = ExtensionID.ID;
		if (ExtensionsByID.IsValidIndex(Index))
		{
			ExtensionsByID.RemoveAt(Index);
		}
	}

public:

	// Internal API
	
	MOVIESCENE_API void SystemLinked(UMovieSceneEntitySystem* InSystem);
	MOVIESCENE_API void SystemUnlinked(UMovieSceneEntitySystem* InSystem);

	MOVIESCENE_API bool HasLinkedSystem(const uint16 GlobalDependencyGraphID);

	MOVIESCENE_API void LinkRelevantSystems();
	MOVIESCENE_API void UnlinkIrrelevantSystems();
	MOVIESCENE_API void AutoLinkRelevantSystems();
	MOVIESCENE_API void AutoUnlinkIrrelevantSystems();

	MOVIESCENE_API bool HasStructureChangedSinceLastRun() const;

	MOVIESCENE_API void InvalidateObjectBinding(const FGuid& ObjectBindingID, FInstanceHandle InstanceHandle);
	MOVIESCENE_API void CleanupInvalidBoundObjects();

	MOVIESCENE_API bool StartEvaluation();
	MOVIESCENE_API TSharedRef<FMovieSceneEntitySystemRunner> GetRunner() const;
	MOVIESCENE_API void PostInstantation();
	MOVIESCENE_API void EndEvaluation();

	MOVIESCENE_API void ResetRunner();

	MOVIESCENE_API void DestroyInstanceImmediately(UE::MovieScene::FRootInstanceHandle Instance);

	UE_DEPRECATED(5.5, "Please use GetRunner()")
	FMovieSceneEntitySystemRunner* GetActiveRunner() const { return &GetRunner().Get(); }

private:

	MOVIESCENE_API UMovieSceneEntitySystem* LinkSystemImpl(TSubclassOf<UMovieSceneEntitySystem> InClassType);

	MOVIESCENE_API void HandlePreGarbageCollection();
	MOVIESCENE_API void HandlePostGarbageCollection();

	MOVIESCENE_API void TagInvalidBoundObjects();
	MOVIESCENE_API void CleanGarbage();

	MOVIESCENE_API void OnWorldCleanup(UWorld* InWorld, bool bSessionEnded, bool bCleanupResources);
	MOVIESCENE_API void OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap);

	MOVIESCENE_API virtual void BeginDestroy() override;

	static MOVIESCENE_API UE::MovieScene::FEntitySystemLinkerExtensionID RegisterExtension();

private:

	TUniquePtr<FInstanceRegistry> InstanceRegistry;

	TSparseArray<UMovieSceneEntitySystem*> EntitySystemsByGlobalGraphID;
	TMap<TObjectPtr<UClass>, TObjectPtr<UMovieSceneEntitySystem>> EntitySystemsRecyclingPool;

	TSharedPtr<FMovieSceneEntitySystemRunner> Runner;
	TBitArray<> RunnerReentrancyFlags;

	TSparseArray<void*> ExtensionsByID;

	friend struct FMovieSceneEntitySystemEvaluationReentrancyWindow;

public:

	struct
	{
		FMovieSceneEntitySystemLinkerEvent          PreFlush;
		FMovieSceneEntitySystemLinkerPostSpawnEvent PostSpawnEvent;
		FMovieSceneEntitySystemLinkerEvent          TagGarbage;
		FMovieSceneEntitySystemLinkerEvent          CleanTaggedGarbage;
		FMovieSceneEntitySystemLinkerAROEvent       AddReferencedObjects;
		FMovieSceneEntitySystemLinkerEvent          AbandonLinker;
	} Events;

private:

	uint64 LastSystemLinkVersion;
	uint64 LastSystemUnlinkVersion;
	uint64 LastInstantiationVersion;

	TWeakPtr<bool> GlobalStateCaptureToken;
	TWeakObjectPtr<UWorld> WeakWorld;

protected:

	UE::MovieScene::EEntitySystemLinkerRole Role;
	UE::MovieScene::EAutoLinkRelevantSystems AutoLinkMode;
	UE::MovieScene::FSystemFilter SystemFilter;
};
