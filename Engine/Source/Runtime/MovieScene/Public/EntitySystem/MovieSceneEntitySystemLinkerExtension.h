// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace UE
{
namespace MovieScene
{

/** Base extension identifier for a UMovieSceneEntitySystemLinker */
struct FEntitySystemLinkerExtensionID
{
	int32 ID;
};

/** Typed extension identifier for a UMovieSceneEntitySystemLinker */
template<typename ExtensionType>
struct TEntitySystemLinkerExtensionID : FEntitySystemLinkerExtensionID
{
	explicit TEntitySystemLinkerExtensionID(int32 InID)
		: FEntitySystemLinkerExtensionID{ InID }
	{}
};


} // namespace MovieScene
} // namespace UE
