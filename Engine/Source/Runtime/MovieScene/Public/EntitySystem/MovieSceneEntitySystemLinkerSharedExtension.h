// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystemLinker.h"

namespace UE
{
namespace MovieScene
{

/*  */
template<typename ExtensionType>
struct TSharedEntitySystemLinkerExtension : public TSharedFromThis<ExtensionType>
{
protected:

	explicit TSharedEntitySystemLinkerExtension(UMovieSceneEntitySystemLinker* Linker)
		: WeakLinker(Linker)
	{
		Linker->AddExtension(ExtensionType::GetExtensionID(), static_cast<ExtensionType*>(this));
	}

	virtual ~TSharedEntitySystemLinkerExtension()
	{
		if (UMovieSceneEntitySystemLinker* Linker = WeakLinker.Get())
		{
			Linker->RemoveExtension(ExtensionType::GetExtensionID());
		}
	}

	TWeakObjectPtr<UMovieSceneEntitySystemLinker> WeakLinker;
};


} // namespace MovieScene
} // namespace UE
