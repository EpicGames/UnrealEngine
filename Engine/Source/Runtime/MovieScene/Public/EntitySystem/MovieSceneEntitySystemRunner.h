// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneEntityIDs.h"
#include "MovieSceneSequenceID.h"
#include "Evaluation/MovieScenePlayback.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "EntitySystem/MovieSceneSequenceInstance.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "Evaluation/MovieScenePlayback.h"
#include "HAL/Platform.h"
#include "MovieSceneEntityIDs.h"
#include "MovieSceneSequenceID.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UMovieSceneEntitySystemLinker;

namespace UE::MovieScene
{

class FEntityManager;
struct FInstanceRegistry;

/** Bit-mask enumeration that defines tasks that need running */
enum class ERunnerFlushState
{
	None                 = 0,			// Signifies no evaluation is currently underway
	Start                = 1 << 0,		// Sets up initial evaluation flags for external players and listeners
	ConditionalRecompile = 1 << 1,		// Conditional recompile of dirtied sequences.
	Import               = 1 << 2,		// Update sequence instances and import entities into the entity manager
	ReimportAfterCompile = 1 << 3,		// Re-update sequence instances after a recompile occurred on a partially evaluated sequence. Not normally run.
	Spawn                = 1 << 4,		// Perform the Spawn phase in the Entity System Graph
	Instantiation        = 1 << 5,		// Perform the Instantiation phase in the Entity System Graph
	Evaluation           = 1 << 6,		// Perform the Evaluation phase in the Entity System Graph
	Finalization         = 1 << 7,		// Perform the Finalization phase in the Entity System Graph and trigger any external events
	EventTriggers        = 1 << 8,		// (re-entrant) Triggers any bound event triggers - skipped by Finalization if the delegate is not bound
	PostEvaluation       = 1 << 9,		// (re-entrant) Call post evaluation callbacks on sequence instances
	End                  = 1 << 10,		// Counterpart for Start - resets external players and listeners
};
ENUM_CLASS_FLAGS(ERunnerFlushState)

/** Bit-field enumeration that defines flags for an asynchronous update request */
enum class ERunnerUpdateFlags
{
	None           = 0,			// No special behavior - just update the instance along with everything else when the runner is next flushed
	Flush          = 1<<0,		// Indicate that the runner should be completely flushed next update, ignoring any budgets
	Finish         = 1<<1,		// Finish the sequence instance and restore state (if enabled) after this update
	Destroy        = 1<<2,		// Destroy the sequence instance once this request has been fulfilled

	FinalDissectionMask = Finish|Destroy, // A mask that includes flags that should only ever be included on final (or non-)dissected updates, not intermediate updates
};
ENUM_CLASS_FLAGS(ERunnerUpdateFlags)

/** Result from a runner flush state that indicates how to proceed */
enum class ERunnerFlushResult
{
	/** Continue evaluation, allow budgeting */
	ContinueAllowBudget,
	/** Continue evaluation without budgeting the next step. */
	ContinueNoBudgeting,
	/** Do not continue evaluation and break out of our loop. */
	Break,
};

} // namespace UE::MovieScene

DECLARE_MULTICAST_DELEGATE(FMovieSceneEntitySystemEventTriggers);

class FMovieSceneEntitySystemRunner : public TSharedFromThis<FMovieSceneEntitySystemRunner>
{
public:

	using FEntityManager = UE::MovieScene::FEntityManager;
	using FInstanceHandle = UE::MovieScene::FInstanceHandle;
	using FInstanceRegistry = UE::MovieScene::FInstanceRegistry;

public:

	/** Destructor */
	MOVIESCENE_API ~FMovieSceneEntitySystemRunner();

	/** Gets how many instances are in the update queue. */
	MOVIESCENE_API int32 GetQueuedUpdateCount() const;
	/** Returns whether this runner has any outstanding updates. */
	MOVIESCENE_API bool HasQueuedUpdates() const;
	/** Returns whether the given instance is queued for any updates. */
	MOVIESCENE_API bool HasQueuedUpdates(FInstanceHandle Instance) const;
	/** Queue the given instance for an update with the given context. */
	MOVIESCENE_API void QueueUpdate(const FMovieSceneContext& Context, FInstanceHandle Instance, UE::MovieScene::ERunnerUpdateFlags UpdateFlags = UE::MovieScene::ERunnerUpdateFlags::None);
	MOVIESCENE_API void QueueUpdate(const FMovieSceneContext& Context, FInstanceHandle Instance, FSimpleDelegate&& InOnFlushedDelegate, UE::MovieScene::ERunnerUpdateFlags UpdateFlags = UE::MovieScene::ERunnerUpdateFlags::None);

	/**
	 * Queue a final update for the specified instance, optionally destroying it after finishing it
	 * @return true if the update requires a flush, or false if it was finished and/or destroyed immediately without requiring a flush
	 */
	MOVIESCENE_API bool QueueFinalUpdate(FInstanceHandle Instance);
	MOVIESCENE_API bool QueueFinalUpdate(FInstanceHandle Instance, FSimpleDelegate&& InOnLastFlushDelegate);
	MOVIESCENE_API bool QueueFinalUpdateAndDestroy(FInstanceHandle Instance);
	MOVIESCENE_API bool QueueFinalUpdateAndDestroy(FInstanceHandle Instance, FSimpleDelegate&& InOnLastFlushDelegate);

	/**
	 * Abandon the specified instance handle and destroy it immediately. May flush this runner if necessary
	 */
	MOVIESCENE_API void AbandonAndDestroyInstance(FInstanceHandle Instance);
	MOVIESCENE_API void AbandonAndDestroyInstance(FInstanceHandle Instance, FSimpleDelegate&& InOnLastFlushDelegate);

	/**
	 * Flushes the update queue and applies any outstanding evaluation logic
	 * 
	 * @param BudgetMs      A budget (in milliseconds) to use for evaluation. Evaluation will cease prematurely once this budget is spent
	 *                      and will process the outstanding work on the next call to Flush. A value of 0.0 signifies no budget - the queue
	 *                      will be fully processed without leaving any outstanding work
	 */
	MOVIESCENE_API void Flush(double BudgetMs = 0.f, UE::MovieScene::ERunnerFlushState TargetState = UE::MovieScene::ERunnerFlushState::None);

	/**
	 * Flushes any outstanding update tasks in the current evaluation scope with a given budget. Only performs work if this runner is part-way through an evaluation
	 * 
	 * @param BudgetMs      A budget (in milliseconds) to use for evaluation. Evaluation will cease prematurely once this budget is spent
	 *                      and will process the outstanding work on the next call to Flush. A value of 0.0 signifies no budget - the queue
	 *                      will be fully processed without leaving any outstanding work
	 * @param TargetState   The desired state to reach. The runner will stop flushing as soon as all the states specified in TargetState have been flushed.
	 */
	MOVIESCENE_API void FlushOutstanding(double BudgetMs = 0.f, UE::MovieScene::ERunnerFlushState TargetState = UE::MovieScene::ERunnerFlushState::None);

	/**
	 * Called in the event that the structure of the entity manager has been unexpectedly changed while this runner is active.
	 * This allows the runner to be reset so it runs from the start of its evaluation loop next time it is evaluated, rather than half-way through
	 */
	MOVIESCENE_API void ResetFlushState();

	/**
	 * Discard any queued updates for the specified sequence instance.
	 */
	MOVIESCENE_API void DiscardQueuedUpdates(FInstanceHandle Instance);

	/** Access this runner's currently executing phase */
	UE::MovieScene::ESystemPhase GetCurrentPhase() const
	{
		return CurrentPhase;
	}

	/**
	 * Check whether this runner is currently inside an active evaluation loop
	 */
	MOVIESCENE_API bool IsCurrentlyEvaluating() const;

	/**
	 * Check whether this runner is currently updating a sequence
	 */
	MOVIESCENE_API bool IsUpdatingSequence() const;

	/**
	 * Run a single evaluation phase
	 *
	 * @return Whether the phase was run
	 */
	MOVIESCENE_API bool FlushSingleEvaluationPhase();

public:

	UMovieSceneEntitySystemLinker* GetLinker() const { return WeakLinker.Get(); }
	bool IsValid() const { return WeakLinker.IsValid(); }

	MOVIESCENE_API FEntityManager* GetEntityManager() const;
	MOVIESCENE_API FInstanceRegistry* GetInstanceRegistry() const;

public:

	UE_DEPRECATED(5.5, "Runners are now owned by linkers, and always attached")
	void AttachToLinker(UMovieSceneEntitySystemLinker* InLinker) {}
	UE_DEPRECATED(5.5, "Runners are now owned by linkers, and always attached")
	bool IsAttachedToLinker() const { return IsValid(); }
	UE_DEPRECATED(5.5, "Runners are now owned by linkers, and always attached")
	void DetachFromLinker() {}

public:
	
	// Internal API

	void MarkForUpdate(FInstanceHandle InInstanceHandle, UE::MovieScene::ERunnerUpdateFlags UpdateFlags);
	MOVIESCENE_API FMovieSceneEntitySystemEventTriggers& GetQueuedEventTriggers();

	/** Creates an unbound runner */
	FMovieSceneEntitySystemRunner();
	/** Create a bound runner */
	FMovieSceneEntitySystemRunner(UMovieSceneEntitySystemLinker* InLinker);

private:

	/**
	 * Queue the final update of a given instance, optionally destroying it after is finishes
	 */
	MOVIESCENE_API bool QueueFinalUpdateImpl(FInstanceHandle Instance, FSimpleDelegate&& InOnLastFlushDelegate, bool bDestroyInstance);

	/**
	 * Flush the next item in our update loop based off the contents of FlushState
	 *
	 * @param  Linker   The linker we are arrached to
	 * @return True if the loop is allowed to continue, or false if we should not flush any more
	 */
	MOVIESCENE_API UE::MovieScene::ERunnerFlushResult FlushNext(UMovieSceneEntitySystemLinker* Linker);

	/**
	 * Set up initial state before any evaluation runs. Only called once regardless of the number of pending updates we have to process
	 * Primarily used for setting up external 'is evaluating' flags for re-entrancy and async checks.
	 */
	MOVIESCENE_API UE::MovieScene::ERunnerFlushResult StartEvaluation(UMovieSceneEntitySystemLinker* Linker);

	/** Execute any pending conditional recompiles on the currently queued update requests */
	MOVIESCENE_API UE::MovieScene::ERunnerFlushResult GameThread_ConditionalRecompile(UMovieSceneEntitySystemLinker* Linker);
	/** Update sequence instances based on currently queued update requests, or outstanding dissected updates */
	MOVIESCENE_API UE::MovieScene::ERunnerFlushResult GameThread_UpdateSequenceInstances(UMovieSceneEntitySystemLinker* Linker);
	/** Re-update sequence instances after a recompile on a partially evaluated sequence */
	MOVIESCENE_API UE::MovieScene::ERunnerFlushResult GameThread_ReimportSequenceInstances(UMovieSceneEntitySystemLinker* Linker);
	/** Execute the spawn phase of the entity system graph, if there is anything to do */
	MOVIESCENE_API UE::MovieScene::ERunnerFlushResult GameThread_SpawnPhase(UMovieSceneEntitySystemLinker* Linker);
	/** Execute the instantiation phase of the entity system graph, if there is anything to do */
	MOVIESCENE_API UE::MovieScene::ERunnerFlushResult GameThread_InstantiationPhase(UMovieSceneEntitySystemLinker* Linker);
	/** Called immediately after instantiation to execute cleanup and bookkeeping tasks. Skipped if instantiation is skipped.*/
	MOVIESCENE_API UE::MovieScene::ERunnerFlushResult GameThread_PostInstantiation(UMovieSceneEntitySystemLinker* Linker);
	/** Main entity-system evaluation phase. Blocks this thread until completion. */
	MOVIESCENE_API UE::MovieScene::ERunnerFlushResult GameThread_EvaluationPhase(UMovieSceneEntitySystemLinker* Linker);
	/** Finalization phase for triggering external events and other behavior. */
	MOVIESCENE_API void GameThread_EvaluationFinalizationPhase(UMovieSceneEntitySystemLinker* Linker);
	/** Post-evaluation phase for triggering events. */
	MOVIESCENE_API void GameThread_EventTriggerPhase(UMovieSceneEntitySystemLinker* Linker);
	/** Post-evaluation phase for clean up. */
	MOVIESCENE_API void GameThread_PostEvaluationPhase(UMovieSceneEntitySystemLinker* Linker);

	/**
	 * Counterpart for StartEvaluation.
	 * Called only when our UpdateQueue and DissectedUpdates have been fully processed and there is nothing left to do.
	 */
	MOVIESCENE_API void EndEvaluation(UMovieSceneEntitySystemLinker* Linker);

	/** Called to set/unset the necessary flags to enter a new flush state and progress to the next */
	MOVIESCENE_API void EnterFlushState(UE::MovieScene::ERunnerFlushState EnteredFlushState);

	/** Skip the specified flush states if they are currently pending */
	MOVIESCENE_API void SkipFlushState(UE::MovieScene::ERunnerFlushState FlushStateToSkip);

private:

	friend struct FMovieSceneEntitySystemEvaluationReentrancyWindow;

	struct FDissectedUpdate
	{
		FSimpleDelegate OnFlushed;
		FMovieSceneContext Context;

		FInstanceHandle InstanceHandle;
		int32 Order;
		UE::MovieScene::ERunnerUpdateFlags UpdateFlags = UE::MovieScene::ERunnerUpdateFlags::None;
	};

	/** Owning linker */
	TWeakObjectPtr<UMovieSceneEntitySystemLinker> WeakLinker;

	struct FQueuedUpdateParams
	{
		FInstanceHandle InstanceHandle;
		UE::MovieScene::ERunnerUpdateFlags UpdateFlags = UE::MovieScene::ERunnerUpdateFlags::None;
	};
	struct FUpdateParamsAndContext
	{
		FSimpleDelegate OnFlushed;
		FMovieSceneContext Context;
		FQueuedUpdateParams Params;
	};
	/** Queue of sequence instances to be updated */
	TArray<FUpdateParamsAndContext> UpdateQueue;

	/** When an update is running, the list of actual instances being updated */
	TArray<FQueuedUpdateParams> CurrentInstances;
	/** When an update is running, the list of sub-contexts for the requested update */
	TArray<FDissectedUpdate> DissectedUpdates;
	TArray<FSimpleDelegate> OnFlushedDelegates;
	FMovieSceneEntitySystemEventTriggers EventTriggers;
	/** Update flags that are accumulated for all currently active sequence instances being evaluated */
	UE::MovieScene::ESequenceInstanceUpdateFlags AccumulatedUpdateFlags;

	/** The number of times this runner has been re-entrant */
	uint32 ReentrancyCount;

	ENamedThreads::Type GameThread;

	UE::MovieScene::ESystemPhase CurrentPhase;

	/**
	 * Defines a bitmask of outstanding tasks that need running.
	 * When a task has completed its bit becomes unset.
	 * Evaluation can trigger tasks to run again by setting previously cleared bits in this mask.
	 * A state of ERunnerFlushState::Everything means an evaluation has not yet started.
	 * A state of ERunnerFlushState::None means an evaluation has just finished.
	 */
	UE::MovieScene::ERunnerFlushState FlushState;

	/** The current FlushState that we are running in the current callstack. Used to detect re-entrancy */
	UE::MovieScene::ERunnerFlushState CurrentFlushState;

	bool bCanQueueEventTriggers;

	/** True if any update has been queued with the ERunnerUpdateFlags::Flush flag since our last flush */
	bool bRequireFullFlush;

	bool bIsUpdatingSequence;
};
