// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"
#include "AutoRTFM.h"
#include "Misc/GeneratedTypeName.h"
#include "MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/EntityAllocationIterator.h"
#include "EntitySystem/MovieSceneEntityRange.h"
#include "EntitySystem/MovieSceneComponentAccessors.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/MovieSceneSystemTaskDependencies.h"
#include "EntitySystem/MovieSceneComponentPtr.h"
#include "EntitySystem/IMovieSceneTaskScheduler.h"

#include "Templates/AndOrNot.h"

#include <initializer_list>

namespace UE
{
namespace MovieScene
{

DECLARE_CYCLE_STAT(TEXT("Aquire Component Access Locks"), MovieSceneEval_AquireComponentAccessLocks, STATGROUP_MovieSceneECS);
DECLARE_CYCLE_STAT(TEXT("Release Component Access Locks"), MovieSceneEval_ReleaseComponentAccessLocks, STATGROUP_MovieSceneECS);

template<typename> struct TReadAccess;
template<typename> struct TOptionalReadAccess;
template<typename> struct TWriteAccess;
template<typename> struct TOptionalWriteAccess;
template<typename...> struct TReadOneOfAccessor;
template<typename...> struct TReadOneOrMoreOfAccessor;

template<typename...> struct TPrelockedDataOffsets;

template<typename...> struct TFilteredEntityTask;
template<typename...> struct TEntityTaskComponents;
template<typename...> struct TEntityTaskComponentsImpl;

template<typename, typename...> struct TEntityTask;
template<typename, typename...> struct TEntityTaskBase;
template<typename, typename...> struct TScheduledEntityTask;
template<typename, typename...> struct TEntityAllocationTask;
template<typename, typename...> struct TEntityAllocationTaskBase;
template<typename> struct TUnstructuredTask;

namespace Private
{
	template<typename...> struct TComponentTypeIDsImpl;
}
template<typename...T>
using TComponentTypeIDs = Private::TComponentTypeIDsImpl<TMakeIntegerSequence<int, sizeof...(T)>, T...>;


template<int32 NumComponents, bool AutoExpandAccessors>
struct TEntityTaskCaller;

struct FCommonEntityTaskParams
{
	FCommonEntityTaskParams()
		: TaskParams(nullptr)
	{}

	/** Task parameters */
	FTaskParams TaskParams;

	/** (deprecated) The thread that this task wants to run on */
	ENamedThreads::Type DesiredThread = ENamedThreads::AnyHiPriThreadHiPriTask;

	/** Useful for debugging to break the debugger when this task is run */
	bool bBreakOnRun = false;
};

/** Default traits specialized for each user TaskImplInstance */
template<typename T>
struct TDefaultEntityTaskTraits
{
	enum
	{
		/**
		 * When true, the various component accessors are passed to the task callback as separate parameters. When false, they are passed through as a combined template
		 * 
		 * For example:
		 *
		 * struct FForEach_Expanded
		 * { 
		 *     void ForEachEntity(float, uint16, UObject*);
		 *     void ForEachAllocation(const FEntityAllocation*, TRead<float>, TRead<uint16>, TRead<UObject*>);
		 * };
		 * struct FForEach_NoExpansion
		 * {
		 *     void ForEachEntity(const TEntityPtr<const float, const uint16, const UObject*>&);
		 *     void ForEachAllocation(const FEntityAllocation*, const TEntityTaskComponents<TRead<float>, TRead<uint16>, TRead<UObject*>>&);
		 * };
		 * template<> struct TEntityTaskTraits<FForEach_NoExpansion> : TDefaultEntityTaskTraits<FForEach_NoExpansion> { enum { AutoExpandAccessors = false }; };
		 * 
		 * FEntityTaskBuilder().Read<float>().Read<uint16>().Read<UObject*>().Dispatch_PerEntity<FForEach_Expanded>(...);
		 * FEntityTaskBuilder().Read<float>().Read<uint16>().Read<UObject*>().Dispatch_PerEntity<FForEach_NoExpansion>(...);
		 * FEntityTaskBuilder().Read<float>().Read<uint16>().Read<UObject*>().Dispatch_PerAllocation<FForEach_Expanded>(...);
		 * FEntityTaskBuilder().Read<float>().Read<uint16>().Read<UObject*>().Dispatch_PerAllocation<FForEach_NoExpansion>(...);
		 */
		AutoExpandAccessors = true,
	};
};

/** Optionally specialized traits for user TaskImplInstances */
template<typename T>
struct TEntityTaskTraits : TDefaultEntityTaskTraits<T>
{
};

/** Utility that promotes callbacks that return void to always return 'true' when iterating entities*/
struct FEntityIterationResult
{
	template<typename T>
	friend inline FEntityIterationResult operator,(T, FEntityIterationResult)
	{
		return FEntityIterationResult { true };
	}

	friend inline FEntityIterationResult operator,(bool In, FEntityIterationResult)
	{
		return FEntityIterationResult{ In };
	}

	inline explicit operator bool() const
	{
		return Value;
	}

	bool Value = true;
};

template <typename TaskType>
static FTaskID SchedulePreTask(IEntitySystemScheduler*, const FTaskParams&, const TSharedPtr<TaskType>&, void*)
{
	return FTaskID::None();
}
template <typename TaskType>
static FTaskID SchedulePostTask(IEntitySystemScheduler*, const FTaskParams&, const TSharedPtr<TaskType>&, void*)
{
	return FTaskID::None();
}

template <typename TaskType, typename TaskImplType>
static FTaskID SchedulePreTask(IEntitySystemScheduler* InScheduler, const FTaskParams& Params, const TSharedPtr<TaskType>& Context, TaskImplType* Unused, decltype(&TaskImplType::PreTask)* = 0)
{
	struct FCallPreTask
	{
		static void Run(const ITaskContext* Context, FEntityAllocationWriteContext WriteContext)
		{
			static_cast<const TaskType*>(Context)->PreTask();
		}
	};

	TaskFunctionPtr Function(TInPlaceType<UnboundTaskFunctionPtr>(), &FCallPreTask::Run);
	return InScheduler->AddTask(Params, Context, Function);
}
template <typename TaskType, typename TaskImplType>
static FTaskID SchedulePostTask(IEntitySystemScheduler* InScheduler, const FTaskParams& Params, const TSharedPtr<TaskType>& Context, TaskImplType* Unused, decltype(&TaskImplType::PostTask)* = 0)
{
	struct FCallPostTask
	{
		static void Run(const ITaskContext* Context, FEntityAllocationWriteContext WriteContext)
		{
			static_cast<const TaskType*>(Context)->PostTask();
		}
	};

	TaskFunctionPtr Function(TInPlaceType<UnboundTaskFunctionPtr>(), &FCallPostTask::Run);
	return InScheduler->AddTask(Params, Context, Function);
}

/**
 * Defines the accessors for each desired component of an entity task
 */
template<typename... T>
struct TEntityTaskComponents : TEntityTaskComponentsImpl<TMakeIntegerSequence<int, sizeof...(T)>, T...>
{
	using Super = TEntityTaskComponentsImpl<TMakeIntegerSequence<int, sizeof...(T)>, T...>;


	/**
	 * Constrain this task to only run for entities that have all the specified components or tags
	 */
	TFilteredEntityTask< T... > FilterAll(const FComponentMask& InComponentMask)
	{
		TFilteredEntityTask< T... > Filtered(*this);
		Filtered.FilterAll(InComponentMask);
		return Filtered;
	}

	/**
	 * Constrain this task to only run for entities that have all the specified components or tags
	 */
	TFilteredEntityTask< T... > FilterAll(std::initializer_list<FComponentTypeID> InComponentTypes)
	{
		TFilteredEntityTask< T... > Filtered(*this);
		Filtered.FilterAll(InComponentTypes);
		return Filtered;
	}

	/**
	 * Constrain this task to only run for entities that have none the specified components or tags
	 */
	TFilteredEntityTask< T... > FilterNone(const FComponentMask& InComponentMask)
	{
		TFilteredEntityTask< T... > Filtered(*this);
		Filtered.FilterNone(InComponentMask);
		return Filtered;
	}

	/**
	 * Constrain this task to only run for entities that have none the specified components or tags
	 */
	TFilteredEntityTask< T... > FilterNone(std::initializer_list<FComponentTypeID> InComponentTypes)
	{
		TFilteredEntityTask< T... > Filtered(*this);
		Filtered.FilterNone(InComponentTypes);
		return Filtered;
	}

	/**
	 * Constrain this task to only run for entities that have at least one of the specified components or tags
	 */
	TFilteredEntityTask< T... > FilterAny(const FComponentMask& InComponentMask)
	{
		TFilteredEntityTask< T... > Filtered(*this);
		Filtered.FilterAny(InComponentMask);
		return Filtered;
	}

	/**
	 * Constrain this task to only run for entities that have at least one of the specified components or tags
	 */
	TFilteredEntityTask< T... > FilterAny(std::initializer_list<FComponentTypeID> InComponentTypes)
	{
		TFilteredEntityTask< T... > Filtered(*this);
		Filtered.FilterAny(InComponentTypes);
		return Filtered;
	}

	/**
	 * Constrain this task to only run for entities that do not have the specific combination of components or tags
	 */
	TFilteredEntityTask< T... > FilterOut(const FComponentMask& InComponentMask)
	{
		TFilteredEntityTask< T... > Filtered(*this);
		Filtered.FilterOut(InComponentMask);
		return Filtered;
	}

	/**
	 * Constrain this task to only run for entities that do not have the specific combination of components or tags
	 */
	TFilteredEntityTask< T... > FilterOut(std::initializer_list<FComponentTypeID> InComponentTypes)
	{
		TFilteredEntityTask< T... > Filtered(*this);
		Filtered.FilterOut(InComponentTypes);
		return Filtered;
	}

	/**
	 * Combine this task's filter with the specified filter
	 */
	TFilteredEntityTask< T... > CombineFilter(const FEntityComponentFilter& InFilter)
	{
		return TFilteredEntityTask< T... >(*this, InFilter);
	}

	/**
	 * Assign a desired thread for this task to run on
	 */
	TEntityTaskComponents< T... >& SetDesiredThread(ENamedThreads::Type InDesiredThread)
	{
		this->CommonParams.DesiredThread = InDesiredThread;
		this->CommonParams.TaskParams.bForceGameThread = (InDesiredThread == ENamedThreads::GameThread || InDesiredThread == ENamedThreads::GameThread_Local);
		return *this;
	}

	/**
	 * Assign a stat ID for this task
	 */
	TEntityTaskComponents< T... >& SetStat(TStatId InStatId)
	{
		this->CommonParams.TaskParams.StatId = InStatId;
		return *this;
	}

	/**
	 * Assign the scheduled task parameters for this task
	 */
	TEntityTaskComponents< T... >& SetParams(const FTaskParams& InOtherParams)
	{
		this->CommonParams.TaskParams = InOtherParams;
		return *this;
	}

	/**
	 * Dispatch a custom task that runs non-structured logic.
	 * Tasks must implement a Run function that doesn't take any argument.
	 *
	 * @param EntityManager    The entity manager to run the task on
	 * @param Prerequisites    Prerequisite tasks that must run before this one, or nullptr if there are no prerequisites
	 * @param Subsequents      (Optional) Subsequent task tracking that this task should be added to for each writable component type
	 * @param InArgs           Optional arguments that are forwarded to the constructor of TaskImpl
	 * @return A pointer to the graph event for the task, or nullptr if this task is not valid (ie contains invalid component types that would be necessary for the task to run), or threading is disabled
	 */
	template<typename TaskImpl, typename... TaskConstructionArgs>
	FGraphEventRef Dispatch(FEntityManager* EntityManager, const FSystemTaskPrerequisites& Prerequisites, FSystemSubsequentTasks* Subsequents, TaskConstructionArgs&&... InArgs) const
	{
		static_assert(sizeof...(T) == 0, "Dispatch() is only for non-structured logic, which means that any call to Read or Write (or their variants) won't do anything -- please remove them");

		checkfSlow(IsInGameThread(), TEXT("Tasks can only be dispatched from the game thread."));

		const bool bRunInline = !ensure(EntityManager->IsLockedDown()) || EntityManager->GetThreadingModel() == EEntityThreadingModel::NoThreading;
		if (bRunInline)
		{
			TaskImpl Task{ Forward<TaskConstructionArgs>(InArgs)... };
			Task.Run();
			return nullptr;
		}
		else
		{
			FGraphEventArray GatheredPrereqs;
			this->PopulatePrerequisites(Prerequisites, &GatheredPrereqs);

			ENamedThreads::Type ThisThread = EntityManager->GetDispatchThread();
			checkSlow(ThisThread != ENamedThreads::AnyThread);

			FGraphEventRef NewTask = TGraphTask< TUnstructuredTask<TaskImpl> >::CreateTask(GatheredPrereqs.Num() != 0 ? &GatheredPrereqs : nullptr, ThisThread)
				.ConstructAndDispatchWhenReady( this->CommonParams, Forward<TaskConstructionArgs>(InArgs)... );

			if (Subsequents)
			{
				this->PopulateSubsequents(NewTask, *Subsequents);
			}

			return NewTask;
		}
	}

	/**
	 * Dispatch a task for every allocation that matches the filters and component types. Must be explicitly instantiated with the task type to dispatch. Construction arguments are deduced.
	 * Tasks must implement a ForEachAllocation function that matches this task's component accessor types.
	 * 
	 * For example:
	 * struct FForEachAllocation
	 * {
	 *     void ForEachAllocation(FEntityAllocation* InAllocation, const TFilteredEntityTask< FEntityIDAccess, TRead<FMovieSceneFloatChannel> >& InputTask);
	 * };
	 *
	 * TComponentTypeID<FMovieSceneFloatChannel> FloatChannelComponent = ...;
	 *
	 * FGraphEventRef Task = FEntityTaskBuilder()
	 * .ReadEntityIDs()
	 * .Read(FloatChannelComponent)
	 * .SetStat(GET_STATID(MyStatName))
	 * .SetDesiredThread(ENamedThreads::AnyThread)
	 * .Dispatch_PerAllocation<FForEachAllocation>(EntityManager, Prerequisites);
	 *
	 * @param EntityManager    The entity manager to run the task on. All component types *must* relate to this entity manager.
	 * @param Prerequisites    Prerequisite tasks that must run before this one
	 * @param Subsequents      (Optional) Subsequent task tracking that this task should be added to for each writable component type
	 * @param InArgs           Optional arguments that are forwarded to the constructor of TaskImpl
	 * @return A pointer to the graph event for the task, or nullptr if this task is not valid (ie contains invalid component types that would be necessary for the task to run), or threading is disabled
	 */
	template<typename TaskImpl, typename... TaskConstructionArgs>
	FGraphEventRef Dispatch_PerAllocation(FEntityManager* EntityManager, const FSystemTaskPrerequisites& Prerequisites, FSystemSubsequentTasks* Subsequents, TaskConstructionArgs&&... InArgs) const
	{
		checkfSlow(IsInGameThread(), TEXT("Tasks can only be dispatched from the game thread."));

		if (!this->IsValid())
		{
			return nullptr;
		}

		// Quick check to prevent dispatching a task that might not have any work to do. We only check the accessors
		// here, so the task could still early-return with no work done if some custom filters end up matching
		// nothing. Callers should in this case do their best to prevent useless tasks being dispatched.
		if (!this->HasAnyWork(EntityManager))
		{
			return nullptr;
		}

		// If this ensure triggers, we are not in the evaluation phase - the callee should be using RunInline_ or Iterate_ variants
		const bool bRunInline = !ensure(EntityManager->IsLockedDown()) || EntityManager->GetThreadingModel() == EEntityThreadingModel::NoThreading;
		if (bRunInline)
		{
			TaskImpl Task{ Forward<TaskConstructionArgs>(InArgs)... };
			TEntityAllocationTaskBase<TaskImpl, T...>(EntityManager, *this).Run(Task);
			return nullptr;
		}
		else
		{
			FGraphEventArray GatheredPrereqs;
			this->PopulatePrerequisites(Prerequisites, &GatheredPrereqs);

			ENamedThreads::Type ThisThread = EntityManager->GetDispatchThread();
			checkSlow(ThisThread != ENamedThreads::AnyThread);

			FGraphEventRef NewTask = TGraphTask< TEntityAllocationTask<TaskImpl, T...> >::CreateTask(GatheredPrereqs.Num() != 0 ? &GatheredPrereqs : nullptr, ThisThread)
				.ConstructAndDispatchWhenReady( EntityManager, *this, Forward<TaskConstructionArgs>(InArgs)... );

			if (Subsequents)
			{
				this->PopulateSubsequents(NewTask, *Subsequents);
			}

			return NewTask;
		}
	}

	template<typename TaskImpl>
	void RunInline_PerAllocation(FEntityManager* EntityManager, TaskImpl& Task) const
	{
		if (this->IsValid())
		{
			TEntityAllocationTaskBase<TaskImpl, T...>(EntityManager, *this).Run(Task);
		}
	}

	/**
	 * Dispatch a task for every entity that matches the filters and component types. Must be explicitly instantiated with the task type to dispatch. Construction arguments are deduced.
	 * Tasks must implement a ForEachEntity function that matches this task's component accessor types.
	 * 
	 * For example:
	 * struct FForEachEntity
	 * {
	 *     void ForEachEntity(FMovieSceneEntityID InEntityID, const FMovieSceneFloatChannel& Channel);
	 * };
	 *
	 * TComponentTypeID<FMovieSceneFloatChannel> FloatChannelComponent = ...;
	 *
	 * FGraphEventRef Task = FEntityTaskBuilder()
	 * .ReadEntityIDs()
	 * .Read(FloatChannelComponent)
	 * .SetStat(GET_STATID(MyStatName))
	 * .SetDesiredThread(ENamedThreads::AnyThread)
	 * .Dispatch_PerEntity<FForEachEntity>(EntityManager, nullptr);
	 *
	 * @param EntityManager    The entity manager to run the task on. All component types *must* relate to this entity manager.
	 * @param Prerequisites    Prerequisite tasks that must run before this one, or nullptr if there are no prerequisites
	 * @param Subsequents      (Optional) Subsequent task tracking that this task should be added to for each writable component type
	 * @param InArgs           Optional arguments that are forwarded to the constructor of TaskImpl
	 * @return A pointer to the graph event for the task, or nullptr if this task is not valid (ie contains invalid component types that would be necessary for the task to run), or threading is disabled
	 */
	template<typename TaskImpl, typename... TaskConstructionArgs>
	FGraphEventRef Dispatch_PerEntity(FEntityManager* EntityManager, const FSystemTaskPrerequisites& Prerequisites, FSystemSubsequentTasks* Subsequents, TaskConstructionArgs&&... InArgs) const
	{
		checkfSlow(IsInGameThread(), TEXT("Tasks can only be dispatched from the game thread."));

		if (!this->IsValid())
		{
			return nullptr;
		}

		// See comment in Dispatch_PerAllocation()
		if (!this->HasAnyWork(EntityManager))
		{
			return nullptr;
		}

		// If this ensure triggers, we are not in the evaluation phase - the callee should be using RunInline_ or Iterate_ variants
		const bool bRunInline = !ensure(EntityManager->IsLockedDown()) || EntityManager->GetThreadingModel() == EEntityThreadingModel::NoThreading;
		if (bRunInline)
		{
			TaskImpl Task{ Forward<TaskConstructionArgs>(InArgs)... };
			TEntityTaskBase<TaskImpl, T...>(EntityManager, *this).Run(Task);
			return nullptr;
		}
		else
		{
			FGraphEventArray GatheredPrereqs;
			this->PopulatePrerequisites(Prerequisites, &GatheredPrereqs);

			ENamedThreads::Type ThisThread = EntityManager->GetDispatchThread();
			checkSlow(ThisThread != ENamedThreads::AnyThread);

			FGraphEventRef NewTask = TGraphTask< TEntityTask<TaskImpl, T...> >::CreateTask(GatheredPrereqs.Num() != 0 ? &GatheredPrereqs : nullptr, ThisThread)
				.ConstructAndDispatchWhenReady( EntityManager, *this, Forward<TaskConstructionArgs>(InArgs)... );

			if (Subsequents)
			{
				this->PopulateSubsequents(NewTask, *Subsequents);
				Subsequents->AddRootTask(NewTask);
			}

			return NewTask;
		}
	}

	template<typename TaskImpl, typename... TaskConstructionArgs>
	FTaskID Fork_PerEntity(FEntityManager* EntityManager, IEntitySystemScheduler* InScheduler, TaskConstructionArgs&&... InArgs) const
	{
		FTaskParams FinalParams = this->CommonParams.TaskParams;
		FinalParams.bSerialTasks = false;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		FinalParams.DebugName = GetGeneratedTypeName<TaskImpl>();
#endif

		return ScheduleImpl<TaskImpl>(
			EntityManager,
			InScheduler,
			FinalParams,
			TaskFunctionPtr(TInPlaceType<PreLockedAllocationItemFunctionPtr>(), TScheduledEntityTask<TaskImpl, T...>::ScheduledRun_PerEntity),
			Forward<TaskConstructionArgs>(InArgs)...);
	}

	template<typename TaskImpl, typename... TaskConstructionArgs>
	FTaskID Fork_PerAllocation(FEntityManager* EntityManager, IEntitySystemScheduler* InScheduler, TaskConstructionArgs&&... InArgs) const
	{
		FTaskParams FinalParams = this->CommonParams.TaskParams;
		FinalParams.bSerialTasks = false;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		FinalParams.DebugName = GetGeneratedTypeName<TaskImpl>();
#endif

		return ScheduleImpl<TaskImpl>(
			EntityManager,
			InScheduler,
			FinalParams,
			TaskFunctionPtr(TInPlaceType<PreLockedAllocationItemFunctionPtr>(), TScheduledEntityTask<TaskImpl, T...>::ScheduledRun_PerAllocation),
			Forward<TaskConstructionArgs>(InArgs)...);
	}

	template<typename TaskImpl, typename... TaskConstructionArgs>
	FTaskID Schedule_PerEntity(FEntityManager* EntityManager, IEntitySystemScheduler* InScheduler, TaskConstructionArgs&&... InArgs) const
	{
		FTaskParams FinalParams = this->CommonParams.TaskParams;
		FinalParams.bSerialTasks = true;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		FinalParams.DebugName = GetGeneratedTypeName<TaskImpl>();
#endif

		return ScheduleImpl<TaskImpl>(
			EntityManager,
			InScheduler,
			FinalParams,
			TaskFunctionPtr(TInPlaceType<PreLockedAllocationItemFunctionPtr>(), TScheduledEntityTask<TaskImpl, T...>::ScheduledRun_PerEntity),
			Forward<TaskConstructionArgs>(InArgs)...);
	}

	template<typename TaskImpl, typename... TaskConstructionArgs>
	FTaskID Schedule_PerAllocation(FEntityManager* EntityManager, IEntitySystemScheduler* InScheduler, TaskConstructionArgs&&... InArgs) const
	{
		FTaskParams FinalParams = this->CommonParams.TaskParams;
		FinalParams.bSerialTasks = true;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		FinalParams.DebugName = GetGeneratedTypeName<TaskImpl>();
#endif

		return ScheduleImpl<TaskImpl>(
			EntityManager,
			InScheduler,
			FinalParams,
			TaskFunctionPtr(TInPlaceType<PreLockedAllocationItemFunctionPtr>(), TScheduledEntityTask<TaskImpl, T...>::ScheduledRun_PerAllocation),
			Forward<TaskConstructionArgs>(InArgs)...);
	}

	template<typename TaskImpl>
	void RunInline_PerEntity(FEntityManager* EntityManager, TaskImpl& Task) const
	{
		if (this->IsValid())
		{
			TEntityTaskBase<TaskImpl, T...>(EntityManager, *this).Run(Task);
		}
	}

public:

	TEntityTaskComponents()
	{
		static_assert(sizeof...(T) == 0, "Default construction is only supported for TEntityTaskComponents<>");
	}

	template<typename... ConstructionTypes>
	explicit TEntityTaskComponents(const FCommonEntityTaskParams& InCommonParams, ConstructionTypes&&... InTypes)
		: Super(InCommonParams, Forward<ConstructionTypes>(InTypes)...)
	{}

private:

	template<typename TaskImpl, typename... TaskConstructionArgs>
	FTaskID ScheduleImpl(FEntityManager* EntityManager, IEntitySystemScheduler* InScheduler, const FTaskParams& TaskParams, TaskFunctionPtr InFunction, TaskConstructionArgs&&... InArgs) const
	{
		if (!this->IsValid())
		{
			return FTaskID::None();
		}

		if (!this->HasAnyWork(EntityManager))
		{
			return FTaskID::None();
		}

		using TaskType = TScheduledEntityTask<TaskImpl, T...>;
		TSharedPtr<TaskType> SharedTask = MakeShared<TaskType>(*this, Forward<TaskConstructionArgs>(InArgs)...);

		FComponentMask ReadDependencies, WriteDependencies;
		this->PopulateReadWriteDependencies(ReadDependencies, WriteDependencies);

		FEntityComponentFilter Filter;
		this->PopulateFilter(&Filter);

		auto PrelockComponentData = [this](FEntityAllocationIteratorItem Allocation, TArray<FPreLockedDataPtr>& OutComponentHeaders)
		{
			this->PreLockComponentHeaders(Allocation, OutComponentHeaders);
		};
		FTaskID TaskID = InScheduler->CreateForkedAllocationTask(TaskParams, SharedTask, InFunction, PrelockComponentData, Filter, ReadDependencies, WriteDependencies);

		if (!TaskID && TaskParams.bForcePrePostTask)
		{
			TaskID = InScheduler->AddNullTask();
		}

		if (TaskID)
		{
			FTaskID PreTask  = SchedulePreTask(InScheduler, TaskParams, SharedTask, (TaskImpl*)0);
			InScheduler->AddChildFront(TaskID, PreTask);

			FTaskID PostTask = SchedulePostTask(InScheduler, TaskParams, SharedTask, (TaskImpl*)0);
			InScheduler->AddChildBack(TaskID, PostTask);
		}

		return TaskID;
	}
};


template<int... Indices, typename... T>
struct TEntityTaskComponentsImpl<TIntegerSequence<int, Indices...>, T...>
{
	/**
	 * Read the entity ID along with this task. Passed to the task as an FMovieSceneEntityID
	 */
	TEntityTaskComponents< T..., FEntityIDAccess > ReadEntityIDs() const
	{
		return TEntityTaskComponents< T..., FEntityIDAccess >( CommonParams, Accessors.template Get<Indices>()..., FEntityIDAccess{} );
	}

	/**
	 * Read the value of a component. Passed to the task as either a const T& or T depending on the type specified in the task function.
	 * @note Supplying an invalid ComponentType will be handled gracefully, but will result in no task being dispatched.
	 *
	 * @param ComponentType   A valid component type to read.
	 */
	template<typename U>
	TEntityTaskComponents<T..., TReadAccess<U>> Read(TComponentTypeID<U> ComponentType) const
	{
		return TEntityTaskComponents<T..., TReadAccess<U> >(CommonParams, Accessors.template Get<Indices>()..., TReadAccess<U>(ComponentType));
	}

	/**
	 * Read the value of only one of the specified components. Only entities with exactly one of these components will be read. Per-entity iteration not supported with this accessor.
	 * @note Supplying an invalid ComponentType will be handled gracefully, but will result in no task being dispatched.
	 *
	 * @param ComponentTypes  The component types to visit
	 */
	template<typename... U>
	TEntityTaskComponents<T..., TReadOneOfAccessor<U...>> ReadOneOf(TComponentTypeID<U>... ComponentTypes) const
	{
		return TEntityTaskComponents<T..., TReadOneOfAccessor<U...> >( CommonParams, Accessors.template Get<Indices>()..., TReadOneOfAccessor<U...>(ComponentTypes...) );
	}

	/**
	 * Read the value of one or more of the specified components. Entities with at least one of these components will be read. Per-entity iteration not supported with this accessor.
	 *
	 * @param ComponentTypes  The component types to visit
	 */
	template<typename... U>
	TEntityTaskComponents<T..., TReadOneOrMoreOfAccessor<U...>> ReadOneOrMoreOf(TComponentTypeID<U>... ComponentTypes) const
	{
		return TEntityTaskComponents<T..., TReadOneOrMoreOfAccessor<U...> >(CommonParams, Accessors.template Get<Indices>()..., TReadOneOrMoreOfAccessor<U...>(ComponentTypes...));
	}

	/**
	 * Read all of the specified components and pass them through to the task as individual parameters
	 *
	 * @param ComponentTypes  The component types to visit
	 */
	template<typename... U>
	TEntityTaskComponents<T..., TReadAccess<U>...> ReadAllOf(TComponentTypeID<U>... ComponentTypes) const
	{
		return TEntityTaskComponents<T..., TReadAccess<U>... >( CommonParams, Accessors.template Get<Indices>()..., TReadAccess<U>(ComponentTypes)... );
	}

	/**
	 * Read all of the specified components and pass them through to the task as individual parameters
	 *
	 * @param ComponentTypes  The component types to visit
	 */
	template<typename... U>
	TEntityTaskComponents<T..., TReadAccess<U>...> ReadAllOf(const TComponentTypeIDs<U...>& TypeIDs) const
	{
		return ReadAllOf(TypeIDs, TMakeIntegerSequence<int, sizeof...(U)>());
	}
	template<typename... U, int... ExtraIndices>
	TEntityTaskComponents<T..., TReadAccess<U>...> ReadAllOf(const TComponentTypeIDs<U...>& TypeIDs, TIntegerSequence<int, ExtraIndices...>&& In) const
	{
		return TEntityTaskComponents<T..., TReadAccess<U>... >( CommonParams, Accessors.template Get<Indices>()..., TReadAccess<U>(TypeIDs.GetType(ExtraIndices).template ReinterpretCast<U>())... );
	}

	/**
	 * Read any of the specified components and pass them through to the task as individual optional parameters
	 *
	 * @param ComponentTypes  The component types to visit
	 */
	template<typename... U>
	TEntityTaskComponents<T..., TOptionalReadAccess<U>...> ReadAnyOf(TComponentTypeID<U>... ComponentTypes) const
	{
		return TEntityTaskComponents<T..., TOptionalReadAccess<U>... >( CommonParams, Accessors.template Get<Indices>()..., TOptionalReadAccess<U>(ComponentTypes)... );
	}

	/**
	 * Read the type-erased value of a component. Passed to the task as a const void*
	 * @note Supplying an invalid ComponentType will be handled gracefully, but will result in no task being dispatched.
	 *
	 * @param ComponentType   A valid component type to read.
	 */
	TEntityTaskComponents<T..., FErasedReadAccess> ReadErased(FComponentTypeID ComponentType) const
	{
		return TEntityTaskComponents<T..., FErasedReadAccess >(CommonParams, Accessors.template Get<Indices>()..., FErasedReadAccess(ComponentType));
	}

	/**
	 * Optionally read the type-erased value of a component. Passed to the task as a const void*
	 * @note Supplying an invalid ComponentType will be handled gracefully, but will result in no task being dispatched.
	 *
	 * @param ComponentType   A valid component type to read.
	 */
	TEntityTaskComponents<T..., FErasedOptionalReadAccess> ReadErasedOptional(FComponentTypeID ComponentType, FComponentTypeIDFilter InConditionType = FComponentTypeIDFilter()) const
	{
		return TEntityTaskComponents<T..., FErasedOptionalReadAccess >(CommonParams, Accessors.template Get<Indices>()..., FErasedOptionalReadAccess(ComponentType, InConditionType));
	}

	/**
	 * Optionally read the value of a component. ComponentType may be invalid, and the component may or may not exist for some/all of the entities in the resulting task
	 * @note Always passed to the task as a const T* pointer which must be checked for null
	 */
	template<typename U>
	TEntityTaskComponents<T..., TOptionalReadAccess<U>> ReadOptional(TComponentTypeID<U> ComponentType, FComponentTypeIDFilter InConditionType = FComponentTypeIDFilter()) const
	{
		return TEntityTaskComponents<T..., TOptionalReadAccess<U> >(CommonParams, Accessors.template Get<Indices>()..., TOptionalReadAccess<U>(ComponentType, InConditionType));
	}

	/**
	 * Write the value of a component in a thread safe manner. Passed to the task as a T& so the value can be modified or overwritten.
	 * @note Supplying an invalid ComponentType will be handled gracefully, but will result in no task being dispatched.
	 *
	 * @param ComponentType   A valid component type to read.
	 */
	template<typename U>
	TEntityTaskComponents<T..., TWriteAccess<U>> Write(TComponentTypeID<U> ComponentType) const
	{
		return TEntityTaskComponents<T..., TWriteAccess<U> >(CommonParams, Accessors.template Get<Indices>()..., TWriteAccess<U>(ComponentType));
	}

	/**
	 * Write all of the specified components and pass them through to the task as individual parameters
	 *
	 * @param ComponentTypes  The component types to visit
	 */
	template<typename... U>
	TEntityTaskComponents<T..., TWriteAccess<U>...> WriteAllOf(TComponentTypeID<U>... ComponentTypes) const
	{
		return TEntityTaskComponents<T..., TWriteAccess<U>... >( CommonParams, Accessors.template Get<Indices>()..., TWriteAccess<U>(ComponentTypes)... );
	}

	/**
	 * Write the type-erased value of a component. Passed to the task as a void*
	 * @note Supplying an invalid ComponentType will be handled gracefully, but will result in no task being dispatched.
	 *
	 * @param ComponentType   A valid component type to read.
	 */
	TEntityTaskComponents<T..., FErasedWriteAccess> WriteErased(FComponentTypeID ComponentType) const
	{
		return TEntityTaskComponents<T..., FErasedWriteAccess >(CommonParams, Accessors.template Get<Indices>()..., FErasedWriteAccess(ComponentType));
	}


	/**
	 * Optionally write the value of a component in a thread safe manner if it exists. Passed to the task as a T* which must be checked for nullptr.
	 * @note Always passed to the task as a T* pointer which must be checked for null
	 */
	template<typename U>
	TEntityTaskComponents<T..., TOptionalWriteAccess<U>> WriteOptional(TComponentTypeID<U> ComponentType, FComponentTypeIDFilter InConditionType = FComponentTypeIDFilter()) const
	{
		return TEntityTaskComponents<T..., TOptionalWriteAccess<U> >(CommonParams, Accessors.template Get<Indices>()..., TOptionalWriteAccess<U>(ComponentType, InConditionType));
	}

	TEntityTaskComponents<T..., FFilterMatchPassthrough> PassthroughFilter(const FEntityComponentFilter& InFilter) const
	{
		return TEntityTaskComponents<T..., FFilterMatchPassthrough >(CommonParams, Accessors.template Get<Indices>()..., FFilterMatchPassthrough{ InFilter });
	}

	bool HasBeenWrittenToSince(uint32 InSystemVersion)
	{
		bool bAnyWrittenTo = true;

		int Temp[] = { ( bAnyWrittenTo |= HasBeenWrittenToSince(&Accessors.template Get<Indices>(), InSystemVersion), 0)... };
		(void)Temp;

		return bAnyWrittenTo;
	}

	/**
	 * Check whether this task data is well-formed in the sense that it can perform meaningful work.
	 */
	bool IsValid() const
	{
		bool bAllValid = true;

		int Temp[] = { ( bAllValid &= IsAccessorValid(&Accessors.template Get<Indices>()), 0)..., 0 };
		(void)Temp;

		return bAllValid;
	}

	/**
	 * Check whether all required accessors correspond to component types that are present in the given entity manager.
	 */
	bool HasAnyWork(const FEntityManager* EntityManager) const
	{
		bool bAllHaveWork = true;

		int Temp[] = { ( bAllHaveWork &= HasAccessorWork(EntityManager, &Accessors.template Get<Indices>()), 0)..., 0 };
		(void)Temp;

		return bAllHaveWork;
	}

	/** Utility function called when the task is dispatched to populate the filter based on our component typs */
	void PopulateFilter(FEntityComponentFilter* OutFilter) const
	{
		int Temp[] = { (AddAccessorToFilter(&Accessors.template Get<Indices>(), OutFilter), 0)..., 0 };
		(void)Temp;
	}

	/** Utility function called when the task is dispatched to populate the filter based on our component typs */
	void PopulatePrerequisites(const FSystemTaskPrerequisites& InPrerequisites, FGraphEventArray* OutGatheredPrereqs) const
	{
		// Gather any root tasks
		InPrerequisites.FilterByComponent(*OutGatheredPrereqs, FComponentTypeID::Invalid());

		int Temp[] = { (UE::MovieScene::PopulatePrerequisites(&Accessors.template Get<Indices>(), InPrerequisites, OutGatheredPrereqs), 0)..., 0 };
		(void)Temp;
	}

	/** Utility function called when the task is dispatched to populate the filter based on our component typs */
	void PopulateSubsequents(const FGraphEventRef& InEvent, FSystemSubsequentTasks& OutSubsequents) const
	{
		OutSubsequents.AddRootTask(InEvent);

		int Temp[] = { (UE::MovieScene::PopulateSubsequents(&Accessors.template Get<Indices>(), InEvent, OutSubsequents), 0)..., 0 };
		(void)Temp;
	}

	void PreLockComponentHeaders(FEntityAllocationIteratorItem Item, TArray<FPreLockedDataPtr>& OutComponentHeaders) const
	{
		constexpr TPrelockedDataOffsets<T...> PrelockedDataOffsets;
		int32 StartIndex = OutComponentHeaders.Num();
		OutComponentHeaders.AddDefaulted(PrelockedDataOffsets.Num);
		( Accessors.template Get<Indices>().PreLockComponentData(Item, &OutComponentHeaders[StartIndex + PrelockedDataOffsets.StartOffset[Indices]]), ... );
	}

	void PopulateReadWriteDependencies(FComponentMask& OutReadDependencies, FComponentMask& OutWriteDependencies) const
	{
		int Temp[] = { (UE::MovieScene::PopulateReadWriteDependencies(&Accessors.template Get<Indices>(), OutReadDependencies, OutWriteDependencies), 0)..., 0 };
		(void)Temp;
	}

	/**
	 * Perform a thread-safe iteration of all matching allocations within the specified entity manager using this task, inline on the current thread
	 *
	 * @param EntityManager  The entity manager to iterate allocations for. All component type IDs in this class must relate to this entity manager
	 * @param InCallback     A callable type that matches the signature defined by ForEachAllocation ie void(FEntityAllocation*, const TFilteredEntityTask<T...>&)
	 */
	template<typename Callback>
	void Iterate_PerAllocation(FEntityManager* EntityManager, Callback&& InCallback) const
	{
		FEntityComponentFilter Filter;
		PopulateFilter(&Filter);
		Iterate_PerAllocationImpl(EntityManager, Filter, InCallback);
	}

	/**
	 * Perform a thread-safe iteration of all matching entities specified entity manager using this task, inline on the current thread
	 *
	 * @param EntityManager  The entity manager to iterate allocations for. All component type IDs in this class must relate to this entity manager
	 * @param InCallback     A callable type that matches the signature defined by ForEachEntity ie void(typename T::AccessType...)
	 */
	template<typename Callback>
	void Iterate_PerEntity(FEntityManager* EntityManager, Callback&& InCallback) const
	{
		FEntityComponentFilter Filter;
		PopulateFilter(&Filter);
		Iterate_PerEntityImpl(EntityManager, Filter, InCallback);
	}

	/**
	 * Implementation function for Iterate_PerEntity
	 *
	 * @param EntityManager  The entity manager to iterate allocations for. All component type IDs in this class must relate to this entity manager
	 * @param Filter         Filter that at least must match the types specified by this task
	 * @param InCallback     A callable type that matches the signature defined by ForEachEntity ie void(typename T::AccessType...)
	 */
	template<typename Callback>
	void Iterate_PerEntityImpl(FEntityManager* EntityManager, const FEntityComponentFilter& Filter, Callback&& InCallback) const
	{
		using TupleType = TTuple< decltype(Accessors.template Get<Indices>().LockComponentData(DeclVal<FEntityAllocationIteratorItem>(), DeclVal<FEntityAllocationWriteContext>()))... >;

		if (IsValid())
		{
			FEntityAllocationWriteContext WriteContext(*EntityManager);

			EComponentHeaderLockMode LockMode = EntityManager->GetThreadingModel() == EEntityThreadingModel::NoThreading
				? EComponentHeaderLockMode::LockFree
				: EComponentHeaderLockMode::Mutex;

			for (FEntityAllocationIteratorItem Item : EntityManager->Iterate(&Filter))
			{
				FEntityIterationResult Result;

				FEntityAllocation* Allocation = Item.GetAllocation();
				FEntityAllocationMutexGuard LockGuard(Allocation, LockMode);

				// Lock the components we want to access
				TupleType ComponentData( Accessors.template Get<Indices>().LockComponentData(Item, WriteContext)... );

				const int32 Num = Allocation->Num();
				for (int32 ComponentOffset = 0; ComponentOffset < Num && Result.Value; ++ComponentOffset )
				{
					Result = (InCallback( GetComponentAtIndex(&ComponentData.template Get<Indices>(), ComponentOffset)... ), Result);
				}
			}
		}
	}

	/**
	 * Implementation function for Iterate_PerAllocation
	 *
	 * @param EntityManager  The entity manager to iterate allocations for. All component type IDs in this class must relate to this entity manager
	 * @param Filter         Filter that at least must match the types specified by this task
	 * @param InCallback     A callable type that matches the signature defined by ForEachAllocation ie void(FEntityAllocation*, const TFilteredEntityTask<T...>&)
	 */
	template<typename Callback>
	void Iterate_PerAllocationImpl(FEntityManager* EntityManager, const FEntityComponentFilter& Filter, Callback&& InCallback) const
	{
		if (IsValid())
		{
			FEntityAllocationWriteContext WriteContext(*EntityManager);

			EComponentHeaderLockMode LockMode = EntityManager->GetThreadingModel() == EEntityThreadingModel::NoThreading
				? EComponentHeaderLockMode::LockFree
				: EComponentHeaderLockMode::Mutex;

			for (FEntityAllocationIteratorItem Item : EntityManager->Iterate(&Filter))
			{
				FEntityAllocationMutexGuard LockGuard(Item.GetAllocation(), LockMode);

				// Lock on the components we want to access
				auto ComponentData = MakeTuple( Accessors.template Get<Indices>().LockComponentData(Item, WriteContext)... );

				FEntityIterationResult Result = (InCallback(Item, ComponentData.template Get<Indices>()...), FEntityIterationResult{});

				if (!Result)
				{
					break;
				}
			}
		}
	}

public:

	/**
	 * Get the accessor for a specific index within this task
	 */
	template<int Index>
	inline auto GetAccessor() const
	{
		static_assert(Index < sizeof...(T), "Invalid component index specified");
		return Accessors.template Get<Index>();
	}

	FString ToString(FEntityManager* EntityManager) const
	{
#if UE_MOVIESCENE_ENTITY_DEBUG
		FString Result;

		int Unused[] = { (AccessorToString(&Accessors.template Get<Indices>(), EntityManager, Result), 0)... };
		(void)Unused;

		return Result;
#else
		return TEXT("<debug info compiled out> - enable UE_MOVIESCENE_ENTITY_DEBUG");
#endif
	}

protected:

	TEntityTaskComponentsImpl()
	{}

	template<typename... ConstructionTypes>
	explicit TEntityTaskComponentsImpl(const FCommonEntityTaskParams& InCommonParams, ConstructionTypes&&... InTypes)
		: Accessors{ Forward<ConstructionTypes>(InTypes)... }
		, CommonParams(InCommonParams)
	{}

protected:

	template<typename...> friend struct TEntityTaskComponentsImpl;

	TTuple<T...> Accessors;

public:

	FCommonEntityTaskParams CommonParams;
};

struct FNoopTask
{
	template<typename... T>
	inline static void ForEachAllocation(T&&...)
	{}

	template<typename... T>
	inline static void ForEachEntity(T&&...)
	{}
};


/**
 * Main entry point utility for create tasks that run over component data
 */
struct FEntityTaskBuilder : TEntityTaskComponents<>
{
	FEntityTaskBuilder() : TEntityTaskComponents<>() {}
};


template<typename... T>
struct TFilteredEntityTask
{
	TFilteredEntityTask(const TEntityTaskComponents<T...>& InComponents)
		: Components(InComponents)
	{
		Components.PopulateFilter(&Filter);
	}
	TFilteredEntityTask(const TEntityTaskComponents<T...>& InComponents, const FEntityComponentFilter& InFilter)
		: Components(InComponents)
		, Filter(InFilter)
	{
		Components.PopulateFilter(&Filter);
	}


	/**
	 * Constrain this task to only run for entities that have all the specified components or tags
	 */
	TFilteredEntityTask< T... >& FilterAll(const FComponentMask& InComponentMask)
	{
		Filter.All(InComponentMask);
		return *this;
	}

	/**
	 * Constrain this task to only run for entities that have all the specified components or tags
	 */
	TFilteredEntityTask< T... >& FilterAll(std::initializer_list<FComponentTypeID> InComponentTypes)
	{
		Filter.All(InComponentTypes);
		return *this;
	}

	/**
	 * Constrain this task to only run for entities that have none the specified components or tags
	 */
	TFilteredEntityTask< T... >& FilterNone(const FComponentMask& InComponentMask)
	{
		Filter.None(InComponentMask);
		return *this;
	}

	/**
	 * Constrain this task to only run for entities that have none the specified components or tags
	 */
	TFilteredEntityTask< T... >& FilterNone(std::initializer_list<FComponentTypeID> InComponentTypes)
	{
		Filter.None(InComponentTypes);
		return *this;
	}

	/**
	 * Constrain this task to only run for entities that have at least one of the specified components or tags
	 */
	TFilteredEntityTask< T... >& FilterAny(const FComponentMask& InComponentMask)
	{
		Filter.Any(InComponentMask);
		return *this;
	}

	/**
	 * Constrain this task to only run for entities that have at least one of the specified components or tags
	 */
	TFilteredEntityTask< T... >& FilterAny(std::initializer_list<FComponentTypeID> InComponentTypes)
	{
		Filter.Any(InComponentTypes);
		return *this;
	}

	/**
	 * Constrain this task to only run for entities that do not have the specific combination of components or tags
	 */
	TFilteredEntityTask< T... >& FilterOut(const FComponentMask& InComponentMask)
	{
		Filter.Deny(InComponentMask);
		return *this;
	}

	/**
	 * Constrain this task to only run for entities that do not have the specific combination of components or tags
	 */
	TFilteredEntityTask< T... >& FilterOut(std::initializer_list<FComponentTypeID> InComponentTypes)
	{
		Filter.Deny(InComponentTypes);
		return *this;
	}

	/**
	 * Combine this task's filter with the specified filter
	 */
	TFilteredEntityTask< T... >& CombineFilter(const FEntityComponentFilter& InFilter)
	{
		Filter.Combine(InFilter);
		return *this;
	}

	/**
	 * Assign a desired thread for this task to run on
	 */
	TFilteredEntityTask< T... >& SetDesiredThread(ENamedThreads::Type InDesiredThread)
	{
		Components.CommonParams.DesiredThread = InDesiredThread;
		Components.CommonParams.TaskParams.bForceGameThread = (InDesiredThread == ENamedThreads::GameThread || InDesiredThread == ENamedThreads::GameThread_Local);
		return *this;
	}

	/**
	 * Assign a stat ID for this task
	 */
	TFilteredEntityTask< T... >& SetStat(TStatId InStatId)
	{
		Components.CommonParams.TaskParams.StatId = InStatId;
		return *this;
	}

	/**
	 * Assign the scheduled task parameters for this task
	 */
	TFilteredEntityTask< T... >& SetParams(const FTaskParams& InOtherParams)
	{
		Components.CommonParams.TaskParams = InOtherParams;
		return *this;
	}

	/**
	 * Get the desired thread for this task to run on
	 */
	ENamedThreads::Type GetDesiredThread() const
	{
		return Components.CommonParams.DesiredThread;
	}

	/**
	 * Return this task's stat id
	 */
	TStatId GetStatId() const
	{
		return Components.CommonParams.TaskParams.StatId;
	}

	/**
	 * Check whether we should break the debugger when this task is run
	 */
	bool ShouldBreakOnRun() const
	{
		return Components.CommonParams.bBreakOnRun;
	}

	/**
	 * Access the pre-populated filter that should be used for iterating relevant entities for this task
	 */
	const FEntityComponentFilter& GetFilter() const
	{
		return Filter;
	}

	/**
	 * Access the underlying component access definitions
	 */
	const TEntityTaskComponents<T...>& GetComponents() const
	{
		return Components;
	}

	TFilteredEntityTask<T...>& AddDynamicReadDependency(std::initializer_list<FComponentTypeID> InReadDependencies)
	{
		DynamicReadMask.SetAll(InReadDependencies);
		return *this;
	}
	template<typename ComponentType>
	TFilteredEntityTask<T...>& AddDynamicReadDependency(TArrayView<ComponentType> InReadDependencies)
	{
		for (FComponentTypeID Component : InReadDependencies)
		{
			DynamicReadMask.Set(Component);
		}
		return *this;
	}
	TFilteredEntityTask<T...>& AddDynamicReadDependency(const FComponentMask& InDynamicReadDependency)
	{
		DynamicReadMask.CombineWithBitwiseOR(InDynamicReadDependency, EBitwiseOperatorFlags::MaxSize);
		return *this;
	}

	TFilteredEntityTask<T...>& AddDynamicWriteDependency(std::initializer_list<FComponentTypeID> InWriteDependencies)
	{
		DynamicWriteMask.SetAll(InWriteDependencies);
		return *this;
	}
	template<typename ComponentType>
	TFilteredEntityTask<T...>& AddDynamicWriteDependency(TArrayView<ComponentType> InWriteDependencies)
	{
		for (FComponentTypeID Component : InWriteDependencies)
		{
			DynamicWriteMask.Set(Component);
		}
		return *this;
	}
	TFilteredEntityTask<T...>& AddDynamicWriteDependency(const FComponentMask& InDynamicWriteDependency)
	{
		DynamicWriteMask.CombineWithBitwiseOR(InDynamicWriteDependency, EBitwiseOperatorFlags::MaxSize);
		return *this;
	}

	/**
	 * Dispatch a task for every entity that matches the filters and component types. Must be explicitly instantiated with the task type to dispatch. Construction arguments are deduced.
	 * Tasks must implement a ForEachEntity function that matches this task's component accessor types.
	 * 
	 * For example:
	 * struct FForEachEntity
	 * {
	 *     void ForEachEntity(FMovieSceneEntityID InEntityID, const FMovieSceneFloatChannel& Channel);
	 * };
	 *
	 * TComponentTypeID<FMovieSceneFloatChannel> FloatChannelComponent = ...;
	 *
	 * FGraphEventRef Task = FEntityTaskBuilder()
	 * .ReadEntityIDs()
	 * .Read(FloatChannelComponent)
	 * .SetStat(GET_STATID(MyStatName))
	 * .SetDesiredThread(ENamedThreads::AnyThread)
	 * .Dispatch_PerEntity<FForEachEntity>(EntityManager, Prerequisites);
	 *
	 * @param EntityManager    The entity manager to run the task on. All component types *must* relate to this entity manager.
	 * @param Prerequisites    Prerequisite tasks that must run before this one, or nullptr if there are no prerequisites
	 * @param Subsequents      (Optional) Subsequent task tracking that this task should be added to for each writable component type
	 * @param InArgs           Optional arguments that are forwarded to the constructor of TaskImpl
	 * @return A pointer to the graph event for the task, or nullptr if this task is not valid (ie contains invalid component types that would be necessary for the task to run), or threading is disabled
	 */
	template<typename TaskImpl, typename... TaskConstructionArgs>
	FGraphEventRef Dispatch_PerAllocation(FEntityManager* EntityManager, const FSystemTaskPrerequisites& Prerequisites, FSystemSubsequentTasks* Subsequents, TaskConstructionArgs&&... InArgs) const
	{
		checkfSlow(IsInGameThread(), TEXT("Tasks can only be dispatched from the game thread."));

		if (!Components.IsValid())
		{
			return nullptr;
		}

		// See comment in TEntityTaskComponents' Dispatch_PerAllocation()
		if (!Components.HasAnyWork(EntityManager))
		{
			return nullptr;
		}

		// If this ensure triggers, we are not in the evaluation phase - the callee should be using RunInline_ or Iterate_ variants
		const bool bRunInline = !ensure(EntityManager->IsLockedDown()) || EntityManager->GetThreadingModel() == EEntityThreadingModel::NoThreading;
		if (bRunInline)
		{
			TaskImpl Task{ Forward<TaskConstructionArgs>(InArgs)... };
			TEntityAllocationTaskBase<TaskImpl, T...>(EntityManager, *this).Run(Task);
			return nullptr;
		}
		else
		{
			FGraphEventArray GatheredPrereqs;
			Components.PopulatePrerequisites(Prerequisites, &GatheredPrereqs);

			ENamedThreads::Type ThisThread = EntityManager->GetDispatchThread();
			checkSlow(ThisThread != ENamedThreads::AnyThread);

			FGraphEventRef NewTask = TGraphTask< TEntityAllocationTask<TaskImpl, T...> >::CreateTask(GatheredPrereqs.Num() != 0 ? &GatheredPrereqs : nullptr, ThisThread)
				.ConstructAndDispatchWhenReady( EntityManager, *this, Forward<TaskConstructionArgs>(InArgs)... );

			if (Subsequents)
			{
				Components.PopulateSubsequents(NewTask, *Subsequents);
			}

			return NewTask;
		}
	}

	template<typename TaskImpl>
	void RunInline_PerAllocation(FEntityManager* EntityManager, TaskImpl& Task) const
	{
		if (Components.IsValid())
		{
			TEntityAllocationTaskBase<TaskImpl, T...>(EntityManager, *this).Run(Task);
		}
	}

	/**
	 * Dispatch a task for every entity that matches the filters and component types. Must be explicitly instantiated with the task type to dispatch. Construction arguments are deduced.
	 * Tasks must implement a ForEachEntity function that matches this task's component accessor types.
	 * 
	 * For example:
	 * struct FForEachEntity
	 * {
	 *     void ForEachEntity(FMovieSceneEntityID InEntityID, const FMovieSceneFloatChannel& Channel);
	 * };
	 *
	 * TComponentTypeID<FMovieSceneFloatChannel> FloatChannelComponent = ...;
	 *
	 * FGraphEventRef Task = FEntityTaskBuilder()
	 * .ReadEntityIDs()
	 * .Read(FloatChannelComponent)
	 * .SetStat(GET_STATID(MyStatName))
	 * .SetDesiredThread(ENamedThreads::AnyThread)
	 * .Dispatch_PerEntity<FForEachEntity>(EntityManager, Prerequisites);
	 *
	 * @param EntityManager    The entity manager to run the task on. All component types *must* relate to this entity manager.
	 * @param Prerequisites    Prerequisite tasks that must run before this one, or nullptr if there are no prerequisites
	 * @param Subsequents      (Optional) Subsequent task tracking that this task should be added to for each writable component type
	 * @param InArgs           Optional arguments that are forwarded to the constructor of TaskImpl
	 * @return A pointer to the graph event for the task, or nullptr if this task is not valid (ie contains invalid component types that would be necessary for the task to run), or threading is disabled
	 */
	template<typename TaskImpl, typename... TaskConstructionArgs>
	FGraphEventRef Dispatch_PerEntity(FEntityManager* EntityManager, const FSystemTaskPrerequisites& Prerequisites, FSystemSubsequentTasks* Subsequents, TaskConstructionArgs&&... InArgs) const
	{
		checkfSlow(IsInGameThread(), TEXT("Tasks can only be dispatched from the game thread."));

		if (!Components.IsValid())
		{
			return nullptr;
		}

		// See comment in TEntityTaskComponents' Dispatch_PerAllocation()
		if (!Components.HasAnyWork(EntityManager))
		{
			return nullptr;
		}

		// If this ensure triggers, we are not in the evaluation phase - the callee should be using RunInline_ or Iterate_ variants
		const bool bRunInline = !ensure(EntityManager->IsLockedDown()) || EntityManager->GetThreadingModel() == EEntityThreadingModel::NoThreading;
		if (bRunInline)
		{
			TaskImpl Task{ Forward<TaskConstructionArgs>(InArgs)... };
			TEntityTaskBase<TaskImpl, T...>(EntityManager, *this).Run(Task);
			return nullptr;
		}
		else
		{
			FGraphEventArray GatheredPrereqs;
			Components.PopulatePrerequisites(Prerequisites, &GatheredPrereqs);

			ENamedThreads::Type ThisThread = EntityManager->GetDispatchThread();
			checkSlow(ThisThread != ENamedThreads::AnyThread);

			FGraphEventRef NewTask = TGraphTask< TEntityTask<TaskImpl, T...> >::CreateTask(GatheredPrereqs.Num() != 0 ? &GatheredPrereqs : nullptr, ThisThread)
				.ConstructAndDispatchWhenReady( EntityManager, *this, Forward<TaskConstructionArgs>(InArgs)... );

			if (Subsequents)
			{
				Components.PopulateSubsequents(NewTask, *Subsequents);
			}

			return NewTask;
		}
	}

	template<typename TaskImpl, typename... TaskConstructionArgs>
	FTaskID Fork_PerEntity(FEntityManager* EntityManager, IEntitySystemScheduler* InScheduler, TaskConstructionArgs&&... InArgs) const
	{
		FTaskParams FinalParams = this->Components.CommonParams.TaskParams;
		FinalParams.bSerialTasks = false;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		FinalParams.DebugName = GetGeneratedTypeName<TaskImpl>();
#endif

		return ScheduleImpl<TaskImpl>(
			EntityManager,
			InScheduler,
			FinalParams,
			TaskFunctionPtr(TInPlaceType<PreLockedAllocationItemFunctionPtr>(), TScheduledEntityTask<TaskImpl, T...>::ScheduledRun_PerEntity),
			Forward<TaskConstructionArgs>(InArgs)...);
	}

	template<typename TaskImpl, typename... TaskConstructionArgs>
	FTaskID Fork_PerAllocation(FEntityManager* EntityManager, IEntitySystemScheduler* InScheduler, TaskConstructionArgs&&... InArgs) const
	{
		FTaskParams FinalParams = this->Components.CommonParams.TaskParams;
		FinalParams.bSerialTasks = false;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		FinalParams.DebugName = GetGeneratedTypeName<TaskImpl>();
#endif

		return ScheduleImpl<TaskImpl>(
			EntityManager,
			InScheduler,
			FinalParams,
			TaskFunctionPtr(TInPlaceType<PreLockedAllocationItemFunctionPtr>(), TScheduledEntityTask<TaskImpl, T...>::ScheduledRun_PerAllocation),
			Forward<TaskConstructionArgs>(InArgs)...);
	}

	template<typename TaskImpl, typename... TaskConstructionArgs>
	FTaskID Schedule_PerEntity(FEntityManager* EntityManager, IEntitySystemScheduler* InScheduler, TaskConstructionArgs&&... InArgs) const
	{
		FTaskParams FinalParams = this->Components.CommonParams.TaskParams;
		FinalParams.bSerialTasks = true;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		FinalParams.DebugName = GetGeneratedTypeName<TaskImpl>();
#endif

		return ScheduleImpl<TaskImpl>(
			EntityManager,
			InScheduler,
			FinalParams,
			TaskFunctionPtr(TInPlaceType<PreLockedAllocationItemFunctionPtr>(), TScheduledEntityTask<TaskImpl, T...>::ScheduledRun_PerEntity),
			Forward<TaskConstructionArgs>(InArgs)...);
	}

	template<typename TaskImpl, typename... TaskConstructionArgs>
	FTaskID Schedule_PerAllocation(FEntityManager* EntityManager, IEntitySystemScheduler* InScheduler, TaskConstructionArgs&&... InArgs) const
	{
		FTaskParams FinalParams = this->Components.CommonParams.TaskParams;
		FinalParams.bSerialTasks = true;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		FinalParams.DebugName = GetGeneratedTypeName<TaskImpl>();
#endif

		return ScheduleImpl<TaskImpl>(
			EntityManager,
			InScheduler,
			FinalParams,
			TaskFunctionPtr(TInPlaceType<PreLockedAllocationItemFunctionPtr>(), TScheduledEntityTask<TaskImpl, T...>::ScheduledRun_PerAllocation),
			Forward<TaskConstructionArgs>(InArgs)...);
	}

	template<typename TaskImpl>
	void RunInline_PerEntity(FEntityManager* EntityManager, TaskImpl Task) const
	{
		if (Components.IsValid())
		{
			TEntityTaskBase<TaskImpl, T...>(EntityManager, *this).Run(Task);
		}
	}

	/**
	 * Perform a thread-safe iteration of all matching entities specified entity manager using this task, inline on the current thread
	 *
	 * @param EntityManager  The entity manager to iterate allocations for. All component type IDs in this class must relate to this entity manager
	 * @param InCallback     A callable type that matches the signature defined by ForEachEntity ie void(typename T::AccessType...)
	 */
	template<typename Callback>
	void Iterate_PerEntity(FEntityManager* EntityManager, Callback&& InCallback) const
	{
		Components.Iterate_PerEntityImpl(EntityManager, Filter, Forward<Callback>(InCallback));
	}

	/**
	 * Perform a thread-safe iteration of all matching allocations within the specified entity manager using this task, inline on the current thread
	 *
	 * @param EntityManager  The entity manager to iterate allocations for. All component type IDs in this class must relate to this entity manager
	 * @param InCallback     A callable type that matches the signature defined by ForEachAllocation ie void(FEntityAllocation*, const TFilteredEntityTask<T...>&)
	 */
	template<typename Callback>
	void Iterate_PerAllocation(FEntityManager* EntityManager, Callback&& InCallback) const
	{
		Components.Iterate_PerAllocationImpl(EntityManager, Filter, Forward<Callback>(InCallback));
	}

private:

	template<typename TaskImpl, typename... TaskConstructionArgs>
	FTaskID ScheduleImpl(FEntityManager* EntityManager, IEntitySystemScheduler* InScheduler, const FTaskParams& TaskParams, TaskFunctionPtr InFunction, TaskConstructionArgs&&... InArgs) const
	{
		if (!Components.IsValid())
		{
			return FTaskID::None();
		}

		if (!Components.HasAnyWork(EntityManager))
		{
			return FTaskID::None();
		}

		using TaskType = TScheduledEntityTask<TaskImpl, T...>;
		TSharedPtr<TaskType> SharedTask = MakeShared<TaskType>(Components, Forward<TaskConstructionArgs>(InArgs)...);

		FComponentMask ReadDependencies = DynamicReadMask, WriteDependencies = DynamicWriteMask;
		Components.PopulateReadWriteDependencies(ReadDependencies, WriteDependencies);

		auto PrelockComponentData = [this](FEntityAllocationIteratorItem Allocation, TArray<FPreLockedDataPtr>& OutComponentHeaders)
		{
			this->Components.PreLockComponentHeaders(Allocation, OutComponentHeaders);
		};

		FTaskID TaskID = InScheduler->CreateForkedAllocationTask(TaskParams, SharedTask, InFunction, PrelockComponentData, Filter, ReadDependencies, WriteDependencies);

		if (!TaskID && TaskParams.bForcePrePostTask)
		{
			TaskID = InScheduler->AddNullTask();
		}

		if (TaskID)
		{
			FTaskID PreTask  = SchedulePreTask(InScheduler, TaskParams, SharedTask, (TaskImpl*)0);
			InScheduler->AddChildFront(TaskID, PreTask);

			FTaskID PostTask = SchedulePostTask(InScheduler, TaskParams, SharedTask, (TaskImpl*)0);
			InScheduler->AddChildBack(TaskID, PostTask);
		}

		return TaskID;
	}

private:

	TEntityTaskComponents<T...> Components;

	FEntityComponentFilter Filter;
	FComponentMask DynamicReadMask;
	FComponentMask DynamicWriteMask;
};

template<typename TaskImpl, typename... ComponentTypes>
struct TEntityTaskBase
{
	explicit TEntityTaskBase(FEntityManager* InEntityManager, const TEntityTaskComponents<ComponentTypes...>& InComponents)
		: FilteredTask(InComponents)
		, EntityManager(InEntityManager)
		, WriteContext(*InEntityManager)
	{}

	explicit TEntityTaskBase(FEntityManager* InEntityManager, const TFilteredEntityTask<ComponentTypes...>& InFilteredTask)
		: FilteredTask(InFilteredTask)
		, EntityManager(InEntityManager)
		, WriteContext(*InEntityManager)
	{}

	TStatId GetStatId() const
	{
		return FilteredTask.GetStatId();
	}

	ENamedThreads::Type GetDesiredThread() const
	{
		return FilteredTask.GetDesiredThread();
	}

	void Run(TaskImpl& TaskImplInstance)
	{
		PreTask(&TaskImplInstance);

		EComponentHeaderLockMode LockMode = EntityManager->GetThreadingModel() == EEntityThreadingModel::NoThreading
			? EComponentHeaderLockMode::LockFree
			: EComponentHeaderLockMode::Mutex;

		for (FEntityAllocationIteratorItem Item : EntityManager->Iterate(&FilteredTask.GetFilter()))
		{
			FEntityAllocationMutexGuard LockGuard(Item, LockMode);
			Caller::ForEachEntityImpl(TaskImplInstance, Item, WriteContext, FilteredTask.GetComponents());
		}

		PostTask(&TaskImplInstance);
	}

protected:

	static void PreTask(void*, ...){}
	template <typename T> static void PreTask(T* InTask, decltype(&T::PreTask)* = 0)
	{
		InTask->PreTask();
	}

	static void PostTask(void*, ...){}
	template <typename T> static void PostTask(T* InTask, decltype(&T::PostTask)* = 0)
	{
		InTask->PostTask();
	}

	using Caller = TEntityTaskCaller<sizeof...(ComponentTypes), TEntityTaskTraits<TaskImpl>::AutoExpandAccessors >;

	TFilteredEntityTask<ComponentTypes...> FilteredTask;
	FEntityManager* EntityManager;
	FEntityAllocationWriteContext WriteContext;
};

template<typename TaskImpl, typename... ComponentTypes>
struct TEntityTask : TEntityTaskBase<TaskImpl, ComponentTypes...>
{
	template<typename... ArgTypes>
	explicit TEntityTask(FEntityManager* InEntityManager, const TEntityTaskComponents<ComponentTypes...>& InComponents, ArgTypes&&... InArgs)
		: TEntityTaskBase<TaskImpl, ComponentTypes...>(InEntityManager, InComponents)
		, TaskImplInstance{ Forward<ArgTypes>(InArgs)... }
	{}

	template<typename... ArgTypes>
	explicit TEntityTask(FEntityManager* InEntityManager, const TFilteredEntityTask<ComponentTypes...>& InFilteredTask, ArgTypes&&... InArgs)
		: TEntityTaskBase<TaskImpl, ComponentTypes...>(InEntityManager, InFilteredTask)
		, TaskImplInstance{ Forward<ArgTypes>(InArgs)... }
	{}

	static ESubsequentsMode::Type GetSubsequentsMode()
	{
		return ESubsequentsMode::TrackSubsequents;
	}

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& CompletionGraphEvent)
	{
		if (this->FilteredTask.ShouldBreakOnRun())
		{
			UE_DEBUG_BREAK();
		}

		if ((this->FilteredTask.GetDesiredThread() & ENamedThreads::AnyThread) == 0)
		{
			checkf(CurrentThread == this->FilteredTask.GetDesiredThread(), TEXT("MovieScene evaluation task is not being run on its desired thread"));
		}

		this->Run(TaskImplInstance);
	}

private:

	TaskImpl TaskImplInstance;
};

template<typename TaskImpl, typename... ComponentTypes>
struct TScheduledEntityTask : ITaskContext
{
	template<typename... ArgTypes>
	explicit TScheduledEntityTask(const TEntityTaskComponents<ComponentTypes...>& InComponents, ArgTypes&&... InArgs)
		: TaskImplInstance{ Forward<ArgTypes>(InArgs)... }
		, Components(InComponents)
	{}

	void PreTask() const
	{
		// This const_cast is ok because we know nothing else can touch this task in PreTask
		PreTaskImpl(const_cast<TaskImpl*>(&TaskImplInstance));
	}
	void PostTask() const
	{
		// This const_cast is ok because we know nothing else can touch this task in PostTask
		PostTaskImpl(const_cast<TaskImpl*>(&TaskImplInstance));
	}

	AUTORTFM_INFER static void ScheduledRun_PerEntity(FEntityAllocationIteratorItem Item, TArrayView<const FPreLockedDataPtr> PreLockedData, const ITaskContext* Context, FEntityAllocationWriteContext WriteContext)
	{
		const TScheduledEntityTask<TaskImpl, ComponentTypes...>* This = static_cast<const TScheduledEntityTask<TaskImpl, ComponentTypes...>*>(Context);
		Caller::ForEachEntityImpl(This->TaskImplInstance, Item, PreLockedData, WriteContext, This->Components);
	}
	AUTORTFM_INFER static void ScheduledRun_PerAllocation(FEntityAllocationIteratorItem Item, TArrayView<const FPreLockedDataPtr> PreLockedData, const ITaskContext* Context, FEntityAllocationWriteContext WriteContext)
	{
		const TScheduledEntityTask<TaskImpl, ComponentTypes...>* This = static_cast<const TScheduledEntityTask<TaskImpl, ComponentTypes...>*>(Context);
		Caller::ForEachAllocationImpl(This->TaskImplInstance, Item, PreLockedData, WriteContext, This->Components);
	}
private:

	static void PreTaskImpl(void*, ...){}
	template <typename T> static void PreTaskImpl(T* InTask, decltype(&T::PreTask)* = 0)
	{
		InTask->PreTask();
	}

	static void PostTaskImpl(void*, ...){}
	template <typename T> static void PostTaskImpl(T* InTask, decltype(&T::PostTask)* = 0)
	{
		InTask->PostTask();
	}

	using Caller = TEntityTaskCaller<sizeof...(ComponentTypes), TEntityTaskTraits<TaskImpl>::AutoExpandAccessors >;

	TaskImpl TaskImplInstance;
	TEntityTaskComponents<ComponentTypes...> Components;
};

template<typename TaskImpl, typename... ComponentTypes>
struct TEntityAllocationTaskBase
{
	explicit TEntityAllocationTaskBase(FEntityManager* InEntityManager, const TEntityTaskComponents<ComponentTypes...>& InComponents)
		: ComponentFilter(InComponents)
		, EntityManager(InEntityManager)
		, WriteContext(*InEntityManager)
	{}

	explicit TEntityAllocationTaskBase(FEntityManager* InEntityManager, const TFilteredEntityTask<ComponentTypes...>& InComponentFilter)
		: ComponentFilter(InComponentFilter)
		, EntityManager(InEntityManager)
		, WriteContext(*InEntityManager)
	{}

	TStatId GetStatId() const
	{
		return ComponentFilter.GetStatId();
	}

	ENamedThreads::Type GetDesiredThread() const
	{
		return ComponentFilter.GetDesiredThread();
	}

	void Run(TaskImpl& TaskImplInstance)
	{
		PreTask(&TaskImplInstance);

		EComponentHeaderLockMode LockMode = EntityManager->GetThreadingModel() == EEntityThreadingModel::NoThreading
			? EComponentHeaderLockMode::LockFree
			: EComponentHeaderLockMode::Mutex;

		for (FEntityAllocationIteratorItem Item : EntityManager->Iterate(&ComponentFilter.GetFilter()))
		{
			FEntityAllocationMutexGuard LockGuard(Item.GetAllocation(), LockMode);
			Caller::ForEachAllocationImpl(TaskImplInstance, Item, WriteContext, ComponentFilter.GetComponents());
		}

		PostTask(&TaskImplInstance);
	}

private:

	static void PreTask(void*, ...){}
	template <typename T> static void PreTask(T* InTask, decltype(&T::PreTask)* = 0)
	{
		InTask->PreTask();
	}

	static void PostTask(void*, ...){}
	template <typename T> static void PostTask(T* InTask, decltype(&T::PostTask)* = 0)
	{
		InTask->PostTask();
	}

protected:

	using Caller = TEntityTaskCaller<sizeof...(ComponentTypes), TEntityTaskTraits<TaskImpl>::AutoExpandAccessors >;

	TFilteredEntityTask<ComponentTypes...> ComponentFilter;
	FEntityManager* EntityManager;
	FEntityAllocationWriteContext WriteContext;
};

template<typename TaskImpl, typename... ComponentTypes>
struct TEntityAllocationTask : TEntityAllocationTaskBase<TaskImpl, ComponentTypes...>
{
	template<typename... ArgTypes>
	explicit TEntityAllocationTask(FEntityManager* InEntityManager, const TEntityTaskComponents<ComponentTypes...>& InComponents, ArgTypes&&... InArgs)
		: TEntityAllocationTaskBase<TaskImpl, ComponentTypes...>(InEntityManager, InComponents)
		, TaskImplInstance{ Forward<ArgTypes>(InArgs)... }
	{}

	template<typename... ArgTypes>
	explicit TEntityAllocationTask(FEntityManager* InEntityManager, const TFilteredEntityTask<ComponentTypes...>& InComponentFilter, ArgTypes&&... InArgs)
		: TEntityAllocationTaskBase<TaskImpl, ComponentTypes...>(InEntityManager, InComponentFilter)
		, TaskImplInstance{ Forward<ArgTypes>(InArgs)... }
	{}

	static ESubsequentsMode::Type GetSubsequentsMode()
	{
		return ESubsequentsMode::TrackSubsequents;
	}

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& CompletionGraphEvent)
	{
		if (this->ComponentFilter.ShouldBreakOnRun())
		{
			UE_DEBUG_BREAK();
		}

		if ((this->ComponentFilter.GetDesiredThread() & ENamedThreads::AnyThread) == 0)
		{
			checkf(CurrentThread == this->ComponentFilter.GetDesiredThread(), TEXT("MovieScene evaluation task is not being run on its desired thread"));
		}

		this->Run(TaskImplInstance);
	}

private:

	TaskImpl TaskImplInstance;
};

template<typename TaskImpl>
struct TUnstructuredTask
{
	template<typename... ArgTypes>
	explicit TUnstructuredTask(const FCommonEntityTaskParams& InCommonParams, ArgTypes&&... InArgs)
		: TaskImplInstance{ Forward<ArgTypes>(InArgs)... }
		, CommonParams(InCommonParams)
	{}

	TStatId GetStatId() const
	{
		return CommonParams.TaskParams.StatId;
	}

	ENamedThreads::Type GetDesiredThread() const
	{
		return CommonParams.DesiredThread;
	}

	static ESubsequentsMode::Type GetSubsequentsMode()
	{
		return ESubsequentsMode::TrackSubsequents;
	}

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& CompletionGraphEvent)
	{
		if (CommonParams.bBreakOnRun)
		{
			UE_DEBUG_BREAK();
		}

		if ((CommonParams.DesiredThread & ENamedThreads::AnyThread) == 0)
		{
			checkf(CurrentThread == CommonParams.DesiredThread, TEXT("MovieScene evaluation task is not being run on its desired thread"));
		}

		TaskImplInstance.Run();
	}

private:

	TaskImpl TaskImplInstance;

	FCommonEntityTaskParams CommonParams;
};

template<typename...> struct TEntityTaskCaller_AutoExpansion;


template<typename ...AccessorTypes>
struct TPrelockedDataOffsets
{
	static constexpr int32 Num = (... + AccessorTypes::PreLockedDataNum);

	int32 StartOffset[sizeof...(AccessorTypes)]={};

	constexpr TPrelockedDataOffsets()
	{
		int Index = 0, DataIndex = 0;
		(..., Assign(Index, DataIndex, AccessorTypes::PreLockedDataNum));
	}
	constexpr void Assign(int& Index, int& DataIndex, int DataSize)
	{
		StartOffset[Index] = DataIndex;
		DataIndex += DataSize;
		++Index;
	}
};

template<int... Indices>
struct AUTORTFM_INFER TEntityTaskCaller_AutoExpansion<TIntegerSequence<int, Indices...>>
{
	template<typename TaskImpl, typename... AccessorTypes>
	static void ForEachEntityImpl(TaskImpl& TaskImplInstance, FEntityAllocationIteratorItem Item, TArrayView<const FPreLockedDataPtr> PreLockedData, FEntityAllocationWriteContext WriteContext, const TEntityTaskComponents<AccessorTypes...>& Components)
	{
		FEntityIterationResult Result;

		constexpr TPrelockedDataOffsets<AccessorTypes...> PrelockedDataOffsets;
		auto ResolvedComponentData = MakeTuple( Components.template GetAccessor<Indices>().ResolvePreLockedComponentData(Item, &PreLockedData[PrelockedDataOffsets.StartOffset[Indices]], WriteContext)... );

		const int32 Num = Item.GetAllocation()->Num();
		for (int32 ComponentOffset = 0; ComponentOffset < Num && Result.Value; ++ComponentOffset )
		{
			Result = (TaskImplInstance.ForEachEntity(GetComponentAtIndex(&ResolvedComponentData.template Get<Indices>(), ComponentOffset)... ), Result);
		}
	}

	template<typename TaskImpl, typename... AccessorTypes>
	static void ForEachEntityImpl(TaskImpl& TaskImplInstance, FEntityAllocationIteratorItem Item, FEntityAllocationWriteContext WriteContext, const TEntityTaskComponents<AccessorTypes...>& Components)
	{
		FEntityIterationResult Result;

		// Lock the components we want to access
		auto LockedComponentData = MakeTuple( Components.template GetAccessor<Indices>().LockComponentData(Item, WriteContext)... );

		const int32 Num = Item.GetAllocation()->Num();
		for (int32 ComponentOffset = 0; ComponentOffset < Num && Result.Value; ++ComponentOffset )
		{
			Result = (TaskImplInstance.ForEachEntity(GetComponentAtIndex(&LockedComponentData.template Get<Indices>(), ComponentOffset)... ), Result);
		}
	}

	template<typename TaskImpl, typename... AccessorTypes>
	static void ForEachAllocationImpl(TaskImpl& TaskImplInstance, FEntityAllocationIteratorItem Item, TArrayView<const FPreLockedDataPtr> PreLockedData, FEntityAllocationWriteContext WriteContext, const TEntityTaskComponents<AccessorTypes...>& Components)
	{
		constexpr TPrelockedDataOffsets<AccessorTypes...> PrelockedDataOffsets;
		const FEntityAllocation* Allocation = Item.GetAllocation();
		TaskImplInstance.ForEachAllocation(Item, Components.template GetAccessor<Indices>().ResolvePreLockedComponentData(Item, &PreLockedData[PrelockedDataOffsets.StartOffset[Indices]], WriteContext)...);
	}

	template<typename TaskImpl, typename... AccessorTypes>
	static void ForEachAllocationImpl(TaskImpl& TaskImplInstance, FEntityAllocationIteratorItem Item, FEntityAllocationWriteContext WriteContext, const TEntityTaskComponents<AccessorTypes...>& Components)
	{
		auto LockedComponentData = MakeTuple( Components.template GetAccessor<Indices>().LockComponentData(Item, WriteContext)... );
		TaskImplInstance.ForEachAllocation(Item, LockedComponentData.template Get<Indices>()...);
	}
};

template<int NumComponents>
struct TEntityTaskCaller<NumComponents, true> : TEntityTaskCaller_AutoExpansion<TMakeIntegerSequence<int, NumComponents>>
{
};

template<int32 NumComponents>
struct TEntityTaskCaller<NumComponents, false>
{
	template<typename TaskImpl>
	inline static void ForEachEntityImpl(TaskImpl& TaskImplInstance, ...)
	{
		static_assert(!std::is_same_v<TaskImpl, TaskImpl>, "non-expanded entity iteration is not supported");
	}

	template<typename TaskImpl, typename... AccessorTypes>
	inline static void ForEachAllocationImpl(TaskImpl& TaskImplInstance, FEntityAllocationIteratorItem Item, TArrayView<const FPreLockedDataPtr> PreLockedData, FEntityAllocationWriteContext WriteContext, const TEntityTaskComponents<AccessorTypes...>& Components)
	{
		TaskImplInstance.ForEachAllocation(Item, Components, PreLockedData);
	}

	template<typename TaskImpl, typename... AccessorTypes>
	inline static void ForEachAllocationImpl(TaskImpl& TaskImplInstance, FEntityAllocationIteratorItem Item, const TEntityTaskComponents<AccessorTypes...>& Components)
	{
		TaskImplInstance.ForEachAllocation(Item, Components);
	}
};


} // namespace MovieScene
} // namespace UE
