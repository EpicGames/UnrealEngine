// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Find.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/BitArray.h"
#include "MovieSceneFwd.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "HAL/CriticalSection.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Stats/Stats.h"
#include "Templates/UnrealTemplate.h"

#include <atomic>
#include <initializer_list>

class UMovieSceneSequence;
namespace UE { namespace MovieScene { class FEntityManager; } }


DECLARE_STATS_GROUP(TEXT("Movie Scene Evaluation Systems"), STATGROUP_MovieSceneECS, STATCAT_Advanced)

namespace UE
{
namespace MovieScene
{

struct FReadErased;
struct FReadErasedOptional;
struct FWriteErased;
struct FWriteErasedOptional;
template<typename T> struct TComponentLock;
template<typename T> struct TRead;
template<typename T> struct TReadOptional;
template<typename T> struct TWrite;
template<typename T> struct TWriteOptional;

enum class ESystemPhase : uint8
{
	/** Null phase which indicates that the system never runs, but still exists in the reference graph */
	None = 0,

	/**  */
	Import = 1 << 0,

	/** Expensive: Phase that is run before instantiation any time any boundary is crossed in the sequence. Used to spawn new objects and trigger pre/post-spawn events. */
	Spawn = 1 << 1,

	/** Expensive: Houses any system that needs to instantiate global entities into the linker, or make meaningful changes to entity structures.. */
	Instantiation = 1 << 2,

	/**  */
	Scheduling = 1 << 3,

	/** Fast, distributed: Houses the majority of evaluation systems that compute animation data. Entity manager is locked down for the duration of this phase. */
	Evaluation = 1 << 4,

	/** Finalization phase for enything that wants to run after everything else. */
	Finalization = 1 << 5,
};
ENUM_CLASS_FLAGS(ESystemPhase);

enum class EComponentTypeFlags : uint8
{
	None = 0x00,

	/** This component type should be preserved when an entity is replaced with another during linking */
	Preserved = 0x1,

	/** Automatically copy this component to child components when being constructed through the component factory */
	CopyToChildren = 0x2,

	/** Indicates that this component type represents a cached value that should be copied to blend outputs */
	CopyToOutput = 0x4,

	/** Indicates that this component type represents a cached value that should be migrated to blend outputs (and removed from blend inputs) */
	MigrateToOutput = 0x8,
};
ENUM_CLASS_FLAGS(EComponentTypeFlags);


enum class EComplexFilterMode : uint8
{
	OneOf       = 1 << 0,
	OneOrMoreOf = 1 << 1,
	AllOf       = 1 << 2,

	// High bit modifiers
	Negate      = 1 << 7, 
};
ENUM_CLASS_FLAGS(EComplexFilterMode);

enum class EMutuallyInclusiveComponentType : uint8
{
	Mandatory = 1u << 0,
	Optional  = 1u << 1,

	All = Mandatory | Optional,
};
ENUM_CLASS_FLAGS(EMutuallyInclusiveComponentType)

/**
 * Enumeration specifying the locking mechanism to use when accessing component data
 */
enum class EComponentHeaderLockMode
{
	LockFree,
	Mutex,
};

/**
 * A numeric identifier used to represent a specific 'channel' within an interrogation linker.
 * Interrogation channels are used to identify groupings of tracks or entities that relate to the same output (eg: a property on an object; a root track etc).
 * See FInterrogationKey for a combination of a channel with a specific interrogation index (or time)
 */
struct FInterrogationChannel
{
	/**
	 * Default construction to an invalid channel
	 */
	FInterrogationChannel()
		: Value(INDEX_NONE)
	{}

	/**
	 * Get the next channel after this one
	 */
	FInterrogationChannel operator++()
	{
		check( Value != INDEX_NONE );
		return FInterrogationChannel(++Value);
	}

	/**
	 * Check whether this channel is valid
	 */
	explicit operator bool() const
	{
		return IsValid();
	}

	/**
	 * Check whether this channel is valid
	 */
	bool IsValid() const
	{
		return Value != INDEX_NONE;
	}

	/**
	 * Get the underlying numeric representation of this channel
	 */
	int32 AsIndex() const
	{
		check(Value != INDEX_NONE);
		return Value;
	}

	/**
	 * A default interrogation channel that can be used when interrogation is only operating on a single set of tracks
	 */
	static FInterrogationChannel Default()
	{
		return FInterrogationChannel(0);
	}

	/**
	 * Retrieve the first allocatable channel when dealing with multiple groupings of tracks
	 */
	static FInterrogationChannel First()
	{
		return FInterrogationChannel(1);
	}

	/**
	 * Make a channel from an index known to already relate to a valid channel
	 */
	static FInterrogationChannel FromIndex(int32 InIndex)
	{
		return FInterrogationChannel(InIndex);
	}

	/**
	 * Retrieve the last allocatable channel when dealing with multiple groupings of tracks
	 */
	static FInterrogationChannel Last()
	{
		return FInterrogationChannel(MAX_int32);
	}

	/**
	 * An invalid channel. Should only be used for comparison.
	 */
	static FInterrogationChannel Invalid()
	{
		return FInterrogationChannel(INDEX_NONE);
	}

	friend uint32 GetTypeHash(FInterrogationChannel In)
	{
		return In.Value;
	}

	friend bool operator==(FInterrogationChannel A, FInterrogationChannel B)
	{
		return A.Value == B.Value;
	}

	friend bool operator!=(FInterrogationChannel A, FInterrogationChannel B)
	{
		return A.Value != B.Value;
	}

	friend bool operator<(FInterrogationChannel A, FInterrogationChannel B)
	{
		return A.Value < B.Value;
	}

private:

	explicit FInterrogationChannel(int32 InValue) : Value(InValue) {}

	int32 Value;
};


/**
 * A key that uniquely identifies a specific interrogation time with a group of entities allocated within a channel
 */
struct FInterrogationKey
{
	/** The channel that the entities are allocated within */
	FInterrogationChannel Channel;

	/** The index of the interrogation (see FSystemInterrogator::InterrogateTime, and FSystemInterrogator::InterrogationData) */
	int32 InterrogationIndex;

	FInterrogationKey()
		: InterrogationIndex(-1)
	{}

	FInterrogationKey(FInterrogationChannel InChannel, int32 InInterrogationIndex)
		: Channel(InChannel)
		, InterrogationIndex(InInterrogationIndex)
	{}

	bool IsValid() const
	{
		return Channel.IsValid() && InterrogationIndex != INDEX_NONE;
	}

	friend uint32 GetTypeHash(const FInterrogationKey& In)
	{
		return GetTypeHash(In.Channel) ^ In.InterrogationIndex;
	}

	friend bool operator==(const FInterrogationKey& A, const FInterrogationKey& B)
	{
		return A.Channel == B.Channel && A.InterrogationIndex == B.InterrogationIndex;
	}

	friend bool operator!=(const FInterrogationKey& A, const FInterrogationKey& B)
	{
		return A.Channel != B.Channel || A.InterrogationIndex != B.InterrogationIndex;
	}

	static FInterrogationKey Default(int32 InInterrogationIndex = 0)
	{
		return FInterrogationKey(FInterrogationChannel::Default(), InInterrogationIndex);
	}
};


/**
 * Sequence instance information for interrogations
 */
struct FInterrogationInstance
{
	UMovieSceneSequence* Sequence = nullptr;

	bool IsValid() const
	{
		return Sequence != nullptr;
	}
};


struct FEntityComponentFilter
{

	void Reset()
	{
		AllMask.Reset();
		NoneMask.Reset();
		ComplexMasks.Reset();
	}

	MOVIESCENE_API bool Match(const FComponentMask& Input) const;

	MOVIESCENE_API bool IsValid() const;

	FEntityComponentFilter& All(const FComponentMask& InComponentMask)
	{
		AllMask.CombineWithBitwiseOR(InComponentMask, EBitwiseOperatorFlags::MaxSize);
		return *this;
	}
	FEntityComponentFilter& All(std::initializer_list<FComponentTypeID> InComponentTypes)
	{
		AllMask.SetAll(InComponentTypes);
		return *this;
	}


	FEntityComponentFilter& None(const FComponentMask& InComponentMask)
	{
		NoneMask.CombineWithBitwiseOR(InComponentMask, EBitwiseOperatorFlags::MaxSize);
		return *this;
	}
	FEntityComponentFilter& None(std::initializer_list<FComponentTypeID> InComponentTypes)
	{
		NoneMask.SetAll(InComponentTypes);
		return *this;
	}


	FEntityComponentFilter& Any(const FComponentMask& InComponentMask)
	{
		return Complex(InComponentMask, EComplexFilterMode::OneOrMoreOf);
	}
	FEntityComponentFilter& Any(std::initializer_list<FComponentTypeID> InComponentTypes)
	{
		return Complex(InComponentTypes, EComplexFilterMode::OneOrMoreOf);
	}

	FEntityComponentFilter& AnyLenient(std::initializer_list<FComponentTypeID> InComponentTypes)
	{
		FComponentMask Mask;
		for (FComponentTypeID TypeID : InComponentTypes)
		{
			if (TypeID)
			{
				Mask.Set(TypeID);
			}
		}
		return Any(Mask);
	}


	FEntityComponentFilter& Deny(std::initializer_list<FComponentTypeID> InComponentTypes)
	{
		return Complex(InComponentTypes, EComplexFilterMode::AllOf | EComplexFilterMode::Negate);
	}

	FEntityComponentFilter& Deny(const FComponentMask& InComponentMask)
	{
		return Complex(InComponentMask, EComplexFilterMode::AllOf | EComplexFilterMode::Negate);
	}


	FEntityComponentFilter& Complex(std::initializer_list<FComponentTypeID> InComponentTypes, EComplexFilterMode ComplexMode)
	{
		if (InComponentTypes.size() > 0)
		{
			FComplexMask& ComplexMask = ComplexMasks.Emplace_GetRef(ComplexMode);
			ComplexMask.Mask.SetAll(InComponentTypes);
		}
		return *this;
	}

	FEntityComponentFilter& Complex(const FComponentMask& InComponentMask, EComplexFilterMode ComplexMode)
	{
		if (InComponentMask.Num() > 0)
		{
			ComplexMasks.Emplace(InComponentMask, ComplexMode);
		}
		return *this;
	}

	FEntityComponentFilter& Combine(const FEntityComponentFilter& CombineWith)
	{
		if (CombineWith.AllMask.Num() > 0)
		{
			AllMask.CombineWithBitwiseOR(CombineWith.AllMask, EBitwiseOperatorFlags::MaxSize);
		}

		if (CombineWith.NoneMask.Num() > 0)
		{
			NoneMask.CombineWithBitwiseOR(CombineWith.NoneMask, EBitwiseOperatorFlags::MaxSize);
		}

		if (CombineWith.ComplexMasks.Num() > 0)
		{
			ComplexMasks.Append(CombineWith.ComplexMasks);
		}
		return *this;
	}

private:

	struct FComplexMask
	{
		FComplexMask(EComplexFilterMode InMode)
			: Mode(InMode)
		{}
		FComplexMask(const FComponentMask& InMask, EComplexFilterMode InMode)
			: Mask(InMask), Mode(InMode)
		{}

		FComponentMask Mask;
		EComplexFilterMode Mode;
	};
	FComponentMask AllMask;
	FComponentMask NoneMask;
	TArray<FComplexMask> ComplexMasks;
};


struct FEntityAllocationWriteContext
{
	MOVIESCENE_API FEntityAllocationWriteContext(const FEntityManager& EntityManager);

	static FEntityAllocationWriteContext NewAllocation()
	{
		return FEntityAllocationWriteContext();
	}

	FEntityAllocationWriteContext Add(FEntityAllocationWriteContext InOther) const
	{
		FEntityAllocationWriteContext Context;
		Context.SystemSerial = SystemSerial + InOther.SystemSerial;
		return Context;
	}
	FEntityAllocationWriteContext Subtract(FEntityAllocationWriteContext InOther) const
	{
		FEntityAllocationWriteContext Context;
		Context.SystemSerial = SystemSerial - InOther.SystemSerial;
		return Context;
	}
	uint64 GetSystemSerial() const
	{
		return SystemSerial;
	}

private:

	FEntityAllocationWriteContext()
		: SystemSerial(0)
	{}

	uint64 SystemSerial;
};

struct FComponentHeader
{
	FComponentHeader() = default;

	FComponentHeader(FComponentHeader&&) = delete;
	FComponentHeader& operator=(FComponentHeader&&) = delete;

	FComponentHeader(const FComponentHeader&) = delete;
	FComponentHeader& operator=(const FComponentHeader&) = delete;

#if UE_MOVIESCENE_ENTITY_DEBUG

	// This virtual destructor exists in order to create a vftable for this type
	//    that is used by the debugger to locate and display the derived TComponentHeader<T> type
	//    enabling visualization of the concrete type in the debugger. It is completely compiled
	//    out when UE_MOVIESCENE_ENTITY_DEBUG is disabled ensuring this type is not polymorphic.
	MOVIESCENE_API virtual ~FComponentHeader();

	/** Pointer to a polymorphic view of the component data used for debugging purposes - only populated when GRichComponentDebugging is enabled */
	struct IComponentDebuggingTypedPtr* DebugComponents = nullptr;

	/** Pointer to this header's owning FEntityAllocation::Size member. Only required and used for debugging. */
	const uint16* Size = nullptr;

#endif // UE_MOVIESCENE_ENTITY_DEBUG

	/** Pointer to the first component of this header's type, with array size == FEntityAllocation::Capacity, and stride == FComponentHeader::Sizeof */
	mutable uint8* Components;

	/** Legacy mutex used for locked threaded evaluation */
	mutable FTransactionallySafeRWLock ReadWriteLock;

private:

	/** Serial number that is increased after every time this header is opened for write access. Not currently meaningfully used but may be used for caching in future. */
	mutable uint64 SerialNumber;

public:

	struct FAutoRTFMCompiledOutAtomicInt32 final
	{
		int32 fetch_add(const int32 Value, const std::memory_order) { return 0; }
		int32 fetch_sub(const int32 Value, const std::memory_order) { return 0; }
		int32 exchange(const int32 Value, const std::memory_order) { return 0; }
	};

#if UE_AUTORTFM
	using FScheduledAccessCountType = FAutoRTFMCompiledOutAtomicInt32;
#else
	using FScheduledAccessCountType = std::atomic<int32>;
#endif

	/** Atomic access count used for verifying write exclusivity at runtime. */
	mutable FScheduledAccessCountType ScheduledAccessCount;

	/** sizeof(T) for the component type this header represents (ie: the stride of our component data). 0 if this header represents a tag. */
	uint8 Sizeof;

	/** Component type identifier */
	FComponentTypeID ComponentType;

	/**
	 * Whether this component header describes a tag component (i.e. a component with no data).
	 */
	bool IsTag() const
	{
		return Sizeof == 0;
	}

	/**
	 * Whether this component header is associated with a data buffer.
	 *
	 * Tag components don't have data. Non-tag components could have no data if their data buffer
	 * has been relocated, such as an entity allocation that has moved elsewhere because of a
	 * migration or mutation.
	 */
	bool HasData() const
	{
		return Components != nullptr;
	}

	/**
	 * Get the raw pointer to the associated component data buffer.
	 */
	void* GetValuePtr(int32 Offset) const
	{
		check(!IsTag() && Components != nullptr);
		return Components + Sizeof*Offset;
	}

	void PostWriteComponents(FEntityAllocationWriteContext InWriteContext) const
	{
		SerialNumber = FMath::Max(SerialNumber, InWriteContext.GetSystemSerial());
	}

	bool HasBeenWrittenToSince(uint64 InSystemSerial) const
	{
		return SerialNumber > InSystemSerial;
	}
};


/**
 * Moveable scoped read-lock type for component headers
 */
struct FScopedHeaderReadLock
{
	FScopedHeaderReadLock();
	FScopedHeaderReadLock(const FComponentHeader* InHeader, EComponentHeaderLockMode InLockMode);

	FScopedHeaderReadLock(const FScopedHeaderReadLock& RHS) = delete;
	void operator=(const FScopedHeaderReadLock& RHS) = delete;

	FScopedHeaderReadLock(FScopedHeaderReadLock&& RHS);
	FScopedHeaderReadLock& operator=(FScopedHeaderReadLock&& RHS);

	~FScopedHeaderReadLock();

private:
	const FComponentHeader* Header;
	EComponentHeaderLockMode LockMode;
};


/**
 * Moveable scoped write-lock type for component headers
 */
struct FScopedHeaderWriteLock
{
	FScopedHeaderWriteLock();
	FScopedHeaderWriteLock(const FComponentHeader* InHeader, EComponentHeaderLockMode InLockMode, FEntityAllocationWriteContext InWriteContext);

	FScopedHeaderWriteLock(const FScopedHeaderWriteLock& RHS) = delete;
	void operator=(const FScopedHeaderWriteLock& RHS) = delete;

	FScopedHeaderWriteLock(FScopedHeaderWriteLock&& RHS);
	FScopedHeaderWriteLock& operator=(FScopedHeaderWriteLock&& RHS);

	~FScopedHeaderWriteLock();

private:
	const FComponentHeader* Header;
	FEntityAllocationWriteContext WriteContext;
	EComponentHeaderLockMode LockMode;
};


/**
 * FEntityAllocation is the authoritative storage of entity-component data within an FEntityManager. It stores component data in separate contiguous arrays, aligned to a cache line.
 * Storing component data in this way allows for cache-efficient and concurrent access to each component array in isolation. It also allows for write access to component arrays
 * at the same time as concurrent read-access to other component arrays within the same entity allocation.
 *
 * FEntityAllocations are custom allocated according to the size of its component capacity, which is loosely computed as sizeof(FEntityAllocation) + sizeof(ComponentData), not simply sizeof(FEntityAllocation).
 *
 * A typical allocation will look like this in memory:
 *
 *    uint32 {UniqueID}, uint16 {NumComponents}, uint16 {Size}, uint16 {Capacity}, uint16 {MaxCapacity}, uint32 {SerialNumber},
 *    FMovieSceneEntityID* {EntityIDs},   <-- points to FMovieSceneEntityID array at end of structure 
 *    FComponentHeader[NumComponents],    <-- each component header contains a component array ptr that points to its corresponding type array below
 *    (padding) FMovieSceneEntityID[Capacity],
 *    (padding) ComponentType1[Capacity],
 *    (padding) ComponentType2[Capacity],
 *    (padding) ComponentType3[Capacity],
 */
struct FEntityAllocation
{
	/**
	 * Constructor that initializes the defaults for this structure.
	 * CAUTION: Does not initialize ComponentHeaders - these constructors must be called manually
	 */
	FEntityAllocation()
		: SerialNumber(0)
		, NumComponents(0)
		, Size(0)
		, Capacity(0)
		, MaxCapacity(0)
		, LockMode(EComponentHeaderLockMode::Mutex)
	{}

	/**
	 * Manually invoked destructor that calls the destructor of each component header according to the number of components
	 */
	~FEntityAllocation()
	{
		for (int32 Index = 0; Index < NumComponents; ++Index)
		{
			ComponentHeaders[Index].~FComponentHeader();
		}
	}

	/** Entity allocations are non-copyable */
	FEntityAllocation(const FEntityAllocation&) = delete;
	FEntityAllocation& operator=(const FEntityAllocation&) = delete;


	/**
	 * Retrieve all of this allocation's component and tag headers.
	 */
	TArrayView<const FComponentHeader> GetComponentHeaders() const
	{
		return MakeArrayView(ComponentHeaders, NumComponents);
	}


	/**
	 * Retrieve all of this allocation's component and tag headers.
	 */
	TArrayView<FComponentHeader> GetComponentHeaders()
	{
		return MakeArrayView(ComponentHeaders, NumComponents);
	}


	/**
	 * Check whether this allocation has the specified component type
	 *
	 * @param ComponentTypeID The type ID for the component to check
	 * @return true if these entities have the specified component, false otherwise
	 */
	bool HasComponent(FComponentTypeID ComponentTypeID) const
	{
		return FindComponentHeader(ComponentTypeID) != nullptr;
	}


	/**
	 * Find a component header by its type
	 *
	 * @param ComponentTypeID The type ID for the component header to locate
	 * @return A pointer to the component header, or nullptr if one was not found
	 */
	const FComponentHeader* FindComponentHeader(FComponentTypeID ComponentTypeID) const
	{
		return Algo::FindBy(GetComponentHeaders(), ComponentTypeID, &FComponentHeader::ComponentType);
	}


	/**
	 * Find a component header by its type
	 *
	 * @param ComponentTypeID The type ID for the component header to locate
	 * @return A pointer to the component header, or nullptr if one was not found
	 */
	FComponentHeader* FindComponentHeader(FComponentTypeID ComponentTypeID)
	{
		return Algo::FindBy(GetComponentHeaders(), ComponentTypeID, &FComponentHeader::ComponentType);
	}


	/**
	 * Get a reference to a component header by its type. Will fail an assertion if it does not exist.
	 *
	 * @param ComponentTypeID The type ID for the component header to locate
	 * @return A reference to the component header
	 */
	const FComponentHeader& GetComponentHeaderChecked(FComponentTypeID ComponentTypeID) const
	{
		const FComponentHeader* Header = FindComponentHeader(ComponentTypeID);
		check(Header);
		return *Header;
	}


	/**
	 * Get a reference to a component header by its type. Will fail an assertion if it does not exist.
	 *
	 * @param ComponentTypeID The type ID for the component header to locate
	 * @return A reference to the component header
	 */
	FComponentHeader& GetComponentHeaderChecked(FComponentTypeID ComponentTypeID)
	{
		FComponentHeader* Header = FindComponentHeader(ComponentTypeID);
		check(Header);
		return *Header;
	}


	/**
	 * Retrieve all of this allocation's entity IDs
	 */
	TArrayView<const FMovieSceneEntityID> GetEntityIDs() const
	{
		return MakeArrayView(EntityIDs, Size);
	}


	/**
	 * Retrieve all of this allocation's entity IDs as a raw ptr
	 */
	const FMovieSceneEntityID* GetRawEntityIDs() const
	{
		return EntityIDs;
	}


	/**
	 * Retrieve the address of this allocation's component data. Only to be used for construction of TRelativePtrs.
	 */
	const void* GetComponentDataAddress() const
	{
		return ComponentData;
	}


	/**
	 * Get the unique identifier for this allocation. This identifier is unique to the specific allocation and entity manager, but is not globally unique.
	 * Typically used for caching component data on a per-allocation basis
	 */
	uint32 GetUniqueID() const
	{
		return UniqueID;
	}


	/**
	 * Retrieve this allocation's serial number. The serial number is incremented whenever a component is modified on this allocation, or when an entity is added or removed.
	 * Typically used for caching component data on a per-allocation basis
	 */
	bool HasStructureChangedSince(uint64 InSystemVersion) const
	{
		return SerialNumber > InSystemVersion;
	}


	/**
	 * Called when this allocation has been modified. Will invalidate any cached data based of this allocation's serial number
	 */
	void PostModifyStructureExcludingHeaders(FEntityAllocationWriteContext InWriteContext)
	{
		SerialNumber = FMath::Max(SerialNumber, InWriteContext.GetSystemSerial());
	}


	void PostModifyStructure(FEntityAllocationWriteContext InWriteContext)
	{
		SerialNumber = FMath::Max(SerialNumber, InWriteContext.GetSystemSerial());
		for (FComponentHeader& Header : GetComponentHeaders())
		{
			Header.PostWriteComponents(InWriteContext);
		}
	}

	/**
	 * Get the number of component types and tags that exist within this allocation
	 */
	int32 GetNumComponentTypes() const
	{
		return int32(NumComponents);
	}


	/**
	 * Retrieve the number of entities in this allocation
	 */
	int32 Num() const
	{
		return int32(Size);
	}


	/**
	 * Retrieve the maximum number of entities that this allocation is allowed to grow to until a new one must be made
	 */
	int32 GetMaxCapacity() const
	{
		return int32(MaxCapacity);
	}


	/**
	 * Retrieve the number of entities this allocation can currently house without reallocation
	 */
	int32 GetCapacity() const
	{
		return int32(Capacity);
	}


	/**
	 * Retrieve the amount of empty space within this allocation
	 */
	int32 GetSlack() const
	{
		return int32(Capacity) - int32(Size);
	}

	/**
	 * Get this allocation's current lock mode
	 */
	EComponentHeaderLockMode GetCurrentLockMode() const
	{
		return LockMode;
	}

	/**
	 * Read type-erased component data for the specified component type
	 */
	[[nodiscard]] MOVIESCENE_API TComponentLock<FReadErased> ReadComponentsErased(FComponentTypeID ComponentType) const;

	/**
	 * Write type-erased component data for the specified component type
	 */
	[[nodiscard]] MOVIESCENE_API TComponentLock<FWriteErased> WriteComponentsErased(FComponentTypeID ComponentType, FEntityAllocationWriteContext InWriteContext) const;

	/**
	 * Attempt to read type-erased component data for the specified component type
	 */
	[[nodiscard]] MOVIESCENE_API TComponentLock<FReadErasedOptional> TryReadComponentsErased(FComponentTypeID ComponentType) const;

	/**
	 * Attempt to write type-erased component data for the specified component type
	 */
	[[nodiscard]] MOVIESCENE_API TComponentLock<FWriteErasedOptional> TryWriteComponentsErased(FComponentTypeID ComponentType, FEntityAllocationWriteContext InWriteContext) const;

	/**
	 * Read typed component data for the specified component type
	 */
	template<typename T>
	[[nodiscard]] TComponentLock<TRead<T>> ReadComponents(TComponentTypeID<T> ComponentType) const
	{
		const FComponentHeader& Header = GetComponentHeaderChecked(ComponentType);
		return TComponentLock<TRead<T>>(&Header, LockMode);
	}

	/**
	 * Write typed component data for the specified component type
	 */
	template<typename T>
	[[nodiscard]] TComponentLock<TReadOptional<T>> TryReadComponents(TComponentTypeID<T> ComponentType) const
	{
		if (const FComponentHeader* Header = FindComponentHeader(ComponentType))
		{
			return TComponentLock<TReadOptional<T>>(Header, LockMode);
		}
		return TComponentLock<TReadOptional<T>>();
	}

	/**
	 * Write typed component data for the specified component type
	 */
	template<typename T>
	[[nodiscard]] TComponentLock<TWrite<T>> WriteComponents(TComponentTypeID<T> ComponentType, FEntityAllocationWriteContext InWriteContext) const
	{
		const FComponentHeader& Header = GetComponentHeaderChecked(ComponentType);
		return TComponentLock<TWrite<T>>(&Header, LockMode, InWriteContext);
	}

	/**
	 * Attempt to write typed component data for the specified component type
	 */
	template<typename T>
	[[nodiscard]] TComponentLock<TWriteOptional<T>> TryWriteComponents(TComponentTypeID<T> ComponentType, FEntityAllocationWriteContext InWriteContext) const
	{
		if (const FComponentHeader* Header = FindComponentHeader(ComponentType))
		{
			return TComponentLock<TWriteOptional<T>>(Header, LockMode, InWriteContext);
		}
		return TComponentLock<TWriteOptional<T>>();
	}

private:

	friend struct FEntityInitializer;
	friend struct FEntityAllocationMutexGuard;

	/** Assigned to FEntityManager::GetSystemSerial whenever this allocation is written to */
	uint64 SerialNumber;
	/** Unique Identifier within this allocation's FEntityManager. This ID is never reused. */
	uint32 UniqueID;
	/** The number of component and tag types in this allocation (also defines the number of ComponentHeaders). */
	uint16 NumComponents;
	/** The number of entities currently allocated within this block. Defines the stride of each component array. */
	uint16 Size;
	/** The maximum number of entities currently allocated within this block including slack. Defines the maximum stride of each component array. */
	uint16 Capacity;
	/** The maximum number of entities that this entity is allowed to reallocate to accomodate for. */
	uint16 MaxCapacity;

	/** Lock mode for access - under threading models this will be set to EComponentHeaderLockMode::Mutex, EComponentHeaderLockMode::LockFree otherwise */
	EComponentHeaderLockMode LockMode;

	/** Pointer to the entity ID array (stored in the end padding of this structure). */
	FMovieSceneEntityID* EntityIDs;

	/** Pointer to separately allocated data buffer for components. */
	uint8* ComponentData;

public:
	/** Pointer to array of the component headers of size NumComponents (stored in the end padding of this structure). */
	FComponentHeader* ComponentHeaders;
};


/**
 * Scoped guard that temporarily overrides the locking mechanism for a specific allocation.
 * In order to guarantee thread-safety, this structure only actually does anything if InLockMode is specified as LockFree
 */
struct FEntityAllocationMutexGuard
{
	FEntityAllocationMutexGuard(FEntityAllocation* InAllocation, EComponentHeaderLockMode InLockMode);
	~FEntityAllocationMutexGuard();

	FEntityAllocationMutexGuard(const FEntityAllocationMutexGuard&) = delete;
	void operator=(const FEntityAllocationMutexGuard&) = delete;

	FEntityAllocationMutexGuard(const FEntityAllocationMutexGuard&&) = delete;
	void operator=(const FEntityAllocationMutexGuard&&) = delete;

private:
	FEntityAllocation* Allocation;
};


/**
 * A wrapper around an FEntityAllocation that provides access to other pieces of
 * information such as its component mask.
 */
struct FEntityAllocationProxy
{
	/** Gets the entity allocation */
	MOVIESCENE_API const FEntityAllocation* GetAllocation() const;

	/** Gets the entity allocation */
	MOVIESCENE_API FEntityAllocation* GetAllocation();

	/** Gets the entity allocation component mask */
	MOVIESCENE_API const FComponentMask& GetAllocationType() const;

	static FEntityAllocationProxy MakeInstance(const FEntityManager* InManager, int32 InAllocationIndex)
	{
		return FEntityAllocationProxy(InManager, InAllocationIndex);
	}

	/** Return this allocation's index within the entity manager */
	int32 GetAllocationIndex() const
	{
		return AllocationIndex;
	}

	/** Implicit cast to an entity allocation */
	operator const FEntityAllocation*() const
	{
		return GetAllocation();
	}

	/** Implicit cast to an entity allocation */
	operator FEntityAllocation*()
	{
		return GetAllocation();
	}

	/** Implicit cast to a component mask */
	operator const FComponentMask&() const
	{
		return GetAllocationType();
	}
	
	friend bool operator==(const FEntityAllocationProxy& A, const FEntityAllocationProxy& B)
	{
		return A.Manager == B.Manager && A.AllocationIndex == B.AllocationIndex;
	}
	
	/** Hashing function for storing handles in maps */
	friend uint32 GetTypeHash(FEntityAllocationProxy Proxy)
	{
		return Proxy.AllocationIndex;
	}

private:
	friend struct FEntityAllocationIterator;

	FEntityAllocationProxy(const FEntityManager* InManager, int32 InAllocationIndex)
		: Manager(InManager), AllocationIndex(InAllocationIndex)
	{}

	/** Entity manager being iterated */
	const FEntityManager* Manager;

	/** Current allocation index or Manager->EntityAllocationMasks.GetMaxIndex() when finished */
	int32 AllocationIndex;
};


/**
 * Defines a contiguous range of entities within an allocation
 */
struct FEntityRange
{
	const FEntityAllocation* Allocation = nullptr;
	int32 ComponentStartOffset = 0;
	int32 Num = 0;
};

struct FEntityDataLocation
{
	FEntityAllocation* Allocation;
	int32 ComponentOffset;

	FEntityRange AsRange() const
	{
		return FEntityRange{ Allocation, ComponentOffset, 1 };
	}
};

struct FEntityInfo
{
	FEntityDataLocation Data;
	FMovieSceneEntityID EntityID;
};

inline FEntityAllocationMutexGuard::FEntityAllocationMutexGuard(FEntityAllocation* InAllocation, EComponentHeaderLockMode InLockMode)
{
	// Since FEntityAllocation always defaults to Mutex locking, we only 
	// read/write to the allocation if LockFree is specified here (implying we are always in single-thread mode)
	if (InLockMode == EComponentHeaderLockMode::LockFree)
	{
		Allocation = InAllocation;
		InAllocation->LockMode = EComponentHeaderLockMode::LockFree;
	}
	else
	{
		Allocation = nullptr;
	}
}

inline FEntityAllocationMutexGuard::~FEntityAllocationMutexGuard()
{
	// Always reset back to Mutex
	if (Allocation)
	{
		Allocation->LockMode = EComponentHeaderLockMode::Mutex;
	}
}

inline FScopedHeaderReadLock::FScopedHeaderReadLock()
	: Header(nullptr)
{}
inline FScopedHeaderReadLock::FScopedHeaderReadLock(const FComponentHeader* InHeader, EComponentHeaderLockMode InLockMode)
	: Header(InHeader)
	, LockMode(InLockMode)
{
	if (InLockMode == EComponentHeaderLockMode::Mutex)
	{
		InHeader->ReadWriteLock.ReadLock();
	}

	InHeader->ScheduledAccessCount.fetch_add(1, std::memory_order_relaxed);
}
inline FScopedHeaderReadLock::FScopedHeaderReadLock(FScopedHeaderReadLock&& RHS)
	: Header(RHS.Header)
	, LockMode(RHS.LockMode)
{
	RHS.Header = nullptr;
}
inline FScopedHeaderReadLock& FScopedHeaderReadLock::operator=(FScopedHeaderReadLock&& RHS)
{
	if (Header)
	{
		Header->ScheduledAccessCount.fetch_add(1, std::memory_order_relaxed);
		if (LockMode == EComponentHeaderLockMode::Mutex)
		{
			Header->ReadWriteLock.ReadUnlock();
		}
	}

	Header = RHS.Header;
	LockMode = RHS.LockMode;
	RHS.Header = nullptr;
	return *this;
}

inline FScopedHeaderReadLock::~FScopedHeaderReadLock()
{
	if (Header)
	{
		Header->ScheduledAccessCount.fetch_sub(1, std::memory_order_relaxed);
		if (LockMode == EComponentHeaderLockMode::Mutex)
		{
			Header->ReadWriteLock.ReadUnlock();
		}
	}
}

inline FScopedHeaderWriteLock::FScopedHeaderWriteLock()
	: Header(nullptr)
	, WriteContext(FEntityAllocationWriteContext::NewAllocation())
	, LockMode(EComponentHeaderLockMode::Mutex)
{}

inline FScopedHeaderWriteLock::FScopedHeaderWriteLock(const FComponentHeader* InHeader, EComponentHeaderLockMode InLockMode, FEntityAllocationWriteContext InWriteContext)
	: Header(InHeader)
	, WriteContext(InWriteContext)
	, LockMode(InLockMode)
{
	if (InLockMode == EComponentHeaderLockMode::Mutex)
	{
		InHeader->ReadWriteLock.WriteLock();
	}

	const int32 PreviousAccessCount = InHeader->ScheduledAccessCount.fetch_add(1, std::memory_order_relaxed);
	checkf(PreviousAccessCount == 0, TEXT("Component header is still in use when it is being opened for write!"));
}

inline FScopedHeaderWriteLock::FScopedHeaderWriteLock(FScopedHeaderWriteLock&& RHS)
	: Header(RHS.Header)
	, WriteContext(RHS.WriteContext)
	, LockMode(RHS.LockMode)
{
	RHS.Header = nullptr;
}
inline FScopedHeaderWriteLock& FScopedHeaderWriteLock::operator=(FScopedHeaderWriteLock&& RHS)
{
	if (Header)
	{
		Header->PostWriteComponents(WriteContext);
		Header->ScheduledAccessCount.fetch_sub(1, std::memory_order_relaxed);

		if (LockMode == EComponentHeaderLockMode::Mutex)
		{
			Header->ReadWriteLock.WriteUnlock();
		}
	}

	Header = RHS.Header;
	WriteContext = RHS.WriteContext;
	LockMode = RHS.LockMode;

	RHS.Header = nullptr;
	return *this;
}
inline FScopedHeaderWriteLock::~FScopedHeaderWriteLock()
{
	if (Header)
	{
		Header->PostWriteComponents(WriteContext);
		Header->ScheduledAccessCount.fetch_sub(1, std::memory_order_relaxed);

		if (LockMode == EComponentHeaderLockMode::Mutex)
		{
			Header->ReadWriteLock.WriteUnlock();
		}
	}
}


} // namespace MovieScene
} // namespace UE

