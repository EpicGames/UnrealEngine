// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "Misc/FrameTime.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneEvalTimeSystem.generated.h"

class UObject;
struct FFrameTime;

namespace UE::MovieScene
{
	struct FEvaluatedTime
	{
		FFrameTime FrameTime;
		double Seconds;
		FFrameRate FrameRate;
	};
}

UCLASS(MinimalAPI)
class UMovieSceneEvalTimeSystem : public UMovieSceneEntitySystem
{
public:

	GENERATED_BODY()

	MOVIESCENE_API UMovieSceneEvalTimeSystem(const FObjectInitializer& ObjInit);

	MOVIESCENE_API virtual bool IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const override;
	MOVIESCENE_API virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	MOVIESCENE_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;

private:

	TArray<UE::MovieScene::FEvaluatedTime, TInlineAllocator<16>> EvaluatedTimes;

	UE::MovieScene::FEntityComponentFilter RelevantFilter;
};

