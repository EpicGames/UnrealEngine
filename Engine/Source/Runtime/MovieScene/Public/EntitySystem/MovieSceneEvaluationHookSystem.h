// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "Evaluation/IMovieSceneEvaluationHook.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Misc/FrameTime.h"
#include "MovieSceneSequenceID.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneEvaluationHookSystem.generated.h"

class UMovieSceneEntitySystemLinker;
class UObject;


USTRUCT()
struct FMovieSceneEvaluationHookEvent
{
	GENERATED_BODY()

	UPROPERTY()
	FMovieSceneEvaluationHookComponent Hook;

	UE::MovieScene::FRootInstanceHandle RootInstanceHandle;

	FMovieSceneSequenceID SequenceID;

	int32 TriggerIndex;

	FFrameTime RootTime;

	UE::MovieScene::EEvaluationHookEvent Type;

	bool bRestoreState = false;
};

USTRUCT()
struct FMovieSceneEvaluationHookEventContainer
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FMovieSceneEvaluationHookEvent> Events;
};

USTRUCT()
struct FMovieSceneEvaluationInstanceKey
{
	GENERATED_BODY()

	FMovieSceneEvaluationInstanceKey()
	{}

	FMovieSceneEvaluationInstanceKey(UE::MovieScene::FInstanceHandle InInstanceHandle)
		: InstanceHandle(InInstanceHandle)
	{}

	friend uint32 GetTypeHash(FMovieSceneEvaluationInstanceKey InKey)
	{
		return GetTypeHash(InKey.InstanceHandle);
	}
	friend bool operator==(FMovieSceneEvaluationInstanceKey A, FMovieSceneEvaluationInstanceKey B)
	{
		return A.InstanceHandle == B.InstanceHandle;
	}

	UE::MovieScene::FInstanceHandle InstanceHandle;
};


UCLASS(MinimalAPI)
class UMovieSceneEvaluationHookSystem : public UMovieSceneEntitySystem
{
public:
	GENERATED_BODY()

	MOVIESCENE_API UMovieSceneEvaluationHookSystem(const FObjectInitializer& ObjInit);

	MOVIESCENE_API void AddEvent(UE::MovieScene::FInstanceHandle RootInstance, const FMovieSceneEvaluationHookEvent& InEvent);

	MOVIESCENE_API void SortEvents();

protected:

	MOVIESCENE_API bool HasEvents() const;
	MOVIESCENE_API void TriggerAllEvents();

private:

	MOVIESCENE_API virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	MOVIESCENE_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
	MOVIESCENE_API virtual bool IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const override;

	void UpdateHooks();

	UPROPERTY()
	TMap<FMovieSceneEvaluationInstanceKey, FMovieSceneEvaluationHookEventContainer> PendingEventsByRootInstance;
};
