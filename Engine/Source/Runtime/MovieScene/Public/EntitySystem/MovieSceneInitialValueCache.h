// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/SparseArray.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntitySystemLinkerExtension.h"
#include "EntitySystem/MovieScenePropertySystemTypes.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Optional.h"
#include "Misc/TVariant.h"
#include "MovieSceneCommonHelpers.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "Templates/TypeHash.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectKey.h"

class UObject;

namespace UE
{
namespace MovieScene
{

template<typename PropertyTraits> struct TPropertyValueStorage;

/** Type-safe index that defines a unique index of an initial value within a TPropertyValueStorage instance
 *  This struct is added as a component to all entities with initial values inside non-interrogated linkers */
struct FInitialValueIndex
{
	uint16 Index;
};

/** Base interface for all initial value storage types */
struct IPropertyValueStorage
{
	virtual ~IPropertyValueStorage()= default;

	/** Reset all the initial values for the specified indices */
	virtual void Reset(TArrayView<const FInitialValueIndex> Indices) = 0;
};

/**
 * Container that houses initial values for any properties animated through FEntityManager
 * Each type is stored in its own container, organized by the initial value component ID
 * The cache is stored as a singleton (through GetSharedInitialValues()) and added as an
 * extension to UMovieSceneEntitySystemLinker where it is active.
 */
struct FInitialValueCache
{
	/**
	 * Retrieve the extension ID for this structure when added to a UMovieSceneEntitySystemLinker
	 */
	MOVIESCENE_API static TEntitySystemLinkerExtensionID<FInitialValueCache> GetExtensionID();


	/**
	 * Retrieve a container used for sharing initial values between linkers.
	 * @note: The instance is not referenced internally, so will be destroyed when all
	 * externally owned shared pointers are destroyed.
	 */
	MOVIESCENE_API static TSharedPtr<FInitialValueCache> GetGlobalInitialValues();

public:

	/**
	 * Reset all the initial values that relate to the specified type and indices
	 *
	 * @param InitialValueType      The ComponentTypeID for the initial value that the indices relate to
	 * @param InitialValueIndices   An array containing all the indices to remove
	 */
	void Reset(FComponentTypeID InitialValueType, TArrayView<const FInitialValueIndex> InitialValueIndices)
	{
		if (TUniquePtr<IPropertyValueStorage>* Storage = StorageByComponent.Find(InitialValueType))
		{
			Storage->Get()->Reset(InitialValueIndices);
		}
	}

	/**
	 * Retrieve the initial value storage for a given initial value type, creating it if necessary.
	 * @note: Care should be taken to ensure that the template parameter matches the traits defined
	 * by the TPropertyComponents for the property.
	 *
	 * @param InitialValueType      The ComponentTypeID for the initial value
	 * @return Property storage for the initial values
	 */
	template<typename PropertyTraits>
	TPropertyValueStorage<PropertyTraits>* GetStorage(FComponentTypeID InitialValueType)
	{
		if (TPropertyValueStorage<PropertyTraits>* Storage = FindStorage<PropertyTraits>(InitialValueType))
		{
			return Storage;
		}

		TPropertyValueStorage<PropertyTraits>* NewStorage = new TPropertyValueStorage<PropertyTraits>();
		StorageByComponent.Add(InitialValueType, TUniquePtr<IPropertyValueStorage>(NewStorage));
		return NewStorage;
	}

	/**
	 * Retrieve the initial value storage for a given property type, creating it if necessary
	 *
	 * @param PropertyComponents    The property component definitions for the storage including its traits
	 * @return Property storage for the initial values or nullptr if none exists
	 */
	template<typename PropertyTraits>
	TPropertyValueStorage<PropertyTraits>* GetStorage(const TPropertyComponents<PropertyTraits>& PropertyComponents)
	{
		return GetStorage<PropertyTraits>(PropertyComponents.InitialValueType);
	}


	/**
	 * Retrieve the initial value storage for a given initial value type.
	 * @note: Care should be taken to ensure that the template parameter matches the traits defined
	 * by the TPropertyComponents for the property.
	 *
	 * @param InitialValueType      The ComponentTypeID for the initial value
	 * @return Property storage for the initial values or nullptr if none exists
	 */
	template<typename PropertyTraits>
	TPropertyValueStorage<PropertyTraits>* FindStorage(FComponentTypeID InitialValueType)
	{
		if (TUniquePtr<IPropertyValueStorage>* Existing = StorageByComponent.Find(InitialValueType))
		{
			// If the ptr exists, it cannot be null
			check(Existing->IsValid());
			return static_cast<TPropertyValueStorage<PropertyTraits>*>(Existing->Get());
		}

		return nullptr;
	}


	/**
	 * Retrieve the initial value storage for a given property type.
	 *
	 * @param PropertyComponents    The property component definitions for the storage including its traits
	 * @return Property storage for the initial values or nullptr if none exists
	 */
	template<typename PropertyTraits>
	TPropertyValueStorage<PropertyTraits>* FindStorage(const TPropertyComponents<PropertyTraits>& PropertyComponents)
	{
		return FindStorage<PropertyTraits>(PropertyComponents.InitialValue);
	}

private:

	TMap<FComponentTypeID, TUniquePtr<IPropertyValueStorage>> StorageByComponent;
};

/**
 * Templated storage for any initial value type, templated on the same parameter as TPropertyComponents for correct retrieval of resolved properties
 * Initial values are stored as a sparse array, with stable indices that uniquely identify the value.
 * A look-up-table exists for finding indices based on an object and resolved property.
 */
template<typename PropertyTraits>
struct TPropertyValueStorage : IPropertyValueStorage
{
	using StorageType = typename PropertyTraits::StorageType;

	/**
	 * Reset all the initial values that relate to the specified indices
	 *
	 * @param Indices    A array specifying all of the indices to remove. Such indices will be invalid after this function call returns.
	 */
	virtual void Reset(TArrayView<const FInitialValueIndex> Indices) override
	{
		for (FInitialValueIndex Index : Indices)
		{
			if (PropertyValues.IsValidIndex(Index.Index))
			{
				PropertyValues.RemoveAt(Index.Index);
			}
		}
		bLUTContainsInvalidEntries = true;
	}


	/**
	 * Add a cached value for the specified object and fast property ptr offset, returning a unique index for the value
	 * @note: Value must not have been cached previously - doing so will result in a failed assertion
	 *
	 * @param BoundObject            The object instance to cache the property from
	 * @param InValue                The value to cache
	 * @param ResolvedPropertyOffset The byte offset from BoundObject that defines the address of the property
	 * @return A tuple containing the cached value and its index
	 */
	FInitialValueIndex AddInitialValue(UObject* BoundObject, const StorageType& InValue, uint16 ResolvedPropertyOffset)
	{
		FKeyType Key{ FObjectKey(BoundObject), FPropertyKey(TInPlaceType<uint16>(), ResolvedPropertyOffset) };

		return AddInitialValue(Key, InValue);
	}


	/**
	 * Add a cached value for the specified object and a custom property accessor, returning a unique index for the value
	 * @note: Value must not have been cached previously - doing so will result in a failed assertion
	 *
	 * @param BoundObject            The object instance to cache the property from.
	 * @param InValue                The value to cache
	 * @param AccessorIndex          The index into ICustomPropertyRegistration::GetAccessors to use for resolving the property
	 * @return A tuple containing the cached value and its index
	 */
	FInitialValueIndex AddInitialValue(UObject* BoundObject, const StorageType& InValue, FCustomPropertyIndex AccessorIndex)
	{
		FKeyType Key{ FObjectKey(BoundObject), FPropertyKey(TInPlaceType<FCustomPropertyIndex>(), AccessorIndex) };

		return AddInitialValue(Key, InValue);
	}


	/**
	 * Add a cached value for the specified object and a slow bindings instance, returning a unique index for the value
	 * @note: Value must not have been cached previously - doing so will result in a failed assertion
	 *
	 * @param BoundObject            The object instance to cache the property from.
	 * @param InValue                The value to cache
	 * @param SlowBindings           Pointer to the track instance property bindings object used for retrieving the property value
	 * @return A tuple containing the cached value and its index
	 */
	FInitialValueIndex AddInitialValue(UObject* BoundObject, const StorageType& InValue, FTrackInstancePropertyBindings* SlowBindings)
	{
		FKeyType Key{ FObjectKey(BoundObject), FPropertyKey(TInPlaceType<FName>(), SlowBindings->GetPropertyPath())};

		return AddInitialValue(Key, InValue);
	}


	/**
	 * Add a cached value for the specified object and a propertypath, returning a unique index for the value
	 * @note: Value must not have been cached previously - doing so will result in a failed assertion
	 *
	 * @param BoundObject            The object instance to cache the property from.
	 * @param InValue                The value to cache
	 * @param PropertyPath           The path of the property
	 * @return A tuple containing the cached value and its index
	 */
	FInitialValueIndex AddInitialValue(UObject* BoundObject, const StorageType& InValue, const FName& PropertyPath)
	{
		FKeyType Key{ FObjectKey(BoundObject), FPropertyKey(TInPlaceType<FName>(), PropertyPath)};

		return AddInitialValue(Key, InValue);
	}


	/**
	 * Find an initial value index given its object and fast ptr offset
	 */
	TOptional<FInitialValueIndex> FindPropertyIndex(UObject* BoundObject, uint16 ResolvedPropertyOffset)
	{
		CleanupStaleEntries();
		return FindPropertyIndex(FKeyType{ FObjectKey(BoundObject), FPropertyKey(TInPlaceType<uint16>(), ResolvedPropertyOffset) });
	}


	/**
	 * Find an initial value index given its object and custom accessor index
	 */
	TOptional<FInitialValueIndex> FindPropertyIndex(UObject* BoundObject, FCustomPropertyIndex AccessorIndex)
	{
		CleanupStaleEntries();
		return FindPropertyIndex(FKeyType{ FObjectKey(BoundObject), FPropertyKey(TInPlaceType<FCustomPropertyIndex>(), AccessorIndex) });
	}


	/**
	 * Find an initial value index given its object and property name.
	 * @note: Only properties cached using a FTrackInstancePropertyBindings instance will be retrieved using this method.
	 */
	TOptional<FInitialValueIndex> FindPropertyIndex(UObject* BoundObject, const FName& PropertyPath)
	{
		CleanupStaleEntries();
		return FindPropertyIndex(FKeyType{ FObjectKey(BoundObject), FPropertyKey(TInPlaceType<FName>(), PropertyPath) });
	}


	/**
	 * Find an initial value given its object and property name.
	 */
	const StorageType* FindCachedValue(UObject* BoundObject, uint16 ResolvedPropertyOffset)
	{
		TOptional<FInitialValueIndex> Index = FindPropertyIndex(BoundObject, ResolvedPropertyOffset);
		return Index.IsSet() ? &PropertyValues[Index.GetValue().Index] : nullptr;
	}


	/**
	 * Find an initial value given its object and custom accessor index
	 */
	const StorageType* FindCachedValue(UObject* BoundObject, FCustomPropertyIndex CustomIndex)
	{
		TOptional<FInitialValueIndex> Index = FindPropertyIndex(BoundObject, CustomIndex);
		return Index.IsSet() ? &PropertyValues[Index.GetValue().Index] : nullptr;
	}


	/**
	 * Find an initial value given its object and property name.
	 * @note: Only properties cached using a FTrackInstancePropertyBindings instance will be retrieved using this method.
	 */
	const StorageType* FindCachedValue(UObject* BoundObject, const FName& PropertyPath)
	{
		TOptional<FInitialValueIndex> Index = FindPropertyIndex(BoundObject, PropertyPath);
		return Index.IsSet() ? &PropertyValues[Index.GetValue().Index] : nullptr;
	}

	/**
	 * Find an initial value given its object and property name.
	 * @note: Only properties cached using a FTrackInstancePropertyBindings instance will be retrieved using this method.
	 */
	const StorageType& GetCachedValue(FInitialValueIndex Index)
	{
		return PropertyValues[Index.Index];
	}

private:

	using FPropertyKey = TVariant<uint16, FCustomPropertyIndex, FName>;

	struct FKeyType
	{
		FObjectKey Object;
		FPropertyKey Property;

		friend uint32 GetTypeHash(const FKeyType& InKey)
		{
			// Hash only considers the _type_ of the resolved property
			// which defers the final comparison to the equality operator
			uint32 Hash = GetTypeHash(InKey.Object);
			Hash = HashCombine(Hash, static_cast<uint32>(InKey.Property.GetIndex()));
			return Hash;
		}
		friend bool operator==(const FKeyType& A, const FKeyType& B)
		{
			if (A.Object != B.Object || A.Property.GetIndex() != B.Property.GetIndex())
			{
				return false;
			}
			switch (A.Property.GetIndex())
			{
				case 0: return A.Property.template Get<uint16>() == B.Property.template Get<uint16>();
				case 1: return A.Property.template Get<FCustomPropertyIndex>().Value == B.Property.template Get<FCustomPropertyIndex>().Value;
				case 2: return A.Property.template Get<FName>() == B.Property.template Get<FName>();
			}
			return true;
		}
	};

	inline void CleanupStaleEntries()
	{
		if (!bLUTContainsInvalidEntries)
		{
			return;
		}
		for (auto It = KeyToPropertyIndex.CreateIterator(); It; ++It)
		{
			if (!PropertyValues.IsValidIndex(It.Value()))
			{
				It.RemoveCurrent();
			}
		}
		PropertyValues.Shrink();
		bLUTContainsInvalidEntries = false;
	}

	TOptional<FInitialValueIndex> FindPropertyIndex(const FKeyType& InKey)
	{
		CleanupStaleEntries();

		const uint16* Index = KeyToPropertyIndex.Find(InKey);
		return Index ? TOptional<FInitialValueIndex>(FInitialValueIndex{*Index}) : TOptional<FInitialValueIndex>();
	}

	FInitialValueIndex AddInitialValue(const FKeyType& InKey, const StorageType& InValue)
	{
		checkSlow(!KeyToPropertyIndex.Contains(InKey));

		const int32 NewIndex = PropertyValues.Add(InValue);
		check(NewIndex < int32(uint16(0xFFFF)));

		const uint16 NarrowIndex = static_cast<uint16>(NewIndex);
		KeyToPropertyIndex.Add(InKey, NarrowIndex);

		return FInitialValueIndex{NarrowIndex};
	}

	/** Sparse array containing all cached property values */
	TSparseArray<StorageType> PropertyValues;
	/** LUT from object+property to its index. May contain stale values if bLUTContainsInvalidEntries is true */
	TMap<FKeyType, uint16> KeyToPropertyIndex;
	/** When true, KeyToPropertyIndex contains invalid entries which must be purged before use */
	bool bLUTContainsInvalidEntries = false;
};


} // namespace MovieScene
} // namespace UE

