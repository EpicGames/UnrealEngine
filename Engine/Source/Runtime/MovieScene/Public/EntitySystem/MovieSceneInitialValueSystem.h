// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntityInstantiatorSystem.h"
#include "MovieSceneInitialValueSystem.generated.h"

namespace UE::MovieScene
{
	struct FComponentTypeID;
	struct IInitialValueProcessor;
	struct FEntityComponentFilter;
}


/**
 * System responsible for initializing initial values for all property types
 * Will handle the presence of an FInitialValueCache extension on the linker
 */
UCLASS(MinimalAPI)
class UMovieSceneInitialValueSystem
	: public UMovieSceneEntityInstantiatorSystem
{
public:

	GENERATED_BODY()

	MOVIESCENE_API UMovieSceneInitialValueSystem(const FObjectInitializer& ObjInit);

	MOVIESCENE_API static void RegisterProcessor(
		UE::MovieScene::FComponentTypeID InitialValueComponent,
		TSharedPtr<UE::MovieScene::IInitialValueProcessor> Processor,
		UE::MovieScene::FEntityComponentFilter&& OptionalFilter = UE::MovieScene::FEntityComponentFilter());

private:

	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
	virtual bool IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const override;
	virtual void OnLink() override;
	virtual void OnUnlink() override;
};
