// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Map.h"
#include "Containers/SortedMap.h"
#include "Containers/SparseArray.h"
#include "Evaluation/MovieScenePlayback.h"
#include "EntitySystem/MovieSceneSequenceInstance.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "EntitySystem/MovieSceneComponentDebug.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "MovieSceneSequenceID.h"

#define UE_API MOVIESCENE_API


class UMovieSceneCompiledDataManager;
class UMovieSceneEntitySystemLinker;
class UMovieSceneCompiledDataManager;

namespace UE
{
namespace MovieScene
{

class FEntityManager;

struct FEntityRange;
struct FInstanceRegistry;
struct FResolveObjectTask;
struct FBoundObjectInstances;



/** Core concept that is required by all entities and systems - should be located on the core system manager itself */
struct FInstanceRegistry
{
	MOVIESCENE_API FInstanceRegistry(UMovieSceneEntitySystemLinker* InLinker);

	MOVIESCENE_API ~FInstanceRegistry();

	FInstanceRegistry(const FInstanceRegistry&) = delete;
	void operator=(const FInstanceRegistry&) = delete;

	const TSparseArray<FSequenceInstance>& GetSparseInstances() const
	{
		return Instances;
	}

	UMovieSceneEntitySystemLinker* GetLinker() const
	{
		return Linker;
	}

	bool IsHandleValid(FInstanceHandle InstanceHandle) const
	{
		return Instances.IsValidIndex(InstanceHandle.InstanceID) && Instances[InstanceHandle.InstanceID].GetSerialNumber() == InstanceHandle.InstanceSerial;
	}

	const FSequenceInstance& GetInstance(FInstanceHandle InstanceHandle) const
	{
		checkfSlow(IsHandleValid(InstanceHandle), TEXT("Attempting to access an invalid instance handle."));
		return Instances[InstanceHandle.InstanceID];
	}

	FSequenceInstance& MutateInstance(FInstanceHandle InstanceHandle)
	{
		checkfSlow(IsHandleValid(InstanceHandle), TEXT("Attempting to access an invalid instance handle."));
		return Instances[InstanceHandle.InstanceID];
	}

	/** 
	 * Finds a (sub)sequence with the given ID somewhere in the hierarchy of the given reference instance
	 *
	 * This method will look up the hierarchy root from the given instance handle, and then look for the appropriate sub-instance with the
	 * given sequence ID.
	 *
	 * @param InstanceHandle  The reference instance handle used to locate the relevant sequence hierarchy to explore. Does not have to be a root instance.
	 * @param SequenceID      An absolute (accumulated) sequence ID to look for in the hierarchy.
	 */
	MOVIESCENE_API FInstanceHandle FindRelatedInstanceHandle(FInstanceHandle InstanceHandle, FMovieSceneSequenceID SequenceID) const;

	const FMovieSceneContext& GetContext(FInstanceHandle InstanceHandle) const
	{
		return GetInstance(InstanceHandle).GetContext();
	}

	MOVIESCENE_API FRootInstanceHandle AllocateRootInstance(
			UMovieSceneSequence& InRootSequence,
			UObject* InPlaybackContext = nullptr,
			UMovieSceneCompiledDataManager* InCompiledDataManager = nullptr);

	MOVIESCENE_API FInstanceHandle AllocateSubInstance(
			FMovieSceneSequenceID SequenceID, FRootInstanceHandle RootInstance, FInstanceHandle ParentInstanceHandle);

	MOVIESCENE_API void DestroyInstance(FInstanceHandle InstanceHandle);

	MOVIESCENE_API void CleanupLinkerEntities(const TSet<FMovieSceneEntityID>& LinkerEntities);

	void InvalidateObjectBinding(const FGuid& ObjectBindingID, FInstanceHandle InstanceHandle)
	{
		InvalidatedObjectBindings.Add(MakeTuple(ObjectBindingID, InstanceHandle));
	}

	bool IsBindingInvalidated(const FGuid& ObjectBindingID, FInstanceHandle InstanceHandle) const
	{
		// The binding is invalidated if it is contained within the invalid set, or if an empty GUID with the same instance handle exists (implying _all_ bindings are invalidated for that instance handle)
		return InvalidatedObjectBindings.Contains(MakeTuple(ObjectBindingID, InstanceHandle)) || InvalidatedObjectBindings.Contains(MakeTuple(FGuid(), InstanceHandle));
	}

	bool HasInvalidatedBindings() const
	{
		return InvalidatedObjectBindings.Num() != 0;
	}

	void PostInstantation();

	void TagGarbage();

private:

	UMovieSceneEntitySystemLinker* Linker;

	/** Authoritate array of unique instance combinations */
	TSparseArray<FSequenceInstance> Instances;
	uint16 InstanceSerialNumber;

	/** Set of invalidated object bindings by their instance handle. Empty guids indicate that all bindings for that instance handle are invalid */
	TSet<TTuple<FGuid, FInstanceHandle>> InvalidatedObjectBindings;
};


/**
 * Defines a scope during which a given sequence hierarchy will not be recompiled when changed
 * even if it was set to volatile.
 */
struct FScopedVolatilityManagerSuppression
{
	UE_API FScopedVolatilityManagerSuppression(TSharedPtr<FSharedPlaybackState> PlaybackState);
	UE_API ~FScopedVolatilityManagerSuppression();

private:
	TWeakPtr<FSharedPlaybackState> WeakPlaybackState;
	TUniquePtr<FCompiledDataVolatilityManager> PreviousVolatilityManager;
};

} // namespace MovieScene
} // namespace UE

#undef UE_API
