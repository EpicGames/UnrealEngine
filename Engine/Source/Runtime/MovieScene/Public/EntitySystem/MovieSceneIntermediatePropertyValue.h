// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/InlineValue.h"
#include "UObject/UnrealType.h"


/*~
 * These classes provide a way to pass around type-erased values for properties within Sequencer
 * Two types are provided to distinguish between values intended as 'source' data (ie, actual property data on a UObject),
 *   and intermediate data used by Sequencer for computational purposes at compile time. In this way it is possible to 
 *   write generic property code and APIs without needing to know the specifics of the type being represented.
 */

namespace UE::MovieScene
{

struct FSourcePropertyValue;
struct FIntermediatePropertyValue;
struct FIntermediatePropertyValueConstRef;

namespace Private
{
	/*
	 * Base class for all type-erased intermediate and source value types
	 */
	struct ITypeErasedPropertyConstValueImpl
	{
		int32 SizeofT = 0;

		virtual ~ITypeErasedPropertyConstValueImpl() = default;

		virtual const void* Get() const = 0;
	};


	/*
	 * Base class for all value-storage type-erased intermediate and source value types
	 */
	struct ITypeErasedPropertyValueImpl : ITypeErasedPropertyConstValueImpl
	{
		virtual ~ITypeErasedPropertyValueImpl() = default;

		virtual TInlineValue<ITypeErasedPropertyValueImpl> Copy() const = 0;
	};


	/*
	 * Class that holds a void* pointer to a value and is able to return it
	 */
	struct FTypeErasedConstPropertyPtrBase : ITypeErasedPropertyConstValueImpl
	{
		FTypeErasedConstPropertyPtrBase(const void* In, int32 InSize)
			: Value(In)
		{
			this->SizeofT = InSize;
		}
		const void* Get() const override
		{
			return Value;
		}
	protected:
		const void* Value;
	};

	/*
	 * Class that holds a typed value and is able to return and copy it
	 */
	template<typename T>
	struct TTypeErasedPropertyValueImpl : ITypeErasedPropertyValueImpl
	{
		template<typename U>
		TTypeErasedPropertyValueImpl(U&& In)
			: Value(Forward<U>(In))
		{
			this->SizeofT = sizeof(U);
		}
		TInlineValue<ITypeErasedPropertyValueImpl> Copy() const override
		{
			return TInlineValue<ITypeErasedPropertyValueImpl>(*this);
		}
		const void* Get() const override
		{
			return &Value;
		}
	private:
		T Value;
	};

	/*
	 * Class that holds a pointer to a value and is able to return and copy it
	 */
	template<typename T>
	struct TTypeErasedPropertyPtrImpl : ITypeErasedPropertyValueImpl
	{
		TTypeErasedPropertyPtrImpl(T* In)
			: Value(In)
		{
			this->SizeofT = sizeof(T);
		}
		TInlineValue<ITypeErasedPropertyValueImpl> Copy() const override
		{
			return TInlineValue<ITypeErasedPropertyValueImpl>(
				TTypeErasedPropertyValueImpl<T>(*static_cast<T*>(Value))
			);
		}
		const void* Get() const override
		{
			return Value;
		}
	protected:
		T* Value;
	};

} // namespace Private




/**
 * Provides a way of wrapping UObject property values to generic Sequencer APIs in a way that
 * doesn't need to know the type of the underlying data. This is useful for APIs implemented
 * as polymorphic interfaces or as ADL templates structures to simplify lifetime semantics and type-safety.
 */
struct FSourcePropertyValue
{
	/*
	 * Default constructor
	 */
	FSourcePropertyValue()
	{
	}

	/*
	 * Move construction/assignment
	 */
	FSourcePropertyValue(FSourcePropertyValue&&) = default;
	FSourcePropertyValue& operator=(FSourcePropertyValue&&) = default;

	/*
	 * Check if this value is assigned a valid value
	 * @return true if this value is valid, false otherwise
	 */
	explicit operator bool() const
	{
		return Value.IsValid();
	}

	/*
	 * Create a new FSourcePropertyValue from a memory address and a reflected property that corresponds to the address
	 *
	 * @param Ptr       The value pointer to create this source value from
	 * @param Property  The property that corresponds to the memory addres
	 * @return A new source value
	 */
	static FSourcePropertyValue FromAddress(const void* Ptr, const FProperty& Property)
	{
		return FSourcePropertyValue {
			TInlineValue<Private::ITypeErasedPropertyConstValueImpl>(
				Private::FTypeErasedConstPropertyPtrBase(Ptr, Property.GetSize())
			)
		};
	}


	/*
	 * Create a new FSourcePropertyValue from a typed value
	 *
	 * @param Value     The value to create this source value from
	 * @return A new source value
	 */
	template<typename T>
	static FSourcePropertyValue FromValue(T&& Value)
	{
		return FSourcePropertyValue{
			TInlineValue<Private::ITypeErasedPropertyConstValueImpl>(
				Private::TTypeErasedPropertyValueImpl<typename TDecay<T>::Type>(Forward<T>(Value))
			)
		};
	}

	/*
	 * Retrieve the memory address of the wrapped source value
	 */
	const void* Get() const
	{
		return Value.IsValid() ? Value->Get() : nullptr;
	}


	/*
	 * Cast this source value to another type, assuming the underlying type matches.
	 * @note: Only crude type-checking is performed here: it is the callee's responsibility to ensure the cast is valid
	 */
	template<typename T>
	const T* Cast() const
	{
		const T* Address = static_cast<const T*>(Get());

		check(!Address || Value->SizeofT == sizeof(T));
		return Address;
	}

private:

	FSourcePropertyValue(TInlineValue<Private::ITypeErasedPropertyConstValueImpl>&& InValue)
		: Value(MoveTemp(InValue))
	{
	}

	TInlineValue<Private::ITypeErasedPropertyConstValueImpl> Value;
};




/**
 * Provides a way of wrapping an intermediate property value reference in an abstract way, similar to FSourcePropertyValue,
 * but exclusively reserved for use with intermeidate types within Sequencer's internal computation algorithms to provide
 * compile-time distinction between the two types.
 */
struct FIntermediatePropertyValueConstRef
{
	/**
	 * Construction from a pointer to a value
	 */
	template<typename T, typename = std::enable_if_t<!std::is_same_v<T, void>>> // Disabled for void*
	FIntermediatePropertyValueConstRef(const T* Ptr)
		: Value(Private::TTypeErasedPropertyPtrImpl<const T>(Ptr))
	{
	}


	/**
	 * Construction from a value
	 */
	template<typename T>
	FIntermediatePropertyValueConstRef(T&& InValue)
		: Value(Private::TTypeErasedPropertyValueImpl<T>(Forward<T>(InValue)))
	{
	}


	/**
	 * Move construction/assignment
	 */
	FIntermediatePropertyValueConstRef(FIntermediatePropertyValueConstRef&&) = default;
	FIntermediatePropertyValueConstRef& operator=(FIntermediatePropertyValueConstRef&&) = default;


	/**
	 * Implicit copy construction and assignment is disabled (use Copy() instead)
	 */
	FIntermediatePropertyValueConstRef(const FIntermediatePropertyValueConstRef&) = delete;
	FIntermediatePropertyValueConstRef& operator=(const FIntermediatePropertyValueConstRef&) = delete;


	/**
	 * Copy this value into a new instance.
	 */
	FIntermediatePropertyValue Copy() const;


	/*
	 * Retrieve the address of this value
	 */
	const void* Get() const
	{
		return Value->Get();
	}


	/*
	 * Cast this value to another type, assuming the underlying type matches.
	 * @note: Only crude type-checking is performed here: it is the callee's responsibility to ensure the cast is valid
	 */
	template<typename T>
	const T* Cast() const
	{
		check(Value->SizeofT == sizeof(T));

		const T* Address = static_cast<const T*>(Get());
		return Address;
	}

protected:

	FIntermediatePropertyValueConstRef()
	{
	}

	FIntermediatePropertyValueConstRef(TInlineValue<Private::ITypeErasedPropertyValueImpl>&& InValue)
		: Value(MoveTemp(InValue))
	{
	}

	TInlineValue<Private::ITypeErasedPropertyValueImpl> Value;
};




/**
 * Same as FIntermediatePropertyValueConstRef but instead of wrapping a reference to a value, this class can wrap an instance of a value or a reference.
 */
struct FIntermediatePropertyValue : FIntermediatePropertyValueConstRef
{
	/**
	 * Move construction/assignment
	 */
	FIntermediatePropertyValue(FIntermediatePropertyValue&&) = default;
	FIntermediatePropertyValue& operator=(FIntermediatePropertyValue&&) = default;

	/**
	 * Implicit copy construction and assignment is disabled (use Copy() instead)
	 */
	FIntermediatePropertyValue(const FIntermediatePropertyValue&) = delete;
	FIntermediatePropertyValue& operator=(const FIntermediatePropertyValue&) = delete;


	/*
	 * Create a new FIntermediatePropertyValue from a typed value
	 *
	 * @param Value     The value to create this source value from
	 * @return A new intermediate value
	 */
	template<typename T>
	static FIntermediatePropertyValue FromValue(T&& In)
	{
		return FIntermediatePropertyValue {
			TInlineValue<Private::ITypeErasedPropertyValueImpl>(
				Private::TTypeErasedPropertyValueImpl<typename TDecay<T>::Type>(Forward<T>(In))
			)
		};
	}


	/*
	 * Create a new FIntermediatePropertyValue from an address to a value
	 *
	 * @param Ptr       The address of the value to wrap
	 * @return A new intermediate value
	 */
	template<typename T, typename = std::enable_if_t<!std::is_same_v<T, void>>> // Disabled for void*
	static FIntermediatePropertyValue FromAddress(T* Ptr)
	{
		return FIntermediatePropertyValue {
			TInlineValue<Private::ITypeErasedPropertyValueImpl>(
				Private::TTypeErasedPropertyPtrImpl<T>(Ptr)
			)
		};
	}


	/*
	 * Create a copy of this value
	 */
	FIntermediatePropertyValue Copy() const
	{
		return FIntermediatePropertyValue(Value->Copy());
	}

	// elevate const overloads
	using FIntermediatePropertyValueConstRef::Get;
	using FIntermediatePropertyValueConstRef::Cast;


	/*
	 * Retrieve this value's underlying address
	 */
	void* Get()
	{
		return const_cast<void*>(Value->Get());
	}


	/*
	 * Cast this value to another type, assuming the underlying type matches.
	 * @note: Only crude type-checking is performed here: it is the callee's responsibility to ensure the cast is valid
	 */
	template<typename T>
	T* Cast()
	{
		check(Value->SizeofT == sizeof(T));

		T* Address = static_cast<T*>(Get());
		return Address;
	}

protected:
	friend FIntermediatePropertyValueConstRef;

	FIntermediatePropertyValue(TInlineValue<Private::ITypeErasedPropertyValueImpl>&& InValue)
		: FIntermediatePropertyValueConstRef(MoveTemp(InValue))
	{
	}
};

inline FIntermediatePropertyValue FIntermediatePropertyValueConstRef::Copy() const
{
	return FIntermediatePropertyValue(Value->Copy());
}

} // namespace UE::MovieScene


