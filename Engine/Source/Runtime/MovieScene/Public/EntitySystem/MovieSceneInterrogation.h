// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "UObject/NameTypes.h"
#include "Containers/SortedMap.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "EntitySystem/MovieScenePropertyBinding.h"
#include "EntitySystem/MovieSceneEntitySystemLinkerExtension.h"

struct FMovieSceneSequenceHierarchy;

namespace UE::MovieScene
{


struct FInterrogationChannelInfo
{
	/** The object that relates to the channel */
	TWeakObjectPtr<UObject> WeakObject;

	/** (Optional) property path for this channel */
	FMovieScenePropertyBinding PropertyBinding;

	/** The channel's hierarchical 'parent' - generally only used for transforms */
	FInterrogationChannel ParentChannel;
};


struct FInterrogationUpdateParams
{

	/** The channel's hierarchical 'parent' - generally only used for transforms */
	FInterrogationChannel ParentChannel;
};

struct FSparseInterrogationChannelInfo
{
	FInterrogationChannelInfo& Get(FInterrogationChannel Channel) { return ChannelInfo.FindOrAdd(Channel); }
	const FInterrogationChannelInfo& Get(FInterrogationChannel Channel) const { return ChannelInfo.FindChecked(Channel); }

	FInterrogationChannelInfo* Find(FInterrogationChannel Channel) { return ChannelInfo.Find(Channel); }
	const FInterrogationChannelInfo* Find(FInterrogationChannel Channel) const { return ChannelInfo.Find(Channel); }

	UObject* FindObject(FInterrogationChannel Channel) const
	{
		return ChannelInfo.FindRef(Channel).WeakObject.Get();
	}

	FInterrogationChannel FindParent(FInterrogationChannel Channel) const
	{
		return ChannelInfo.FindRef(Channel).ParentChannel;
	}

	void Empty()
	{
		ChannelInfo.Empty();
	}

private:

	TSortedMap<FInterrogationChannel, FInterrogationChannelInfo> ChannelInfo;
};

struct IInterrogationExtension
{
public:
	MOVIESCENE_API static TEntitySystemLinkerExtensionID<IInterrogationExtension> GetExtensionID();

	virtual const FSparseInterrogationChannelInfo& GetSparseChannelInfo() const = 0;

	virtual const FMovieSceneSequenceHierarchy* GetHierarchy() const = 0;

protected:

	virtual ~IInterrogationExtension() = default;
};

} // namespace UE::MovieScene
