// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformAtomics.h"
#include "Templates/UnrealTemplate.h"

namespace UE
{
namespace MovieScene
{

/**
 * Enumeration that defines a threading model for this entity manager
 */
enum class EEntityThreadingModel : uint8
{
	/** Specified when the data contained within an entity manager does not satisfy the requirements to justify using threaded evaluation */
	NoThreading,

	/** Specified when the data contained within an entity manager is large or complex enough to justify threaded evaluation  */
	TaskGraph,
};

/**
 * A potentially atomic struct that will perform atomic operations if
 * required on an underlying integer, depending on what
 * `EEntityThreadingModel` was specified.
 */
struct FEntitySystemMaybeAtomicInt32 final
{
	FEntitySystemMaybeAtomicInt32() = default;
	FEntitySystemMaybeAtomicInt32(int32 Payload) : Payload(Payload) {}

	int32 Load(EEntityThreadingModel ThreadingModel) const
	{
		if (EEntityThreadingModel::NoThreading == ThreadingModel)
		{
			return Payload;
		}
		else
		{
			return FPlatformAtomics::AtomicRead(&Payload);
		}
	}

	int32 Add(EEntityThreadingModel ThreadingModel, const int32 Value)
	{
		if (EEntityThreadingModel::NoThreading == ThreadingModel)
		{
			const int32 Old = Payload;
			Payload += Value;
			return Old;
		}
		else
		{
			return FPlatformAtomics::InterlockedAdd(&Payload, Value);
		}
	}

	int32 Sub(EEntityThreadingModel ThreadingModel, const int32 Value)
	{
		if (EEntityThreadingModel::NoThreading == ThreadingModel)
		{
			const int32 Old = Payload;
			Payload -= Value;
			return Old;
		}
		else
		{
			return FPlatformAtomics::InterlockedAdd(&Payload, -Value);
		}
	}

	int32 Exchange(EEntityThreadingModel ThreadingModel, int32 Other)
	{
		if (EEntityThreadingModel::NoThreading == ThreadingModel)
		{
			Swap(Payload, Other);
			return Other;
		}
		else
		{
			return FPlatformAtomics::InterlockedExchange(&Payload, Other);
		}
	}

	int32 Increment(EEntityThreadingModel ThreadingModel)
	{
		if (EEntityThreadingModel::NoThreading == ThreadingModel)
		{
			return Payload++;
		}
		else
		{
			return FPlatformAtomics::InterlockedIncrement(&Payload);
		}
	}

	int32 Decrement(EEntityThreadingModel ThreadingModel)
	{
		if (EEntityThreadingModel::NoThreading == ThreadingModel)
		{
			return Payload--;
		}
		else
		{
			return FPlatformAtomics::InterlockedDecrement(&Payload);
		}
	}

private:
	int32 Payload = 0;
};

} // namespace MovieScene
} // namespace UE
