// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "EntitySystem/MovieSceneMutualComponentInitializer.h"
#include "EntitySystem/MovieSceneEntitySystemDirectedGraph.h"

#include <initializer_list>

namespace UE::MovieScene
{

struct FMutualInclusivityGraph;
struct IMutualComponentInitializer;

/** Enumeration specifying how to apply a complex filter for mutual inclusion */
enum class EComplexInclusivityFilterMode
{
	/** Specifies that all the components must be present for a mutal component to be included */
	AllOf,
	/** Specifies that the mutal component should be included if any of the specified components are present */
	AnyOf
};

struct FMutuallyInclusiveComponentParams
{
	FMutuallyInclusiveComponentParams()
	{}

	/** Custom initializer for this included component */
	TUniquePtr<IMutualComponentInitializer> CustomInitializer;

	/** Optional by default because generally we do not want mutual components on Imported entities */
	EMutuallyInclusiveComponentType Type = EMutuallyInclusiveComponentType::Optional;
};


/** Filter used for specifying more complex mutual inclusivity rules that depend on more than one component type */
struct FComplexInclusivityFilter
{
	/** The component mask defining the set of components to test for */
	FComponentMask Mask;
	/** The filter mode used for matching */
	EComplexInclusivityFilterMode Mode;

	FComplexInclusivityFilter(const FComponentMask& InMask, EComplexInclusivityFilterMode InMode)
		: Mask(InMask), Mode(InMode)
	{}

	friend uint32 GetTypeHash(const FComplexInclusivityFilter& Filter)
	{
		return HashCombine(GetTypeHash(Filter.Mask), GetTypeHash(Filter.Mode));
	}

	friend bool operator==(const FComplexInclusivityFilter& A, const FComplexInclusivityFilter& B)
	{
		return A.Mask == B.Mask && A.Mode == B.Mode;
	}

	static FComplexInclusivityFilter All(std::initializer_list<FComponentTypeID> InComponentTypes)
	{
		return FComplexInclusivityFilter(FComponentMask(InComponentTypes), EComplexInclusivityFilterMode::AllOf);
	}

	static FComplexInclusivityFilter Any(std::initializer_list<FComponentTypeID> InComponentTypes)
	{
		return FComplexInclusivityFilter(FComponentMask(InComponentTypes), EComplexInclusivityFilterMode::AnyOf);
	}

	UE_DEPRECATED(5.2, "This function will no longer be called")
	bool Match(FComponentMask Input) const
	{
		switch (Mode)
		{
			case EComplexInclusivityFilterMode::AllOf:
				{
					FComponentMask Temp = Mask;
					Temp.CombineWithBitwiseAND(Input, EBitwiseOperatorFlags::MaintainSize);
					return Temp == Mask;
				}
				break;
			case EComplexInclusivityFilterMode::AnyOf:
				{
					FComponentMask Temp = Mask;
					Temp.CombineWithBitwiseAND(Input, EBitwiseOperatorFlags::MaintainSize);
					return Temp.Find(true) != INDEX_NONE;
				}
				break;
			default:
				checkf(false, TEXT("Not implemented"));
				return false;
		}
	}
};


/**
 * Container for keeping track of which initializers need to be run for a given call to
 * FMutualInclusivityGraphCommandBuffer::ComputeMutuallyInclusiveComponents
 */
struct FMutualComponentInitializers
{
	/**
	 * Run the mutual initializers over the specified entity range
	 */
	MOVIESCENE_API void Execute(const FEntityRange& Range, const FEntityAllocationWriteContext& WriteContext) const;

	/**
	 * Add a new initializer to this cache
	 */
	void Add(IMutualComponentInitializer* InitializerToAdd);

	/**
	 * Reset this container. Will not free memory.
	 */
	void Reset();

private:

	TArray<IMutualComponentInitializer*, TInlineAllocator<2>> Initializers;
};


/**
 * Command buffer used for computing mutually inclusive components based on a complex set of rules and dependencies
 */
struct FMutualInclusivityGraphCommandBuffer
{
	/**
	 * Populate a component mask with all the necessary mutually inclusive components given an input type.
	 * Mutual components that already existed in InMask will not be added or counted.
	 * Can be used to populate a new mask with only the mutual components, or to add the components to an existing mask.
	 *
	 * @param InMask           The input component type. Rules that are relevant to this type (directly or indirectly) will be applied to the output.
	 * @param TypesToCompute   Allows computation of mandatory, optional or all rules
	 * @param OutMask          Component mask to receieve the result. May point to the same mask as InMask in order to add the mutual components directly.
	 * @param OutInitializers  Array to receieve initializers that must be called on the resulting entity range(s)
	 * @return The number of new components added to OutMask. Only components that did not exist before are considered.
	 */
	int32 ComputeMutuallyInclusiveComponents(EMutuallyInclusiveComponentType TypesToCompute, const FComponentMask& InMask, FComponentMask& OutMask, FMutualComponentInitializers& OutInitializers) const;
	
	/**
	 * Check this buffer's invariants are satisfied
	 */
	void CheckInvariants() const;

private:

	// Only FMutualInclusivityGraph can populate this command buffer
	friend FMutualInclusivityGraph;

	enum class ECommandType
	{
		Simple,
		Type,
		MatchAny,
		MatchAll,
		ShortCircuit,
		Include,
		Initialize,
	};

	struct FSimpleCommand
	{
		FComponentTypeID MatchID;
		FComponentTypeID IncludeID;
	};
	struct FTypeCommand
	{
		/** Array index within FMutualInclusivityGraphCommandBuffer::Commands to jump to if this command fails */
		uint16 ShortCircuitIndex;
		/** The type to match */
		EMutuallyInclusiveComponentType Type;
	};

	struct FMatchCommand
	{
		/** The ID of the component we are to match against, or include (depending on the value of CommandType) */
		FComponentTypeID ComponentTypeID;
		/** Array index within FMutualInclusivityGraphCommandBuffer::Commands to jump to if this command fails */
		uint16 ShortCircuitIndex;
	};

	struct FShortCircuitCommand
	{
		/** Array index within FMutualInclusivityGraphCommandBuffer::Commands to jump to */
		uint16 ShortCircuitIndex;
	};

	struct FIncludeCommand
	{
		/** The ID of the component we are to match against, or include (depending on the value of CommandType) */
		FComponentTypeID ComponentTypeID;
	};

	struct FInitializeCommand
	{
		/** Pointer to an initializer to use */
		IMutualComponentInitializer* Initializer;
	};

	/** A single command within a command buffer (FMutualInclusivityGraphCommandBuffer) that either matches or includes a component type ID */
	struct FCommand
	{
		FCommand(FSimpleCommand InSimple)                                   : Simple(InSimple),             CommandType(ECommandType::Simple)       {}
		FCommand(FTypeCommand InType)                                       : Type(InType),                 CommandType(ECommandType::Type)         {}
		FCommand(FMatchCommand InMatch, EComplexInclusivityFilterMode Mode) : Match(InMatch),               CommandType(Mode == EComplexInclusivityFilterMode::AllOf ? ECommandType::MatchAll : ECommandType::MatchAny)   {}
		FCommand(FShortCircuitCommand InShortCircuit)                       : ShortCircuit(InShortCircuit), CommandType(ECommandType::ShortCircuit) {}
		FCommand(FIncludeCommand InInclude)                                 : Include(InInclude),           CommandType(ECommandType::Include)      {}
		FCommand(FInitializeCommand InInitialize)                           : Initialize(InInitialize),     CommandType(ECommandType::Initialize)   {}

		union
		{
			FTypeCommand         Type;
			FSimpleCommand       Simple;
			FMatchCommand        Match;
			FShortCircuitCommand ShortCircuit;
			FIncludeCommand      Include;
			FInitializeCommand   Initialize;
		};

		/** The type of the command: either a match or include */
		ECommandType CommandType;
	};

	/** Ordered command buffer specifying matches and includes */
	TArray<FCommand> Commands;
};


/**
 * A mutual inclusion graph for adding components to an entity based on the presence of other components
 */
struct FMutualInclusivityGraph
{
	/**
	 * Define a new rule specifying that all of Dependents should always exist if Predicate exists
	 */
	void DefineMutualInclusionRule(FComponentTypeID Predicate, std::initializer_list<FComponentTypeID> Dependents);
	void DefineMutualInclusionRule(FComponentTypeID Predicate, std::initializer_list<FComponentTypeID> Dependents, FMutuallyInclusiveComponentParams&& InParams);

	/**
	 * Define a new complex rule specifying that all of Dependents should exist if InFilter is matched
	 */
	void DefineComplexInclusionRule(const FComplexInclusivityFilter& InFilter, std::initializer_list<FComponentTypeID> Dependents);
	void DefineComplexInclusionRule(const FComplexInclusivityFilter& InFilter, std::initializer_list<FComponentTypeID> Dependents, FMutuallyInclusiveComponentParams&& InParams);

public:

	/**
	 * Populate a component mask with all the necessary mutually inclusive components given an input type.
	 * Mutual components that already existed in InMask will not be added or counted.
	 * Can be used to populate a new mask with only the mutual components, or to add the components to an existing mask.
	 *
	 * @param InMask           The input component type. Rules that are relevant to this type (directly or indirectly) will be applied to the output.
	 * @param OutMask          Component mask to receieve the result. May point to the same mask as InMask in order to add the mutual components directly.
	 * @param OutInitializers  Initializers that must be run for any new entities created from OutMask
	 * @return The number of new components added to OutMask. Only components that did not exist before are considered.
	 */
	int32 ComputeMutuallyInclusiveComponents(EMutuallyInclusiveComponentType TypesToCompute, const FComponentMask& InMask, FComponentMask& OutMask, FMutualComponentInitializers& OutInitializers) const;

private:

	/**
	 * Follows the longest chain in the graph starting from Component
	 */
	static int32 FindPrerequisiteChainLength(const FDirectedGraph& Graph, FComponentTypeID Component, TMap<FComponentTypeID, int32>& InOutCache);

	/**
	 * Reconstructs the command buffer
	 */
	void ReconstructCommandBuffer() const;

private:

	struct FIncludes
	{
		void Add(FComponentTypeID Component)
		{
			Includes.AddUnique(Component);
		}
		TArray<FComponentTypeID, TInlineAllocator<1>> Includes;
		TArray<TUniquePtr<IMutualComponentInitializer>> Initializers;
	};
	struct FIncludePair
	{
		FIncludes MandatoryIncludes;
		FIncludes OptionalIncludes;
	};

	/** 1->many simple include relationships */
	TMap<FComponentTypeID, FIncludePair> SimpleIncludes;
	/** many->many complex include relationships */
	TMap<FComplexInclusivityFilter, FIncludePair> ComplexIncludes;

	/** Mutable command buffer that is repopulated by ReconstructCommandBuffer */
	mutable FMutualInclusivityGraphCommandBuffer CommandBuffer;
	/** Flag indicating whether our command buffer needs re-generating or not */
	mutable bool bCommandBufferInvalidated = false;
};


} // namespace UE::MovieScene