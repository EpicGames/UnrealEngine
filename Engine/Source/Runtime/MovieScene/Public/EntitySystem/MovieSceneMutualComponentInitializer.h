// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE::MovieScene
{

struct FEntityRange;
struct FEntityAllocationWriteContext;

struct IMutualComponentInitializer
{
	virtual ~IMutualComponentInitializer()= default;
	virtual void Run(const FEntityRange& Range, const FEntityAllocationWriteContext& WriteContext) = 0;
};

} // namespace UE::MovieScene
