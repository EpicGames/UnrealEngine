// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


namespace UE
{
namespace MovieScene
{

template<typename InType, typename OutType>
void ConvertOperationalProperty(const InType& In, OutType& Out)
{
	Out = In;
}

} // namespace MovieScene
} // namespace UE


