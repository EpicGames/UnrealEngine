// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Map.h"
#include "Containers/SparseArray.h"
#include "Algo/AnyOf.h"

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieSceneComponentAccessors.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneComponentTypeInfo.h"

namespace UE
{
namespace MovieScene
{

/**
 * Aggregate of multiple input entities for an output defined in a TOverlappingEntityTracker
 */
struct FEntityOutputAggregate
{
	bool bNeedsRestoration = false;
};


struct FGarbageTraits
{
	inline static constexpr bool IsGarbage(...)
	{
		return false;
	}
	template<typename T>
	inline static std::enable_if_t<TPointerIsConvertibleFromTo<T, const UObject>::Value, bool> IsGarbage(T* InObject)
	{
		constexpr bool bTypeDependentFalse = !std::is_same_v<T, T>;
		static_assert(bTypeDependentFalse, "Raw object pointers are no longer supported. Please use TObjectPtr<T> instead.");
	}
	inline static bool IsGarbage(FObjectKey InObject)
	{
		return FBuiltInComponentTypes::IsBoundObjectGarbage(InObject.ResolveObjectPtr());
	}
	template<typename T>
	inline static bool IsGarbage(TObjectPtr<T>& InObject)
	{
		return FBuiltInComponentTypes::IsBoundObjectGarbage(InObject);
	}
	inline static bool IsGarbage(FObjectComponent& InComponent)
	{
		return FBuiltInComponentTypes::IsBoundObjectGarbage(InComponent.GetObject());
	}
	
	template<typename T>
	inline static void AddReferencedObjects(FReferenceCollector& ReferenceCollector, T* In)
	{
		if constexpr (THasAddReferencedObjectForComponent<T>::Value)
		{
			AddReferencedObjectForComponent(&ReferenceCollector, In);
		}
	}
};

template<typename... T> struct TGarbageTraitsImpl;
template<typename... T> struct TOverlappingEntityInput;

template<typename... T, int... Indices>
struct TGarbageTraitsImpl<TIntegerSequence<int, Indices...>, T...>
{
	static bool IsGarbage(TOverlappingEntityInput<T...>& InParam)
	{
		return (FGarbageTraits::IsGarbage(InParam.Key.template Get<Indices>()) || ...);
	}

	static void AddReferencedObjects(FReferenceCollector& ReferenceCollector, TOverlappingEntityInput<T...>& InParam)
	{
		(FGarbageTraits::AddReferencedObjects(ReferenceCollector, &InParam.Key.template Get<Indices>()), ...);
	}

	template<typename CallbackType>
	static void Unpack(const TTuple<T...>& InTuple, CallbackType&& Callback)
	{
		Callback(InTuple.template Get<Indices>()...);
	}
};

/** Override for OutputType* in order to provide custom garbage collection logic */
inline void CollectGarbageForOutput(void*)
{
}

template<typename... T>
struct TOverlappingEntityInput
{
	using GarbageTraits = TGarbageTraitsImpl<TMakeIntegerSequence<int, sizeof...(T)>, T...>;

	TTuple<T...> Key;

	template<typename... ArgTypes>
	TOverlappingEntityInput(ArgTypes&&... InArgs)
		: Key(Forward<ArgTypes>(InArgs)...)
	{}

	friend uint32 GetTypeHash(const TOverlappingEntityInput<T...>& In)
	{
		return GetTypeHash(In.Key);
	}
	friend bool operator==(const TOverlappingEntityInput<T...>& A, const TOverlappingEntityInput<T...>& B)
	{
		return A.Key == B.Key;
	}

	template<typename CallbackType>
	void Unpack(CallbackType&& Callback)
	{
		GarbageTraits::Unpack(Key, MoveTemp(Callback));
	}
};


/**
 * Templated utility class that assists in tracking the state of many -> one data relationships in an FEntityManager.
 * InputKeyTypes defines the component type(s) which defines the key that determines whether an entity animates the same output.
 * OutputType defines the user-specfied data to be associated with the multiple inputs (ie, its output)
 */
template<typename OutputType, typename... InputKeyTypes>
struct TOverlappingEntityTrackerImpl
{
	using KeyType = TOverlappingEntityInput<InputKeyTypes...>;
	using ParamType = typename TCallTraits<KeyType>::ParamType;

	bool IsInitialized() const
	{
		return bIsInitialized;
	}

	/**
	 * Update this tracker by iterating any entity that contains InKeyComponent, and matches the additional optional filter
	 * Only entities tagged as NeedsLink or NeedsUnlink are iterated, invalidating their outputs
	 */
	void Update(UMovieSceneEntitySystemLinker* Linker, TComponentTypeID<InputKeyTypes>... InKeyComponents, const FEntityComponentFilter& InFilter)
	{
		UpdateFromComponents(Linker, InFilter, InKeyComponents...);
	}

	/**
	 * Update this tracker by iterating any entity that contains InKeyComponent, and matches the additional optional filter
	 * Only entities tagged as NeedsLink or NeedsUnlink are iterated, invalidating their outputs
	 */
	template<typename ...ComponentTypes>
	void UpdateFromComponents(UMovieSceneEntitySystemLinker* Linker, const FEntityComponentFilter& InFilter, TComponentTypeID<ComponentTypes>... InKeyComponents)
	{
		check(bIsInitialized);

		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

		// Visit unlinked entities
		TFilteredEntityTask<>(TEntityTaskComponents<>())
		.CombineFilter(InFilter)
		.FilterAll({ BuiltInComponents->Tags.NeedsUnlink, FComponentTypeID(InKeyComponents)... })
		.Iterate_PerAllocation(&Linker->EntityManager, [this](const FEntityAllocation* Allocation){ this->VisitUnlinkedAllocation(Allocation); });

		// Visit newly or re-linked entities
		FEntityTaskBuilder()
		.ReadAllOf(InKeyComponents...)
		.CombineFilter(InFilter)
		.FilterAll({ BuiltInComponents->Tags.NeedsLink })
		.Iterate_PerAllocation(&Linker->EntityManager, [this](const FEntityAllocation* Allocation, TRead<ComponentTypes>... ReadKeys){ this->VisitLinkedAllocation(Allocation, ReadKeys...); });
	}

	/**
	 * Update this tracker by iterating any entity that contains InKeyComponent, and matches the additional optional filter
	 * Only entities tagged as NeedsUnlink are iterated, invalidating their outputs
	 */
	template<typename ...ComponentTypes>
	void UpdateUnlinkedOnly(UMovieSceneEntitySystemLinker* Linker, TComponentTypeID<ComponentTypes>... InKeyComponent, const FEntityComponentFilter& InFilter)
	{
		check(bIsInitialized);

		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

		// Visit unlinked entities
		TFilteredEntityTask<>(TEntityTaskComponents<>())
		.CombineFilter(InFilter)
		.FilterAll({ BuiltInComponents->Tags.NeedsUnlink, InKeyComponent... })
		.Iterate_PerAllocation(&Linker->EntityManager, [this](const FEntityAllocation* Allocation){ this->VisitUnlinkedAllocation(Allocation); });
	}

	/**
	 * Update this tracker by (re)linking the specified allocation
	 */
	template<typename ...ComponentTypes>
	void VisitActiveAllocation(const FEntityAllocation* Allocation, TComponentPtr<const ComponentTypes>... ReadKeys)
	{
		VisitActiveAllocationImpl(Allocation, ReadKeys...);
	}
	/**
	 * Update this tracker by (re)linking the specified allocation
	 */
	template<typename ...ComponentTypes>
	void VisitLinkedAllocation(const FEntityAllocation* Allocation, TComponentPtr<const ComponentTypes>... ReadKeys)
	{
		VisitActiveAllocationImpl(Allocation, ReadKeys...);
	}

	/**
	 * Update this tracker by unlinking the specified allocation
	 */
	void VisitUnlinkedAllocation(const FEntityAllocation* Allocation)
	{
		VisitUnlinkedAllocationImpl(Allocation);
	}

	/**
	 * Process any outputs that were invalidated as a result of Update being called using a custom handler.
	 *
	 * InHandler    Any user-defined handler type that contains the following named functions:
	 *                  // Called when an output is first created
	 *                  void InitializeOutput(InputKeyTypes... Inputs, TArrayView<const FMovieSceneEntityID> Inputs, OutputType* Output, FEntityOutputAggregate Aggregate);
	 *
	 *                  // Called when an output has been updated with new inputs
	 *                  void UpdateOutput(InputKeyTypes... Inputs, TArrayView<const FMovieSceneEntityID> Inputs, OutputType* Output, FEntityOutputAggregate Aggregate);
	 *
	 *                  // Called when all an output's inputs are no longer relevant, and as such the output should be destroyed (or restored)
	 *                  void DestroyOutput(InputKeyTypes... Inputs, OutputType* Output, FEntityOutputAggregate Aggregate);
	 */
	template<typename HandlerType>
	void ProcessInvalidatedOutputs(UMovieSceneEntitySystemLinker* Linker, HandlerType&& InHandler)
	{
		if (InvalidatedOutputs.Num() != 0)
		{
			TArray<FMovieSceneEntityID, TInlineAllocator<8>> InputArray;

			FComponentTypeID RestoreStateTag = FBuiltInComponentTypes::Get()->Tags.RestoreState;
			int32 NumRemoved = 0;

			auto RestoreStatePredicate = [Linker, RestoreStateTag](FMovieSceneEntityID InEntityID){ return Linker->EntityManager.HasComponent(InEntityID, RestoreStateTag); };

			check(InvalidatedOutputs.Num() < NO_OUTPUT);
			for (TConstSetBitIterator<> InvalidOutput(InvalidatedOutputs); InvalidOutput; ++InvalidOutput)
			{
				const uint16 OutputIndex = static_cast<uint16>(InvalidOutput.GetIndex());

				InputArray.Reset();
				for (auto Inputs = OutputToEntity.CreateKeyIterator(OutputIndex); Inputs; ++Inputs)
				{
					if (ensure(Linker->EntityManager.IsAllocated(Inputs.Value())))
					{
						InputArray.Add(Inputs.Value());
					}
					else
					{
						EntityToOutput.Remove(Inputs.Value());
						Inputs.RemoveCurrent();
					}
				}

				FOutput& Output = Outputs[OutputIndex];
				if (InputArray.Num() > 0)
				{
					Output.Aggregate.bNeedsRestoration = Algo::AnyOf(InputArray, RestoreStatePredicate);

					if (NewOutputs.IsValidIndex(OutputIndex) && NewOutputs[OutputIndex] == true)
					{
						Output.Key.Unpack([&InHandler, &InputArray, &Output](typename TCallTraits<InputKeyTypes>::ParamType... InKeys){
							InHandler.InitializeOutput(InKeys..., InputArray, &Output.OutputData, Output.Aggregate);
						});
					}
					else
					{
						Output.Key.Unpack([&InHandler, &InputArray, &Output](typename TCallTraits<InputKeyTypes>::ParamType... InKeys){
							InHandler.UpdateOutput(InKeys..., InputArray, &Output.OutputData, Output.Aggregate);
						});
					}
				}
				else
				{
					Output.Key.Unpack([&InHandler, &Output](typename TCallTraits<InputKeyTypes>::ParamType... InKeys){
						InHandler.DestroyOutput(InKeys..., &Output.OutputData, Output.Aggregate);
					});
				}

				if (InputArray.Num() == 0)
				{
					KeyToOutput.Remove(Outputs[OutputIndex].Key);
					Outputs.RemoveAt(OutputIndex, 1);
				}
			}
		}

		InvalidatedOutputs.Empty();
		NewOutputs.Empty();
	}

	bool IsEmpty() const
	{
		return Outputs.Num() != 0;
	}

	/**
	 * Destroy all the outputs currently being tracked
	 */
	template<typename HandlerType>
	void Destroy(HandlerType&& InHandler)
	{
		for (FOutput& Output : Outputs)
		{
			Output.Key.Unpack([&InHandler, &Output](typename TCallTraits<InputKeyTypes>::ParamType... InKeys){
				InHandler.DestroyOutput(InKeys..., &Output.OutputData, Output.Aggregate);
			});
		}

		EntityToOutput.Empty();
		OutputToEntity.Empty();

		KeyToOutput.Empty();
		Outputs.Empty();

		InvalidatedOutputs.Empty();
		NewOutputs.Empty();
	}

	void FindEntityIDs(ParamType Key, TArray<FMovieSceneEntityID>& OutEntityIDs) const
	{
		if (const uint16* OutputIndex = KeyToOutput.Find(Key))
		{
			OutputToEntity.MultiFind(*OutputIndex, OutEntityIDs);
		}
	}

	const OutputType* FindOutput(FMovieSceneEntityID EntityID) const
	{
		if (const uint16* OutputIndex = EntityToOutput.Find(EntityID))
		{
			if (ensure(Outputs.IsValidIndex(*OutputIndex)))
			{
				return &Outputs[*OutputIndex].OutputData;
			}
		}
		return nullptr;
	}

	const OutputType* FindOutput(ParamType Key) const
	{
		if (const uint16* OutputIndex = KeyToOutput.Find(Key))
		{
			if (ensure(Outputs.IsValidIndex(*OutputIndex)))
			{
				return &Outputs[*OutputIndex].OutputData;
			}
		}
		return nullptr;
	}

	bool NeedsRestoration(ParamType Key, bool bEnsureOutput = false) const
	{
		const uint16 ExistingOutput = FindOutputByKey(Key);
		const bool bIsOutputValid = IsOutputValid(ExistingOutput);
		ensure(bIsOutputValid || !bEnsureOutput);
		if (bIsOutputValid)
		{
			return Outputs[ExistingOutput].Aggregate.bNeedsRestoration;
		}
		return false;
	}

	void SetNeedsRestoration(ParamType Key, bool bNeedsRestoration, bool bEnsureOutput = false)
	{
		const uint16 ExistingOutput = FindOutputByKey(Key);
		const bool bIsOutputValid = IsOutputValid(ExistingOutput);
		ensure(bIsOutputValid || !bEnsureOutput);
		if (bIsOutputValid)
		{
			Outputs[ExistingOutput].Aggregate.bNeedsRestoration = bNeedsRestoration;
		}
	}

protected:

	template<typename ...ComponentTypes>
	void VisitActiveAllocationImpl(const FEntityAllocation* Allocation, TComponentPtr<const ComponentTypes>... Keys)
	{
		check(bIsInitialized);

		const int32 Num = Allocation->Num();

		const FMovieSceneEntityID* EntityIDs = Allocation->GetRawEntityIDs();

		const bool bNeedsLink = Allocation->HasComponent(FBuiltInComponentTypes::Get()->Tags.NeedsLink);
		if (Allocation->HasComponent(FBuiltInComponentTypes::Get()->Tags.RestoreState))
		{
			for (int32 Index = 0; Index < Num; ++Index)
			{
				const uint16 OutputIndex = MakeOutput(EntityIDs[Index], TOverlappingEntityInput<InputKeyTypes...>(Keys[Index]...), bNeedsLink);
				Outputs[OutputIndex].Aggregate.bNeedsRestoration = true;
			}
		}
		else
		{
			for (int32 Index = 0; Index < Num; ++Index)
			{
				MakeOutput(EntityIDs[Index], TOverlappingEntityInput<InputKeyTypes...>(Keys[Index]...), bNeedsLink);
			}
		}
	}

	void VisitUnlinkedAllocationImpl(const FEntityAllocation* Allocation)
	{
		check(bIsInitialized);

		const int32 Num = Allocation->Num();
		const FMovieSceneEntityID* EntityIDs = Allocation->GetRawEntityIDs();

		for (int32 Index = 0; Index < Num; ++Index)
		{
			ClearOutputByEntity(EntityIDs[Index]);
		}
	}

	uint16 MakeOutput(FMovieSceneEntityID EntityID, ParamType InKey, bool bAlwaysInvalidate)
	{
		const uint16 PreviousOutputIndex = FindOutputByEntity(EntityID);
		const uint16 DesiredOutputIndex  = CreateOutputByKey(InKey);

		if (PreviousOutputIndex == DesiredOutputIndex)
		{
			if (bAlwaysInvalidate)
			{
				// Previous output is now invalidated since we're removing this entity
				InvalidatedOutputs.PadToNum(DesiredOutputIndex + 1, false);
				InvalidatedOutputs[DesiredOutputIndex] = true;
			}

			return DesiredOutputIndex;
		}

		if (PreviousOutputIndex != NO_OUTPUT)
		{
			// Previous output is now invalidated since we're removing this entity
			InvalidatedOutputs.PadToNum(PreviousOutputIndex + 1, false);
			InvalidatedOutputs[PreviousOutputIndex] = true;

			// Remove the entitiy's contribution from the previous output
			OutputToEntity.Remove(PreviousOutputIndex, EntityID);
			EntityToOutput.Remove(EntityID);
		}

		// Invalidate the new output
		InvalidatedOutputs.PadToNum(DesiredOutputIndex + 1, false);
		InvalidatedOutputs[DesiredOutputIndex] = true;

		EntityToOutput.Add(EntityID, DesiredOutputIndex);
		OutputToEntity.Add(DesiredOutputIndex, EntityID);

		return DesiredOutputIndex;
	}

	uint16 CreateOutputByKey(ParamType Key)
	{
		const uint16 ExistingOutput = FindOutputByKey(Key);
		if (ExistingOutput != NO_OUTPUT)
		{
			return ExistingOutput;
		}

		const int32 Index = Outputs.Add(FOutput{ Key, OutputType{} });
		check(Index < NO_OUTPUT);

		const uint16 NewOutput = static_cast<uint16>(Index);

		NewOutputs.PadToNum(NewOutput + 1, false);
		NewOutputs[NewOutput] = true;

		KeyToOutput.Add(Key, NewOutput);
		return NewOutput;
	}

	uint16 FindOutputByKey(ParamType Key) const
	{
		const uint16* OutputIndex = KeyToOutput.Find(Key);
		return OutputIndex ? *OutputIndex : NO_OUTPUT;
	}

	uint16 FindOutputByEntity(FMovieSceneEntityID EntityID) const
	{
		const uint16* OutputIndex = EntityToOutput.Find(EntityID);
		return OutputIndex ? *OutputIndex : NO_OUTPUT;
	}

	void ClearOutputByEntity(FMovieSceneEntityID EntityID)
	{
		const uint16 OutputIndex = FindOutputByEntity(EntityID);
		if (OutputIndex != NO_OUTPUT)
		{
			OutputToEntity.Remove(OutputIndex, EntityID);
			EntityToOutput.Remove(EntityID);

			InvalidatedOutputs.PadToNum(OutputIndex + 1, false);
			InvalidatedOutputs[OutputIndex] = true;
		}
	}

	bool IsOutputValid(uint16 OutputIndex)
	{
		return OutputIndex != NO_OUTPUT &&
			(!InvalidatedOutputs.IsValidIndex(OutputIndex) ||
			 !InvalidatedOutputs[OutputIndex]);
	}

	struct FOutput
	{
		KeyType Key;
		OutputType OutputData;
		FEntityOutputAggregate Aggregate;
	};

	TMap<FMovieSceneEntityID, uint16> EntityToOutput;
	TMultiMap<uint16, FMovieSceneEntityID> OutputToEntity;

	TMap<KeyType, uint16> KeyToOutput;
	TSparseArray< FOutput > Outputs;

	TBitArray<> InvalidatedOutputs, NewOutputs;

	bool bIsInitialized = false;

	static constexpr uint16 NO_OUTPUT = MAX_uint16;
};


template<typename OutputType, typename... InputTypes>
struct TOverlappingEntityTracker_NoGarbage : TOverlappingEntityTrackerImpl<OutputType, InputTypes...>
{
	void Initialize(UMovieSceneEntitySystem* OwningSystem)
	{
		this->bIsInitialized = true;
	}
};

template<typename OutputType, typename... InputTypes>
struct TOverlappingEntityTracker_WithGarbage : TOverlappingEntityTrackerImpl<OutputType, InputTypes...>
{
	using ThisType = TOverlappingEntityTracker_WithGarbage<OutputType, InputTypes...>;
	using Super = TOverlappingEntityTrackerImpl<OutputType, InputTypes...>;
	using typename Super::FOutput;
	using typename Super::KeyType;

	~TOverlappingEntityTracker_WithGarbage()
	{
		UMovieSceneEntitySystem* OwningSystem = WeakOwningSystem.GetEvenIfUnreachable();
		UMovieSceneEntitySystemLinker* Linker = OwningSystem ? OwningSystem->GetLinker() : nullptr;
		if (Linker)
		{
			Linker->Events.TagGarbage.RemoveAll(this);
			Linker->Events.CleanTaggedGarbage.RemoveAll(this);
			Linker->Events.AddReferencedObjects.RemoveAll(this);
		}
	}

	void Initialize(UMovieSceneEntitySystem* OwningSystem)
	{
		if (this->bIsInitialized)
		{
			return;
		}

		this->bIsInitialized = true;
		WeakOwningSystem = OwningSystem;

		OwningSystem->GetLinker()->Events.TagGarbage.AddRaw(this, &ThisType::TagGarbage);
		OwningSystem->GetLinker()->Events.CleanTaggedGarbage.AddRaw(this, &ThisType::CleanTaggedGarbage);
		OwningSystem->GetLinker()->Events.AddReferencedObjects.AddRaw(this, &ThisType::AddReferencedObjects);
	}

	void TagGarbage(UMovieSceneEntitySystemLinker* Linker)
	{
		for (int32 Index = this->Outputs.GetMaxIndex()-1; Index >= 0; --Index)
		{
			if (!this->Outputs.IsValidIndex(Index))
			{
				continue;
			}
			const uint16 OutputIndex = static_cast<uint16>(Index);

			FOutput& Output = this->Outputs[Index];
			if (KeyType::GarbageTraits::IsGarbage(Output.Key))
			{
				CollectGarbageForOutput(&this->Outputs[Index].OutputData);

				this->Outputs.RemoveAt(Index, 1);

				// Make sure this output is not flagged as invalidated because it is being destroyed.
				// This prevents us from blindly processing it in ProcessInvalidatedOutputs
				if (this->InvalidatedOutputs.IsValidIndex(OutputIndex))
				{
					this->InvalidatedOutputs[OutputIndex] = false;
				}

				for (auto It = this->OutputToEntity.CreateKeyIterator(OutputIndex); It; ++It)
				{
					this->EntityToOutput.Remove(It.Value());
					It.RemoveCurrent();
				}
			}
		}

		for (auto It = this->KeyToOutput.CreateIterator(); It; ++It)
		{
			if (KeyType::GarbageTraits::IsGarbage(It.Key()) || !this->Outputs.IsValidIndex(It.Value()))
			{
				It.RemoveCurrent();
			}
		}
	}

	void CleanTaggedGarbage(UMovieSceneEntitySystemLinker* Linker)
	{
		FComponentTypeID NeedsUnlink = FBuiltInComponentTypes::Get()->Tags.NeedsUnlink;

		// Check whether any of our inputs are about to be destroyed
		for (auto EntityToOutputIt = this->EntityToOutput.CreateIterator(); EntityToOutputIt; ++EntityToOutputIt)
		{
			const FMovieSceneEntityID EntityID    = EntityToOutputIt.Key();
			const uint16              OutputIndex = EntityToOutputIt.Value();
			if (OutputIndex != Super::NO_OUTPUT && Linker->EntityManager.HasComponent(EntityID, NeedsUnlink))
			{
				this->OutputToEntity.Remove(OutputIndex, EntityID);

				this->InvalidatedOutputs.PadToNum(OutputIndex + 1, false);
				this->InvalidatedOutputs[OutputIndex] = true;

				EntityToOutputIt.RemoveCurrent();
			}
		}
	}

	void AddReferencedObjects(UMovieSceneEntitySystemLinker* Linker, FReferenceCollector& ReferenceCollector)
	{
		constexpr bool bKeyCanBeGarbage = (THasAddReferencedObjectForComponent<InputTypes>::Value || ...);
		constexpr bool bOutputCanBeGarbage = THasAddReferencedObjectForComponent<OutputType>::Value;

		if constexpr (bKeyCanBeGarbage)
		{
			for (TPair<KeyType, uint16>& Pair : this->KeyToOutput)
			{
				KeyType::GarbageTraits::AddReferencedObjects(ReferenceCollector, Pair.Key);
			}
		}

		for (FOutput& Output : this->Outputs)
		{
			if constexpr (bKeyCanBeGarbage)
			{
				KeyType::GarbageTraits::AddReferencedObjects(ReferenceCollector, Output.Key);
			}
			if constexpr (bOutputCanBeGarbage)
			{
				FGarbageTraits::AddReferencedObjects(ReferenceCollector, Output.OutputData);
			}
		}
	}

protected:

	TWeakObjectPtr<UMovieSceneEntitySystem> WeakOwningSystem;
};



template<typename OutputType, typename... KeyType>
using TOverlappingEntityTracker = std::conditional_t<
	(THasAddReferencedObjectForComponent<KeyType>::Value || ...) || THasAddReferencedObjectForComponent<OutputType>::Value,
	TOverlappingEntityTracker_WithGarbage<OutputType, KeyType...>,
	TOverlappingEntityTracker_NoGarbage<OutputType, KeyType...>
>;

} // namespace MovieScene
} // namespace UE
