// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "EntitySystem/MovieSceneComponentAccessors.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "EntitySystem/MovieScenePropertySystemTypes.h"
#include "EntitySystem/MovieScenePropertyRegistry.h"


class UMovieSceneEntitySystemLinker;
class FTrackInstancePropertyBindings;

namespace UE
{
namespace MovieScene
{

template<typename CompositeType>
inline void PatchComposite(uint8* OutValueBase, const CompositeType* Composite, uint16 PtrOffset)
{
	if (Composite)
	{
		*reinterpret_cast<CompositeType*>(OutValueBase + PtrOffset) = *Composite;
	}
}

template<typename... CompositeTypes, int ...CompositeIndices>
inline void PatchCompositeValueImpl(const TIntegerSequence<int, CompositeIndices...>&, TArrayView<const FPropertyCompositeDefinition> CompositeDefinitions, uint8* OutValueBase, const CompositeTypes*... Composites)
{
	int Tmp[] = {
		( PatchComposite(OutValueBase, Composites, CompositeDefinitions[CompositeIndices].CompositeOffset), 0 )..., 0
	};
	(void)Tmp;
}

template<typename... CompositeTypes>
void PatchCompositeValue(TArrayView<const FPropertyCompositeDefinition> CompositeDefinitions, void* OutValueAddress, const CompositeTypes*... Composites)
{
	uint8* ValueBase = static_cast<uint8*>(OutValueAddress);
	PatchCompositeValueImpl(TMakeIntegerSequence<int, sizeof...(CompositeTypes)>(), CompositeDefinitions, ValueBase, Composites...);
}


template<typename PropertyTraits, typename MetaDataType, typename CompositeIntegers, typename ...CompositeTypes>
struct TSetPartialPropertyValuesImpl;

template<typename PropertyTraits, typename ...MetaDataTypes, int ...CompositeIndices, typename ...CompositeTypes>
struct TSetPartialPropertyValuesImpl<PropertyTraits, TPropertyMetaData<MetaDataTypes...>, TIntegerSequence<int, CompositeIndices...>, CompositeTypes...>
{
	using StorageType = typename PropertyTraits::StorageType;

	using FThreeWayAccessor  = TMultiReadOptional<FCustomPropertyIndex, uint16, TSharedPtr<FTrackInstancePropertyBindings>>;
	using FTwoWayAccessor    = TMultiReadOptional<uint16, TSharedPtr<FTrackInstancePropertyBindings>>;

	explicit TSetPartialPropertyValuesImpl(const PropertyTraits* InTraits, const ICustomPropertyRegistration* InCustomProperties, TArrayView<const FPropertyCompositeDefinition> InCompositeDefinitions)
		: Traits(InTraits)
		, CustomProperties(InCustomProperties)
		, CompositeDefinitions(InCompositeDefinitions)
	{
		if (CustomProperties)
		{
			CustomAccessors = CustomProperties->GetAccessors();
		}
	}

	void ForEachAllocation(const FEntityAllocation* Allocation, TRead<UObject*> BoundObjectComponents, FThreeWayAccessor PropertyBindingComponents, TRead<MetaDataTypes>... InMetaData, TReadOptional<CompositeTypes>... InCompositeComponents) const;
	void ForEachAllocation(const FEntityAllocation* Allocation, TRead<UObject*> BoundObjectComponents, FTwoWayAccessor PropertyBindingComponents, TRead<MetaDataTypes>... InMetaData, TReadOptional<CompositeTypes>... InCompositeComponents) const;

private:

	void ForEachCustom(const FEntityAllocation* Allocation, UObject* const* Objects, const FCustomPropertyIndex* Custom, const MetaDataTypes*... InMetaData, const CompositeTypes*... InCompositeComponents) const;

	void ForEachFast(const FEntityAllocation* Allocation, UObject* const* Objects, const uint16* Fast, const MetaDataTypes*... InMetaData, const CompositeTypes*... InCompositeComponents) const;

	void ForEachSlow(const FEntityAllocation* Allocation, UObject* const* Objects, const TSharedPtr<FTrackInstancePropertyBindings>* Slow, const MetaDataTypes*... InMetaData, const CompositeTypes*... InCompositeComponents) const;

private:
	const PropertyTraits* Traits;
	const ICustomPropertyRegistration* CustomProperties;
	FCustomAccessorView CustomAccessors;
	TArrayView<const FPropertyCompositeDefinition> CompositeDefinitions;
};

template<typename PropertyTraits, typename ...CompositeTypes>
using TSetPartialPropertyValues = TSetPartialPropertyValuesImpl<PropertyTraits, typename PropertyTraits::MetaDataType, TMakeIntegerSequence<int, sizeof...(CompositeTypes)>, CompositeTypes...>;


} // namespace MovieScene
} // namespace UE