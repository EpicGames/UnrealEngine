// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieScenePartialProperties.h"
#include "EntitySystem/MovieSceneOperationalTypeConversions.h"
#include "MovieSceneCommonHelpers.h"

namespace UE
{
namespace MovieScene
{


template<typename PropertyTraits, typename ...MetaDataTypes, int ...CompositeIndices, typename ...CompositeTypes>
void TSetPartialPropertyValuesImpl<PropertyTraits, TPropertyMetaData<MetaDataTypes...>, TIntegerSequence<int, CompositeIndices...>, CompositeTypes...>::ForEachAllocation(
	const FEntityAllocation* Allocation,
	TRead<UObject*> BoundObjectComponents,
	FThreeWayAccessor PropertyBindingComponents,
	TRead<MetaDataTypes>... InMetaData,
	TReadOptional<CompositeTypes>... InCompositeComponents) const
{
	// ----------------------------------------------------------------------------------------------------------------------------
	// For partially animated composites, we first retrieve the current properties for the allocation, then go through and patch in
	// All the animated values, then apply the properties to objects

	const int32 Num = Allocation->Num();

	if (const FCustomPropertyIndex* Custom = PropertyBindingComponents.template Get<0>())
	{
		ForEachCustom(Allocation, BoundObjectComponents.AsPtr(), Custom, InMetaData.AsPtr()..., InCompositeComponents.AsPtr()...);
	}
	else if (const uint16* Fast = PropertyBindingComponents.template Get<1>())
	{
		ForEachFast(Allocation, BoundObjectComponents.AsPtr(), Fast, InMetaData.AsPtr()..., InCompositeComponents.AsPtr()...);
	}
	else if (const TSharedPtr<FTrackInstancePropertyBindings>* Slow = PropertyBindingComponents.template Get<2>())
	{
		ForEachSlow(Allocation, BoundObjectComponents.AsPtr(), Slow, InMetaData.AsPtr()..., InCompositeComponents.AsPtr()...);
	}
}


template<typename PropertyTraits, typename ...MetaDataTypes, int ...CompositeIndices, typename ...CompositeTypes>
void TSetPartialPropertyValuesImpl<PropertyTraits, TPropertyMetaData<MetaDataTypes...>, TIntegerSequence<int, CompositeIndices...>, CompositeTypes...>::ForEachAllocation(
	const FEntityAllocation* Allocation,
	TRead<UObject*> BoundObjectComponents,
	FTwoWayAccessor PropertyBindingComponents,
	TRead<MetaDataTypes>... InMetaData,
	TReadOptional<CompositeTypes>... InCompositeComponents) const
{
	// ----------------------------------------------------------------------------------------------------------------------------
	// For partially animated composites, we first retrieve the current properties for the allocation, then go through and patch in
	// All the animated values, then apply the properties to objects

	const int32 Num = Allocation->Num();
	if (const uint16* Fast = PropertyBindingComponents.template Get<0>())
	{
		ForEachFast(Allocation, BoundObjectComponents.AsPtr(), Fast, InMetaData.AsPtr()..., InCompositeComponents.AsPtr()...);
	}
	else if (const TSharedPtr<FTrackInstancePropertyBindings>* Slow = PropertyBindingComponents.template Get<1>())
	{
		ForEachSlow(Allocation, BoundObjectComponents.AsPtr(), Slow, InMetaData.AsPtr()..., InCompositeComponents.AsPtr()...);
	}
}


template<typename PropertyTraits, typename ...MetaDataTypes, int ...CompositeIndices, typename ...CompositeTypes>
void TSetPartialPropertyValuesImpl<PropertyTraits, TPropertyMetaData<MetaDataTypes...>, TIntegerSequence<int, CompositeIndices...>, CompositeTypes...>::ForEachCustom(
	const FEntityAllocation* Allocation,
	UObject* const* Objects,
	const FCustomPropertyIndex* Custom,
	const MetaDataTypes*... InMetaData,
	const CompositeTypes*... InCompositeComponents) const
{
	const int32 Num = Allocation->Num();

	for (int32 Index = 0; Index < Num; ++Index)
	{
		const FCustomPropertyIndex PropertyIndex = Custom[Index];

		StorageType Storage{};
		Traits->GetObjectPropertyValue(Objects[Index], InMetaData[Index]..., CustomAccessors[PropertyIndex.Value], Storage);

		PatchCompositeValue(CompositeDefinitions, &Storage, InCompositeComponents ? &InCompositeComponents[Index] : nullptr...);

		Traits->SetObjectPropertyValue(Objects[Index], InMetaData[Index]..., CustomAccessors[PropertyIndex.Value], Storage);
	}
}


template<typename PropertyTraits, typename ...MetaDataTypes, int ...CompositeIndices, typename ...CompositeTypes>
void TSetPartialPropertyValuesImpl<PropertyTraits, TPropertyMetaData<MetaDataTypes...>, TIntegerSequence<int, CompositeIndices...>, CompositeTypes...>::ForEachFast(
	const FEntityAllocation* Allocation,
	UObject* const* Objects,
	const uint16* Fast,
	const MetaDataTypes*... InMetaData,
	const CompositeTypes*... InCompositeComponents) const
{
	const int32 Num = Allocation->Num();

	for (int32 Index = 0; Index < Num; ++Index)
	{
		const uint16 PropertyOffset = Fast[Index];
		checkSlow(PropertyOffset != 0);

		StorageType Storage{};
		Traits->GetObjectPropertyValue(Objects[Index], InMetaData[Index]..., PropertyOffset, Storage);

		PatchCompositeValue(CompositeDefinitions, &Storage, InCompositeComponents ? &InCompositeComponents[Index] : nullptr...);

		Traits->SetObjectPropertyValue(Objects[Index], InMetaData[Index]..., PropertyOffset, Storage);
	}
}


template<typename PropertyTraits, typename ...MetaDataTypes, int ...CompositeIndices, typename ...CompositeTypes>
void TSetPartialPropertyValuesImpl<PropertyTraits, TPropertyMetaData<MetaDataTypes...>, TIntegerSequence<int, CompositeIndices...>, CompositeTypes...>::ForEachSlow(
	const FEntityAllocation* Allocation,
	UObject* const* Objects,
	const TSharedPtr<FTrackInstancePropertyBindings>* Slow,
	const MetaDataTypes*... InMetaData,
	const CompositeTypes*... InCompositeComponents) const
{
	const int32 Num = Allocation->Num();

	for (int32 Index = 0; Index < Num; ++Index)
	{
		FTrackInstancePropertyBindings* Bindings = Slow[Index].Get();

		StorageType Storage{};
		Traits->GetObjectPropertyValue(Objects[Index], InMetaData[Index]..., Bindings, Storage);

		PatchCompositeValue(CompositeDefinitions, &Storage, InCompositeComponents ? &InCompositeComponents[Index] : nullptr...);

		Traits->SetObjectPropertyValue(Objects[Index], InMetaData[Index]..., Bindings, Storage);
	}
}


} // namespace MovieScene
} // namespace UE