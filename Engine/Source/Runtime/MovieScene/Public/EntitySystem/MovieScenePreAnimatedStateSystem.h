// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "EntitySystem/MovieSceneEntityInstantiatorSystem.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateStorage.h"
#include "Templates/SharedPointer.h"
#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieScenePreAnimatedStateSystem.generated.h"

class UMovieSceneEntitySystemLinker;
class UObject;
namespace UE { namespace MovieScene { struct FPreAnimatedStateExtension; } }
namespace UE { namespace MovieScene { struct FSystemSubsequentTasks; } }
namespace UE { namespace MovieScene { struct FSystemTaskPrerequisites; } }


UINTERFACE()
class UMovieScenePreAnimatedStateSystemInterface : public UInterface
{
	GENERATED_BODY()
};


/**
 * Interface that can be added to any entity system in the 'instantiation' phase to implement save / restore state
 * with its system dependencies strictly saved in order, and restored in reverse order
 */
class IMovieScenePreAnimatedStateSystemInterface
{
public:
	GENERATED_BODY()

	struct FPreAnimationParameters
	{
		UE::MovieScene::FSystemTaskPrerequisites* Prerequisites;
		UE::MovieScene::FSystemSubsequentTasks* Subsequents;
		UE::MovieScene::FPreAnimatedStateExtension* CacheExtension;
	};

	virtual void SavePreAnimatedState(const FPreAnimationParameters& InParameters) {}
	virtual void RestorePreAnimatedState(const FPreAnimationParameters& InParameters) {}
};


/**
 * System that becomes relevant if there are any entites tagged RestoreState,
 * or UMovieSceneEntitySystemLinker::ShouldCaptureGlobalState is true.
 * When run this system will iterate the instantiation phase in order, and call 
 * IMovieScenePreAnimatedStateSystemInterface::Save(Global)PreAnimatedState on any
 * systems that implement the necessary interface
 */
UCLASS(MinimalAPI)
class UMovieSceneCachePreAnimatedStateSystem
	: public UMovieSceneEntityInstantiatorSystem
{
public:

	GENERATED_BODY()

	UMovieSceneCachePreAnimatedStateSystem(const FObjectInitializer& ObjInit);

private:

	virtual bool IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const override;
	virtual void OnLink() override;
	virtual void OnUnlink() override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override final;
};



/**
 * System that becomes relevant if there are any entites tagged RestoreState,
 * or UMovieSceneEntitySystemLinker::ShouldCaptureGlobalState is true.
 * When run this system will iterate the instantiation phase in reverse order, and call 
 * IMovieScenePreAnimatedStateSystemInterface::Restore(Global)PreAnimatedState on any
 * systems that implement the necessary interface.
 */
UCLASS(MinimalAPI)
class UMovieSceneRestorePreAnimatedStateSystem
	: public UMovieSceneEntityInstantiatorSystem
{
public:

	GENERATED_BODY()

	UMovieSceneRestorePreAnimatedStateSystem(const FObjectInitializer& ObjInit);

private:

	virtual bool IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const override;
	virtual void OnLink() override;
	virtual void OnUnlink() override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override final;

	TSharedPtr<UE::MovieScene::FPreAnimatedStateExtension> PreAnimatedStateRef;
};
