// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "EntitySystem/MovieSceneComponentDebug.h"

#include "MovieScenePropertyBinding.generated.h"


/**
 * Source property binding information for an entity on a moviescene timeline
 * Comprises a leaf property name and a path and a cached boolean signifying whether the binding is allowed to perform a fast class-wise property lookup
 */
USTRUCT()
struct FMovieScenePropertyBinding
{
	GENERATED_BODY()

	FMovieScenePropertyBinding()
		: bCanUseClassLookup(false)
	{}

	MOVIESCENE_API FMovieScenePropertyBinding(FName InPropertyName, const FString& InPropertyPath);

	MOVIESCENE_API static FMovieScenePropertyBinding FromPath(const FString& InPropertyPath);

	friend bool operator==(FMovieScenePropertyBinding A, FMovieScenePropertyBinding B)
	{
		return A.PropertyName == B.PropertyName && A.PropertyPath == B.PropertyPath;
	}

	bool CanUseClassLookup() const
	{
		return bCanUseClassLookup;
	}

#if WITH_EDITORONLY_DATA
	void PostSerialize(const FArchive& Ar);
#endif

	/** Leaf name of the property to animate */
	UPROPERTY()
	FName PropertyName;

	/** Full path to the property from the object including struct and array indirection */
	UPROPERTY()
	FName PropertyPath;

	/** True if this property can be considered for fast property offset resolution(ie the property address is _always_ a constant offset from the obejct ptr), false otherwise */
	UPROPERTY()
	bool bCanUseClassLookup;
};

#if WITH_EDITORONLY_DATA
template<>
struct TStructOpsTypeTraits<FMovieScenePropertyBinding> : public TStructOpsTypeTraitsBase2<FMovieScenePropertyBinding>
{
	enum { WithPostSerialize = true };
};
#endif // WITH_EDITORONLY_DATA

