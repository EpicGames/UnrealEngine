// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/DelegateCombinations.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieScenePropertyRegistry.h"
#include "EntitySystem/MovieScenePartialProperties.inl"
#include "EntitySystem/MovieSceneDecompositionQuery.h"
#include "EntitySystem/MovieSceneBlenderSystem.h"
#include "EntitySystem/MovieSceneInitialValueCache.h"
#include "EntitySystem/MovieScenePropertySystemTypes.inl"
#include "EntitySystem/MovieSceneOperationalTypeConversions.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationExtension.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateExtension.h"

#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedPropertyStorage.h"

class UMovieSceneTrack;

namespace UE
{
namespace MovieScene
{


template<typename PropertyTraits, typename MetaDataType, typename MetaDataIndices, typename CompositeIndices, typename ...CompositeTypes>
struct TPropertyComponentHandlerImpl;

template<typename PropertyTraits, typename ...CompositeTypes>
struct TPropertyComponentHandler
	: TPropertyComponentHandlerImpl<PropertyTraits, typename PropertyTraits::MetaDataType, TMakeIntegerSequence<int, PropertyTraits::MetaDataType::Num>, TMakeIntegerSequence<int, sizeof...(CompositeTypes)>, CompositeTypes...>
{
};

template<typename, typename, typename>
struct TInitialValueProcessorImpl;

template<typename PropertyTraits, typename ...MetaDataTypes, int ...MetaDataIndices>
struct TInitialValueProcessorImpl<PropertyTraits, TPropertyMetaData<MetaDataTypes...>, TIntegerSequence<int, MetaDataIndices...>> : IInitialValueProcessor
{
	using StorageType = typename PropertyTraits::StorageType;

	TSortedMap<FInterrogationChannel, StorageType> ValuesByChannel;

	FBuiltInComponentTypes* BuiltInComponents;

	ICustomPropertyRegistration* CustomPropertyRegistration;
	TComponentTypeID<StorageType> InitialValueType;
	TTuple<TComponentTypeID<MetaDataTypes>...> MetaDataComponents;
	FCustomAccessorView CustomAccessors;

	// Transient properties reset on each usage.
	IInterrogationExtension* Interrogation;
	const PropertyTraits* Traits;
	TPropertyValueStorage<PropertyTraits>* CacheStorage;

	FEntityAllocationWriteContext WriteContext;

	TInitialValueProcessorImpl(
		const PropertyTraits* InTraits,
		TComponentTypeID<StorageType> InInitialValueType,
		TArrayView<const FComponentTypeID> InMetaDataComponents,
		ICustomPropertyRegistration* InCustomPropertyRegistration
	)
		: BuiltInComponents(FBuiltInComponentTypes::Get())
		, CustomPropertyRegistration(InCustomPropertyRegistration)
		, InitialValueType(InInitialValueType)
		, MetaDataComponents(InMetaDataComponents[MetaDataIndices].ReinterpretCast<MetaDataTypes>()...)
		, WriteContext(FEntityAllocationWriteContext::NewAllocation())
	{
		Traits = InTraits;
		Interrogation = nullptr;
		CacheStorage = nullptr;
	}

	virtual void Initialize(UMovieSceneEntitySystemLinker* Linker, FInitialValueCache* InitialValueCache) override
	{
		Interrogation = Linker->FindExtension<IInterrogationExtension>();
		WriteContext  = FEntityAllocationWriteContext(Linker->EntityManager);

		if (CustomPropertyRegistration)
		{
			CustomAccessors = CustomPropertyRegistration->GetAccessors();
		}

		if (InitialValueCache)
		{
			CacheStorage = InitialValueCache->GetStorage<PropertyTraits>(InitialValueType);
		}
	}

	virtual void PopulateFilter(FEntityComponentFilter& OutFilter) const override
	{
		// These initializers must have bound objects or an interrogation key
		OutFilter.Any({ BuiltInComponents->BoundObject, BuiltInComponents->Interrogation.OutputKey });
	}

	virtual void Process(const FEntityAllocation* Allocation, const FComponentMask& AllocationType) override
	{
		if (Interrogation && AllocationType.Contains(BuiltInComponents->Interrogation.OutputKey))
		{
			VisitInterrogationAllocation(Allocation);
		}
		else if (CacheStorage)
		{
			VisitAllocationCached(Allocation);
		}
		else
		{
			VisitAllocation(Allocation);
		}
	}

	virtual void Finalize() override
	{
		ValuesByChannel.Empty();
		Interrogation = nullptr;
		CacheStorage = nullptr;
	}

	void VisitAllocation(const FEntityAllocation* Allocation)
	{
		const int32 Num = Allocation->Num();

		TComponentWriter<StorageType> InitialValues = Allocation->WriteComponents(InitialValueType, WriteContext);
		TComponentReader<UObject*>    BoundObjects  = Allocation->ReadComponents(BuiltInComponents->BoundObject);

		TTuple< TComponentReader<MetaDataTypes>... > MetaData(
			Allocation->ReadComponents(MetaDataComponents.template Get<MetaDataIndices>())...
		);

		if (TOptionalComponentReader<FCustomPropertyIndex> CustomIndices = Allocation->TryReadComponents(BuiltInComponents->CustomPropertyIndex))
		{
			const FCustomPropertyIndex* RawIndices = CustomIndices.AsPtr();
			for (int32 Index = 0; Index < Num; ++Index)
			{
				Traits->GetObjectPropertyValue(BoundObjects[Index], MetaData.template Get<MetaDataIndices>()[Index]..., CustomAccessors[RawIndices[Index].Value], InitialValues[Index]);
			}
		}

		else if (TOptionalComponentReader<uint16> FastOffsets = Allocation->TryReadComponents(BuiltInComponents->FastPropertyOffset))
		{
			const uint16* RawOffsets = FastOffsets.AsPtr();
			for (int32 Index = 0; Index < Num; ++Index)
			{
				Traits->GetObjectPropertyValue(BoundObjects[Index], MetaData.template Get<MetaDataIndices>()[Index]..., RawOffsets[Index], InitialValues[Index]);
			}
		}

		else if (TOptionalComponentReader<TSharedPtr<FTrackInstancePropertyBindings>> SlowProperties = Allocation->TryReadComponents(BuiltInComponents->SlowProperty))
		{
			const TSharedPtr<FTrackInstancePropertyBindings>* RawProperties = SlowProperties.AsPtr();
			for (int32 Index = 0; Index < Num; ++Index)
			{
				Traits->GetObjectPropertyValue(BoundObjects[Index], MetaData.template Get<MetaDataIndices>()[Index]..., RawProperties[Index].Get(), InitialValues[Index]);
			}
		}
	}

	void VisitAllocationCached(const FEntityAllocation* Allocation)
	{
		const int32 Num = Allocation->Num();

		TComponentWriter<FInitialValueIndex> InitialValueIndices = Allocation->WriteComponents(BuiltInComponents->InitialValueIndex, WriteContext);
		TComponentWriter<StorageType>        InitialValues       = Allocation->WriteComponents(InitialValueType, WriteContext);
		TComponentReader<UObject*>           BoundObjects        = Allocation->ReadComponents(BuiltInComponents->BoundObject);

		TTuple< TComponentReader<MetaDataTypes>... > MetaData(
			Allocation->ReadComponents(MetaDataComponents.template Get<MetaDataIndices>())...
		);

		if (TOptionalComponentReader<FCustomPropertyIndex> CustomIndices = Allocation->TryReadComponents(BuiltInComponents->CustomPropertyIndex))
		{
			const FCustomPropertyIndex* RawIndices = CustomIndices.AsPtr();
			for (int32 Index = 0; Index < Num; ++Index)
			{
				TOptional<FInitialValueIndex> ExistingIndex = CacheStorage->FindPropertyIndex(BoundObjects[Index], RawIndices[Index]);
				if (ExistingIndex)
				{
					InitialValues[Index] = CacheStorage->GetCachedValue(ExistingIndex.GetValue());
				}
				else
				{
					StorageType Value{};
					Traits->GetObjectPropertyValue(BoundObjects[Index], MetaData.template Get<MetaDataIndices>()[Index]..., CustomAccessors[RawIndices[Index].Value], Value);

					InitialValues[Index] = Value;
					InitialValueIndices[Index] = CacheStorage->AddInitialValue(BoundObjects[Index], Value, RawIndices[Index]);
				}
			}
		}

		else if (TOptionalComponentReader<uint16> FastOffsets = Allocation->TryReadComponents(BuiltInComponents->FastPropertyOffset))
		{
			const uint16* RawOffsets = FastOffsets.AsPtr();
			for (int32 Index = 0; Index < Num; ++Index)
			{
				TOptional<FInitialValueIndex> ExistingIndex = CacheStorage->FindPropertyIndex(BoundObjects[Index], FastOffsets[Index]);
				if (ExistingIndex)
				{
					InitialValues[Index] = CacheStorage->GetCachedValue(ExistingIndex.GetValue());
				}
				else
				{
					StorageType Value{};
					Traits->GetObjectPropertyValue(BoundObjects[Index], MetaData.template Get<MetaDataIndices>()[Index]..., RawOffsets[Index], Value);

					InitialValues[Index] = Value;
					InitialValueIndices[Index] = CacheStorage->AddInitialValue(BoundObjects[Index], Value, RawOffsets[Index]);
				}
			}
		}

		else if (TOptionalComponentReader<TSharedPtr<FTrackInstancePropertyBindings>> SlowProperties = Allocation->TryReadComponents(BuiltInComponents->SlowProperty))
		{
			const TSharedPtr<FTrackInstancePropertyBindings>* RawProperties = SlowProperties.AsPtr();
			for (int32 Index = 0; Index < Num; ++Index)
			{
				TOptional<FInitialValueIndex> ExistingIndex = CacheStorage->FindPropertyIndex(BoundObjects[Index], *RawProperties[Index]->GetPropertyPath());
				if (ExistingIndex)
				{
					InitialValues[Index] = CacheStorage->GetCachedValue(ExistingIndex.GetValue());
				}
				else
				{
					StorageType Value{};
					Traits->GetObjectPropertyValue(BoundObjects[Index], MetaData.template Get<MetaDataIndices>()[Index]..., RawProperties[Index].Get(), Value);

					InitialValues[Index] = Value;
					InitialValueIndices[Index] = CacheStorage->AddInitialValue(BoundObjects[Index], Value, RawProperties[Index].Get());
				}
			}
		}
	}

	void VisitInterrogationAllocation(const FEntityAllocation* Allocation)
	{
		const int32 Num = Allocation->Num();

		TComponentWriter<StorageType>       InitialValues = Allocation->WriteComponents(InitialValueType, WriteContext);
		TComponentReader<FInterrogationKey> OutputKeys    = Allocation->ReadComponents(BuiltInComponents->Interrogation.OutputKey);

		TTuple< TComponentReader<MetaDataTypes>... > MetaData(
			Allocation->ReadComponents(MetaDataComponents.template Get<MetaDataIndices>())...
		);

		const FSparseInterrogationChannelInfo& SparseChannelInfo = Interrogation->GetSparseChannelInfo();

		for (int32 Index = 0; Index < Num; ++Index)
		{
			FInterrogationChannel Channel = OutputKeys[Index].Channel;

			// Did we already cache this value?
			if (const StorageType* CachedValue = ValuesByChannel.Find(Channel))
			{
				InitialValues[Index] = *CachedValue;
				continue;
			}

			const FInterrogationChannelInfo* ChannelInfo = SparseChannelInfo.Find(Channel);
			UObject* Object = ChannelInfo ? ChannelInfo->WeakObject.Get() : nullptr;
			if (!ChannelInfo || !Object || ChannelInfo->PropertyBinding.PropertyName.IsNone())
			{
				continue;
			}

			TOptional< FResolvedFastProperty > Property = FPropertyRegistry::ResolveFastProperty(Object, ChannelInfo->PropertyBinding, CustomAccessors);

			// Retrieve a cached value if possible
			if (CacheStorage)
			{
				const StorageType* CachedValue = nullptr;
				if (!Property.IsSet())
				{
					CachedValue = CacheStorage->FindCachedValue(Object, ChannelInfo->PropertyBinding.PropertyPath);
				}
				else if (const FCustomPropertyIndex* CustomIndex = Property->TryGet<FCustomPropertyIndex>())
				{
					CachedValue = CacheStorage->FindCachedValue(Object, *CustomIndex);
				}
				else
				{
					CachedValue = CacheStorage->FindCachedValue(Object, Property->Get<uint16>());
				}
				if (CachedValue)
				{
					InitialValues[Index] = *CachedValue;
					ValuesByChannel.Add(Channel, *CachedValue);
					continue;
				}
			}

			// No cached value available, must retrieve it now
			TOptional<StorageType> CurrentValue;

			if (!Property.IsSet())
			{
				FTrackInstancePropertyBindings Bindings(ChannelInfo->PropertyBinding.PropertyName, ChannelInfo->PropertyBinding.PropertyPath.ToString());
				Traits->GetObjectPropertyValue(Object, MetaData.template Get<MetaDataIndices>()[Index]..., &Bindings, CurrentValue.Emplace());
			}
			else if (const FCustomPropertyIndex* Custom = Property->TryGet<FCustomPropertyIndex>())
			{
				Traits->GetObjectPropertyValue(Object, MetaData.template Get<MetaDataIndices>()[Index]..., CustomAccessors[Custom->Value], CurrentValue.Emplace());
			}
			else
			{
				const uint16 FastPtrOffset = Property->Get<uint16>();
				Traits->GetObjectPropertyValue(Object, MetaData.template Get<MetaDataIndices>()[Index]..., FastPtrOffset, CurrentValue.Emplace());
			}

			InitialValues[Index] = CurrentValue.GetValue();
			ValuesByChannel.Add(Channel, CurrentValue.GetValue());
		};
	}
};

template<typename PropertyTraits>
struct TInitialValueProcessor : TInitialValueProcessorImpl<PropertyTraits, typename PropertyTraits::MetaDataType, TMakeIntegerSequence<int, PropertyTraits::MetaDataType::Num>>
{
	using Super = TInitialValueProcessorImpl<PropertyTraits, typename PropertyTraits::MetaDataType, TMakeIntegerSequence<int, PropertyTraits::MetaDataType::Num>>;

	TInitialValueProcessor() = delete;

	TInitialValueProcessor(
		const PropertyTraits* InTraits,
		TComponentTypeID<typename PropertyTraits::StorageType> InInitialValueType,
		TArrayView<const FComponentTypeID> InMetaDataComponents,
		ICustomPropertyRegistration* InCustomPropertyRegistration
	) : Super(InTraits, InInitialValueType, InMetaDataComponents, InCustomPropertyRegistration)
	{}
};

template<typename T, typename U = decltype(T::bIsComposite)>
constexpr bool IsCompositePropertyTraits(T*)
{
	return T::bIsComposite;
}
constexpr bool IsCompositePropertyTraits(...)
{
	return true;
}

template<typename PropertyTraits, typename ...MetaDataTypes, int ...MetaDataIndices, typename ...CompositeTypes, int ...CompositeIndices>
struct TPropertyComponentHandlerImpl<PropertyTraits, TPropertyMetaData<MetaDataTypes...>, TIntegerSequence<int, MetaDataIndices...>, TIntegerSequence<int, CompositeIndices...>, CompositeTypes...>
	: IPropertyComponentHandler
{
	static constexpr bool bIsComposite = IsCompositePropertyTraits((PropertyTraits*)nullptr);

	using StorageType        = typename PropertyTraits::StorageType;
	using CompleteSetterTask = std::conditional_t<bIsComposite, TSetCompositePropertyValues<PropertyTraits, CompositeTypes...>, TSetPropertyValues<PropertyTraits>>;

	using PreAnimatedStorageType = TPreAnimatedPropertyStorage<PropertyTraits>;

	TAutoRegisterPreAnimatedStorageID<PreAnimatedStorageType> StorageID;

	TPropertyComponentHandlerImpl()
	{
	}

	virtual bool SupportsProperty(const FPropertyDefinition& Definition, const FProperty& InProperty) const override
	{
		return static_cast<const PropertyTraits*>(Definition.TraitsInstance)->SupportsProperty(InProperty);
	}
	virtual FIntermediatePropertyValue CoercePropertyValue(const FPropertyDefinition& Definition, const FProperty& InProperty, const FSourcePropertyValue& InPropertyValue) const override
	{
		return static_cast<const PropertyTraits*>(Definition.TraitsInstance)->CoercePropertyValue(InProperty, InPropertyValue);
	}
	virtual void UnpackChannels(const FPropertyDefinition& Definition, const FProperty& Property, const FIntermediatePropertyValueConstRef& Value, FUnpackedChannelValues& OutUnpackedValues) const override
	{
		const PropertyTraits* Traits = static_cast<const PropertyTraits*>(Definition.TraitsInstance);
		return Traits->UnpackChannels(*Value.Cast<StorageType>(), Property, OutUnpackedValues);
	}

	virtual TSharedPtr<IPreAnimatedStorage> GetPreAnimatedStateStorage(const FPropertyDefinition& Definition, FPreAnimatedStateExtension* Container) override
	{
		TSharedPtr<PreAnimatedStorageType> Existing = Container->FindStorage(StorageID);
		if (!Existing)
		{
			Existing = MakeShared<PreAnimatedStorageType>(Definition);
			Existing->Initialize(StorageID, Container);
			Container->AddStorage(StorageID, Existing);
		}

		return Existing;
	}

	virtual void ScheduleSetterTasks(const FPropertyDefinition& Definition, TArrayView<const FPropertyCompositeDefinition> Composites, const FPropertyStats& Stats, IEntitySystemScheduler* TaskScheduler, UMovieSceneEntitySystemLinker* Linker)
	{
		ScheduleSetterTasksImpl(Definition, Composites, Stats, TaskScheduler, Linker, FEntityComponentFilter());
	}

	void ScheduleSetterTasksImpl(const FPropertyDefinition& Definition, TArrayView<const FPropertyCompositeDefinition> Composites, const FPropertyStats& Stats, IEntitySystemScheduler* TaskScheduler, UMovieSceneEntitySystemLinker* Linker, const FEntityComponentFilter& AdditionalFilter)
	{
		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

		const PropertyTraits* TraitsInstance = static_cast<const PropertyTraits*>(Definition.TraitsInstance);

		FEntityTaskBuilder()
		.Read(BuiltInComponents->BoundObject)
		.ReadOneOf(BuiltInComponents->CustomPropertyIndex, BuiltInComponents->FastPropertyOffset, BuiltInComponents->SlowProperty)
		.ReadAllOf(Definition.GetMetaDataComponent<MetaDataTypes>(MetaDataIndices)...)
		.ReadAllOf(Composites[CompositeIndices].ComponentTypeID.ReinterpretCast<CompositeTypes>()...)
		.FilterAll({ Definition.PropertyType })
		.FilterNone({ BuiltInComponents->Tags.Ignored })
		.CombineFilter(AdditionalFilter)
		.SetStat(Definition.StatID)
		.SetDesiredThread(Linker->EntityManager.GetGatherThread())
		.template Fork_PerAllocation<CompleteSetterTask>(&Linker->EntityManager, TaskScheduler, TraitsInstance, Definition.CustomPropertyRegistration);

		if constexpr (bIsComposite)
		{
			if (Stats.NumPartialProperties > 0)
			{
				using PartialSetterTask  = TSetPartialPropertyValues<PropertyTraits, CompositeTypes...>;

				FComponentMask CompletePropertyMask;
				for (const FPropertyCompositeDefinition& Composite : Composites)
				{
					CompletePropertyMask.Set(Composite.ComponentTypeID);
				}

				FEntityTaskBuilder()
				.Read(BuiltInComponents->BoundObject)
				.ReadOneOf(BuiltInComponents->CustomPropertyIndex, BuiltInComponents->FastPropertyOffset, BuiltInComponents->SlowProperty)
				.ReadAllOf(Definition.GetMetaDataComponent<MetaDataTypes>(MetaDataIndices)...)
				.ReadAnyOf(Composites[CompositeIndices].ComponentTypeID.ReinterpretCast<CompositeTypes>()...)
				.FilterAny({ CompletePropertyMask })
				.FilterAll({ Definition.PropertyType })
				.FilterNone({ BuiltInComponents->Tags.Ignored })
				.CombineFilter(AdditionalFilter)
				.FilterOut(CompletePropertyMask)
				.SetStat(Definition.StatID)
				.SetDesiredThread(Linker->EntityManager.GetGatherThread())
				.template Fork_PerAllocation<PartialSetterTask>(&Linker->EntityManager, TaskScheduler, TraitsInstance, Definition.CustomPropertyRegistration, Composites);
			}
		}
	}

	virtual void DispatchSetterTasks(const FPropertyDefinition& Definition, TArrayView<const FPropertyCompositeDefinition> Composites, const FPropertyStats& Stats, FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents, UMovieSceneEntitySystemLinker* Linker)
	{
		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

		const PropertyTraits* TraitsInstance = static_cast<const PropertyTraits*>(Definition.TraitsInstance);

		FEntityTaskBuilder()
		.Read(BuiltInComponents->BoundObject)
		.ReadOneOf(BuiltInComponents->CustomPropertyIndex, BuiltInComponents->FastPropertyOffset, BuiltInComponents->SlowProperty)
		.ReadAllOf(Definition.GetMetaDataComponent<MetaDataTypes>(MetaDataIndices)...)
		.ReadAllOf(Composites[CompositeIndices].ComponentTypeID.ReinterpretCast<CompositeTypes>()...)
		.FilterAll({ Definition.PropertyType })
		.FilterNone({ BuiltInComponents->Tags.Ignored })
		.SetStat(Definition.StatID)
		.SetDesiredThread(Linker->EntityManager.GetGatherThread())
		.template Dispatch_PerAllocation<CompleteSetterTask>(&Linker->EntityManager, InPrerequisites, &Subsequents, TraitsInstance, Definition.CustomPropertyRegistration);

		if constexpr (bIsComposite)
		{
			if (Stats.NumPartialProperties > 0)
			{
				using PartialSetterTask  = TSetPartialPropertyValues<PropertyTraits, CompositeTypes...>;

				FComponentMask CompletePropertyMask;
				for (const FPropertyCompositeDefinition& Composite : Composites)
				{
					CompletePropertyMask.Set(Composite.ComponentTypeID);
				}

				FEntityTaskBuilder()
				.Read(BuiltInComponents->BoundObject)
				.ReadOneOf(BuiltInComponents->CustomPropertyIndex, BuiltInComponents->FastPropertyOffset, BuiltInComponents->SlowProperty)
				.ReadAllOf(Definition.GetMetaDataComponent<MetaDataTypes>(MetaDataIndices)...)
				.ReadAnyOf(Composites[CompositeIndices].ComponentTypeID.ReinterpretCast<CompositeTypes>()...)
				.FilterAny({ CompletePropertyMask })
				.FilterAll({ Definition.PropertyType })
				.FilterOut(CompletePropertyMask)
				.FilterNone({ BuiltInComponents->Tags.Ignored })
				.SetStat(Definition.StatID)
				.SetDesiredThread(Linker->EntityManager.GetGatherThread())
				.template Dispatch_PerAllocation<PartialSetterTask>(&Linker->EntityManager, InPrerequisites, &Subsequents, TraitsInstance, Definition.CustomPropertyRegistration, Composites);
			}
		}
	}

	virtual TSharedPtr<IInitialValueProcessor> MakeInitialValueProcessor(const FPropertyDefinition& Definition) override
	{
		return MakeShared<TInitialValueProcessor<PropertyTraits>>(
			static_cast<const PropertyTraits*>(Definition.TraitsInstance),
			Definition.InitialValueType.ReinterpretCast<StorageType>(),
			Definition.MetaDataTypes,
			Definition.CustomPropertyRegistration
			);
	}

	virtual void RecomposeBlendOperational(const FPropertyDefinition& PropertyDefinition, TArrayView<const FPropertyCompositeDefinition> Composites, const FValueDecompositionParams& InParams, UMovieSceneBlenderSystem* Blender, const FIntermediatePropertyValueConstRef& InCurrentValue, TArrayView<FIntermediatePropertyValue> OutResult) override
	{
		RecomposeBlendImpl(PropertyDefinition, Composites, InParams, Blender, *InCurrentValue.Cast<StorageType>(), OutResult);
	}

	void RecomposeBlendImpl(const FPropertyDefinition& PropertyDefinition, TArrayView<const FPropertyCompositeDefinition> Composites, const FValueDecompositionParams& InParams, UMovieSceneBlenderSystem* Blender, const StorageType& InCurrentValue, TArrayView<FIntermediatePropertyValue> OutResults)
	{
		check(OutResults.Num() == InParams.Query.Entities.Num());

		IMovieSceneValueDecomposer* ValueDecomposer = Cast<IMovieSceneValueDecomposer>(Blender);
		if (!ValueDecomposer)
		{
			return;
		}

		FEntityManager& EntityManager = Blender->GetLinker()->EntityManager;
		EntityManager.LockDown();

		constexpr int32 NumComposites = sizeof...(CompositeTypes);
		check(Composites.Num() == NumComposites);

		FAlignedDecomposedValue AlignedOutputs[NumComposites];

		FValueDecompositionParams LocalParams = InParams;

		FGraphEventArray Tasks;
		for (int32 Index = 0; Index < NumComposites; ++Index)
		{
			if ((PropertyDefinition.DoubleCompositeMask & (1 << Index)) == 0)
			{
				continue;
			}

			LocalParams.ResultComponentType = Composites[Index].ComponentTypeID;
			FGraphEventRef Task = ValueDecomposer->DispatchDecomposeTask(LocalParams, &AlignedOutputs[Index]);
			if (Task)
			{
				Tasks.Add(Task);
			}
		}

		if (Tasks.Num() != 0)
		{
			FTaskGraphInterface::Get().WaitUntilTasksComplete(Tasks, ENamedThreads::GameThread);
		}

		// Get the initial value in case we have a value without a full-weighted absolute channel.
		TOptionalComponentReader<StorageType> InitialValueComponent;
		if (InParams.PropertyEntityID)
		{
			TComponentTypeID<StorageType> InitialValueType = PropertyDefinition.InitialValueType.ReinterpretCast<StorageType>();
			InitialValueComponent = EntityManager.ReadComponent(InParams.PropertyEntityID, InitialValueType);
		}

		for (int32 Index = 0; Index < LocalParams.Query.Entities.Num(); ++Index)
		{
			FMovieSceneEntityID EntityID = LocalParams.Query.Entities[Index];

			uint8* Result = reinterpret_cast<uint8*>(OutResults[Index].Cast<StorageType>());

			for (int32 CompositeIndex = 0; CompositeIndex < NumComposites; ++CompositeIndex)
			{
				if ((PropertyDefinition.DoubleCompositeMask & ( 1 << CompositeIndex)) != 0)
				{
					const double* InitialValueComposite = nullptr;
					FAlignedDecomposedValue& AlignedOutput = AlignedOutputs[CompositeIndex];
					if (InitialValueComponent)
					{
						const StorageType* InitialValuePtr = InitialValueComponent.AsPtr();
						InitialValueComposite = reinterpret_cast<const double*>(reinterpret_cast<const uint8*>(InitialValuePtr) + Composites[CompositeIndex].CompositeOffset);
					}

					const double NewComposite = *reinterpret_cast<const double*>(reinterpret_cast<const uint8*>(&InCurrentValue) + Composites[CompositeIndex].CompositeOffset);

					double* RecomposedComposite = reinterpret_cast<double*>(Result + Composites[CompositeIndex].CompositeOffset);
					*RecomposedComposite = AlignedOutput.Value.Recompose(EntityID, NewComposite, InitialValueComposite);
				}
			}
		}

		EntityManager.ReleaseLockDown();
	}

	virtual void RecomposeBlendChannel(const FPropertyDefinition& PropertyDefinition, TArrayView<const FPropertyCompositeDefinition> Composites, int32 CompositeIndex, const FValueDecompositionParams& InParams, UMovieSceneBlenderSystem* Blender, double InCurrentValue, TArrayView<double> OutResults) override
	{
		check(OutResults.Num() == InParams.Query.Entities.Num());

		constexpr int32 NumComposites = sizeof...(CompositeTypes);
		check(Composites.Num() == NumComposites);
		const FPropertyCompositeDefinition& Composite = Composites[CompositeIndex];

		IMovieSceneValueDecomposer* ValueDecomposer = Cast<IMovieSceneValueDecomposer>(Blender);
		if (!ValueDecomposer)
		{
			return;
		}

		FEntityManager& EntityManager = Blender->GetLinker()->EntityManager;
		EntityManager.LockDown();

		FAlignedDecomposedValue AlignedOutput;

		FValueDecompositionParams LocalParams = InParams;

		LocalParams.ResultComponentType = Composite.ComponentTypeID;
		FGraphEventRef Task = ValueDecomposer->DispatchDecomposeTask(LocalParams, &AlignedOutput);
		if (Task)
		{
			FTaskGraphInterface::Get().WaitUntilTaskCompletes(Task, ENamedThreads::GameThread);
		}

		// Get the initial value in case we have a value without a full-weighted absolute channel.
		TOptionalComponentReader<StorageType> InitialValueComponent;
		if (InParams.PropertyEntityID)
		{
			TComponentTypeID<StorageType> InitialValueType = PropertyDefinition.InitialValueType.ReinterpretCast<StorageType>();
			InitialValueComponent = EntityManager.ReadComponent(InParams.PropertyEntityID, InitialValueType);
		}

		for (int32 Index = 0; Index < LocalParams.Query.Entities.Num(); ++Index)
		{
			FMovieSceneEntityID EntityID = LocalParams.Query.Entities[Index];

			if ((PropertyDefinition.DoubleCompositeMask & (1 << CompositeIndex)) != 0)
			{
				const double* InitialValueComposite = nullptr;
				if (InitialValueComponent)
				{
					const StorageType* InitialValuePtr = InitialValueComponent.AsPtr();
					InitialValueComposite = reinterpret_cast<const double*>(reinterpret_cast<const uint8*>(InitialValuePtr) + Composite.CompositeOffset);
				}

				const double RecomposedComposite = AlignedOutput.Value.Recompose(EntityID, InCurrentValue, InitialValueComposite);
				OutResults[Index] = RecomposedComposite;
			}
		}

		EntityManager.ReleaseLockDown();
	}

	virtual void RebuildOperational(const FPropertyDefinition& PropertyDefinition, TArrayView<const FPropertyCompositeDefinition> Composites, const TArrayView<FMovieSceneEntityID>& EntityIDs, UMovieSceneEntitySystemLinker* Linker, FPropertyComponentArrayView OutResult) override
	{
		TArrayView<StorageType> TypedResults = OutResult.ReinterpretCast<StorageType>();

		constexpr int32 NumComposites = sizeof...(CompositeTypes);
		check(Composites.Num() == NumComposites);

		check(TypedResults.Num() == EntityIDs.Num());

		FEntityManager& EntityManager = Linker->EntityManager;

		for (int32 Index = 0; Index < EntityIDs.Num(); ++Index)
		{
			FMovieSceneEntityID EntityID = EntityIDs[Index];
			if (!EntityID)
			{
				continue;
			}

			FEntityDataLocation Location = EntityManager.GetEntity(EntityIDs[Index]).Data;

			PatchCompositeValue(Composites, &TypedResults[Index],
				Location.Allocation->TryReadComponents(Composites[CompositeIndices].ComponentTypeID.ReinterpretCast<CompositeTypes>()).ComponentAtIndex(Location.ComponentOffset)...
			);
		}
	}
};



struct CPropertyTraitsDefineHandler
{
	template<typename T, typename ...Composites>
	auto Requires() -> decltype(T::MakeHandler());
};


template<typename PropertyTraits>
struct TPropertyDefinitionBuilder
{
	TPropertyDefinitionBuilder<PropertyTraits>& AddSoleChannel(TComponentTypeID<typename PropertyTraits::StorageType> InComponent)
	{
		checkf(Definition == &Registry->GetProperties().Last(), TEXT("Cannot re-define a property type after another has been added."));
		checkf(Definition->CompositeSize == 0, TEXT("Property already has a composite."));

		FPropertyCompositeDefinition NewChannel = { InComponent, 0 };
		Registry->CompositeDefinitions.Add(NewChannel);

		Definition->CompositeSize = 1;

		static_assert(!std::is_same_v<typename PropertyTraits::StorageType, float>, "Please use double-precision composites");

		if constexpr (std::is_same_v<typename PropertyTraits::StorageType, double>)
		{
			Definition->DoubleCompositeMask = 1;
		}

		return *this;
	}

	template<int InlineSize>
	TPropertyDefinitionBuilder<PropertyTraits>& SetCustomAccessors(TCustomPropertyRegistration<PropertyTraits, InlineSize>* InCustomAccessors)
	{
		Definition->CustomPropertyRegistration = InCustomAccessors;
		return *this;
	}

	TPropertyDefinitionBuilder<PropertyTraits>& SetStat(TStatId InStatID)
	{
		Definition->StatID = InStatID;
		return *this;
	}

	template<typename BlenderSystemType>
	TPropertyDefinitionBuilder<PropertyTraits>& SetBlenderSystem()
	{
		Definition->BlenderSystemClass = BlenderSystemType::StaticClass();
		return *this;
	}

	TPropertyDefinitionBuilder<PropertyTraits>& SetBlenderSystem(UClass* BlenderSystemClass)
	{
		Definition->BlenderSystemClass = BlenderSystemClass;
		return *this;
	}

	TPropertyDefinitionBuilder<PropertyTraits>& SetDefaultTrackType(TSubclassOf<UMovieSceneTrack> InDefaultTrackType)
	{
		Definition->DefaultTrackType = InDefaultTrackType;
		return *this;
	}

	void Commit()
	{
		if constexpr (TModels_V<CPropertyTraitsDefineHandler, PropertyTraits>)
		{
			Definition->Handler = PropertyTraits::template MakeHandler<typename PropertyTraits::StorageType>();
		}
		else
		{
			Definition->Handler = TPropertyComponentHandler<PropertyTraits, typename PropertyTraits::StorageType>();
		}
		Definition->SetupInitialValueProcessor();
	}

	template<typename HandlerType>
	void Commit(HandlerType&& InHandler)
	{
		Definition->Handler = Forward<HandlerType>(InHandler);
		Definition->SetupInitialValueProcessor();
	}

protected:

	friend FPropertyRegistry;

	TPropertyDefinitionBuilder(FPropertyDefinition* InDefinition, FPropertyRegistry* InRegistry)
		: Definition(InDefinition), Registry(InRegistry)
	{}

	FPropertyDefinition* Definition;
	FPropertyRegistry* Registry;
};


template<typename PropertyTraits, typename... Composites>
struct TCompositePropertyDefinitionBuilder
{
	using StorageType = typename PropertyTraits::StorageType;

	static_assert(sizeof...(Composites) <= 32, "More than 32 composites is not supported");

	TCompositePropertyDefinitionBuilder(FPropertyDefinition* InDefinition, FPropertyRegistry* InRegistry)
		: Definition(InDefinition), Registry(InRegistry)
	{}

	template<typename T>
	TCompositePropertyDefinitionBuilder<PropertyTraits, Composites..., T> AddComposite(TComponentTypeID<T> InComponent, T StorageType::*DataPtr)
	{
		checkf(Definition == &Registry->GetProperties().Last(), TEXT("Cannot re-define a property type after another has been added."));

		const PTRINT CompositeOffset = (PTRINT)&(((StorageType*)0)->*DataPtr);

		FPropertyCompositeDefinition NewChannel = { InComponent, static_cast<uint16>(CompositeOffset) };
		Registry->CompositeDefinitions.Add(NewChannel);

		static_assert(!std::is_same_v<T, float>, "Please use double-precision composites");

		if constexpr (std::is_same_v<T, double>)
		{
			Definition->DoubleCompositeMask |= 1 << Definition->CompositeSize;
		}

		++Definition->CompositeSize;
		return TCompositePropertyDefinitionBuilder<PropertyTraits, Composites..., T>(Definition, Registry);
	}

	TCompositePropertyDefinitionBuilder<PropertyTraits, Composites..., double> AddComposite(TComponentTypeID<double> InComponent, double StorageType::*DataPtr)
	{
		checkf(Definition == &Registry->GetProperties().Last(), TEXT("Cannot re-define a property type after another has been added."));

		const PTRINT CompositeOffset = (PTRINT)&(((StorageType*)0)->*DataPtr);

		FPropertyCompositeDefinition NewChannel = { InComponent, static_cast<uint16>(CompositeOffset) };
		Registry->CompositeDefinitions.Add(NewChannel);

		Definition->DoubleCompositeMask |= 1 << Definition->CompositeSize;

		++Definition->CompositeSize;
		return TCompositePropertyDefinitionBuilder<PropertyTraits, Composites..., double>(Definition, Registry);
	}

	template<int InlineSize>
	TCompositePropertyDefinitionBuilder<PropertyTraits, Composites...>& SetCustomAccessors(TCustomPropertyRegistration<PropertyTraits, InlineSize>* InCustomAccessors)
	{
		Definition->CustomPropertyRegistration = InCustomAccessors;
		return *this;
	}

	template<typename BlenderSystemType>
	TCompositePropertyDefinitionBuilder<PropertyTraits, Composites...>& SetBlenderSystem()
	{
		Definition->BlenderSystemClass = BlenderSystemType::StaticClass();
		return *this;
	}

	TCompositePropertyDefinitionBuilder<PropertyTraits, Composites...>& SetBlenderSystem(UClass* BlenderSystemClass)
	{
		Definition->BlenderSystemClass = BlenderSystemClass;
		return *this;
	}

	TCompositePropertyDefinitionBuilder<PropertyTraits, Composites...>& SetDefaultTrackType(TSubclassOf<UMovieSceneTrack> InDefaultTrackType)
	{
		Definition->DefaultTrackType = InDefaultTrackType;
		return *this;
	}

	void Commit()
	{
		if constexpr (TModels_V<CPropertyTraitsDefineHandler, PropertyTraits, Composites...>)
		{
			Definition->Handler = PropertyTraits::template MakeHandler<Composites...>();
		}
		else
		{
			Definition->Handler = TPropertyComponentHandler<PropertyTraits, Composites...>();
		}
		Definition->SetupInitialValueProcessor();
	}
	
	template<typename HandlerType>
	void Commit(HandlerType&& InHandler)
	{
		Definition->Handler = Forward<HandlerType>(InHandler);
		Definition->SetupInitialValueProcessor();
	}

private:

	FPropertyDefinition* Definition;
	FPropertyRegistry* Registry;
};


struct FPropertyRecomposerPropertyInfo
{
	static constexpr uint16 INVALID_BLEND_CHANNEL = uint16(-1);

	uint16 BlendChannel = INVALID_BLEND_CHANNEL;
	UMovieSceneBlenderSystem* BlenderSystem = nullptr;
	FMovieSceneEntityID PropertyEntityID;

	static FPropertyRecomposerPropertyInfo Invalid()
	{ 
		return FPropertyRecomposerPropertyInfo { INVALID_BLEND_CHANNEL, nullptr, FMovieSceneEntityID::Invalid() };
	}
};

DECLARE_DELEGATE_RetVal_TwoParams(FPropertyRecomposerPropertyInfo, FOnGetPropertyRecomposerPropertyInfo, FMovieSceneEntityID, UObject*);

struct FPropertyRecomposerImpl
{
	template<typename PropertyTraits>
	TRecompositionResult<typename PropertyTraits::StorageType> RecomposeBlendOperational(const TPropertyComponents<PropertyTraits>& InComponents, const FDecompositionQuery& InQuery, const typename PropertyTraits::StorageType& InCurrentValue);

	MOVIESCENE_API FRecompositionResult RecomposeBlendOperational(const FPropertyDefinition& InPropertyDefinition, const FDecompositionQuery& InQuery, const FIntermediatePropertyValueConstRef& InCurrentValue);

	FOnGetPropertyRecomposerPropertyInfo OnGetPropertyInfo;
};

template<typename PropertyTraits>
TRecompositionResult<typename PropertyTraits::StorageType> FPropertyRecomposerImpl::RecomposeBlendOperational(const TPropertyComponents<PropertyTraits>& Components, const FDecompositionQuery& InQuery, const typename PropertyTraits::StorageType& InCurrentValue)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FPropertyDefinition& PropertyDefinition = BuiltInComponents->PropertyRegistry.GetDefinition(Components.CompositeID);

	TRecompositionResult<typename PropertyTraits::StorageType> Result(InCurrentValue, InQuery.Entities.Num());

	if (InQuery.Entities.Num() == 0)
	{
		return Result;
	}

	const FPropertyRecomposerPropertyInfo Property = OnGetPropertyInfo.Execute(InQuery.Entities[0], InQuery.Object);

	if (Property.BlendChannel == FPropertyRecomposerPropertyInfo::INVALID_BLEND_CHANNEL)
	{
		return Result;
	}

	UMovieSceneBlenderSystem* Blender = Property.BlenderSystem;
	if (!Blender)
	{
		return Result;
	}

	FValueDecompositionParams Params;
	Params.Query = InQuery;
	Params.PropertyEntityID = Property.PropertyEntityID;
	Params.DecomposeBlendChannel = Property.BlendChannel;
	Params.PropertyTag = PropertyDefinition.PropertyType;

	TArrayView<const FPropertyCompositeDefinition> Composites = BuiltInComponents->PropertyRegistry.GetComposites(PropertyDefinition);

	// Make an array of type-erased values that reference each element of Result.Values.
	// These will be written to inside RecomposeBlendOperational on success
	TArray<FIntermediatePropertyValue, TInlineAllocator<16>> ErasedValues;
	ErasedValues.Reserve(Result.Values.Num());
	for (int32 Index = 0; Index < Result.Values.Num(); ++Index)
	{
		ErasedValues.Emplace(FIntermediatePropertyValue::FromAddress(&Result.Values[Index]));
	}

	FIntermediatePropertyValueConstRef CurrentValueErased(&InCurrentValue);
	PropertyDefinition.Handler->RecomposeBlendOperational(PropertyDefinition, Composites, Params, Blender, CurrentValueErased, ErasedValues);
	return Result;
}


} // namespace MovieScene
} // namespace UE


