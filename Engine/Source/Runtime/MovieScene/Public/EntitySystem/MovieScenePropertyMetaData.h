// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Models.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneComponentTypeIDs.h"

namespace UE::MovieScene
{

struct FComponentRegistry;

template<typename>
struct TPropertyMetaDataComponents;

template<typename ...MetaDataTypes>
struct TPropertyMetaData
{
	static constexpr int32 Num = sizeof...(MetaDataTypes);
};

struct CPublicMetaDataRetrievable
{
	template<typename T>
	auto Requires() -> typename T::PublicMetaData;
};

template<typename PropertyTraits, bool B = TModels_V<CPublicMetaDataRetrievable, PropertyTraits>>
struct TGetPublicPropertyMetaData;

template<typename PropertyTraits>
struct TGetPublicPropertyMetaData<PropertyTraits, true>
{
	using Type = typename PropertyTraits::PublicMetaData;
};
template<typename PropertyTraits>
struct TGetPublicPropertyMetaData<PropertyTraits, false>
{
	using Type = typename PropertyTraits::MetaDataType;
};

template<typename PropertyTraits>
using TGetPublicPropertyMetaDataT = typename TGetPublicPropertyMetaData<PropertyTraits>::Type;

template<typename ...MetaDataTypes>
struct TPropertyMetaDataComponents<TPropertyMetaData<MetaDataTypes...>> : TComponentTypeIDs<MetaDataTypes...>
{
	template<typename T> using MakeTCHARPtr = const TCHAR*;

	using TComponentTypeIDs<MetaDataTypes...>::Initialize;

	// #include "EntitySystem/MovieScenePropertyMetaData.inl" for definition
	//
	void Initialize(FComponentRegistry* ComponentRegistry, MakeTCHARPtr<MetaDataTypes>... DebugNames);
};

} // namespace UE::MovieScene


