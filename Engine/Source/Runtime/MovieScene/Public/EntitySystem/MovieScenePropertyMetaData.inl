// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieScenePropertyMetaData.h"
#include "EntitySystem/MovieScenePropertyRegistry.h"
#include "EntitySystem/MovieSceneComponentRegistry.h"

namespace UE
{
namespace MovieScene
{


template<typename ...MetaDataTypes>
void TPropertyMetaDataComponents<TPropertyMetaData<MetaDataTypes...>>::Initialize(FComponentRegistry* ComponentRegistry, MakeTCHARPtr<MetaDataTypes>... DebugNames)
{
	Initialize(ComponentRegistry->NewComponentType<MetaDataTypes...>(DebugNames, EComponentTypeFlags::CopyToOutput)...);
}


} // namespace MovieScene
} // namespace UE


