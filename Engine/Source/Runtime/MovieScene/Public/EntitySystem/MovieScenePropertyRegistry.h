// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreTypes.h"
#include "EntitySystem/IMovieScenePropertyComponentHandler.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "EntitySystem/MovieScenePropertySystemTypes.h"
#include "EntitySystem/MovieSceneSystemTaskDependencies.h"
#include "Math/NumericLimits.h"
#include "Misc/AssertionMacros.h"
#include "Misc/GeneratedTypeName.h"
#include "Misc/InlineValue.h"
#include "Misc/Optional.h"
#include "Misc/TVariant.h"
#include "Stats/Stats.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/NameTypes.h"
#include "Templates/SubclassOf.h"

#include <initializer_list>

class FTrackInstancePropertyBindings;
class UClass;
class UMovieSceneBlenderSystem;
class UMovieSceneTrack;
class UMovieSceneEntitySystemLinker;
class UObject;
struct FMovieScenePropertyBinding;

namespace UE
{
namespace MovieScene
{

struct FFloatDecompositionParams;
struct FPropertyCompositeDefinition;
struct FPropertyDefinition;
struct IPropertyTraits;

DECLARE_CYCLE_STAT(TEXT("Apply properties"), MovieSceneEval_ApplyProperties,  STATGROUP_MovieSceneECS);

/**
 * Stats pertaining to a given type of property including how many properties exist in the linker,
 * and how many of those are partially animated
 */
struct FPropertyStats
{
	/** The total number of properties currently animated, including partial properties */
	int32 NumProperties = 0;

	/** The number of properties partially animated */
	int32 NumPartialProperties = 0;
};


/**
 * Structure defining a type of property that can be animated by sequencer
 */
struct FPropertyDefinition
{
	FPropertyDefinition() = default;

	FPropertyDefinition(
			uint16 InVariableSizeCompositeOffset,
			FComponentTypeID InPropertyType, FComponentTypeID InInitialValueType)
		: CustomPropertyRegistration(nullptr)
		, TraitsInstance(nullptr)
		, DoubleCompositeMask(0)
		, VariableSizeCompositeOffset(InVariableSizeCompositeOffset)
		, CompositeSize(0)
		, PropertyType(InPropertyType)
		, InitialValueType(InInitialValueType)
		, BlenderSystemClass(nullptr)
	{
	}

	FPropertyDefinition(FPropertyDefinition&&) = default;
	FPropertyDefinition(const FPropertyDefinition&) = delete;

	/**
	 * Return a typed component ID for the meta data at the specified index.
	 * Care should obviously be taken here to ensure that the meta data type being used
	 * matches the TPropertyComponents traits that constructed this property definition.
	 */
	template<typename T>
	TComponentTypeID<T> GetMetaDataComponent(int32 Index) const
	{
		return MetaDataTypes[Index].ReinterpretCast<T>();
	}

	/**
	 * Sets up an initial value processor for this property type
	 */
	MOVIESCENE_API void SetupInitialValueProcessor() const;

	/** Pointer to a custom getter/setter registry for short circuiting the UObject VM. Must outlive this definitions lifetime (usually these are static or singletons) */
	ICustomPropertyRegistration* CustomPropertyRegistration = nullptr;

	/** Pointer to the traits instance */
	const IPropertyTraits* TraitsInstance = nullptr;

	/** A default track type to use for this property */
	TSubclassOf<UMovieSceneTrack> DefaultTrackType;

	/** Stat ID for this property type */
	TStatId StatID;

	/** A mask of which composite indices pertain to doubles */
	uint32 DoubleCompositeMask = 0;

	/** The number of channels that this property comprises */
	uint16 VariableSizeCompositeOffset = INDEX_NONE;

	/** The number of channels that this property comprises */
	uint16 CompositeSize = 0;

	/** The component type or tag of the property itself */
	FComponentTypeID PropertyType;

	/** The component type for this property's inital value (used for relative and/or additive blending) */
	FComponentTypeID InitialValueType;

	/** MetaData types */
	TArrayView<const FComponentTypeID> MetaDataTypes;

	/** The blender system to use by default (if specified here) to blend composites of this property (it can be overriden per-entity with a blender system component) */
	UClass* BlenderSystemClass;

	/** Implementation of type specific property actions such as applying properties from entities or recomposing values */
	TInlineValue<IPropertyComponentHandler, 32> Handler;
};

/** A generic definition of a composite channel that contributes to a property */
struct FPropertyCompositeDefinition
{
	/** The type of component that contains the value for this channel (ie a TComponentTypeID<float>) */
	FComponentTypeID ComponentTypeID;

	/**
	 * The offset of the member variable within the operational type of the property in bytes.
	 * Ie for FIntermediate3DTransform::T_Z, the composite offset is 8 Bytes.
	 */
	uint16 CompositeOffset;
};

/** Type aliases for a property that resolved to either a fast pointer offset (type index 0), or a custom property index (specific to the path of the property - type index 1) */
using FResolvedFastProperty = TVariant<uint16, UE::MovieScene::FCustomPropertyIndex>;
/** Type aliases for a property that resolved to either a fast pointer offset (type index 0), or a custom property index (specific to the path of the property - type index 1) with a fallback to a slow property binding (type index 2) */
using FResolvedProperty = TVariant<uint16, UE::MovieScene::FCustomPropertyIndex, TSharedPtr<FTrackInstancePropertyBindings>>;

template<typename PropertyTraits, typename... Composites> struct TCompositePropertyDefinitionBuilder;
template<typename PropertyTraits> struct TPropertyDefinitionBuilder;

/**
 * Central registry of all property types animatable by sequencer.
 * Once registered, properties cannot be de-registered. This vastly simplifies the lifetime and ID management of the class
 */
class FPropertyRegistry
{
public:

	FPropertyRegistry() = default;

	FPropertyRegistry(FPropertyRegistry&&) = delete;
	FPropertyRegistry(const FPropertyRegistry&) = delete;

	/**
	 * Resolve a property to either a fast ptr offset, or a custom property accessor based on the specified array
	 *
	 * @param Object          The object to resolve the property for
	 * @param PropertyBinding The property binding to resolve
	 * @param CustomAccessors A view to an array of custom accessors (as retrieved from ICustomPropertyRegistration::GetAccessors)
	 * @return An optional variant specifying the resolved property if it resolved successfully
	 */
	static MOVIESCENE_API TOptional< FResolvedFastProperty > ResolveFastProperty(UObject* Object, const FMovieScenePropertyBinding& PropertyBinding, FCustomAccessorView CustomAccessors);

	/**
	 * Resolve a property to either a fast ptr offset, or a custom property accessor based on the specified array falling back to a slow instance binding if possible
	 *
	 * @param Object          The object to resolve the property for
	 * @param PropertyBinding The property binding to resolve
	 * @param CustomAccessors A view to an array of custom accessors (as retrieved from ICustomPropertyRegistration::GetAccessors)
	 * @return An optional variant specifying the resolved property if it resolved successfully
	 */
	static MOVIESCENE_API TOptional< FResolvedProperty > ResolveProperty(UObject* Object, const FMovieScenePropertyBinding& PropertyBinding, FCustomAccessorView CustomAccessors);

	/**
	 * Define a new animatable composite property type from its components.
	 * 
	 * @param InOutPropertyComponents  The property's components that are used for animating this property. TPropertyComponents::CompositeID is written to.
	 * @return A builder class that should be used to define the composites that contribute to this property
	 */
	template<typename PropertyTraits>
	TCompositePropertyDefinitionBuilder<PropertyTraits> DefineCompositeProperty(TPropertyComponents<PropertyTraits>& InOutPropertyComponents, const TCHAR* InStatName)
	{
		DefinePropertyImpl(InOutPropertyComponents, InStatName);
		FPropertyDefinition* Property = &Properties[InOutPropertyComponents.CompositeID.AsIndex()];
		Property->TraitsInstance = &InOutPropertyComponents.PropertyTraits;
		return TCompositePropertyDefinitionBuilder<PropertyTraits>(Property, this);
	}

	/**
	 * Define a new animatable property type from its components.
	 * 
	 * @param InOutPropertyComponents  The property's components that are used for animating this property. TPropertyComponents::CompositeID is written to.
	 * @return A builder class that should be used to define the composites that contribute to this property
	 */
	template<typename PropertyTraits>
	TPropertyDefinitionBuilder<PropertyTraits> DefineProperty(TPropertyComponents<PropertyTraits>& InOutPropertyComponents, const TCHAR* InStatName)
	{
		DefinePropertyImpl(InOutPropertyComponents, InStatName);
		FPropertyDefinition* Property = &Properties[InOutPropertyComponents.CompositeID.AsIndex()];
		Property->TraitsInstance = &InOutPropertyComponents.PropertyTraits;
		return TPropertyDefinitionBuilder<PropertyTraits>(Property, this);
	}

	/**
	 * Retrieve a property definition from its ID
	 */
	const FPropertyDefinition& GetDefinition(FCompositePropertyTypeID PropertyID) const
	{
		return Properties[PropertyID.TypeIndex];
	}

	/**
	 * Access all the properties currently registered
	 */
	TArrayView<const FPropertyDefinition> GetProperties() const
	{
		return Properties;
	}

	/**
	 * Access all the properties currently registered
	 */
	MOVIESCENE_API const FPropertyDefinition* FindPropertyDefinition(FComponentTypeID ComponentTypeID) const;

	/**
	 * Retrieve a generic representation of all the composites that contribute to a given property
	 */
	TArrayView<const FPropertyCompositeDefinition> GetComposites(const FPropertyDefinition& Property) const
	{
		const int32 CompositeOffset = Property.VariableSizeCompositeOffset;
		const int32 NumComposites   = Property.CompositeSize;
		return MakeArrayView(CompositeDefinitions.GetData() + CompositeOffset, NumComposites);
	}

	/**
	 * Retrieve a generic representation of all the composites that contribute to a given property
	 */
	TArrayView<const FPropertyCompositeDefinition> GetComposites(FCompositePropertyTypeID PropertyID) const
	{
		return GetComposites(GetDefinition(PropertyID));
	}

private:

	template<typename PropertyTraits>
	friend struct TPropertyDefinitionBuilder;
	
	template<typename PropertyTraits, typename... Composites>
	friend struct TCompositePropertyDefinitionBuilder;

	/**
	 * Define a new animatable property type from its components.
	 * 
	 * @param InOutPropertyComponents  The property's components that are used for animating this property. TPropertyComponents::CompositeID is written to.
	 * @return A builder class that should be used to define the composites that contribute to this property
	 */
	template<typename PropertyTraits>
	void DefinePropertyImpl(TPropertyComponents<PropertyTraits>& InOutPropertyComponents, const TCHAR* InStatName)
	{
		using StorageType = typename PropertyTraits::StorageType;

		const int32 CompositeOffsetInt32 = CompositeDefinitions.Num();
		checkf(CompositeOffsetInt32 <= MAX_uint16, TEXT("Maximum number of composite definitions reached"));

		const uint16 CompositeOffset = static_cast<uint16>(CompositeOffsetInt32);

		TStatId StatID;

#if STATS || ENABLE_STATNAMEDEVENTS

	#if STATS
		// Use FDynamicStats to create the stat in the right stat group if possible
		StatID = FDynamicStats::CreateStatId<STAT_GROUP_TO_FStatGroup(STATGROUP_MovieSceneECS)>( FName(InStatName) );
	#else
		// Otherwise just make a named stat
		const auto& ConversionData = StringCast<PROFILER_CHAR>(InStatName);
		const int32 NumStorageChars = (ConversionData.Length() + 1);	//length doesn't include null terminator

		// We leak this string
		PROFILER_CHAR* StoragePtr = new PROFILER_CHAR[NumStorageChars];
		FMemory::Memcpy(StoragePtr, ConversionData.Get(), NumStorageChars * sizeof(PROFILER_CHAR));

		StatID = TStatId(StoragePtr);
	#endif
#endif

		FPropertyDefinition NewDefinition(
			CompositeOffset, 
			InOutPropertyComponents.PropertyTag,
			InOutPropertyComponents.InitialValue);

		NewDefinition.StatID = StatID;

		NewDefinition.MetaDataTypes = InOutPropertyComponents.MetaDataComponents.GetTypes();
		checkf(!NewDefinition.MetaDataTypes.Contains(FComponentTypeID()), TEXT("Property meta-data component is not defined"));

		const int32 NewPropertyIndex = Properties.Add(MoveTemp(NewDefinition));

		checkf(!InOutPropertyComponents.CompositeID, TEXT("Property already defined"));
		static_cast<FCompositePropertyTypeID&>(InOutPropertyComponents.CompositeID) = FCompositePropertyTypeID::FromIndex(NewPropertyIndex);
	}

	TArray<FPropertyDefinition> Properties;

	TArray<FPropertyCompositeDefinition> CompositeDefinitions;
};




} // namespace MovieScene
} // namespace UE
