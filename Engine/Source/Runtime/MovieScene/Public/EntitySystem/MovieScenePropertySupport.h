// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Class.h"
#include "UObject/UnrealType.h"
#include "UObject/TextProperty.h"
#include "UObject/NoExportTypes.h"
#include "UObject/StrProperty.h"
#include "UObject/EnumProperty.h"

namespace  UE::MovieScene
{

template<typename T>
concept CBaseStructureAccessible = requires()
{
	TBaseStructure<T>::Get();
};

template<typename CppType>
struct TPropertyMatch;


template<typename T>
struct TVariantPropertyMatch
{
	static bool SupportsProperty(const FProperty& InProperty)
	{
		if (const FStructProperty* StructProp = CastField<FStructProperty>(&InProperty))
		{
			return StructProp->Struct == TVariantStructure<T>::Get();
		}
		return false;
	}
};

template<typename T> requires CBaseStructureAccessible<T>
struct TPropertyMatch<T>
{
	static bool SupportsProperty(const FProperty& InProperty)
	{
		if (const FStructProperty* StructProp = CastField<FStructProperty>(&InProperty))
		{
			return StructProp->Struct == TBaseStructure<T>::Get();
		}
		return false;
	}
};
template<>
struct TPropertyMatch<float>
{
	static bool SupportsProperty(const FProperty& InProperty)
	{
		return InProperty.IsA<FFloatProperty>();
	}
};
template<>
struct TPropertyMatch<double>
{
	static bool SupportsProperty(const FProperty& InProperty)
	{
		return InProperty.IsA<FDoubleProperty>();
	}
};
template<>
struct TPropertyMatch<FString>
{
	static bool SupportsProperty(const FProperty& InProperty)
	{
		return InProperty.IsA<FStrProperty>();
	}
};
template<>
struct TPropertyMatch<FText>
{
	static bool SupportsProperty(const FProperty& InProperty)
	{
		return InProperty.IsA<FTextProperty>();
	}
};
template<>
struct TPropertyMatch<bool>
{
	static bool SupportsProperty(const FProperty& InProperty)
	{
		return InProperty.IsA<FBoolProperty>();
	}
};

template<>
struct TPropertyMatch<uint8>
{
	static bool SupportsProperty(const FProperty& InProperty)
	{
		return InProperty.IsA<FByteProperty>() || InProperty.IsA<FEnumProperty>();
	}
};

template<>
struct TPropertyMatch<int8>
{
	static bool SupportsProperty(const FProperty& InProperty)
	{
		return InProperty.IsA<FInt8Property>();
	}
};
template<>
struct TPropertyMatch<int16>
{
	static bool SupportsProperty(const FProperty& InProperty)
	{
		return InProperty.IsA<FInt16Property>();
	}
};
template<>
struct TPropertyMatch<int32>
{
	static bool SupportsProperty(const FProperty& InProperty)
	{
		return InProperty.IsA<FIntProperty>();
	}
};
template<>
struct TPropertyMatch<int64>
{
	static bool SupportsProperty(const FProperty& InProperty)
	{
		return InProperty.IsA<FInt64Property>();
	}
};
template<>
struct TPropertyMatch<UObject*>
{
	static bool SupportsProperty(const FProperty& InProperty)
	{
		return InProperty.IsA<FObjectPropertyBase>();
	}
};

template<>
struct TPropertyMatch<FVector2f> : TVariantPropertyMatch<FVector2f>
{
};
template<>
struct TPropertyMatch<FVector3f> : TVariantPropertyMatch<FVector3f>
{
};
template<>
struct TPropertyMatch<FVector4f> : TVariantPropertyMatch<FVector4f>
{
};
template<>
struct TPropertyMatch<FVector2d>
{
	static bool SupportsProperty(const FProperty& InProperty)
	{
		if (const FStructProperty* StructProp = CastField<FStructProperty>(&InProperty))
		{
			return StructProp->Struct == TBaseStructure<FVector2D>::Get();
		}
		return false;
	}
};
template<>
struct TPropertyMatch<FVector3d>
{
	static bool SupportsProperty(const FProperty& InProperty)
	{
		if (const FStructProperty* StructProp = CastField<FStructProperty>(&InProperty))
		{
			return StructProp->Struct == TVariantStructure<FVector3d>::Get() ||
				StructProp->Struct == TBaseStructure<FVector>::Get();
		}
		return false;
	}
};
template<>
struct TPropertyMatch<FVector4d>
{
	static bool SupportsProperty(const FProperty& InProperty)
	{
		if (const FStructProperty* StructProp = CastField<FStructProperty>(&InProperty))
		{
			return StructProp->Struct == TVariantStructure<FVector4d>::Get() ||
				StructProp->Struct == TBaseStructure<FVector4>::Get();
		}
		return false;
	}
};

} // UE::MovieScene