// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/NameTypes.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTypeTraits.h"
#include "Containers/StringView.h"
#include "EntitySystem/MovieSceneComponentAccessors.h"
#include "EntitySystem/MovieScenePropertyMetaData.h"

class UClass;
class FTrackInstancePropertyBindings;


namespace UE::MovieScene
{

struct FCustomPropertyIndex
{
	uint16 Value;
};

struct FCompositePropertyTypeID
{
	FCompositePropertyTypeID() : TypeIndex(INDEX_NONE) {}

	static FCompositePropertyTypeID FromIndex(int32 Index)
	{
		return FCompositePropertyTypeID{ Index };
	}

	int32 AsIndex() const
	{
		return TypeIndex;
	}

	explicit operator bool() const
	{
		return TypeIndex != INDEX_NONE;
	}

private:
	FCompositePropertyTypeID(int32 InTypeIndex) : TypeIndex(InTypeIndex) {}

	friend class FPropertyRegistry;
	int32 TypeIndex;
};


template<typename PropertyTraits>
struct TCompositePropertyTypeID : FCompositePropertyTypeID
{};

namespace Private
{
	/** Utility global flag to determine whether a given type has a nested type called CustomAccessorStorageType */
	template<typename, typename = void>
	constexpr bool PropertyTraitsHaveCustomAccessorStorageType = false;

	template<typename T>
	constexpr bool PropertyTraitsHaveCustomAccessorStorageType<T, std::void_t<decltype(sizeof(typename T::CustomAccessorStorageType))>> = true;

	/** Utility class for selecting a property traits' CustomAccessorStorageType, or StorageType if not defined */
	template<typename PropertyTraits, bool Custom>
	struct TCustomPropertyAccessorStorageTypeImpl;

	template<typename PropertyTraits>
	struct TCustomPropertyAccessorStorageTypeImpl<PropertyTraits, false>
	{
		using Value = typename PropertyTraits::StorageType;
	};
	
	template<typename PropertyTraits>
	struct TCustomPropertyAccessorStorageTypeImpl<PropertyTraits, true>
	{
		using Value = typename PropertyTraits::CustomAccessorStorageType;
	};
	
	template<typename PropertyTraits>
	struct TCustomPropertyAccessorStorageType : TCustomPropertyAccessorStorageTypeImpl<PropertyTraits, PropertyTraitsHaveCustomAccessorStorageType<PropertyTraits>>
	{};

	namespace Tests
	{
		struct TestNormal
		{
			using StorageType = bool;
		};
		static_assert(PropertyTraitsHaveCustomAccessorStorageType<TestNormal> == false, "Normal has no custom storage type");
		static_assert(std::is_same_v<TCustomPropertyAccessorStorageType<TestNormal>::Value, bool>, "Normal has bool storage type");

		struct TestCustom
		{
			using StorageType = bool;
			using CustomAccessorStorageType = int;
		};
		static_assert(PropertyTraitsHaveCustomAccessorStorageType<TestCustom> == true, "Custom does have custom storage type");
		static_assert(std::is_same_v<TCustomPropertyAccessorStorageType<TestCustom>::Value, int>, "Custom has int storage type");
	}
}

/**
 * Structure that defines 2 static function pointers that are to be used for retrieving and applying properties of a given type
 */
template<typename PropertyTraits, typename MetaDataType>
struct TCustomPropertyAccessorFunctionsImpl;

template<typename PropertyTraits, typename ...MetaDataTypes>
struct TCustomPropertyAccessorFunctionsImpl<PropertyTraits, TPropertyMetaData<MetaDataTypes...>>
{
	using StorageType = typename Private::TCustomPropertyAccessorStorageType<PropertyTraits>::Value;
	using ParamType   = typename TCallTraits<StorageType>::ParamType;

	using GetterFunc = StorageType (*)(const UObject* Object, MetaDataTypes...);
	using SetterFunc = void        (*)(UObject* Object, MetaDataTypes..., ParamType Value);

	/** Function pointer to be used for retrieving an object's current property */
	GetterFunc Getter;

	/** Function pointer to be used for applying a new value to an object's property */
	SetterFunc Setter;
};


template<typename PropertyTraits>
struct TCustomPropertyAccessorFunctions
	: TCustomPropertyAccessorFunctionsImpl<PropertyTraits, TGetPublicPropertyMetaDataT<PropertyTraits>>
{
};

struct FCustomPropertyAccessor
{
	/** The class of the object that the accessor applies to */
	UClass* Class;

	/** The complete path name to the property from the class specified above */
	FName PropertyPath;

	/** (Optional) An additional tag that should be applied alongside this property accessor component */
	FComponentTypeID AdditionalTag;
};

/**
 * Complete information required for applying a custom getter/setter to an object
 */
template<typename PropertyTraits>
struct TCustomPropertyAccessor : FCustomPropertyAccessor
{
	TCustomPropertyAccessor(UClass* InClass, FName InPropertyPath, const TCustomPropertyAccessorFunctions<PropertyTraits>& InFunctions)
		: FCustomPropertyAccessor{ InClass, InPropertyPath }
		, Functions(InFunctions)
	{}

	/** Function pointers to use for interacting with the property */
	TCustomPropertyAccessorFunctions<PropertyTraits> Functions;
};


struct FCustomAccessorView
{
	FCustomAccessorView()
		: Base(nullptr)
		, ViewNum(0)
		, Stride(0)
	{}

	template<typename T, typename Allocator>
	explicit FCustomAccessorView(const TArray<T, Allocator>& InArray)
		: Base(reinterpret_cast<const uint8*>(InArray.GetData()))
		, ViewNum(InArray.Num())
		, Stride(sizeof(T))
	{}

	const FCustomPropertyAccessor& operator[](int32 InIndex) const
	{
		return *reinterpret_cast<const FCustomPropertyAccessor*>(Base + InIndex*Stride);
	}

	int32 Num() const
	{
		return ViewNum;
	}

	int32 FindCustomAccessorIndex(UClass* ClassType, FName PropertyPath) const
	{
		UClass* StopIterationAt = UObject::StaticClass();

		while (ClassType != StopIterationAt)
		{
			for (int32 Index = 0; Index < ViewNum; ++Index)
			{
				const FCustomPropertyAccessor& Accessor = (*this)[Index];
				if (Accessor.Class == ClassType && Accessor.PropertyPath == PropertyPath)
				{
					return Index;
				}
			}
			ClassType = ClassType->GetSuperClass();
		}

		return INDEX_NONE;
	}

private:
	const uint8* Base;
	int32 ViewNum;
	int32 Stride;
};


struct ICustomPropertyRegistration
{
	virtual ~ICustomPropertyRegistration() = default;

	virtual FCustomAccessorView GetAccessors() const = 0;
};

#if WITH_EDITOR
MOVIESCENE_API void AddGlobalCustomAccessor(const UClass* ClassType, FName PropertyPath);
MOVIESCENE_API void RemoveGlobalCustomAccessor(const UClass* ClassType, FName PropertyPath);
MOVIESCENE_API bool GlobalCustomAccessorExists(const UClass* ClassType, TStringView<WIDECHAR> PropertyPath);
MOVIESCENE_API bool GlobalCustomAccessorExists(const UClass* ClassType, TStringView<ANSICHAR> PropertyPath);
#endif // WITH_EDITOR

/** Generally static collection of accessors for a given type of property */
template<typename PropertyTraits, int InlineSize = 8>
struct TCustomPropertyRegistration : ICustomPropertyRegistration
{
	using GetterFunc = typename TCustomPropertyAccessorFunctions<PropertyTraits>::GetterFunc;
	using SetterFunc = typename TCustomPropertyAccessorFunctions<PropertyTraits>::SetterFunc;

	virtual FCustomAccessorView GetAccessors() const override
	{
		return FCustomAccessorView(CustomAccessors);
	}

	void Add(UClass* ClassType, FName PropertyName, GetterFunc Getter, SetterFunc Setter)
	{
		CustomAccessors.Add(TCustomPropertyAccessor<PropertyTraits>{ ClassType, PropertyName, { Getter, Setter } });
#if WITH_EDITOR
		AddGlobalCustomAccessor(ClassType, PropertyName);
#endif
	}

	void Remove(UClass* ClassType, FName PropertyName)
	{
		for (int32 Index = CustomAccessors.Num()-1; Index >= 0; --Index)
		{
			TCustomPropertyAccessor<PropertyTraits>& Accessor = CustomAccessors[Index];
			if (Accessor.Class == ClassType && Accessor.PropertyPath == PropertyName)
			{
#if WITH_EDITOR
				RemoveGlobalCustomAccessor(Accessor.Class, Accessor.PropertyPath);
#endif
				// Null out the entry rather than remove it because we don't want to invalidate any cached array indices
				Accessor.Class = nullptr;
				Accessor.PropertyPath = NAME_None;
			}
		}
	}

	void RemoveAll(UClass* ClassType)
	{
		for (int32 Index = CustomAccessors.Num()-1; Index >= 0; --Index)
		{
			TCustomPropertyAccessor<PropertyTraits>& Accessor = CustomAccessors[Index];
			if (Accessor.Class == ClassType)
			{
#if WITH_EDITOR
				RemoveGlobalCustomAccessor(Accessor.Class, Accessor.PropertyPath);
#endif
				// Null out the entry rather than remove it because we don't want to invalidate any cached array indices
				Accessor.Class = nullptr;
				Accessor.PropertyPath = NAME_None;
			}
		}
	}

protected:

	/** */
	TArray<TCustomPropertyAccessor<PropertyTraits>, TInlineAllocator<InlineSize>> CustomAccessors;
};

/**
 * User-defined property type that is represented as an UE::MovieScene::FPropertyDefinition within UE::MovieScene::FPropertyRegistry
 *
 * This type must be templated on a traits class that defines the storage type for the property, and methods for retrieving and assigning
 * the property value from a UObject*. See TPropertyTraits for an example. TRuntimePropertyTraits shows
 * an example trait for a property that requires additional meta-data components where multiple property types can be represented by
 * a single property definition at runtime (ie, for multi-channel vectors, or colors)
 */
template<typename InPropertyTraits>
struct TPropertyComponents
{
	using MetaDataType = typename InPropertyTraits::MetaDataType;

	FComponentTypeID PropertyTag;
	TComponentTypeID<typename InPropertyTraits::StorageType> InitialValue;

	TPropertyMetaDataComponents<typename InPropertyTraits::MetaDataType> MetaDataComponents;

	TCompositePropertyTypeID<InPropertyTraits> CompositeID;

	InPropertyTraits PropertyTraits;
};

template<typename PropertyTraits, typename MetaDataType>
struct TSetPropertyValuesImpl;



template<typename PropertyTraits, typename ...MetaDataTypes>
struct TSetPropertyValuesImpl<PropertyTraits, TPropertyMetaData<MetaDataTypes...>>
{
	using StorageType = typename PropertyTraits::StorageType;
	using InParamType = typename TCallTraits<typename PropertyTraits::StorageType>::ParamType;

	explicit TSetPropertyValuesImpl(const PropertyTraits* InTraits, const ICustomPropertyRegistration* InCustomProperties)
		: Traits(InTraits)
		, CustomProperties(InCustomProperties)
	{
		if (CustomProperties)
		{
			CustomAccessors = CustomProperties->GetAccessors();
		}
	}

	/**
	 * Task callback that applies a value to an object property via a custom native setter function
	 * Must be invoked with a task builder with the specified parameters:
	 *
	 *     FEntityTaskBuilder()
	 *     .Read( TComponentTypeID<UObject*>(...) )
	 *     .Read( TComponentTypeID<FCustomPropertyIndex>(...) )
	 *     .Read( TComponentTypeID<PropertyType>(...) )
	 *     .Dispatch_PerEntity<TSetPropertyValues<PropertyType>>(...);
	 */
	void ForEachEntity(UObject* InObject, FCustomPropertyIndex CustomIndex, typename TCallTraits<MetaDataTypes>::ParamType... MetaData, InParamType ValueToSet) const;

	/**
	 * Task callback that applies a value to an object property via a fast pointer offset
	 * Must be invoked with a task builder with the specified parameters:
	 *
	 *     FEntityTaskBuilder()
	 *     .Read( TComponentTypeID<UObject*>(...) )
	 *     .Read( TComponentTypeID<uint16>(...) )
	 *     .Read( TComponentTypeID<PropertyType>(...) )
	 *     .Dispatch_PerEntity<TSetPropertyValues<PropertyType>>(...);
	 */
	void ForEachEntity(UObject* InObject, uint16 PropertyOffset, typename TCallTraits<MetaDataTypes>::ParamType... MetaData, InParamType ValueToSet) const;


	/**
	 * Task callback that applies a value to an object property via a slow (legacy) track instance binding
	 * Must be invoked with a task builder with the specified parameters:
	 *
	 *     FEntityTaskBuilder()
	 *     .Read( TComponentTypeID<UObject*>(...) )
	 *     .Read( TComponentTypeID<TSharedPtr<FTrackInstancePropertyBindings>>(...) )
	 *     .Read( TComponentTypeID<PropertyType>(...) )
	 *     .Dispatch_PerEntity<TSetPropertyValues<PropertyType>>(...);
	 */
	void ForEachEntity(UObject* InObject, const TSharedPtr<FTrackInstancePropertyBindings>& PropertyBindings, typename TCallTraits<MetaDataTypes>::ParamType... MetaData, InParamType ValueToSet) const;

public:

	using FThreeWayAccessor  = TMultiReadOptional<FCustomPropertyIndex, uint16, TSharedPtr<FTrackInstancePropertyBindings>>;
	using FTwoWayAccessor    = TMultiReadOptional<uint16, TSharedPtr<FTrackInstancePropertyBindings>>;

	/**
	 * Task callback that applies properties for a whole allocation of entities with either an FCustomPropertyIndex, uint16 or TSharedPtr<FTrackInstancePropertyBindings> property component
	 * Must be invoked with a task builder with the specified parameters:
	 *
	 *     FEntityTaskBuilder()
	 *     .Read(      TComponentTypeID<UObject*>(...) )
	 *     .ReadOneOf( TComponentTypeID<FCustomPropertyIndex>(...), TComponentTypeID<uint16>(...), TComponentTypeID<TSharedPtr<FTrackInstancePropertyBindings>>(...) )
	 *     .Read(      TComponentTypeID<PropertyType>(...) )
	 *     .Dispatch_PerAllocation<TSetPropertyValues<PropertyType>>(...);
	 */
	void ForEachAllocation(const FEntityAllocation* Allocation, TRead<UObject*> BoundObjectComponents, FThreeWayAccessor ResolvedPropertyComponents, TRead<MetaDataTypes>... MetaDataComponents, TRead<StorageType> PropertyValueComponents) const;


	/**
	 * Task callback that applies properties for a whole allocation of entities with either a uint16 or TSharedPtr<FTrackInstancePropertyBindings> property component
	 * Must be invoked with a task builder with the specified parameters:
	 *
	 *     FEntityTaskBuilder()
	 *     .Read(      TComponentTypeID<UObject*>(...) )
	 *     .ReadOneOf( TComponentTypeID<uint16>(...), TComponentTypeID<TSharedPtr<FTrackInstancePropertyBindings>>(...) )
	 *     .Read(      TComponentTypeID<PropertyType>(...) )
	 *     .Dispatch_PerAllocation<TSetPropertyValues<PropertyType>>(...);
	 */
	void ForEachAllocation(const FEntityAllocation* Allocation, TRead<UObject*> BoundObjectComponents, FTwoWayAccessor ResolvedPropertyComponents, TRead<MetaDataTypes>... MetaDataComponents, TRead<StorageType> PropertyValueComponents) const;

private:

	const PropertyTraits* Traits;
	const ICustomPropertyRegistration* CustomProperties;
	FCustomAccessorView CustomAccessors;
};


/**
 * Stateless entity task that will apply values to properties. Three types of property are supported: Custom native accessor functions, fast pointer offset, or FTrackInstancePropertyBindings
 * 
 * Can be invoked in one of 2 ways: either with a specific property type through a per-entity iteration:
 *
 *     TComponentTypeID<FCustomPropertyIndex> CustomProperty = ...;
 *     TComponentTypeID<FTransform> TransformComponent = ...;
 *     TComponentTypeID<UObject*> BoundObject = ...;
 *
 *     FEntityTaskBuilder()
 *     .Read(BoundObject)
 *     .Read(CustomProperty)
 *     .Read(TransformComponent)
 *     .Dispatch_PerEntity<TSetPropertyValues<FTransform>>( ... );
 *
 * Or via a combinatorial task that iterates all entities with any one of the property components:
 *
 *     TComponentTypeID<uint16> FastPropertyOffset = ...;
 *     TComponentTypeID<TSharedPtr<FTrackInstancePropertyBindings>> SlowProperty = ...;
 *
 *     FEntityTaskBuilder()
 *     .Read(BoundObject)
 *     .ReadOneOf(CustomProperty, FastProperty, SlowProperty)
 *     .Read(TransformComponent)
 *     .Dispatch_PerAllocation<TSetPropertyValues<FTransform>>( ... );
 */
template<typename PropertyTraits>
struct TSetPropertyValues : TSetPropertyValuesImpl<PropertyTraits, typename PropertyTraits::MetaDataType>
{
	using ParamType = typename TCallTraits<typename PropertyTraits::StorageType>::ParamType;

	explicit TSetPropertyValues(const PropertyTraits* InTraits, const ICustomPropertyRegistration* InCustomProperties)
		: TSetPropertyValuesImpl<PropertyTraits, typename PropertyTraits::MetaDataType>(InTraits, InCustomProperties)
	{}
};




template<typename PropertyTraits, typename MetaDataType>
struct TGetPropertyValuesImpl;



template<typename PropertyTraits, typename ...MetaDataTypes>
struct TGetPropertyValuesImpl<PropertyTraits, TPropertyMetaData<MetaDataTypes...>>
{
	using StorageType = typename PropertyTraits::StorageType;

	explicit TGetPropertyValuesImpl(const PropertyTraits* InTraits, const ICustomPropertyRegistration* InCustomProperties)
		: Traits(InTraits)
		, CustomProperties(InCustomProperties)
	{
		if (CustomProperties)
		{
			CustomAccessors = CustomProperties->GetAccessors();
		}
	}

	/**
	 * Task callback that retrieves the object's current value via a custom native setter function, and writes it to the specified output variable
	 * Must be invoked with a task builder with the specified parameters:
	 *
	 *     FEntityTaskBuilder()
	 *     .Read( TComponentTypeID<UObject*>(...) )
	 *     .Read( TComponentTypeID<FCustomPropertyIndex>(...) )
	 *     .Write( TComponentTypeID<StorageType>(...) )
	 *     .Dispatch_PerEntity<TGetPropertyValues<PropertyType, StorageType>>(...);
	 */
	void ForEachEntity(UObject* InObject, FCustomPropertyIndex CustomPropertyIndex, typename TCallTraits<MetaDataTypes>::ParamType... MetaData, StorageType& OutValue) const;

	/**
	 * Task callback that retrieves the object's current value via a fast pointer offset, and writes it to the specified output variable
	 * Must be invoked with a task builder with the specified parameters:
	 *
	 *     FEntityTaskBuilder()
	 *     .Read( TComponentTypeID<UObject*>(...) )
	 *     .Read( TComponentTypeID<uint16>(...) )
	 *     .ReadAllOf( TComponentTypeID<MetaData>(...)... )
	 *     .Read( TComponentTypeID<StorageType>(...) )
	 *     .Dispatch_PerEntity<TGetPropertyValues<PropertyType, StorageType>>(...);
	 */
	void ForEachEntity(UObject* InObject, uint16 PropertyOffset, typename TCallTraits<MetaDataTypes>::ParamType... MetaData, StorageType& OutValue) const;

	/**
	 * Task callback that retrieves the object's current value via a slow (legacy) track instance binding, and writes it to the specified output variable
	 * Must be invoked with a task builder with the specified parameters:
	 *
	 *     FEntityTaskBuilder()
	 *     .Read( TComponentTypeID<UObject*>(...) )
	 *     .Read( TComponentTypeID<TSharedPtr<FTrackInstancePropertyBindings>>(...) )
	 *     .ReadAllOf( TComponentTypeID<MetaData>(...)... )
	 *     .Read( TComponentTypeID<StorageType>(...) )
	 *     .Dispatch_PerEntity<TGetPropertyValues<PropertyType, StorageType>>(...);
	 */
	void ForEachEntity(UObject* InObject, const TSharedPtr<FTrackInstancePropertyBindings>& PropertyBindings, typename TCallTraits<MetaDataTypes>::ParamType... MetaData, StorageType& OutValue) const;

public:

	using FThreeWayAccessor  = TMultiReadOptional<FCustomPropertyIndex, uint16, TSharedPtr<FTrackInstancePropertyBindings>>;
	using FTwoWayAccessor    = TMultiReadOptional<uint16, TSharedPtr<FTrackInstancePropertyBindings>>;


	/**
	 * Task callback that writes current property values for objects into an output component for a whole allocation of entities with either an FCustomPropertyIndex, uint16 or TSharedPtr<FTrackInstancePropertyBindings> property component
	 * Must be invoked with a task builder with the specified parameters:
	 *
	 *     FEntityTaskBuilder()
	 *     .Read(      TComponentTypeID<UObject*>(...) )
	 *     .ReadOneOf( TComponentTypeID<FCustomPropertyIndex>(...), TComponentTypeID<uint16>(...), TComponentTypeID<TSharedPtr<FTrackInstancePropertyBindings>>(...) )
	 *     .ReadAllOf( TComponentTypeID<MetaData>(...)... )
	 *     .Write(     TComponentTypeID<StorageType>(...) )
	 *     .Dispatch_PerAllocation<TGetPropertyValues<PropertyType, StorageType>>(...);
	 */
	void ForEachAllocation(const FEntityAllocation* Allocation, TRead<UObject*> BoundObjectComponents, FThreeWayAccessor ResolvedPropertyComponents, TRead<MetaDataTypes>... MetaDataComponents, TWrite<StorageType> OutValueComponents) const;


	/**
	 * Task callback that writes current property values for objects into an output component for a whole allocation of entities with either a uint16 or TSharedPtr<FTrackInstancePropertyBindings> property component
	 * Must be invoked with a task builder with the specified parameters:
	 *
	 *     FEntityTaskBuilder()
	 *     .Read(      TComponentTypeID<UObject*>(...) )
	 *     .ReadOneOf( TComponentTypeID<uint16>(...), TComponentTypeID<TSharedPtr<FTrackInstancePropertyBindings>>(...) )
	 *     .ReadAllOf( TComponentTypeID<MetaData>(...)... )
	 *     .Write(     TComponentTypeID<StorageType>(...) )
	 *     .Dispatch_PerAllocation<TGetPropertyValues<PropertyType ,StorageType>>(...);
	 */
	void ForEachAllocation(const FEntityAllocation* Allocation, TRead<UObject*> BoundObjectComponents, FTwoWayAccessor ResolvedPropertyComponents, TRead<MetaDataTypes>... MetaDataComponents, TWrite<StorageType> OutValueComponents) const;

private:

	const PropertyTraits* Traits;
	const ICustomPropertyRegistration* CustomProperties;
	FCustomAccessorView CustomAccessors;
};


template<typename PropertyTraits>
struct TGetPropertyValues : TGetPropertyValuesImpl<PropertyTraits, typename PropertyTraits::MetaDataType>
{
	explicit TGetPropertyValues(const PropertyTraits* InTraits, const ICustomPropertyRegistration* InCustomProperties)
		: TGetPropertyValuesImpl<PropertyTraits, typename PropertyTraits::MetaDataType>(InTraits, InCustomProperties)
	{}
};



/**
 * Task implementation that combines a specific set of input components (templated as CompositeTypes) through a projection, and applies the result to an object property
 * Three types of property are supported: Custom native accessor functions, fast pointer offset, or FTrackInstancePropertyBindings.
 * 
 * Can be invoked in one of 2 ways: either with a specific property type and input components through a per-entity iteration:
 *
 *     TComponentTypeID<FCustomPropertyIndex> CustomProperty = ...;
 *     TComponentTypeID<UObject*> BoundObject = ...;
 *
 *     FEntityTaskBuilder()
 *     .Read( BoundObject )
 *     .Read( CustomProperty )
 *     .Read( TComponentTypeID<float>(...) )
 *     .Read( TComponentTypeID<float>(...) )
 *     .Read( TComponentTypeID<float>(...) )
 *     .Dispatch_PerEntity<TSetCompositePropertyValues<FVector, float, float, float>>( ..., &UKismetMathLibrary::MakeVector );
 *
 * Or via a combinatorial task that iterates all entities with any one of the property components:
 *
 *     TComponentTypeID<uint16> FastPropertyOffset = ...;
 *     TComponentTypeID<TSharedPtr<FTrackInstancePropertyBindings>> SlowProperty = ...;
 *
 *     FEntityTaskBuilder()
 *     .Read(      BoundObject )
 *     .ReadOneOf( CustomProperty, FastPropertyOffset, SlowProperty )
 *     .Read(      TComponentTypeID<float>(...) )
 *     .Read(      TComponentTypeID<float>(...) )
 *     .Read(      TComponentTypeID<float>(...) )
 *     .Dispatch_Perllocation<TSetCompositePropertyValues<FVector, float, float, float>>( ..., &UKismetMathLibrary::MakeVector );
 */
template<typename PropertyTraits, typename MetaDataType, typename... CompositeTypes>
struct TSetCompositePropertyValuesImpl;

template<typename PropertyTraits, typename... MetaDataTypes, typename... CompositeTypes>
struct TSetCompositePropertyValuesImpl<PropertyTraits, TPropertyMetaData<MetaDataTypes...>, CompositeTypes...>
{
	using StorageType = typename PropertyTraits::StorageType;

	explicit TSetCompositePropertyValuesImpl(const PropertyTraits* InTraits, const ICustomPropertyRegistration* InCustomProperties)
		: Traits(InTraits)
		, CustomProperties(InCustomProperties)
	{
		if (CustomProperties)
		{
			CustomAccessors = CustomProperties->GetAccessors();
		}
	}

	/**
	 * Task callback that applies a value to an object property via a custom native setter function
	 * Must be invoked with a task builder with the specified parameters:
	 *
	 *     FEntityTaskBuilder()
	 *     .Read( TComponentTypeID<UObject*>(...) )
	 *     .Read( TComponentTypeID<FCustomPropertyIndex>(...) )
	 *     .Read( TComponentTypeID<CompositeType[0  ]>(...) )
	 *     .Read( TComponentTypeID<CompositeType[...]>(...) )
	 *     .Read( TComponentTypeID<CompositeType[N-1]>(...) )
	 *     .Dispatch_PerEntity<TSetCompositePropertyValues<PropertyType, CompositeTypes...>>(...);
	 */
	void ForEachEntity(UObject* InObject, FCustomPropertyIndex CustomPropertyIndex, typename TCallTraits<MetaDataTypes>::ParamType... MetaData, typename TCallTraits<CompositeTypes>::ParamType... CompositeResults) const;


	/**
	 * Task callback that applies a value to an object property via a fast pointer offset
	 * Must be invoked with a task builder with the specified parameters:
	 *
	 *     FEntityTaskBuilder()
	 *     .Read( TComponentTypeID<UObject*>(...) )
	 *     .Read( TComponentTypeID<uint16>(...) )
	 *     .Read( TComponentTypeID<CompositeType[0  ]>(...) )
	 *     .Read( TComponentTypeID<CompositeType[...]>(...) )
	 *     .Read( TComponentTypeID<CompositeType[N-1]>(...) )
	 *     .Dispatch_PerEntity<TSetCompositePropertyValues<PropertyType, CompositeTypes...>>(...);
	 */
	void ForEachEntity(UObject* InObject, uint16 PropertyOffset, typename TCallTraits<MetaDataTypes>::ParamType... MetaData, typename TCallTraits<CompositeTypes>::ParamType... CompositeResults) const;


	/**
	 * Task callback that applies a value to an object property via a slow (legacy) track instance binding
	 * Must be invoked with a task builder with the specified parameters:
	 *
	 *     FEntityTaskBuilder()
	 *     .Read( TComponentTypeID<UObject*>(...) )
	 *     .Read( TComponentTypeID<TSharedPtr<FTrackInstancePropertyBindings>>(...) )
	 *     .Read( TComponentTypeID<CompositeType[0  ]>(...) )
	 *     .Read( TComponentTypeID<CompositeType[...]>(...) )
	 *     .Read( TComponentTypeID<CompositeType[N-1]>(...) )
	 *     .Dispatch_PerEntity<TSetCompositePropertyValues<PropertyType, CompositeTypes...>>(...);
	 */
	void ForEachEntity(UObject* InObject, const TSharedPtr<FTrackInstancePropertyBindings>& PropertyBindings, typename TCallTraits<MetaDataTypes>::ParamType... MetaData, typename TCallTraits<CompositeTypes>::ParamType... CompositeResults) const;

public:

	using FThreeWayAccessor  = TMultiReadOptional<FCustomPropertyIndex, uint16, TSharedPtr<FTrackInstancePropertyBindings>>;
	using FTwoWayAccessor    = TMultiReadOptional<uint16, TSharedPtr<FTrackInstancePropertyBindings>>;


	/**
	 * Task callback that applies properties for a whole allocation of entities with either an FCustomPropertyIndex, uint16 or TSharedPtr<FTrackInstancePropertyBindings> property component
	 * Must be invoked with a task builder with the specified parameters:
	 *
	 *     FEntityTaskBuilder()
	 *     .Read(      TComponentTypeID<UObject*>(...) )
	 *     .ReadOneOf( TComponentTypeID<FCustomPropertyIndex>(...), TComponentTypeID<uint16>(...), TComponentTypeID<TSharedPtr<FTrackInstancePropertyBindings>>(...) )
	 *     .Read( TComponentTypeID<CompositeType[0  ]>(...) )
	 *     .Read( TComponentTypeID<CompositeType[...]>(...) )
	 *     .Read( TComponentTypeID<CompositeType[N-1]>(...) )
	 *     .Dispatch_PerAllocation<TSetCompositePropertyValues<PropertyType, CompositeTypes...>>(...);
	 */
	void ForEachAllocation(const FEntityAllocation* Allocation, TRead<UObject*> BoundObjectComponents, FThreeWayAccessor ResolvedPropertyComponents, TRead<MetaDataTypes>... InMetaData, TRead<CompositeTypes>... VariadicComponents) const;


	/**
	 * Task callback that applies properties for a whole allocation of entities with either a uint16 or TSharedPtr<FTrackInstancePropertyBindings> property component
	 * Must be invoked with a task builder with the specified parameters:
	 *
	 *     FEntityTaskBuilder()
	 *     .Read(      TComponentTypeID<UObject*>(...) )
	 *     .ReadOneOf( TComponentTypeID<uint16>(...), TComponentTypeID<TSharedPtr<FTrackInstancePropertyBindings>>(...) )
	 *     .Read( TComponentTypeID<CompositeType[0  ]>(...) )
	 *     .Read( TComponentTypeID<CompositeType[...]>(...) )
	 *     .Read( TComponentTypeID<CompositeType[N-1]>(...) )
	 *     .Dispatch_PerAllocation<TSetCompositePropertyValues<PropertyType, CompositeTypes...>>(...);
	 */
	void ForEachAllocation(const FEntityAllocation* Allocation, TRead<UObject*> BoundObjectComponents, FTwoWayAccessor ResolvedPropertyComponents, TRead<MetaDataTypes>... InMetaData, TRead<CompositeTypes>... VariadicComponents) const;

private:

	const PropertyTraits* Traits;
	const ICustomPropertyRegistration* CustomProperties;
	FCustomAccessorView CustomAccessors;
};

template<typename PropertyTraits, typename ...CompositeTypes>
using TSetCompositePropertyValues = TSetCompositePropertyValuesImpl<PropertyTraits, typename PropertyTraits::MetaDataType, CompositeTypes...>;


} // namespace UE::MovieScene
