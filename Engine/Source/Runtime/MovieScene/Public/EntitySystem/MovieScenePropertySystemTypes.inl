// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieScenePropertySystemTypes.h"
#include "CoreTypes.h"
#include "UObject/NameTypes.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTypeTraits.h"
#include "Containers/SparseArray.h"
#include "MovieSceneCommonHelpers.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "EntitySystem/MovieSceneComponentRegistry.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneComponentAccessors.h"
#include "EntitySystem/MovieSceneOperationalTypeConversions.h"
#include "EntitySystem/MovieScenePropertyBinding.h"


namespace UE::MovieScene
{

template<typename PropertyTraits, typename ...MetaDataTypes>
void TSetPropertyValuesImpl<PropertyTraits, TPropertyMetaData<MetaDataTypes...>>::ForEachEntity(
	UObject* InObject,
	FCustomPropertyIndex CustomIndex,
	typename TCallTraits<MetaDataTypes>::ParamType... MetaData,
	InParamType ValueToSet) const
{
	Traits->SetObjectPropertyValue(InObject, MetaData..., CustomAccessors[CustomIndex.Value], ValueToSet);
}

template<typename PropertyTraits, typename ...MetaDataTypes>
void TSetPropertyValuesImpl<PropertyTraits, TPropertyMetaData<MetaDataTypes...>>::ForEachEntity(
	UObject* InObject,
	uint16 PropertyOffset,
	typename TCallTraits<MetaDataTypes>::ParamType... MetaData,
	InParamType ValueToSet) const
{
	// Would really like to avoid branching here, but if we encounter this data the options are either handle it gracefully, stomp a vtable, or report a fatal error.
	if (ensureAlwaysMsgf(PropertyOffset != 0, TEXT("Invalid property offset specified (ptr+%d bytes) for property on object %s. This would otherwise overwrite the object's vfptr."), PropertyOffset, *InObject->GetName()))
	{
		Traits->SetObjectPropertyValue(InObject, MetaData..., PropertyOffset, ValueToSet);
	}
}

template<typename PropertyTraits, typename ...MetaDataTypes>
void TSetPropertyValuesImpl<PropertyTraits, TPropertyMetaData<MetaDataTypes...>>::ForEachEntity(
	UObject* InObject,
	const TSharedPtr<FTrackInstancePropertyBindings>& PropertyBindings,
	typename TCallTraits<MetaDataTypes>::ParamType... MetaData,
	InParamType ValueToSet) const
{
	Traits->SetObjectPropertyValue(InObject, MetaData..., PropertyBindings.Get(), ValueToSet);
}

template<typename PropertyTraits, typename ...MetaDataTypes>
void TSetPropertyValuesImpl<PropertyTraits, TPropertyMetaData<MetaDataTypes...>>::ForEachAllocation(
	const FEntityAllocation* Allocation,
	TRead<UObject*> BoundObjectComponents,
	FThreeWayAccessor ResolvedPropertyComponents,
	TRead<MetaDataTypes>... MetaDataComponents,
	TRead<StorageType> PropertyValueComponents) const
{
	const int32 Num = Allocation->Num();
	if (const FCustomPropertyIndex* Custom = ResolvedPropertyComponents.template Get<0>())
	{
		for (int32 Index = 0; Index < Num; ++Index)
		{
			ForEachEntity(BoundObjectComponents[Index], Custom[Index], MetaDataComponents[Index]..., PropertyValueComponents[Index]);
		}
	}
	else if (const uint16* Fast = ResolvedPropertyComponents.template Get<1>())
	{
		for (int32 Index = 0; Index < Num; ++Index)
		{
			ForEachEntity(BoundObjectComponents[Index], Fast[Index], MetaDataComponents[Index]..., PropertyValueComponents[Index]);
		}
	}
	else if (const TSharedPtr<FTrackInstancePropertyBindings>* Slow = ResolvedPropertyComponents.template Get<2>())
	{
		for (int32 Index = 0; Index < Num; ++Index)
		{
			ForEachEntity(BoundObjectComponents[Index], Slow[Index], MetaDataComponents[Index]..., PropertyValueComponents[Index]);
		}
	}
}

template<typename PropertyTraits, typename ...MetaDataTypes>
void TSetPropertyValuesImpl<PropertyTraits, TPropertyMetaData<MetaDataTypes...>>::ForEachAllocation(
	const FEntityAllocation* Allocation,
	TRead<UObject*> BoundObjectComponents,
	FTwoWayAccessor ResolvedPropertyComponents,
	TRead<MetaDataTypes>... MetaDataComponents,
	TRead<StorageType> PropertyValueComponents) const
{
	const int32 Num = Allocation->Num();
	if (const uint16* Fast = ResolvedPropertyComponents.template Get<0>())
	{
		for (int32 Index = 0; Index < Num; ++Index)
		{
			ForEachEntity(BoundObjectComponents[Index], *Fast++, MetaDataComponents[Index]..., PropertyValueComponents[Index]);
		}
	}
	else if (const TSharedPtr<FTrackInstancePropertyBindings>* Slow = ResolvedPropertyComponents.template Get<1>())
	{
		for (int32 Index = 0; Index < Num; ++Index)
		{
			ForEachEntity(BoundObjectComponents[Index], *Slow++, MetaDataComponents[Index]..., PropertyValueComponents[Index]);
		}
	}
}

template<typename PropertyTraits, typename ...MetaDataTypes>
void TGetPropertyValuesImpl<PropertyTraits, TPropertyMetaData<MetaDataTypes...>>::ForEachEntity(
	UObject* InObject,
	FCustomPropertyIndex CustomPropertyIndex,
	typename TCallTraits<MetaDataTypes>::ParamType... MetaData,
	StorageType& OutValue) const
{
	Traits->GetObjectPropertyValue(InObject, MetaData..., CustomAccessors[CustomPropertyIndex.Value], OutValue);
}

template<typename PropertyTraits, typename ...MetaDataTypes>
void TGetPropertyValuesImpl<PropertyTraits, TPropertyMetaData<MetaDataTypes...>>::ForEachEntity(
	UObject* InObject,
	uint16 PropertyOffset,
	typename TCallTraits<MetaDataTypes>::ParamType... MetaData,
	StorageType& OutValue) const
{
	// Would really like to avoid branching here, but if we encounter this data the options are either handle it gracefully, stomp a vtable, or report a fatal error.
	if (ensureAlwaysMsgf(PropertyOffset != 0, TEXT("Invalid property offset specified (ptr+%d bytes) for property on object %s. This would otherwise overwrite the object's vfptr."), PropertyOffset, *InObject->GetName()))
	{
		Traits->GetObjectPropertyValue(InObject, MetaData..., PropertyOffset, OutValue);
	}
}

template<typename PropertyTraits, typename ...MetaDataTypes>
void TGetPropertyValuesImpl<PropertyTraits, TPropertyMetaData<MetaDataTypes...>>::ForEachEntity(
	UObject* InObject,
	const TSharedPtr<FTrackInstancePropertyBindings>& PropertyBindings,
	typename TCallTraits<MetaDataTypes>::ParamType... MetaData,
	StorageType& OutValue) const
{
	Traits->GetObjectPropertyValue(InObject, MetaData..., PropertyBindings.Get(), OutValue);
}

template<typename PropertyTraits, typename ...MetaDataTypes>
void TGetPropertyValuesImpl<PropertyTraits, TPropertyMetaData<MetaDataTypes...>>::ForEachAllocation(
	const FEntityAllocation* Allocation,
	TRead<UObject*> BoundObjectComponents,
	FThreeWayAccessor ResolvedPropertyComponents,
	TRead<MetaDataTypes>... MetaData,
	TWrite<StorageType> OutValueComponents) const
{
	const int32 Num = Allocation->Num();
	if (const FCustomPropertyIndex* Custom = ResolvedPropertyComponents.template Get<0>())
	{
		for (int32 Index = 0; Index < Num; ++Index)
		{
			ForEachEntity(BoundObjectComponents[Index], Custom[Index], MetaData[Index]..., OutValueComponents[Index]);
		}
	}
	else if (const uint16* Fast = ResolvedPropertyComponents.template Get<1>())
	{
		for (int32 Index = 0; Index < Num; ++Index)
		{
			ForEachEntity(BoundObjectComponents[Index], Fast[Index], MetaData[Index]..., OutValueComponents[Index]);
		}
	}
	else if (const TSharedPtr<FTrackInstancePropertyBindings>* Slow = ResolvedPropertyComponents.template Get<2>())
	{
		for (int32 Index = 0; Index < Num; ++Index)
		{
			ForEachEntity(BoundObjectComponents[Index], Slow[Index], MetaData[Index]..., OutValueComponents[Index]);
		}
	}
}

template<typename PropertyTraits, typename ...MetaDataTypes>
void TGetPropertyValuesImpl<PropertyTraits, TPropertyMetaData<MetaDataTypes...>>::ForEachAllocation(
	const FEntityAllocation* Allocation,
	TRead<UObject*> BoundObjectComponents,
	FTwoWayAccessor ResolvedPropertyComponents,
	TRead<MetaDataTypes>... MetaData,
	TWrite<StorageType> OutValueComponents) const
{
	const int32 Num = Allocation->Num();
	if (const uint16* Fast = ResolvedPropertyComponents.template Get<0>())
	{
		for (int32 Index = 0; Index < Num; ++Index)
		{
			ForEachEntity(BoundObjectComponents[Index], Fast[Index], MetaData[Index]..., OutValueComponents[Index]);
		}
	}
	else if (const TSharedPtr<FTrackInstancePropertyBindings>* Slow = ResolvedPropertyComponents.template Get<1>())
	{
		for (int32 Index = 0; Index < Num; ++Index)
		{
			ForEachEntity(BoundObjectComponents[Index], Slow[Index], MetaData[Index]..., OutValueComponents[Index]);
		}
	}
}


template<typename PropertyTraits, typename... MetaDataTypes, typename... CompositeTypes>
void TSetCompositePropertyValuesImpl<PropertyTraits, TPropertyMetaData<MetaDataTypes...>, CompositeTypes...>::ForEachEntity(
	UObject* InObject,
	FCustomPropertyIndex CustomPropertyIndex,
	typename TCallTraits<MetaDataTypes>::ParamType... MetaData,
	typename TCallTraits<CompositeTypes>::ParamType... CompositeResults) const
{
	StorageType Result = Traits->CombineComposites(MetaData..., CompositeResults...);
	Traits->SetObjectPropertyValue(InObject, MetaData..., CustomAccessors[CustomPropertyIndex.Value], Result);
}

template<typename PropertyTraits, typename... MetaDataTypes, typename... CompositeTypes>
void TSetCompositePropertyValuesImpl<PropertyTraits, TPropertyMetaData<MetaDataTypes...>, CompositeTypes...>::ForEachEntity(
	UObject* InObject,
	uint16 PropertyOffset,
	typename TCallTraits<MetaDataTypes>::ParamType... MetaData,
	typename TCallTraits<CompositeTypes>::ParamType... CompositeResults) const
{
	// Would really like to avoid branching here, but if we encounter this data the options are either handle it gracefully, stomp a vtable, or report a fatal error.
	if (ensureAlwaysMsgf(PropertyOffset != 0, TEXT("Invalid property offset specified (ptr+%d bytes) for property on object %s. This would otherwise overwrite the object's vfptr."), PropertyOffset, *InObject->GetName()))
	{
		StorageType Result = Traits->CombineComposites(MetaData..., CompositeResults...);
		Traits->SetObjectPropertyValue(InObject, MetaData..., PropertyOffset, Result);
	}
}

template<typename PropertyTraits, typename... MetaDataTypes, typename... CompositeTypes>
void TSetCompositePropertyValuesImpl<PropertyTraits, TPropertyMetaData<MetaDataTypes...>, CompositeTypes...>::ForEachEntity(
	UObject* InObject,
	const TSharedPtr<FTrackInstancePropertyBindings>& PropertyBindings,
	typename TCallTraits<MetaDataTypes>::ParamType... MetaData,
	typename TCallTraits<CompositeTypes>::ParamType... CompositeResults) const
{
	StorageType Result = Traits->CombineComposites(MetaData..., CompositeResults...);
	Traits->SetObjectPropertyValue(InObject, MetaData..., PropertyBindings.Get(), Result);
}

template<typename PropertyTraits, typename... MetaDataTypes, typename... CompositeTypes>
void TSetCompositePropertyValuesImpl<PropertyTraits, TPropertyMetaData<MetaDataTypes...>, CompositeTypes...>::ForEachAllocation(
	const FEntityAllocation* Allocation,
	TRead<UObject*> BoundObjectComponents,
	FThreeWayAccessor ResolvedPropertyComponents,
	TRead<MetaDataTypes>... InMetaData,
	TRead<CompositeTypes>... VariadicComponents) const
{
	const int32 Num = Allocation->Num();
	if (const FCustomPropertyIndex* Custom = ResolvedPropertyComponents.template Get<0>())
	{
		for (int32 Index = 0; Index < Num; ++Index )
		{
			ForEachEntity( BoundObjectComponents[Index], Custom[Index], InMetaData[Index]..., VariadicComponents[Index]... );
		}
	}
	else if (const uint16* Fast = ResolvedPropertyComponents.template Get<1>())
	{
		for (int32 Index = 0; Index < Num; ++Index )
		{
			ForEachEntity( BoundObjectComponents[Index], Fast[Index], InMetaData[Index]..., VariadicComponents[Index]... );
		}
	}
	else if (const TSharedPtr<FTrackInstancePropertyBindings>* Slow = ResolvedPropertyComponents.template Get<2>())
	{
		for (int32 Index = 0; Index < Num; ++Index )
		{
			ForEachEntity( BoundObjectComponents[Index], Slow[Index], InMetaData[Index]..., VariadicComponents[Index]... );
		}
	}
}

template<typename PropertyTraits, typename... MetaDataTypes, typename... CompositeTypes>
void TSetCompositePropertyValuesImpl<PropertyTraits, TPropertyMetaData<MetaDataTypes...>, CompositeTypes...>::ForEachAllocation(
	const FEntityAllocation* Allocation,
	TRead<UObject*> BoundObjectComponents,
	FTwoWayAccessor ResolvedPropertyComponents,
	TRead<MetaDataTypes>... InMetaData,
	TRead<CompositeTypes>... VariadicComponents) const
{
	const int32 Num = Allocation->Num();
	if (const uint16* Fast = ResolvedPropertyComponents.template Get<0>())
	{
		for (int32 Index = 0; Index < Num; ++Index )
		{
			ForEachEntity( BoundObjectComponents[Index], Fast[Index], InMetaData[Index]..., VariadicComponents[Index]... );
		}
	}
	else if (const TSharedPtr<FTrackInstancePropertyBindings>* Slow = ResolvedPropertyComponents.template Get<1>())
	{
		for (int32 Index = 0; Index < Num; ++Index )
		{
			ForEachEntity( BoundObjectComponents[Index], Slow[Index], InMetaData[Index]..., VariadicComponents[Index]... );
		}
	}
}

} // namespace UE::MovieScene