// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieScenePropertySystemTypes.h"
#include "EntitySystem/MovieScenePropertyMetaData.h"
#include "EntitySystem/MovieSceneOperationalTypeConversions.h"
#include "EntitySystem/MovieSceneIntermediatePropertyValue.h"
#include "EntitySystem/MovieSceneVariantPropertyTypeIndex.h"
#include "EntitySystem/MovieScenePropertySupport.h"
#include "Channels/MovieSceneUnpackedChannelValues.h"
#include "MovieSceneCommonHelpers.h"
#include "Misc/GeneratedTypeName.h"


#include "UObject/Class.h"
#include "UObject/UnrealType.h"
#include "UObject/TextProperty.h"
#include "UObject/NoExportTypes.h"

class UMovieSceneTrack;


namespace UE
{
namespace MovieScene
{

template<typename PropertyTraits, typename ...CompositeTypes>
struct TVariantPropertyComponentHandler;


template<typename T>
void UnpackChannelsFromOperational(T&&, const FProperty& Property, FUnpackedChannelValues&)
{
	static_assert(!std::is_same_v<T, T>, "Please implement this function for the necessary types to support unpacking the property type into its constituent channels");
}

struct IPropertyTraits
{
	using MetaDataType = TPropertyMetaData<>;

	virtual ~IPropertyTraits() = default;

	virtual TSubclassOf<UMovieSceneTrack> GetTrackClass(const FProperty* InProperty) const
	{
		return nullptr;
	}
	virtual bool InitializeTrackFromProperty(UMovieSceneTrack* InTrack, const FProperty* InProperty) const
	{
		return false;
	}
};


template<typename ...UObjectPropertyTypes>
struct TSupportedPropertyTypes
{

};
template<typename ...CompositeTypes>
struct TCompositeChannelMapping
{
	
};


/**
 * Property accessor traits that talk directly to the reflected UObject property type
 */
template<typename UObjectPropertyType, typename InMemoryType, bool bInIsComposite = true>
struct TPropertyTraits : IPropertyTraits
{
	static constexpr bool bIsComposite     = bInIsComposite;
	static constexpr bool bNeedsConversion = !std::is_same_v<InMemoryType, UObjectPropertyType>;

	using TraitsType          = TPropertyTraits<UObjectPropertyType, InMemoryType, bInIsComposite>;
	using StorageType         = InMemoryType;
	using StorageTypeParam    = typename TCallTraits<StorageType>::ParamType;

	/** IPropertyTraits interface  */
	FORCEINLINE static bool SupportsProperty(const FProperty& InProperty)
	{
		return TPropertyMatch<UObjectPropertyType>::SupportsProperty(InProperty);
	}
	FORCEINLINE static FIntermediatePropertyValue CoercePropertyValue(const FProperty& InProperty, const FSourcePropertyValue& InPropertyValue)
	{
		StorageType Value;
		ConvertOperationalProperty(*InPropertyValue.Cast<UObjectPropertyType>(), Value);
		return FIntermediatePropertyValue::FromValue(Value);
	}
	FORCEINLINE static void UnpackChannels(const InMemoryType& Value, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues)
	{
		UnpackChannelsFromOperational(Value, Property, OutUnpackedValues);
	}

	/** Property Value Getters  */
	FORCEINLINE static void GetObjectPropertyValue(const UObject* InObject, const FCustomPropertyAccessor& BaseCustomAccessor, StorageType& OutValue)
	{
		const TCustomPropertyAccessor<TraitsType>& CustomAccessor = static_cast<const TCustomPropertyAccessor<TraitsType>&>(BaseCustomAccessor);

		OutValue = (*CustomAccessor.Functions.Getter)(InObject);
	}
	FORCEINLINE static void GetObjectPropertyValue(const UObject* InObject, uint16 PropertyOffset, StorageType& OutValue)
	{
		const UObjectPropertyType* PropertyAddress = reinterpret_cast<const UObjectPropertyType*>( reinterpret_cast<const uint8*>(InObject) + PropertyOffset );

		if constexpr (bNeedsConversion)
		{
			ConvertOperationalProperty(*PropertyAddress, OutValue);
		}
		else
		{
			OutValue = *PropertyAddress;
		}
	}
	FORCEINLINE static void GetObjectPropertyValue(const UObject* InObject, FTrackInstancePropertyBindings* PropertyBindings, StorageType& OutValue)
	{
		if constexpr (bNeedsConversion)
		{
			UObjectPropertyType Value = PropertyBindings->GetCurrentValue<UObjectPropertyType>(*InObject);
			ConvertOperationalProperty(Value, OutValue);
		}
		else
		{
			OutValue = PropertyBindings->GetCurrentValue<UObjectPropertyType>(*InObject);
		}
	}

	FORCEINLINE static void GetObjectPropertyValue(const UObject* InObject, const FName& PropertyPath, StorageType& OutValue)
	{
		TOptional<UObjectPropertyType> Property = FTrackInstancePropertyBindings::StaticValue<UObjectPropertyType>(InObject, *PropertyPath.ToString());
		if (Property)
		{
			if constexpr (bNeedsConversion)
			{
				ConvertOperationalProperty(Property.GetValue(), OutValue);
			}
			else
			{
				OutValue = Property.GetValue();
			}
		}
	}

	/** Property Value Setters  */
	FORCEINLINE static void SetObjectPropertyValue(UObject* InObject, const FCustomPropertyAccessor& BaseCustomAccessor, StorageTypeParam InValue)
	{
		const TCustomPropertyAccessor<TraitsType>& CustomAccessor = static_cast<const TCustomPropertyAccessor<TraitsType>&>(BaseCustomAccessor);
		(*CustomAccessor.Functions.Setter)(InObject, InValue);
	}
	FORCEINLINE static void SetObjectPropertyValue(UObject* InObject, uint16 PropertyOffset, StorageTypeParam InValue)
	{
		UObjectPropertyType* PropertyAddress = reinterpret_cast<UObjectPropertyType*>( reinterpret_cast<uint8*>(InObject) + PropertyOffset );
		if constexpr (bNeedsConversion)
		{
			ConvertOperationalProperty(InValue, *PropertyAddress);
		}
		else
		{
			*PropertyAddress = InValue;
		}
	}
	FORCEINLINE static void SetObjectPropertyValue(UObject* InObject, FTrackInstancePropertyBindings* PropertyBindings, StorageTypeParam InValue)
	{
		if constexpr (bNeedsConversion)
		{
			UObjectPropertyType NewValue{};
			ConvertOperationalProperty(InValue, NewValue);

			PropertyBindings->CallFunction<UObjectPropertyType>(*InObject, NewValue);
		}
		else
		{
			PropertyBindings->CallFunction<UObjectPropertyType>(*InObject, InValue);
		}
	}

	template<typename ...T>
	FORCEINLINE static StorageType CombineComposites(T&&... InComposites)
	{
		return StorageType{ Forward<T>(InComposites)... };
	}
};

template<typename StorageType>
struct TDynamicVariantTraitsBase
{
	using StorageTypeParam        = typename TCallTraits<StorageType>::ParamType;

	using CastToOperationalPtr    = void(*)(const void*, StorageType&);
	using CastToFinalPtr          = void(*)(StorageTypeParam, void*);
	using RetrieveSlowPropertyPtr = void(*)(const UObject*, FTrackInstancePropertyBindings*, StorageType&);
	using ApplySlowPropertyPtr    = void(*)(UObject*, FTrackInstancePropertyBindings*, StorageTypeParam);
	using InitializeNewTrackPtr   = void(*)(UMovieSceneTrack*, const FProperty*);

	UScriptStruct*                Struct              = nullptr;
	TSubclassOf<UMovieSceneTrack> TrackClass          = nullptr;
	CastToOperationalPtr          CastToOperational   = nullptr;
	CastToFinalPtr                CastToFinal         = nullptr;
	RetrieveSlowPropertyPtr       RetrieveSlowProperty= nullptr;
	ApplySlowPropertyPtr          ApplySlowProperty   = nullptr;
	InitializeNewTrackPtr         InitializeNewTrack  = nullptr;
};

template<typename UObjectPropertyType, typename StorageType>
struct TDynamicVariantTraits : TDynamicVariantTraitsBase<StorageType>
{
	using StorageTypeParam = typename TCallTraits<StorageType>::ParamType;

	using InitializeNewTrackPtr   = void(*)(UMovieSceneTrack*, const FProperty*);

	TDynamicVariantTraits()
	{
		this->Struct               = TBaseStructure<UObjectPropertyType>::Get();
		this->CastToOperational    = CastToOperationalImpl;
		this->CastToFinal          = CastToFinalImpl;
		this->RetrieveSlowProperty = RetrieveSlowPropertyImpl;
		this->ApplySlowProperty    = ApplySlowPropertyImpl;
	}
	TDynamicVariantTraits(UScriptStruct* InStruct)
	{
		this->Struct               = InStruct;
		this->CastToOperational    = CastToOperationalImpl;
		this->CastToFinal          = CastToFinalImpl;
		this->RetrieveSlowProperty = RetrieveSlowPropertyImpl;
		this->ApplySlowProperty    = ApplySlowPropertyImpl;
	}

	TDynamicVariantTraits<UObjectPropertyType, StorageType>& SetTrackClass(TSubclassOf<UMovieSceneTrack> InTrackClass)
	{
		this->TrackClass = InTrackClass;
		return *this;
	}

	TDynamicVariantTraits<UObjectPropertyType, StorageType>& SetTrackInitializer(InitializeNewTrackPtr InFunc)
	{
		this->InitializeNewTrack = InFunc;
		return *this;
	}

	static void CastToOperationalImpl(const void* In, StorageType& OutValue)
	{
		const UObjectPropertyType* PropertyAddress = static_cast<const UObjectPropertyType*>(In);
		ConvertOperationalProperty(*PropertyAddress, OutValue);
	}
	static void CastToFinalImpl(StorageTypeParam InValue, void* Out)
	{
		UObjectPropertyType* PropertyAddress = static_cast<UObjectPropertyType*>(Out);
		ConvertOperationalProperty(InValue, *PropertyAddress);
	}

	static void RetrieveSlowPropertyImpl(const UObject* InObject, FTrackInstancePropertyBindings* PropertyBindings, StorageType& OutValue)
	{
		UObjectPropertyType Value = PropertyBindings->GetCurrentValue<UObjectPropertyType>(*InObject);
		ConvertOperationalProperty(Value, OutValue);
	}
	static void ApplySlowPropertyImpl(UObject* InObject, FTrackInstancePropertyBindings* PropertyBindings, StorageTypeParam InValue)
	{
		UObjectPropertyType NewValue{};
		ConvertOperationalProperty(InValue, NewValue);

		PropertyBindings->CallFunction<UObjectPropertyType>(*InObject, NewValue);
	}
};

template<typename InMemoryType, typename ...UObjectPropertyTypes>
struct TVariantPropertyTraits
	: IPropertyTraits
{
	static constexpr int32 CompileTimeNum = sizeof...(UObjectPropertyTypes);
	static constexpr bool  bIsComposite   = true;

	using StorageType    = InMemoryType;
	using StorageTypeParam = typename TCallTraits<StorageType>::ParamType;

	using MetaDataType   = TPropertyMetaData<FVariantPropertyTypeIndex>;
	using PublicMetaData = TPropertyMetaData<>;

	using TraitsType     = TVariantPropertyTraits<StorageType, UObjectPropertyTypes...>;

	static bool SupportsProperty(const FProperty& InProperty)
	{
		FVariantPropertyTypeIndex Unused;
		return ComputeVariantIndex(InProperty, Unused);
	}

	static bool ComputeVariantIndex(const FProperty& Property, FVariantPropertyTypeIndex& OutTypeIndex)
	{
		using FuncDef = bool (*)(const FProperty& InProperty);
		static constexpr FuncDef Funcs[] = { &TPropertyMatch<UObjectPropertyTypes>::SupportsProperty... };

		for (int32 Index = 0; Index < CompileTimeNum; ++Index)
		{
			if (Funcs[Index](Property))
			{
				OutTypeIndex.Index = Index;
				return true;
			}
		}
		return false;
	}

	static FIntermediatePropertyValue CoercePropertyValue(const FProperty& InProperty, const FSourcePropertyValue& InPropertyValue)
	{
		FVariantPropertyTypeIndex VariantIndex;

		const bool bIsSupported = ComputeVariantIndex(InProperty, VariantIndex);
		check(bIsSupported);

		using FuncDef = FIntermediatePropertyValue (*)(const FProperty& InProperty, const FSourcePropertyValue& InPropertyValue);
		static constexpr FuncDef Funcs[] = { &TPropertyTraits<UObjectPropertyTypes, StorageType>::CoercePropertyValue... };

		return CoercePropertyValueChecked(InProperty, VariantIndex, InPropertyValue);
	}

	static FIntermediatePropertyValue CoercePropertyValueChecked(const FProperty& InProperty, FVariantPropertyTypeIndex VariantIndex, const FSourcePropertyValue& InPropertyValue)
	{
		using FuncDef = FIntermediatePropertyValue (*)(const FProperty& InProperty, const FSourcePropertyValue& InPropertyValue);
		static constexpr FuncDef Funcs[] = { &TPropertyTraits<UObjectPropertyTypes, StorageType>::CoercePropertyValue... };

		return Funcs[VariantIndex.Index](InProperty, InPropertyValue);
	}

	static void UnpackChannels(const InMemoryType& Value, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues)
	{
		UnpackChannelsFromOperational(Value, Property, OutUnpackedValues);
	}

	template<typename ...Composites>
	static TVariantPropertyComponentHandler<TraitsType, Composites...> MakeHandler()
	{
		return TVariantPropertyComponentHandler<TraitsType, Composites...>();
	}

	static bool NeedsMetaData()
	{
		return sizeof...(UObjectPropertyTypes) > 1;
	}

	static void GetObjectPropertyValue(const UObject* InObject, FVariantPropertyTypeIndex VariantTypeIndex, const FCustomPropertyAccessor& BaseCustomAccessor, StorageType& OutValue)
	{
		const TCustomPropertyAccessor<TraitsType>& CustomAccessor = static_cast<const TCustomPropertyAccessor<TraitsType>&>(BaseCustomAccessor);

		OutValue = (*CustomAccessor.Functions.Getter)(InObject);
	}

	static void GetObjectPropertyValue(const UObject* InObject, FVariantPropertyTypeIndex VariantTypeIndex, uint16 PropertyOffset, StorageType& OutValue)
	{
		using FuncDef = void (*)(const UObject*, uint16, StorageType&);
		static constexpr FuncDef Funcs[] = { &TPropertyTraits<UObjectPropertyTypes, StorageType>::GetObjectPropertyValue... };

		if (VariantTypeIndex.Index >= 0 && VariantTypeIndex.Index < CompileTimeNum)
		{
			Funcs[VariantTypeIndex.Index](InObject, PropertyOffset, OutValue);
		}
	}
	static void GetObjectPropertyValue(const UObject* InObject, FVariantPropertyTypeIndex VariantTypeIndex, FTrackInstancePropertyBindings* PropertyBindings, StorageType& OutValue)
	{
		using FuncDef = void (*)(const UObject*, FTrackInstancePropertyBindings*, StorageType&);
		static constexpr FuncDef Funcs[] = { &TPropertyTraits<UObjectPropertyTypes, StorageType>::GetObjectPropertyValue... };

		if (VariantTypeIndex.Index >= 0 && VariantTypeIndex.Index < CompileTimeNum)
		{
			Funcs[VariantTypeIndex.Index](InObject, PropertyBindings, OutValue);
		}
	}

	static void SetObjectPropertyValue(UObject* InObject, FVariantPropertyTypeIndex VariantTypeIndex, const FCustomPropertyAccessor& BaseCustomAccessor, StorageTypeParam InValue)
	{
		const TCustomPropertyAccessor<TraitsType>& CustomAccessor = static_cast<const TCustomPropertyAccessor<TraitsType>&>(BaseCustomAccessor);
		(*CustomAccessor.Functions.Setter)(InObject, InValue);
	}
	static void SetObjectPropertyValue(UObject* InObject, FVariantPropertyTypeIndex VariantTypeIndex, uint16 PropertyOffset, StorageTypeParam InValue)
	{
		using FuncDef = void (*)(UObject*, uint16, StorageTypeParam);
		static constexpr FuncDef Funcs[] = { &TPropertyTraits<UObjectPropertyTypes, StorageType>::SetObjectPropertyValue... };

		if (VariantTypeIndex.Index >= 0 && VariantTypeIndex.Index < CompileTimeNum)
		{
			Funcs[VariantTypeIndex.Index](InObject, PropertyOffset, InValue);
		}
	}
	static void SetObjectPropertyValue(UObject* InObject, FVariantPropertyTypeIndex VariantTypeIndex, FTrackInstancePropertyBindings* PropertyBindings, StorageTypeParam InValue)
	{
		using FuncDef = void (*)(UObject*, FTrackInstancePropertyBindings*, StorageTypeParam);
		static constexpr FuncDef Funcs[] = { &TPropertyTraits<UObjectPropertyTypes, StorageType>::SetObjectPropertyValue... };

		if (VariantTypeIndex.Index >= 0 && VariantTypeIndex.Index < CompileTimeNum)
		{
			Funcs[VariantTypeIndex.Index](InObject, PropertyBindings, InValue);
		}
	}

	template<typename ...T>
	static StorageType CombineComposites(FVariantPropertyTypeIndex VariantTypeIndex, T&&... InComposites)
	{
		return StorageType{ Forward<T>(InComposites)... };
	}
};


template<typename InMemoryType, typename ...UObjectPropertyTypes>
struct TDynamicVariantPropertyTraits : IPropertyTraits
{
	using StaticTraits     = TVariantPropertyTraits<InMemoryType, UObjectPropertyTypes...>;
	using TraitsType       = TDynamicVariantPropertyTraits<InMemoryType, UObjectPropertyTypes...>;
	using StorageType      = InMemoryType;
	using StorageTypeParam = typename TCallTraits<StorageType>::ParamType;

	using MetaDataType   = TPropertyMetaData<FVariantPropertyTypeIndex>;
	using PublicMetaData = TPropertyMetaData<>;

	static constexpr bool  bIsComposite   = true;

	TArray<TDynamicVariantTraitsBase<StorageType>> DynamicTraits;

	bool NeedsMetaData() const
	{
		return sizeof...(UObjectPropertyTypes) + DynamicTraits.Num() > 1;
	}

	static void UnpackChannels(const InMemoryType& Value, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues)
	{
		StaticTraits::UnpackChannels(Value, Property, OutUnpackedValues);
	}

	template<typename ...Composites>
	static TVariantPropertyComponentHandler<TraitsType, Composites...> MakeHandler()
	{
		return TVariantPropertyComponentHandler<TraitsType, Composites...>();
	}

	virtual TSubclassOf<UMovieSceneTrack> GetTrackClass(const FProperty* InProperty) const override
	{
		if (const FStructProperty* StructProperty = CastField<const FStructProperty>(InProperty))
		{
			const TDynamicVariantTraitsBase<StorageType>* Variant = Algo::FindBy(DynamicTraits, StructProperty->Struct, &TDynamicVariantTraitsBase<StorageType>::Struct);
			if (Variant)
			{
				return Variant->TrackClass;
			}
		}
		return nullptr;
	}
	virtual bool InitializeTrackFromProperty(UMovieSceneTrack* InTrack, const FProperty* InProperty) const override
	{
		if (const FStructProperty* StructProperty = CastField<const FStructProperty>(InProperty))
		{
			const TDynamicVariantTraitsBase<StorageType>* Variant = Algo::FindBy(DynamicTraits, StructProperty->Struct, &TDynamicVariantTraitsBase<StorageType>::Struct);
			if (Variant && Variant->InitializeNewTrack)
			{
				(*Variant->InitializeNewTrack)(InTrack, InProperty);
				return true;
			}
		}
		return false;
	}

	bool ComputeVariantIndex(const FProperty& InProperty, FVariantPropertyTypeIndex& OutTypeIndex) const
	{
		if (StaticTraits::ComputeVariantIndex(InProperty, OutTypeIndex))
		{
			return true;
		}

		if (const FStructProperty* StructProperty = CastField<const FStructProperty>(&InProperty))
		{
			for (int32 DynamicIndex = 0; DynamicIndex < DynamicTraits.Num(); ++DynamicIndex)
			{
				if (StructProperty->Struct == DynamicTraits[DynamicIndex].Struct)
				{
					OutTypeIndex.Index = StaticTraits::CompileTimeNum + DynamicIndex;
					return true;
				}
			}
		}
		return false;
	}
	bool SupportsProperty(const FProperty& InProperty) const
	{
		FVariantPropertyTypeIndex Unused;
		return ComputeVariantIndex(InProperty, Unused);
	}
	FIntermediatePropertyValue CoercePropertyValue(const FProperty& InProperty, const FSourcePropertyValue& InPropertyValue) const
	{
		FVariantPropertyTypeIndex StaticVariantIndex;
		if (StaticTraits::ComputeVariantIndex(InProperty, StaticVariantIndex))
		{
			return StaticTraits::CoercePropertyValueChecked(InProperty, StaticVariantIndex, InPropertyValue);
		}

		check(DynamicTraits.Num() > 0);

		const FStructProperty* StructProperty = CastFieldChecked<const FStructProperty>(&InProperty);

		for (int32 DynamicIndex = 0; DynamicIndex < DynamicTraits.Num()-1; ++DynamicIndex)
		{
			if (StructProperty->Struct == DynamicTraits[DynamicIndex].Struct)
			{
				StorageType NewValue;
				DynamicTraits[DynamicIndex].CastToOperational(InPropertyValue.Get(), NewValue);
				return FIntermediatePropertyValue::FromValue(NewValue);
			}
		}

		check(StructProperty->Struct == DynamicTraits.Last().Struct);

		StorageType NewValue;
		DynamicTraits.Last().CastToOperational(InPropertyValue.Get(), NewValue);
		return FIntermediatePropertyValue::FromValue(NewValue);
	}

	void AddDynamicType(const TDynamicVariantTraitsBase<StorageType>& Variant)
	{
		DynamicTraits.Emplace(Variant);
	}

	void GetObjectPropertyValue(const UObject* InObject, FVariantPropertyTypeIndex VariantTypeIndex, uint16 PropertyOffset, StorageType& OutValue) const
	{
		if (VariantTypeIndex.Index >= StaticTraits::CompileTimeNum)
		{
			const int32 DynamicIndex = int32(VariantTypeIndex.Index) - StaticTraits::CompileTimeNum;
			if (DynamicTraits.IsValidIndex(DynamicIndex))
			{
				const void* PropertyAddress = reinterpret_cast<const uint8*>(InObject) + PropertyOffset;
				DynamicTraits[DynamicIndex].CastToOperational(PropertyAddress, OutValue);
			}
		}
		else
		{
			StaticTraits::GetObjectPropertyValue(InObject, VariantTypeIndex, PropertyOffset, OutValue);
		}
	}
	void GetObjectPropertyValue(const UObject* InObject, FVariantPropertyTypeIndex VariantTypeIndex, FTrackInstancePropertyBindings* PropertyBindings, StorageType& OutValue) const
	{
		if (VariantTypeIndex.Index >= StaticTraits::CompileTimeNum)
		{
			const int32 DynamicIndex = int32(VariantTypeIndex.Index) - StaticTraits::CompileTimeNum;
			if (DynamicTraits.IsValidIndex(DynamicIndex))
			{
				DynamicTraits[DynamicIndex].RetrieveSlowProperty(InObject, PropertyBindings, OutValue);
			}
		}
		else
		{
			StaticTraits::GetObjectPropertyValue(InObject, VariantTypeIndex, PropertyBindings, OutValue);
		}
	}

	static void GetObjectPropertyValue(const UObject* InObject, FVariantPropertyTypeIndex VariantTypeIndex, const FCustomPropertyAccessor& BaseCustomAccessor, StorageType& OutValue)
	{
		const TCustomPropertyAccessor<TraitsType>& CustomAccessor = static_cast<const TCustomPropertyAccessor<TraitsType>&>(BaseCustomAccessor);

		OutValue = (*CustomAccessor.Functions.Getter)(InObject);
	}

	void SetObjectPropertyValue(UObject* InObject, FVariantPropertyTypeIndex VariantTypeIndex, uint16 PropertyOffset, StorageTypeParam InValue) const
	{
		if (VariantTypeIndex.Index >= StaticTraits::CompileTimeNum)
		{
			const int32 DynamicIndex = int32(VariantTypeIndex.Index) - StaticTraits::CompileTimeNum;
			if (DynamicTraits.IsValidIndex(DynamicIndex))
			{
				void* PropertyAddress = reinterpret_cast<uint8*>(InObject) + PropertyOffset;
				DynamicTraits[DynamicIndex].CastToFinal(InValue, PropertyAddress);
			}
		}
		else
		{
			StaticTraits::SetObjectPropertyValue(InObject, VariantTypeIndex, PropertyOffset, InValue);
		}
	}
	void SetObjectPropertyValue(UObject* InObject, FVariantPropertyTypeIndex VariantTypeIndex, FTrackInstancePropertyBindings* PropertyBindings, StorageTypeParam InValue) const
	{
		if (VariantTypeIndex.Index >= StaticTraits::CompileTimeNum)
		{
			const int32 DynamicIndex = int32(VariantTypeIndex.Index) - StaticTraits::CompileTimeNum;
			if (DynamicTraits.IsValidIndex(DynamicIndex))
			{
				DynamicTraits[DynamicIndex].ApplySlowProperty(InObject, PropertyBindings, InValue);
			}
		}
		else
		{
			StaticTraits::SetObjectPropertyValue(InObject, VariantTypeIndex, PropertyBindings, InValue);
		}
	}

	static void SetObjectPropertyValue(UObject* InObject, FVariantPropertyTypeIndex VariantTypeIndex, const FCustomPropertyAccessor& BaseCustomAccessor, StorageTypeParam InValue)
	{
		const TCustomPropertyAccessor<TraitsType>& CustomAccessor = static_cast<const TCustomPropertyAccessor<TraitsType>&>(BaseCustomAccessor);
		(*CustomAccessor.Functions.Setter)(InObject, InValue);
	}

	template<typename ...T>
	static StorageType CombineComposites(FVariantPropertyTypeIndex VariantTypeIndex, T&&... InComposites)
	{
		return StorageType{ Forward<T>(InComposites)... };
	}
};



/**
 * Property accessor traits that do not know the underlying UObjectPropertyType until runtime
 */
template<typename RuntimeType, typename ...MetaDataTypes>
struct TRuntimePropertyTraits
{
	using StorageType      = RuntimeType;
	using StorageTypeParam = typename TCallTraits<StorageType>::ParamType;
	using MetaDataType     = TPropertyMetaData<MetaDataTypes...>;


	/** Property Value Getters  */
	static void GetObjectPropertyValue(const UObject* InObject, const FCustomPropertyAccessor& BaseCustomAccessor, typename TCallTraits<MetaDataTypes>::ParamType... MetaData, StorageType& OutValue)
	{}

	static void GetObjectPropertyValue(const UObject* InObject, uint16 PropertyOffset, typename TCallTraits<MetaDataTypes>::ParamType... MetaData, StorageType& OutValue)
	{}

	static void GetObjectPropertyValue(const UObject* InObject, FTrackInstancePropertyBindings* PropertyBindings, typename TCallTraits<MetaDataTypes>::ParamType... MetaData, StorageType& OutValue)
	{}

	static void GetObjectPropertyValue(const UObject* InObject, const FName& PropertyPath, StorageType& OutValue)
	{}

	/** Property Value Setters  */
	static void SetObjectPropertyValue(UObject* InObject, const FCustomPropertyAccessor& BaseCustomAccessor, typename TCallTraits<MetaDataTypes>::ParamType... MetaData, StorageTypeParam InValue)
	{}

	static void SetObjectPropertyValue(UObject* InObject, uint16 PropertyOffset, typename TCallTraits<MetaDataTypes>::ParamType... MetaData, StorageTypeParam InValue)
	{}

	static void SetObjectPropertyValue(UObject* InObject, FTrackInstancePropertyBindings* PropertyBindings, typename TCallTraits<MetaDataTypes>::ParamType... MetaData, StorageTypeParam InValue)
	{}
};



template<typename UObjectPropertyType, bool bInIsComposite = true>
using TDirectPropertyTraits = TPropertyTraits<UObjectPropertyType, UObjectPropertyType, bInIsComposite>;

template<typename UObjectPropertyType, typename InMemoryType, bool bInIsComposite = true>
struct UE_DEPRECATED(5.4, "Please use TPropertyTraits directly.") TIndirectPropertyTraits : TPropertyTraits<UObjectPropertyType, InMemoryType, bInIsComposite>
{};


} // namespace MovieScene
} // namespace UE


