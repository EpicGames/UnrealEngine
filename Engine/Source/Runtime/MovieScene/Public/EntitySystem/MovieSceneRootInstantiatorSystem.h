// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntityInstantiatorSystem.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneRootInstantiatorSystem.generated.h"

class UObject;
namespace UE { namespace MovieScene { struct FEntityAllocation; } }


UCLASS(MinimalAPI)
class UMovieSceneRootInstantiatorSystem : public UMovieSceneEntityInstantiatorSystem
{
	GENERATED_BODY()

public:

	MOVIESCENE_API UMovieSceneRootInstantiatorSystem(const FObjectInitializer& ObjInit);

private:

	MOVIESCENE_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override final;

	void InstantiateAllocation(const UE::MovieScene::FEntityAllocation* ParentAllocation);
};
