// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compilation/MovieSceneCompiledDataID.h"
#include "Containers/Array.h"
#include "Containers/SortedMap.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/IDelegateInstance.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntityLedger.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "Evaluation/MovieScenePlayback.h"
#include "MovieSceneSequenceID.h"
#include "Templates/UniquePtr.h"

class FMovieSceneLegacyPlayer;
class IMovieScenePlayer;
class UMovieSceneEntitySystemLinker;
class UObject;
struct FFrameTime;
struct FMovieSceneTrackEvaluator;
template <typename ElementType> class TRange;

namespace UE
{
namespace MovieScene
{

struct FCompiledDataVolatilityManager;
struct FPreAnimatedStateExtension;
struct FSequenceInstance;
struct FSharedPlaybackState;
struct FSubSequencePath;
struct ISequenceUpdater;

enum class ESequenceInstanceUpdateFlags : uint8
{
	None                = 0,
	NeedsDissection     = 1u << 0,
	NeedsPreEvaluation  = 1u << 1,
	NeedsPostEvaluation = 1u << 2,
	HasLegacyTemplates  = 1u << 3,
};
ENUM_CLASS_FLAGS(ESequenceInstanceUpdateFlags);

/**
 * Specifies what cached data should be invalidated in a sequence instance.
 */
enum class ESequenceInstanceInvalidationType : uint8
{
	/**
	 * Only invalidate cached data for sequences whose data has been dynamically changed.
	 */
	DataChanged,

	/**
	 * Invalidate all cached data for all sequences.
	 */
	All
};

/**
 * A sequence instance represents a specific instance of a currently playing sequence, either as a top-level sequence in an IMovieScenePlayer, or as a sub sequence.
 * Any given sequence asset may have any number of instances created for it at any given time depending on how many times it is referenced by playing sequences
 */
struct FSequenceInstance
{
	/** Ledger that tracks all currently instantiated entities for this instance */
	FEntityLedger Ledger;

public:

	/**
	 * Conditionally recompile this sequence if it needs to be
	 *
	 * @param Linker     The linker that owns this sequence instance
	 * @return true if a recompile has ocurred, false otherwise
	 */
	MOVIESCENE_API bool ConditionalRecompile();

	/**
	 * Called only for top-level sequence instances before any updates to it with the specified context.
	 * This allows the sequence an opportunity to dissect the context into a series of distinct evaluations to force determinism.
	 *
	 * @param Linker     The linker that owns this sequence instance
	 * @param Context    The overall context that this sequence instance is being evaluated with
	 * @param OutDissections   An array to populate with dissected time ranges that should be evaluated separately, in order.
	 */
	MOVIESCENE_API void DissectContext(const FMovieSceneContext& Context, TArray<TRange<FFrameTime>>& OutDissections);

	/**
	 * Called for root level instances that have either never evaluated, or have previously finished evaluating
	 *
	 * @param Linker     The linker that owns this sequence instance
	 * @param InContext  The context that this sequence instance is to be evaluated with
	 */
	MOVIESCENE_API void Start(const FMovieSceneContext& InContext);

	/**
	 * Called when this instance has been queued for evaluation in order for it to do any pre-work setup.
	 *
	 * @param Linker     The linker that owns this sequence instance
	 */
	MOVIESCENE_API void PreEvaluation();

	/**
	 * Called after dissection for root level instances in order for this sequence instacne to update any entities it needs for evaluation.
	 *
	 * @param Linker     The linker that owns this sequence instance
	 * @param InContext  The (potentially dissected) context that this sequence instance is to be evaluated with
	 */
	MOVIESCENE_API void Update(const FMovieSceneContext& InContext);

	/**
	 * Returns whether this instance can be finished immediately without any last update.
	 *
	 * @param Linker     The linker that owns this sequence instance
	 * @return           Whether the instance can be finished immediately
	 */
	MOVIESCENE_API bool CanFinishImmediately() const;

	/**
	 * Mark this instance as finished, causing all its entities to be unlinked and the instance to become inactive at the end of the next update.
	 *
	 * @param Linker     The linker that owns this sequence instance
	 */
	MOVIESCENE_API void Finish();

	/**
	 * Called when this sequence instance has been evaluated in order for it to do any clean-up or other post-update work
	 *
	 * @param Linker     The linker that owns this sequence instance
	 */
	MOVIESCENE_API void PostEvaluation();

	/**
	 * Called to run legacy track templates
	 */
	MOVIESCENE_API void RunLegacyTrackTemplates();

public:

	/**
	 * Retrieve the shared playback state for this instance's hierarchy
	 */
	TSharedRef<FSharedPlaybackState> GetSharedPlaybackState() const
	{
		return SharedPlaybackState;
	}

	/**
	 * Retrieve the IMovieScenePlayer that is playing back the top level sequence for this instance
	 *
	 * @return A pointer to this instance's player
	 */
	UE_DEPRECATED(5.5, "The player interface is now optional and will be deprecated in a future relase. Please use FPlayerIndexPlaybackCapability::GetPlayer()")
	MOVIESCENE_API IMovieScenePlayer* GetPlayer() const;

	/**
	 * Retrieve the IMovieScenePlayer's unique index
	 */
	UE_DEPRECATED(5.5, "The player interface is now optional and will be deprecated in a future relase. Please use FPlayerIndexPlaybackCapability::GetPlayerIndex()")
	MOVIESCENE_API uint16 GetPlayerIndex() const;

	/**
	 * Retrieve the SequenceID for this instance
	 *
	 * @return This sequence instance's SequenceID within the root-sequences hierachy, or MovieSceneSequenceID::Root for root sequence instances.
	 */
	FMovieSceneSequenceID GetSequenceID() const
	{
		return SequenceID;
	}

	/**
	 * Get the evaluation context for the current frame
	 *
	 * @return This sequence instance's playback context
	 */
	const FMovieSceneContext& GetContext() const
	{
		return Context;
	}

	/**
	 * Gets the handle to this instance
	 */
	FInstanceHandle GetInstanceHandle() const
	{
		return InstanceHandle;
	}

	/**
	 * Get a handle to the root instance for this sub sequence instance
	 *
	 * @return A handle to the root sequence instance for this sub sequence, or a handle to this sequence instance if it is not a sub sequence
	 */
	FRootInstanceHandle GetRootInstanceHandle() const
	{
		return RootInstanceHandle;
	}

	/**
	 * Get a handle to the parent instance for this sequence instance, or an invalid handle if it is a root instance
	 *
	 * @return A a handle to the parent instance for this sequence instance, or an invalid handle if it is a root instance
	 */
	FInstanceHandle GetParentInstanceHandle() const
	{
		return ParentInstanceHandle;
	}

	/**
	 * Returns whether this instance is the root instance.
	 */
	bool IsRootSequence() const
	{
		return RootInstanceHandle == InstanceHandle;
	}

	/**
	 * Returns whether this instance is a sub sequence.
	 */
	bool IsSubSequence() const
	{
		return RootInstanceHandle != InstanceHandle;
	}

	/**
	 * Get the serial number for this sequence instance that it was created with on construction.
	 *
	 * @return The serial number of this instance that is used to check that handles to it are still valid.
	 */
	uint16 GetSerialNumber() const
	{
		return InstanceHandle.InstanceSerial;
	}

	/**
	 * Check whether this sequence instance has finished evaluating
	 *
	 * @return True if this sequence has finished evaluation, false otherwise
	 */
	bool HasFinished() const
	{
		return bFinished;
	}

	/**
	 * Check whether this sequence instance has ever been updated or not
	 */
	bool HasEverUpdated() const
	{
		return bHasEverUpdated;
	}

	/**
	 * Retrieve this sequence's update flags
	 */
	ESequenceInstanceUpdateFlags GetUpdateFlags() const
	{
		return UpdateFlags;
	}

	/**
	 * Attempt to locate a sub instance based on its sequence ID
	 */
	MOVIESCENE_API FInstanceHandle FindSubInstance(FMovieSceneSequenceID SequenceID) const;

	/**
	 * Attempt to locate an entity given its owner and ID
	 */
	MOVIESCENE_API FMovieSceneEntityID FindEntity(UObject* Owner, uint32 EntityID) const;

	/**
	 * Attempt to locate all entities given their owner
	 */
	MOVIESCENE_API void FindEntities(UObject* Owner, TArray<FMovieSceneEntityID>& OutEntityIDs) const;

	/**
	 * Retrieve the legacy evaluator for this sequence, if it is available (may return nullptr)
	 */
	const FMovieSceneTrackEvaluator* GetLegacyEvaluator() const
	{
		return LegacyEvaluator.Get();
	}

	/**
	 * Retrieve a path for this sequence instance back to the root
	 */
	MOVIESCENE_API FSubSequencePath GetSubSequencePath() const;

public:

	/**
	 * Get the evaluation context for the current frame
	 *
	 * @return This sequence instance's playback context
	 */
	void SetContext(const FMovieSceneContext& InContext)
	{
		Context = InContext;
	}

	/**
	 * Indicate that this sequence instance has finished evaluation and should remove its entities
	 */
	void SetFinished(bool bInFinished)
	{
		bFinished = bInFinished;
	}

	/**
	 * Invalidate any cached data that may be being used for evaluation due to a change in the source asset data
	 */
	MOVIESCENE_API void InvalidateCachedData(ESequenceInstanceInvalidationType InvalidationType = ESequenceInstanceInvalidationType::All);

	/**
	 * Destroy this sequence instance immediately - Finish must previously have been called
	 */
	MOVIESCENE_API void DestroyImmediately();

	/**
	 * Called to override the simulated root sequence ID for this instance. Only valid for IsRootSequence() instances.
	 */
	MOVIESCENE_API void OverrideRootSequence(FMovieSceneSequenceID NewRootSequenceID);

	/*
	* Evaluate a condition on this sequence instance hierarchy, utilizing any cache as relevant.
	*/
	MOVIESCENE_API bool EvaluateCondition(const FGuid& BindingID, const FMovieSceneSequenceID& SequenceID, const UMovieSceneCondition* Condition, UObject* ConditionOwnerObject) const;

public:

	/** Constructor for top level sequences */
	MOVIESCENE_API explicit FSequenceInstance(TSharedRef<FSharedPlaybackState> PlaybackState);

	/** Constructor for sub sequences */
	MOVIESCENE_API explicit FSequenceInstance(TSharedRef<FSharedPlaybackState> PlaybackState, FInstanceHandle ThisInstanceHandle, FInstanceHandle InParentInstanceHandle, FMovieSceneSequenceID InSequenceID);

	/** Finish initializing this sequence instance */
	MOVIESCENE_API void Initialize();

	/** Destructor */
	MOVIESCENE_API ~FSequenceInstance();

	/** Sequence instances are not copyable */
	FSequenceInstance(const FSequenceInstance&) = delete;
	FSequenceInstance& operator=(const FSequenceInstance&) = delete;

	/** Move constructors implemented in cpp to avoid includes for TUniquePtr */
	MOVIESCENE_API FSequenceInstance(FSequenceInstance&&);
	MOVIESCENE_API FSequenceInstance& operator=(FSequenceInstance&&);

public:

	UE_DEPRECATED(5.4, "Please use the version of this method without a Linker parameter")
	bool ConditionalRecompile(UMovieSceneEntitySystemLinker* Linker);
	UE_DEPRECATED(5.4, "Please use the version of this method without a Linker parameter")
	void DissectContext(UMovieSceneEntitySystemLinker* Linker, const FMovieSceneContext& Context, TArray<TRange<FFrameTime>>& OutDissections);
	UE_DEPRECATED(5.4, "Please use the version of this method without a Linker parameter")
	void Start(UMovieSceneEntitySystemLinker* Linker, const FMovieSceneContext& InContext);
	UE_DEPRECATED(5.4, "Please use the version of this method without a Linker parameter")
	void PreEvaluation(UMovieSceneEntitySystemLinker* Linker);
	UE_DEPRECATED(5.4, "Please use the version of this method without a Linker parameter")
	void Update(UMovieSceneEntitySystemLinker* Linker, const FMovieSceneContext& InContext);
	UE_DEPRECATED(5.4, "Please use the version of this method without a Linker parameter")
	bool CanFinishImmediately(UMovieSceneEntitySystemLinker* Linker) const;
	UE_DEPRECATED(5.4, "Please use the version of this method without a Linker parameter")
	void Finish(UMovieSceneEntitySystemLinker* Linker);
	UE_DEPRECATED(5.4, "Please use the version of this method without a Linker parameter")
	void PostEvaluation(UMovieSceneEntitySystemLinker* Linker);
	UE_DEPRECATED(5.4, "Please use the version of this method without a Linker parameter")
	void InvalidateCachedData(UMovieSceneEntitySystemLinker* Linker);
	UE_DEPRECATED(5.4, "Please use the version of this method without a Linker parameter")
	void DestroyImmediately(UMovieSceneEntitySystemLinker* Linker);
	UE_DEPRECATED(5.4, "Please use the version of this method without a Linker parameter")
	void OverrideRootSequence(UMovieSceneEntitySystemLinker* Linker, FMovieSceneSequenceID NewRootSequenceID);

private:

	void InitializeLegacyEvaluator();
	void InitializeLegacyPlayer();

private:

	FMovieSceneContext Context;

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
	/** Name of the root sequence */
	FString RootSequenceName;
#endif

	/** For top-level sequences only - legacy track template evaluator for the entire sequence */
	TUniquePtr<FMovieSceneTrackEvaluator> LegacyEvaluator;
	/** For top-level sequences only - legacy player */
	TUniquePtr<FMovieSceneLegacyPlayer> LegacyPlayer;
	/** For top-level sequences only - interface for either a flat or hierarchical entity updater */
	TUniquePtr<ISequenceUpdater> SequenceUpdater;
	/** For top-level sequences only - a utility class that is used to ensure that volatile sequences are up to date. Only valid in editor, or for sequences that have the volatile flag. */
	TUniquePtr<FCompiledDataVolatilityManager> VolatilityManager;

	/** Playback state shared by the entire sequence hierarchy */
	TSharedRef<FSharedPlaybackState> SharedPlaybackState;

	/** This sequence instances sequence ID, or MovieSceneSequenceID::Root for top-level sequences. */
	FMovieSceneSequenceID SequenceID;
	/** When SequenceID != MovieSceneSequenceID::Root, specifies an ID to override as a simulated root. */
	FMovieSceneSequenceID RootOverrideSequenceID;
	/** Cached update flags denoting what kinds of updates are required by this instance */
	ESequenceInstanceUpdateFlags UpdateFlags;
	/** This instance's handle. */
	FInstanceHandle InstanceHandle;
	/** This instance's parent handle. */
	FInstanceHandle ParentInstanceHandle;
	/** This instance's root handle, if it is a sub sequence. */
	FRootInstanceHandle RootInstanceHandle;
	/** Flag that indicates whether this instance was initialized */
	bool bInitialized : 1;
	/** Flag that is set when this sequence has or (will be) finished. */
	bool bFinished : 1;
	/** Flag that is set if this sequence has ever updated. */
	bool bHasEverUpdated : 1;

	friend struct FScopedVolatilityManagerSuppression;
};


} // namespace MovieScene
} // namespace UE
