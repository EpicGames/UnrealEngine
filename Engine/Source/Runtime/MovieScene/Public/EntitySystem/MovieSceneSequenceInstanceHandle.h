// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "EntitySystem/MovieSceneComponentDebug.h"

namespace UE
{
namespace MovieScene
{

/** A handle to an active instance of a UMovieSceneSequence */
struct FInstanceHandle
{
	uint16 InstanceID;
	uint16 InstanceSerial;

	FInstanceHandle()
		: InstanceID(uint16(-1))
		, InstanceSerial(0)
	{}

	FInstanceHandle(uint16 InInstanceID, uint16 InInstanceSerial)
		: InstanceID(InInstanceID)
		, InstanceSerial(InInstanceSerial)
	{}

	bool IsValid() const
	{
		return InstanceID != uint16(-1);
	}

	friend bool operator==(FInstanceHandle A, FInstanceHandle B)
	{
		return A.InstanceID == B.InstanceID && A.InstanceSerial == B.InstanceSerial;
	}
	friend bool operator!=(FInstanceHandle A, FInstanceHandle B)
	{
		return !(A == B);
	}
	friend bool operator<(FInstanceHandle A, FInstanceHandle B)
	{
		return A.InstanceID < B.InstanceID;
	}
	friend uint32 GetTypeHash(FInstanceHandle In)
	{
		return uint32(In.InstanceID) | (uint32(In.InstanceSerial) >> 16);
	}
};

/**
 * A handle to a root instance of a UMovieSceneSequence.
 * This type should be used where type-safety needs to be enforced for root instances.
 * FRootInstanceHandle is implicitly castable to FInstanceHandle but not visa-versa (even explicitly so).
 */
struct FRootInstanceHandle : FInstanceHandle
{
	FRootInstanceHandle()
	{}

	FRootInstanceHandle(uint16 InInstanceID, uint16 InInstanceSerial)
		: FInstanceHandle(InInstanceID, InInstanceSerial)
	{}
};


} // namespace MovieScene
} // namespace UE