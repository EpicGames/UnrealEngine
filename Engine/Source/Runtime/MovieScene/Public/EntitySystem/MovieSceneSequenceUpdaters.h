// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "Math/Range.h"
#include "Misc/FrameTime.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

class UMovieSceneCompiledDataManager;
class UMovieSceneEntitySystemLinker;
class UMovieSceneCondition;
class UMovieSceneSequence;
struct FFrameTime;
struct FMovieSceneCompiledDataID;
struct FMovieSceneContext;
struct FMovieSceneSequenceID;
template <typename ElementType> class TRange;

namespace UE
{
namespace MovieScene
{

struct FInstanceHandle;
struct FSharedPlaybackState;

enum class ESequenceInstanceUpdateFlags : uint8;
enum class ESequenceInstanceInvalidationType : uint8;

/**
 * Interface for an object that will update a sequence based on the current context. It holds several responsibilities:
 *     1. Handle dissection of evaluation contexts based on determinism fences
 *     2. To add pending instantiations to FInstanceRegistry for newly-evaluated source entities
 *     3. To add the Unlink tag for any linker entities that are now longer required for evaluation (ie, have just finished evaluating)
 *     4. To handle 1 and 2 for any sub sequences that may or may not be required for the context
 *     5. To handle legacy track template evaluation
 */
struct ISequenceUpdater
{
	/**
	 * Entry-point for creating or updating a new ISequenceUpdater interface based on the requirements of the compiled data.
	 * If OutPtr is null, a new instance will always be assigned.
	 * If OutPtr is valid, but no longer suitable for the compiled data (ie, it does not support hierarchical sequences but the compiled data now has a hierarchy), a new instance will be assigned
	 *
	 * @param OutPtr              Reference to receive the new sequence updater interface
	 * @param CompiledDataManager The manager class that houses the compiled data for the sequence that needs updating
	 * @param CompiledDataID      The ID of the compiled data
	 */
	static void FactoryInstance(TUniquePtr<ISequenceUpdater>& OutPtr, UMovieSceneCompiledDataManager* CompiledDataManager, FMovieSceneCompiledDataID CompiledDataID);

public:

	/** Virtual destructor */
	virtual ~ISequenceUpdater() = default;

	/**
	 * Called to initialize the flag structure that denotes what functions need to be called on this updater
	 */
	virtual void PopulateUpdateFlags(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, ESequenceInstanceUpdateFlags& OutUpdateFlags) = 0;

	/**
	 * Called before any updates to the sequence to allow this updater to dissect the context into smaller ranges that should be evaluated independently for the purpose of ensuring determinism.
	 * If the resulting array is empty the whole context will be used by default.
	 *
	 * @param InLinker         The linker that is evaluating this sequence
	 * @param InPlayer         The movie scene player instance playing this sequence
	 * @param InContext        The total root-level context for the next evaluation to be dissected
	 * @param OutDissections   (Out) Array to populate with dissected ranges
	 */
	virtual void DissectContext(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, const FMovieSceneContext& InContext, TArray<TRange<FFrameTime>>& OutDissections) = 0;


	/**
	 * Called if this updater has not been evaluated before, or has since been Finished.
	 *
	 * @param InLinker         The linker that is evaluating this sequence
	 * @param InInstanceHandle The instance handle for the top level sequence instance that this updater belongs to
	 * @param InPlayer         The movie scene player instance playing this sequence
	 * @param InContext        The root-level context for the current evaluation.
	 */
	virtual void Start(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, const FMovieSceneContext& InContext) = 0;


	/**
	 * Called in order that this updater may make any changes to the linker's environment before its sequence is evaluated (ie, initialize or unlink any entity instantiations)
	 *
	 * @param InLinker         The linker that is evaluating this sequence
	 * @param InInstanceHandle The instance handle for the top level sequence instance that this updater belongs to
	 * @param InPlayer         The movie scene player instance playing this sequence
	 * @param InContext        The root-level context for the current evaluation.
	 */
	virtual void Update(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, const FMovieSceneContext& InContext) = 0;


	/**
	 * Returns whether this instance can be finished immediately without any last update.
	 *
	 * @param Linker              The linker that owns this sequence instance
	 * @param RootInstanceHandle  The handle to the root instance
	 * @return                    Whether the instance can be finished immediately
	 */
	virtual bool CanFinishImmediately(TSharedRef<const FSharedPlaybackState> SharedPlaybackState) const = 0;


	/**
	 * Called before evaluation when this updater's sequence is no longer required to be evaluated
	 *
	 * @param InLinker         The linker that is evaluating this sequence
	 * @param InInstanceHandle The instance handle for the top level sequence instance that this updater belongs to
	 * @param InPlayer         The movie scene player instance playing this sequence
	 */
	virtual void Finish(TSharedRef<const FSharedPlaybackState> SharedPlaybackState) = 0;


	/**
	 * Invalidate any cached information that depends on the compiled sequence data due to the compiled data changing
	 *
	 * @param InLinker         The linker that is evaluating this sequence
	 * @param InInstanceHandle The instance handle for the top level sequence instance that this updater belongs to
	 */
	virtual void InvalidateCachedData(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, ESequenceInstanceInvalidationType InvalidationType) = 0;


	/**
	 * Called right before this updater's sequence instance is about to be destroyed completely
	 *
	 * @param InLinker         The linker that is owns this sequence
	 */
	virtual void Destroy(TSharedRef<const FSharedPlaybackState> SharedPlaybackState) = 0;


	/**
	 * Override the sequence ID that should be considered the root sequence for this updater
	 *
	 * @param InLinker                    The linker that is owns this sequence
	 * @param InstanceHandle              The instance handle for the root sequence
	 * @param NewRootOverrideSequenceID   The new sequence ID to treat as the root
	 */
	virtual void OverrideRootSequence(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, FMovieSceneSequenceID NewRootOverrideSequenceID) = 0;


	/**
	 * Migrate this updater to one that can represent hierarchical data. 
	 *
	 * @return A new instance capable of dealing with hierarchical data, or nullptr if this already does.
	 */
	virtual TUniquePtr<ISequenceUpdater> MigrateToHierarchical() = 0;

	/**
	 * Find a sub sequence instance from its ID (if it exists)
	 */
	virtual FInstanceHandle FindSubInstance(FMovieSceneSequenceID SubSequenceID) const = 0;

	/*
	* Evaluate a condition on this sequence instance hierarchy, utilizing any cache as relevant.
	*/
	virtual bool EvaluateCondition(const FGuid& BindingID, const FMovieSceneSequenceID& SequenceID, const UMovieSceneCondition* Condition, UObject* ConditionOwnerObject, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const = 0;

public:

	UE_DEPRECATED(5.5, "Please use the method that takes a reason parameter.")
	MOVIESCENE_API void InvalidateCachedData(TSharedRef<const FSharedPlaybackState> SharedPlaybackState);
};

} // namespace MovieScene
} // namespace UE
