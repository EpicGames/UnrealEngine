// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compilation/MovieSceneCompiledDataID.h"
#include "CoreTypes.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "Evaluation/MovieSceneEvaluationOperand.h"
#include "Evaluation/MovieScenePlaybackCapabilities.h"
#include "Evaluation/MovieScenePreAnimatedState.h"
#include "MovieSceneSequenceID.h"

#define UE_API MOVIESCENE_API

class UMovieSceneSequence;
class FMovieSceneEntitySystemRunner;
class UMovieSceneCompiledDataManager;
class UMovieSceneEntitySystemLinker;
struct FMovieSceneObjectCache;
struct FMovieSceneSequenceHierarchy;

namespace UE::MovieScene
{

/**
* Playback capability for recording audio from sequences.
*/
struct FMovieSceneAudioRecordingCapability
{
	UE_DECLARE_MOVIESCENE_PLAYBACK_CAPABILITY_API(MOVIESCENE_API, FMovieSceneAudioRecordingCapability)

	// True when recording audio from scene, e.g. via MRQ.
	bool bIsRecordingAudio = false;
};

/**
 * Parameter structure for initializing a new shared playback state.
 */
struct FSharedPlaybackStateCreateParams
{
	/**
	 * The playback context in which the root sequence will be evaluated.
	 */
	UObject* PlaybackContext = nullptr;

	/**
	 * The handle of the root sequence instance, if the created playback state
	 * is meant to relate to an instance that has also been created inside
	 * a runner/linker's instance registry.
	 */
	FRootInstanceHandle RootInstanceHandle;

	/**
	 * The linker that will be evaluating the sequence that the created playback
	 * state relates to.
	 */
	TObjectPtr<UMovieSceneEntitySystemLinker> Linker;

	/**
	 * The compiled data manager with which the root sequence was compiled, or
	 * will be compiled. If unset, the default global manager will be used.
	 */
	UMovieSceneCompiledDataManager* CompiledDataManager = nullptr;
};

/**
 * A structure that stores playback state for an entire sequence hierarchy.
 */
struct FSharedPlaybackState : TSharedFromThis<FSharedPlaybackState>
{
public:

	UE_API FSharedPlaybackState(UMovieSceneEntitySystemLinker* InLinker = nullptr);
	UE_API FSharedPlaybackState(
			UMovieSceneSequence& InRootSequence,
			const FSharedPlaybackStateCreateParams& CreateParams);

	UE_API ~FSharedPlaybackState();

public:
	
	/** Gets the playback context */
	UObject* GetPlaybackContext() const { return WeakPlaybackContext.Get(); }

	/** Gets the root sequence */
	UMovieSceneSequence* GetRootSequence() const { return WeakRootSequence.Get(); }

	/** Gets the linker evaluating this root sequence */
	UMovieSceneEntitySystemLinker* GetLinker() const { return WeakLinker.Get(); }

	/** Gets the compiled data manager that contains the data for the root sequence */
	TObjectPtr<UMovieSceneCompiledDataManager> GetCompiledDataManager() const { return CompiledDataManager; }

	/** Gets the handle of the root sequence */
	const FRootInstanceHandle& GetRootInstanceHandle() const { return RootInstanceHandle; }

	/** Gets the compiled data ID for the root sequence */
	const FMovieSceneCompiledDataID&  GetRootCompiledDataID() const { return RootCompiledDataID; }

	/** Gets the pre-animated state utility for the sequence hierarchy */
	const FMovieSceneInstancePreAnimatedState& GetPreAnimatedState() const { return PreAnimatedState; }

	/** Gets the pre-animated state utility for the sequence hierarchy */
	FMovieSceneInstancePreAnimatedState& GetPreAnimatedState() { return PreAnimatedState; }

public:

	// General utility methods

	/** Gets the runner evaluating this root sequence */
	UE_API TSharedPtr<FMovieSceneEntitySystemRunner> GetRunner() const;

	/** Gets the hierarchy (if any) for this root sequence */
	UE_API const FMovieSceneSequenceHierarchy* GetHierarchy() const;
	/** Gets a sub-sequence given an ID */
	UE_API UMovieSceneSequence* GetSequence(FMovieSceneSequenceIDRef SequenceID) const;

	/** Finds the bound objects for the given object binding in the given (sub)sequence */
	UE_API TArrayView<TWeakObjectPtr<>> FindBoundObjects(const FGuid& ObjectBindingID, FMovieSceneSequenceIDRef SequenceID) const;

	/** Clears object caches for the entire sequence hierarchy. */
	UE_API void ClearObjectCaches();

public:

	/**
	 * Gets the capabilities container.
	 */
	FPlaybackCapabilities& GetCapabilities()
	{
#if !UE_BUILD_SHIPPING
		ensureMsgf(IsInGameThread(), 
				TEXT("Playback capabilities aren't meant to be thread-safe. Do not modify or access their container outside of the game thread."));
#endif
		return Capabilities;
	}

	/**
	 * Gets the capabilities container.
	 */
	const FPlaybackCapabilities& GetCapabilities() const
	{
#if !UE_BUILD_SHIPPING
		ensureMsgf(IsInGameThread(), 
				TEXT("Playback capabilities aren't meant to be thread-safe. Do not modify or access their container outside of the game thread."));
#endif
		return Capabilities;
	}

	/**
	 * Returns whether the root sequence has the specified capability.
	 */
	template<typename T>
	bool HasCapability() const
	{
#if !UE_BUILD_SHIPPING
		ensureMsgf(IsInGameThread(), 
				TEXT("Playback capabilities aren't meant to be thread-safe. Do not modify or access their container outside of the game thread."));
#endif
		return Capabilities.HasCapability<T>();
	}

	/**
	 * Finds the specified capability on the root sequence.
	 */
	template<typename T>
	T* FindCapability() const
	{
#if !UE_BUILD_SHIPPING
		ensureMsgf(IsInGameThread(), 
				TEXT("Playback capabilities aren't meant to be thread-safe. Do not modify or access their container outside of the game thread."));
#endif
		return Capabilities.FindCapability<T>();
	}

	/**
	 * Builds the specified capability for the root sequence.
	 */
	template<typename T, typename ...ArgTypes>
	T& AddCapability(ArgTypes&&... InArgs)
	{
#if !UE_BUILD_SHIPPING
		ensureMsgf(IsInGameThread(), 
				TEXT("Playback capabilities aren't meant to be thread-safe. Do not modify or access their container outside of the game thread."));
#endif
		T& Cap = Capabilities.AddCapability<T>(Forward<ArgTypes>(InArgs)...);
		MaybeInitialize(Cap);
		return Cap;
	}

	/**
	 * Adds the specified capability on the root sequence as a raw pointer.
	 */
	template<typename T>
	T& AddCapabilityRaw(T* InPointer)
	{
#if !UE_BUILD_SHIPPING
		ensureMsgf(IsInGameThread(), 
				TEXT("Playback capabilities aren't meant to be thread-safe. Do not modify or access their container outside of the game thread."));
#endif
		T& Cap = Capabilities.AddCapabilityRaw<T>(InPointer);
		MaybeInitialize(Cap);
		return Cap;
	}

	/**
	 * Adds the specified capability on the root sequence as a shared pointer.
	 */
	template<typename T>
	T& AddCapabilityShared(TSharedRef<T> InSharedRef)
	{
#if !UE_BUILD_SHIPPING
		ensureMsgf(IsInGameThread(), 
				TEXT("Playback capabilities aren't meant to be thread-safe. Do not modify or access their container outside of the game thread."));
#endif
		T& Cap = Capabilities.AddCapabilityShared<T>(InSharedRef);
		MaybeInitialize(Cap);
		return Cap;
	}

	/**
	 * Adds the specified capability on the root sequence.
	 * If the capability already exists, it must be stored inline, and its
	 * value will be replaced by the new object.
	 * If the template parameter is a sub-class of the playback capability, the previously
	 * stored playback capability must not only have been stored inline, but must have 
	 * been of the same sub-class (or a sub-class with the exact same size and alignment).
	 */
	template<typename T, typename ...ArgTypes>
	T& SetOrAddCapability(ArgTypes&&... InArgs)
	{
#if !UE_BUILD_SHIPPING
		ensureMsgf(IsInGameThread(), 
				TEXT("Playback capabilities aren't meant to be thread-safe. Do not modify or access their container outside of the game thread."));
#endif
		if (HasCapability<T>())
		{
			T& Cap = Capabilities.OverwriteCapability<T>(Forward<ArgTypes>(InArgs)...);
			MaybeInitialize(Cap);
			return Cap;
		}
		else
		{
			T& Cap = Capabilities.AddCapability<T>(Forward<ArgTypes>(InArgs)...);
			MaybeInitialize(Cap);
			return Cap;
		}
	}

	/**
	 * Adds the specified capability on the root sequence as a raw pointer.
	 * If the capability already exists, it must be stored as a raw pointer and its
	 * value will be replaced by the new pointer.
	 */
	template<typename T, typename ...ArgTypes>
	T& SetOrAddCapabilityRaw(T* InPointer)
	{
#if !UE_BUILD_SHIPPING
		ensureMsgf(IsInGameThread(), 
				TEXT("Playback capabilities aren't meant to be thread-safe. Do not modify or access their container outside of the game thread."));
#endif
		if (HasCapability<T>())
		{
			T& Cap = Capabilities.OverwriteCapabilityRaw<T>(InPointer);
			MaybeInitialize(Cap);
			return Cap;
		}
		else
		{
			T& Cap = Capabilities.AddCapabilityRaw<T>(InPointer);
			MaybeInitialize(Cap);
			return Cap;
		}
	}

	/**
	 * Adds the specified capability on the root sequence as a shared pointer.
	 * If the capability already exists, it must be stored as a shared pointer and its
	 * value will be replaced by the new pointer.
	 */
	template<typename T, typename ...ArgTypes>
	T& SetOrAddCapabilityShared(TSharedRef<T> InSharedRef)
	{
#if !UE_BUILD_SHIPPING
		ensureMsgf(IsInGameThread(), 
				TEXT("Playback capabilities aren't meant to be thread-safe. Do not modify or access their container outside of the game thread."));
#endif
		if (HasCapability<T>())
		{
			T& Cap = Capabilities.OverwriteCapabilityShared<T>(InSharedRef);
			MaybeInitialize(Cap);
			return Cap;
		}
		else
		{
			T& Cap = Capabilities.AddCapabilityShared<T>(InSharedRef);
			MaybeInitialize(Cap);
			return Cap;
		}
	}

public:

	UE_API void InvalidateCachedData();

#if !UE_BUILD_SHIPPING
	void SetDebugBreakOnDestroy() { bDebugBreakOnDestroy = true; }
#endif

private:

	template<typename T>
	void MaybeInitialize(T& Cap)
	{
		if constexpr (TPointerIsConvertibleFromTo<T, IPlaybackCapability>::Value)
		{
			IPlaybackCapability* InterfacePtr = static_cast<IPlaybackCapability*>(&Cap);
			InterfacePtr->Initialize(SharedThis(this));
		}
	}

private:

	/** The root sequence */
	TWeakObjectPtr<UMovieSceneSequence> WeakRootSequence;

	/** The playback context */
	TWeakObjectPtr<UObject> WeakPlaybackContext;

	/** The linker evaluating this root sequence */
	TWeakObjectPtr<UMovieSceneEntitySystemLinker> WeakLinker;

	/** The compiled data manager that contains the data for the root sequence */
	TObjectPtr<UMovieSceneCompiledDataManager> CompiledDataManager;

	/** The handle of the root sequence */
	FRootInstanceHandle RootInstanceHandle;

	/** The compiled data ID for the root sequence */
	FMovieSceneCompiledDataID  RootCompiledDataID;

	/** Playback capabilities for the root sequence */
	FPlaybackCapabilities Capabilities;

	/** Pre-animated state utility for the sequence hierarchy */
	FMovieSceneInstancePreAnimatedState PreAnimatedState;

#if !UE_BUILD_SHIPPING
	bool bDebugBreakOnDestroy = false;
#endif
};

} // namespace UE::MovieScene

#undef UE_API
