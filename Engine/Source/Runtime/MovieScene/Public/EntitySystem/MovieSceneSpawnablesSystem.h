// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneSpawnablesSystem.generated.h"

class UObject;
struct FMovieSceneAnimTypeID;

UCLASS(MinimalAPI)
class UMovieSceneSpawnablesSystem : public UMovieSceneEntitySystem
{
public:

	GENERATED_BODY()

	UMovieSceneSpawnablesSystem(const FObjectInitializer& ObjInit);

	MOVIESCENE_API static FMovieSceneAnimTypeID GetAnimTypeID();

private:

	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override final;
};



