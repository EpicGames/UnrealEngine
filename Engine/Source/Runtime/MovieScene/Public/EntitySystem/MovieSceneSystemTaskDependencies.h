// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"
#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Stats/Stats.h"
#include "Templates/SharedPointer.h"

#include <initializer_list>

struct FMovieSceneEntitySystemGraph;

DECLARE_CYCLE_STAT_EXTERN(TEXT("System Dependency Cost"), MovieSceneEval_SystemDependencyCost, STATGROUP_MovieSceneECS,);

namespace UE
{
namespace MovieScene
{

enum class EEntityThreadingModel : uint8;

struct FSystemTaskPrerequisites
{
	using FComponentTypeID = UE::MovieScene::FComponentTypeID;

	FSystemTaskPrerequisites()
	{}

	FSystemTaskPrerequisites(std::initializer_list<FGraphEventRef> InEvents)
	{
		for (FGraphEventRef Task : InEvents)
		{
			Prereqs.Add(FPrerequisite{ Task, FComponentTypeID::Invalid() });
			AllTasks.Add(Task);
		}
	}

	int32 Num() const
	{
		return Prereqs.Num();
	}

	const FGraphEventArray* All() const
	{
		if (AllTasks.Num() != Prereqs.Num())
		{
			AllTasks.Reset();
			for (const FPrerequisite& Prereq : Prereqs)
			{
				AllTasks.Add(Prereq.GraphEvent);
			}
		}
		return &AllTasks;
	}

	inline void FilterByComponent(FGraphEventArray& OutArray, FComponentTypeID ComponentType) const
	{
		FilterByComponent(OutArray, { ComponentType });
	}

	MOVIESCENE_API void FilterByComponent(FGraphEventArray& OutArray, std::initializer_list<FComponentTypeID> ComponentTypes) const;

	void AddRootTask(const FGraphEventRef& InNewTask)
	{
		AddComponentTask(FComponentTypeID::Invalid(), InNewTask);
	}

	MOVIESCENE_API void AddComponentTask(FComponentTypeID ComponentType, const FGraphEventRef& InNewTask);

	MOVIESCENE_API void Consume(const FSystemTaskPrerequisites& Other);

	void Empty()
	{
		Prereqs.Reset();
		AllTasks.Reset();
	}

private:

	struct FPrerequisite
	{
		FGraphEventRef   GraphEvent;
		FComponentTypeID ComponentType;
	};
	TArray<FPrerequisite, TInlineAllocator<4>> Prereqs;
	mutable FGraphEventArray AllTasks;
};



struct FSystemSubsequentTasks
{
	using FComponentTypeID = UE::MovieScene::FComponentTypeID;

	MOVIESCENE_API void AddRootTask(FGraphEventRef RootTask);

	MOVIESCENE_API void AddComponentTask(FComponentTypeID ComponentType, FGraphEventRef ComponentTask);

private:

	friend FMovieSceneEntitySystemGraph;

	MOVIESCENE_API FSystemSubsequentTasks(FMovieSceneEntitySystemGraph* InGraph, FGraphEventArray* InAllTasks, EEntityThreadingModel InThreadingModel);

	MOVIESCENE_API void ResetNode(uint16 InNodeID);

	TSharedPtr<UE::MovieScene::FSystemTaskPrerequisites> Subsequents;
	FMovieSceneEntitySystemGraph* Graph;
	FGraphEventArray* AllTasks;
	uint16 NodeID;
	EEntityThreadingModel ThreadingModel;
};


} // namespace MovieScene
} // namespace UE
