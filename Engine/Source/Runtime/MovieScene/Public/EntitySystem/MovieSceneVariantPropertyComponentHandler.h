// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieScenePropertyComponentHandler.h"


namespace UE::MovieScene
{


template<typename PropertyTraits, typename ...CompositeTypes>
struct TVariantPropertyComponentHandler
	: TPropertyComponentHandler<PropertyTraits, CompositeTypes...>
{
	virtual void DispatchInitializePropertyMetaDataTasks(const FPropertyDefinition& Definition, FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents, UMovieSceneEntitySystemLinker* Linker) override
	{
		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

		const PropertyTraits* PropertyTraitsInstance = static_cast<const PropertyTraits*>(Definition.TraitsInstance);

		FEntityTaskBuilder()
		.Read(BuiltInComponents->BoundObject)
		.Read(BuiltInComponents->PropertyBinding)
		.Write(BuiltInComponents->VariantPropertyTypeIndex)
		.FilterAll({ BuiltInComponents->Tags.NeedsLink, Definition.PropertyType })
		.Iterate_PerEntity(&Linker->EntityManager, [PropertyTraitsInstance](UObject* Object, const FMovieScenePropertyBinding& Binding, FVariantPropertyTypeIndex& OutMetaData)
		{
			// @todo: this function is garbage slow - needs optimizing to use string views
			FString PropertyPath = Binding.PropertyPath.ToString();
			FProperty* BoundProperty = FTrackInstancePropertyBindings::FindProperty(Object, PropertyPath);

			if (ensureMsgf(BoundProperty, TEXT("Unable to find property '%s::%s' on bound object '%s'"), *Object->GetClass()->GetName(), *PropertyPath, *Object->GetName()))
			{
				if (!PropertyTraitsInstance->ComputeVariantIndex(*BoundProperty, OutMetaData))
				{
					ensureMsgf(false, TEXT("Property '%s::%s' on bound object '%s' is not of a compatible type with %s"), *Object->GetClass()->GetName(), *PropertyPath, *Object->GetName(), GetGeneratedTypeName<PropertyTraits>());
				}
			}
		});
	}
};


} // namespace UE::MovieScene


