// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace UE::MovieScene
{

struct FVariantPropertyTypeIndex
{
	FVariantPropertyTypeIndex() : Index(255) {}

	bool IsValid() const
	{
		return Index != 255;
	}

	uint8 Index = 255;
};

} // namespace UE::MovieScene