// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/AssertionMacros.h"
#include <limits>
#include <type_traits>

template<typename T, typename OffsetType = int16>
struct TRelativePtr
{
	static constexpr OffsetType NullValue = std::numeric_limits<OffsetType>::max();

	static_assert(std::is_integral_v<OffsetType>, "Offset type must be a signed integer");

	explicit TRelativePtr()
		: Offset(NullValue)
	{}

	explicit TRelativePtr(TYPE_OF_NULLPTR)
		: Offset(NullValue)
	{}

	explicit TRelativePtr(const void* BasePtr, const void* ThisPtr)
	{
		Reset(BasePtr, ThisPtr);
	}

	explicit operator bool() const
	{
		return Offset != NullValue;
	}

	T* Resolve(const void* BasePtr) const
	{
		return Offset == NullValue ? nullptr : reinterpret_cast<T*>(static_cast<uint8*>(const_cast<void*>(BasePtr)) + Offset);
	}

	template<typename U>
	U* Resolve(const void* BasePtr) const
	{
		return static_cast<U*>(Resolve(BasePtr));
	}

	void Reset(TYPE_OF_NULLPTR)
	{
		Offset = NullValue;
	}

	void Reset(const void* BasePtr, const void* ThisPtr)
	{
		const uintptr_t Base = reinterpret_cast<uintptr_t>(BasePtr);
		const uintptr_t This = reinterpret_cast<uintptr_t>(ThisPtr);
		if (This >= Base)
		{
			const uintptr_t OffsetPtr = This - Base;
			checkf(OffsetPtr >= 0 && OffsetPtr < std::numeric_limits<OffsetType>::max(),
				TEXT("Attempting to create a relative pointer outside the bounds of its capacity."));
			Offset = static_cast<OffsetType>(OffsetPtr);
		}
		else if constexpr (std::is_signed_v<OffsetType>)
		{
			const uintptr_t OffsetPtr = Base - This;
			checkf(OffsetPtr >= 0 && OffsetPtr < std::numeric_limits<OffsetType>::max(),
				TEXT("Attempting to create a relative pointer outside the bounds of its capacity."));
			Offset = -static_cast<OffsetType>(OffsetPtr);
		}
		else
		{
			checkf(false, TEXT("Attempting to assign a negative offset to an unsigned relative pointer!!"));
		}
	}
private:

	OffsetType Offset;
};
