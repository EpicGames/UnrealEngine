// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneInstanceRegistry.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "HAL/Platform.h"
#include "HAL/PlatformMisc.h"
#include "Serialization/Archive.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneTrackInstance.generated.h"

class UMovieSceneEntitySystemLinker;
class UMovieSceneSection;
class UWorld;
struct FMovieSceneTrackInstanceComponent;

namespace UE
{
namespace MovieScene
{

struct FPreAnimatedTrackInstanceInputCaptureSources;

}
}

USTRUCT()
struct FMovieSceneTrackInstanceInput
{
	GENERATED_BODY()

	/** The section that acts as an input for this animator */
	UPROPERTY()
	TObjectPtr<UMovieSceneSection> Section = nullptr;

	/** The instance that this input relates to */
	UE::MovieScene::FInstanceHandle InstanceHandle;

	/** Whether the input has been processed by being added to a track instance */
	bool bInputHasBeenProcessed = false;

	/** Check whether this input matches another, ignoring the bInputHasBeenProcessed flag */
	bool IsSameInput(const FMovieSceneTrackInstanceInput& OtherInput) const
	{
		return Section == OtherInput.Section && InstanceHandle == OtherInput.InstanceHandle;
	}

	friend bool operator==(const FMovieSceneTrackInstanceInput& A, const FMovieSceneTrackInstanceInput& B)
	{
		return A.Section == B.Section && A.InstanceHandle == B.InstanceHandle && A.bInputHasBeenProcessed == B.bInputHasBeenProcessed;
	}

	friend bool operator<(const FMovieSceneTrackInstanceInput& A, const FMovieSceneTrackInstanceInput& B)
	{
		// bInputHasBeenProcessed is not considered for sorting
		if (A.Section == B.Section)
		{
			return A.InstanceHandle < B.InstanceHandle;
		}
		return A.Section < B.Section;
	}

	friend uint32 GetTypeHash(const FMovieSceneTrackInstanceInput& Input)
	{
		// Use arbitrary primes to salt this thing.
		uint32 hash = 7211;
		hash = (hash * 307) + GetTypeHash(Input.Section);
		hash = (hash * 307) + GetTypeHash(Input.InstanceHandle);
		return hash;
	}

	/** Serializer that ensures both the section and instance handle are copied over when an input is copied for a reinstanced animator */
	bool Serialize(FArchive& Ar)
	{
		Ar << (UObject*&)Section << InstanceHandle.InstanceID << InstanceHandle.InstanceSerial << bInputHasBeenProcessed;
		return true;
	}
};

template<>
struct TStructOpsTypeTraits<FMovieSceneTrackInstanceInput> : public TStructOpsTypeTraitsBase2<FMovieSceneTrackInstanceInput>
{
	enum
	{
		WithIdenticalViaEquality       = true,
		WithSerializer                 = true,
	};
};


/**
 * Base class for all track instances. Can also be used for root tracks where AnimatedObject will be nullptr
 */
UCLASS(transient, MinimalAPI)
class UMovieSceneTrackInstance : public UObject
{
public:
	GENERATED_BODY()

	/**
	 * Initialize this instance with the object to animate, and a linker
	 *
	 * @param InAnimatedObject     (optional) The object that we should animate
	 * @param InLinker             The entity system linker that owns this animator. Used for retrieving contexts and player information.
	 */
	MOVIESCENE_API void Initialize(UObject* InAnimatedObject, UMovieSceneEntitySystemLinker* InLinker);


	/**
	 * Run this animator for the current frame
	 */
	MOVIESCENE_API void Animate();


	/**
	 * Update the inputs that contribute to this animator
	 */
	MOVIESCENE_API void UpdateInputs(TArray<FMovieSceneTrackInstanceInput>&& InNewInputs);


	/**
	 * Destroy this animator. The animator may still have inputs when it is destroyed.
	 */
	MOVIESCENE_API void Destroy();


public:


	/**
	 * Retrieve the object that is being animated (may be nullptr for root tracks)
	 */
	UObject* GetAnimatedObject() const
	{
		return WeakAnimatedObject.Get();
	}


	/**
	 * Retrieve the linker that owns this animator
	 */
	UMovieSceneEntitySystemLinker* GetLinker() const
	{
		return PrivateLinker;
	}


	/**
	 * Retrieve this animator's inputs
	 */
	TArrayView<const FMovieSceneTrackInstanceInput> GetInputs() const
	{
		return Inputs;
	}

	MOVIESCENE_API virtual UWorld* GetWorld() const override;
#if WITH_EDITOR
	virtual bool ImplementsGetWorld() const override { return true; }
#endif

private:

	/** Called when this animator is first created to perform any initialization */
	virtual void OnInitialize() {}

	/** Called when the sequence is updated to apply animation */
	virtual void OnAnimate() {}

	/** Called when this animator is about to have its inputs updated */
	virtual void OnBeginUpdateInputs() {}

	/** Called when this animator has had an input added */
	virtual void OnInputAdded(const FMovieSceneTrackInstanceInput& InInput) {}

	/** Called when this animator has had an input removed */
	virtual void OnInputRemoved(const FMovieSceneTrackInstanceInput& InInput) {}

	/** Called after this animator has finished updating its inputs */
	virtual void OnEndUpdateInputs() {}

	/** Called when this animator is being destroyed in order that it can perform any final fixup */
	virtual void OnDestroyed() {}

private:

	UPROPERTY()
	TWeakObjectPtr<UObject> WeakAnimatedObject;

	UPROPERTY()
	bool bIsRootTrackInstance;

	UPROPERTY()
	TObjectPtr<UMovieSceneEntitySystemLinker> PrivateLinker;

	UPROPERTY()
	TArray<FMovieSceneTrackInstanceInput> Inputs;
};
