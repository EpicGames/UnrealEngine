// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Containers/SparseArray.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntityInstantiatorSystem.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "EntitySystem/MovieScenePropertySystemTypes.h"
#include "HAL/Platform.h"
#include "Serialization/Archive.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneTrackInstanceSystem.generated.h"

class UClass;
class UMovieSceneSection;
class UMovieSceneTrackInstance;
class UObject;

USTRUCT()
struct FMovieSceneTrackInstanceEntry
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UObject> BoundObject = nullptr;

	UPROPERTY()
	TObjectPtr<UMovieSceneTrackInstance> TrackInstance = nullptr;
};


UCLASS(MinimalAPI)
class UMovieSceneTrackInstanceInstantiator : public UMovieSceneEntityInstantiatorSystem
{
	GENERATED_BODY()

public:

	MOVIESCENE_API UMovieSceneTrackInstanceInstantiator(const FObjectInitializer& ObjInit);

	MOVIESCENE_API int32 MakeOutput(UObject* BoundObject, UClass* TrackInstanceClass);

	MOVIESCENE_API int32 FindOutput(UObject* BoundObject, UClass* TrackInstanceClass) const;

	const TSparseArray<FMovieSceneTrackInstanceEntry>& GetTrackInstances() const
	{
		return TrackInstances;
	}

	static MOVIESCENE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

private:

	MOVIESCENE_API virtual void OnLink() override final;
	MOVIESCENE_API virtual void OnUnlink() override final;
	MOVIESCENE_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override final;

	MOVIESCENE_API virtual void OnTagGarbage() override;
	MOVIESCENE_API virtual void OnCleanTaggedGarbage() override;

	MOVIESCENE_API virtual void Serialize(FArchive& Ar) override;

	TSparseArray<FMovieSceneTrackInstanceEntry> TrackInstances;
	TMultiMap<TObjectPtr<UObject>, int32> BoundObjectToInstances;

	TBitArray<> InvalidatedOutputs;

	int32 ChildInitializerIndex;
};


UCLASS(MinimalAPI)
class UMovieSceneTrackInstanceSystem : public UMovieSceneEntitySystem
{
	GENERATED_BODY()

	UMovieSceneTrackInstanceSystem(const FObjectInitializer& ObjInit);

private:

	MOVIESCENE_API virtual void OnLink() override final;
	MOVIESCENE_API virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override final;
	MOVIESCENE_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override final;

	void EvaluateAllInstances();

	UPROPERTY()
	TObjectPtr<UMovieSceneTrackInstanceInstantiator> Instantiator;
};


