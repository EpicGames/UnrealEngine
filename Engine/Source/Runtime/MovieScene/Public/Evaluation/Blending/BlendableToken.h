// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Evaluation/Blending/MovieSceneBlendType.h"
#include "Misc/InlineValue.h"
#include "Evaluation/MovieScenePlayback.h"
#include "Evaluation/MovieSceneEvaluationScope.h"


template<typename DataType> struct TMovieSceneInitialValueStore;

/**
 * Implementation of custom blend logic should be as follows (using doubles as an example).
 * Specializing TBlendableTokenTraits for a particular input data type causes WorkingDataType to be used during the blending operation.
 * Where WorkingDataType belongs to a namespace, ADL will be employed to discover any relevant overloads for BlendValue that match the necessary types.
 * This allows blending of any arbitrary type into the WorkingDataType.

	namespace MovieScene
	{
		// Define a custom namespaced type that will be used to calculate blends between doubles
		struct FBlendableDouble
		{
			FBlendableDouble()
				: AbsoluteTotal(0.0), AdditiveTotal(0.0)
			{}

			double AbsoluteTotal;
			double AdditiveTotal;

			TOptional<float> TotalWeight;

			double Resolve(TMovieSceneInitialValueStore<int32>& InitialValueStore)
			{
				if (TotalWeight.IsSet())
				{
					if (TotalWeight.GetValue() == 0.f)
					{
						AbsoluteTotal = InitialValueStore.GetInitialValue();
					}
					else
					{
						AbsoluteTotal /= TotalWeight.GetValue();
					}
				}

				return AbsoluteTotal + AdditiveTotal;
			}
		};

		void BlendValue(FBlendableDouble& OutBlend, double InValue, float Weight, EMovieSceneBlendType BlendType, TMovieSceneInitialValueStore<double>& InitialValueStore)
		{
			if (BlendType == EMovieSceneBlendType::Absolute || BlendType == EMovieSceneBlendType::Relative)
			{
				if (BlendType == EMovieSceneBlendType::Relative)
				{
					OutBlend.AbsoluteTotal += (InitialValueStore.GetInitialValue() + InValue) * Weight;
				}
				else
				{
					OutBlend.AbsoluteTotal += InValue * Weight;
				}

				OutBlend.TotalWeight = OutBlend.TotalWeight.Get(0.f) + Weight;
			}
			else if (BlendType == EMovieSceneBlendType::Additive)
			{
				OutBlend.AdditiveTotal += InValue * Weight;
			}
		}
	}
	template<> struct TBlendableTokenTraits<double> { typedef UE::MovieScene::FBlendableDouble WorkingDataType; };
 */

namespace UE
{
namespace MovieScene
{
	template<typename InType, typename WorkingDataType, typename SourceDataType>
	void BlendValue(WorkingDataType& OutBlend, InType InValue, float Weight, EMovieSceneBlendType BlendType, TMovieSceneInitialValueStore<SourceDataType>& InitialValueStore)
	{
		BlendValue(OutBlend, InValue, Weight, BlendType, INDEX_NONE, InitialValueStore);
	}
	//new optional blending priority that supports blending priorities
	template<typename InType, typename WorkingDataType, typename SourceDataType>
	void BlendValue(WorkingDataType& OutBlend, InType InValue, float Weight, EMovieSceneBlendType BlendType, int32 BlendingOrder, TMovieSceneInitialValueStore<SourceDataType>& InitialValueStore)
	{
	
	}
} // namespace MovieScene
} // namespace UE

template<typename DataType> struct TBlendableTokenTraits { typedef DataType WorkingDataType; };

/**
 * Templated structure that encapsulates any blendable data type, and the information required to blend it
 */
template<typename DataType>
struct TBlendableToken
{
	typedef typename TBlendableTokenTraits<DataType>::WorkingDataType WorkingDataType;

	/** Default construction */
	TBlendableToken() = default;

	/** Construction from a value, blend method, and a weight. Scope and bias to be populated later */
	template<typename T>
	TBlendableToken(T&& InValue, EMovieSceneBlendType InBlendType, float InWeight = 1.f, int32 InBlendingOrder = INDEX_NONE)
		: Value(TData<typename TDecay<T>::Type>(Forward<T>(InValue)))
		, HierarchicalBias(0)
		, Weight(InWeight)
		, BlendType(InBlendType)
		, BlendingOrder(InBlendingOrder)
	{}

	/** Construction from a value, scope, context, blend method, and a weight */
	template<typename T>
	TBlendableToken(T&& InValue, const FMovieSceneEvaluationScope& InCurrentScope, const FMovieSceneContext& InContext, EMovieSceneBlendType InBlendType, float InWeight = 1.f,
		int32 InBlendingOrder = INDEX_NONE)
		: Value(TData<typename TDecay<T>::Type>(Forward<T>(InValue)))
		, AnimatingScope(InCurrentScope)
		, HierarchicalBias(InContext.GetHierarchicalBias())
		, Weight(InWeight)
		, BlendType(InBlendType)
		, BlendingOrder(InBlendingOrder)

	{}

	/** Copying is disabled */
	TBlendableToken(const TBlendableToken&) = delete;
	TBlendableToken& operator=(const TBlendableToken&) = delete;

	/** Move construction/assignment */
	TBlendableToken(TBlendableToken&&) = default;
	TBlendableToken& operator=(TBlendableToken&&) = default;

	/**
	 * Add this value into the specified cumulative blend
	 *
	 * @param CumulativeBlend		The working structure to add our value onto
	 */
	void AddTo(WorkingDataType& CumulativeBlend, TMovieSceneInitialValueStore<DataType>& InitialValueStore) const
	{
		check(Value.IsValid());
		Value->AddTo(CumulativeBlend, Weight, BlendType, BlendingOrder, InitialValueStore);
	}

	bool operator<(const TBlendableToken& RHS) const
	{
		return BlendingOrder < RHS.BlendingOrder;
	}
private:

	/** Base class for all value types */
	struct IData
	{
		virtual ~IData() = default;
		virtual void AddTo(WorkingDataType& CumulativeBlend, float Weight, EMovieSceneBlendType BlendType, int32 BlendingOrder, TMovieSceneInitialValueStore<DataType>& InitialValueStore) const = 0;
	};

	/** Templated value data for any other type */
	template<typename T>
	struct TData : IData
	{
		TData(T In) : Data(MoveTemp(In)) {}

		virtual void AddTo(WorkingDataType& CumulativeBlend, float InWeight, EMovieSceneBlendType InBlendType, int32 InBlendingOrder, TMovieSceneInitialValueStore<DataType>& InitialValueStore) const
		{
			// Use the default BlendValue function, or any other BlendValue function found through ADL on WorkingDataType
			using UE::MovieScene::BlendValue;
			if (InBlendingOrder == INDEX_NONE)
			{
				BlendValue(CumulativeBlend, Data, InWeight, InBlendType, InitialValueStore);
			}
			else
			{
				BlendValue(CumulativeBlend, Data, InWeight, InBlendType, InBlendingOrder, InitialValueStore);
			}
		}

		/** The actual value to blend */
		T Data;
	};

	/** The actual user provided value data, stored as inline bytes (unless it's > sizeof(DataType)) */
	TInlineValue<IData, sizeof(DataType)> Value;

public:

	/** The scope from which this token was generated. Used for restoring pre-animated state where necessary. */
	FMovieSceneEvaluationScope AnimatingScope;

	/** The hierarchical bias for this template instance */
	int32 HierarchicalBias;

	/** Weight to apply to the value */
	float Weight;

	/** Enumeration specifying how this token should be blended */
	EMovieSceneBlendType BlendType;

	/** Opitonal Blending Order, if present will be used to sort additives and override blend types*/
	int32 BlendingOrder = INDEX_NONE;

};
