// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneFwd.h"
#include "Evaluation/MovieSceneAnimTypeID.h"
#include "Evaluation/Blending/MovieSceneBlendingActuatorID.h"
#include "Misc/InlineValue.h"
#include "Evaluation/MovieSceneEvaluationScope.h"
#include "Evaluation/MovieSceneEvaluationOperand.h"
#include "Evaluation/Blending/BlendableToken.h"
#include "IMovieScenePlayer.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedCaptureSource.h"

struct FMovieSceneInterrogationData;
struct FMovieSceneContext;
struct FMovieSceneBlendingAccumulator;
template<typename T> struct TMovieSceneBlendingActuator;

/**
 * Template to access the type ID for a given blendable data type
 * Care should be taken to ensure that only a single TypeID maps to each data type.
 * Where shared access to a data type is required, this type should be specialized with a DLL exported (XYZ_API) definition.
 */
template<typename T> FMovieSceneAnimTypeID GetBlendingDataType()
{
	// Always assert on instantiation
	static_assert(std::is_same_v<T, void>, "GetBlendingDataType must be specialized for a type in order to use it with an accumulator.");
	return FMovieSceneAnimTypeID::Unique();
}

/**
 * Base interface for a stack of typed tokens
 */
struct IBlendableTokenStack
{
protected:

	/**
	 * Constructor
	 */
	IBlendableTokenStack(FMovieSceneAnimTypeID InDataTypeID) : DataTypeID(InDataTypeID) {}

public:

	/**
	 * Virtual destructor
	 */
	virtual ~IBlendableTokenStack() = default;

	/**
	 * Implemented by typed stacks to compute the final blended value for its data, and apply that result to the specified object
	 *
	 * @param InObject			The object to apply the result to. Nullptr for root track data.
	 * @param Accumulator 		The accumulator that was used to gather and consolidate the blendable tokens
	 * @param ActuatorTypeID	A unique identifier to the actuator that should be used to apply the result to the object
	 * @param Context			Root-level movie scene evaluation context for the current frame
	 * @param PersistentData	Persistent data store for the evaluation
	 * @param Player			The player that is responsible for playing back the sequence
	 */
	virtual void ComputeAndActuate(UObject* InObject, FMovieSceneBlendingAccumulator& Accumulator, FMovieSceneBlendingActuatorID ActuatorTypeID, const FMovieSceneContext& Context, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) = 0;

	/**
	 * Implemented by typed stacks to interrogate the final blended value for its data.
	 *
	 * @param AnimatedObject	The object that the value should be interrogated for
	 * @param InterrogationData The interrogation data to populate
	 * @param Accumulator		The accumulator that was used to gather and consolidate the blendable tokens
	 * @param ActuatorTypeID	A unique identifier to the actuator that should be used to apply the result to the object
	 * @param Context			Root-level movie scene evaluation context for the current frame
	 */
	virtual void Interrogate(UObject* AnimatedObject, FMovieSceneInterrogationData& InterrogationData, FMovieSceneBlendingAccumulator& Accumulator, FMovieSceneBlendingActuatorID ActuatorTypeID, const FMovieSceneContext& Context) = 0;

	/** The type of data that this stack contains */
	FMovieSceneAnimTypeID DataTypeID;
};

/**
 * Implementation of a blendable token stack for any given data type
 */
template<typename DataType>
struct TBlendableTokenStack : IBlendableTokenStack
{
	/** This stack's typed blendable tokens */
	TArray<const TBlendableToken<DataType>*> Tokens;

	/** The highest encountered hierarchical bias */
	int32 CurrentBias;

	TBlendableTokenStack() : IBlendableTokenStack(GetBlendingDataType<DataType>()), CurrentBias(TNumericLimits<int32>::Lowest()) {}

	/**
	 * Conditionally add a token to this stack if it has a >= hierarchical bias, removing anything of a lower bias
	 *
	 * @param TokenToAdd		The token to add
	 */
	void AddToken(const TBlendableToken<DataType>* TokenToAdd)
	{
		check(TokenToAdd);

		if (TokenToAdd->HierarchicalBias > CurrentBias)
		{
			Tokens.Reset();
			CurrentBias = TokenToAdd->HierarchicalBias;
		}

		if (TokenToAdd->HierarchicalBias == CurrentBias)
		{
			// Just add the token
			Tokens.Add(TokenToAdd);
		}
	}

public:

	/**
	 * Implemented by typed stacks to compute the final blended value for its data, and apply that result to the specified object
	 *
	 * @param InObject			The object to apply the result to. Nullptr for root track data.
	 * @param Accumulator 		The accumulator that was used to gather and consolidate the blendable tokens
	 * @param ActuatorTypeID	A unique identifier to the actuator that should be used to apply the result to the object
	 * @param Context			Root-level movie scene evaluation context for the current frame
	 * @param PersistentData	Persistent data store for the evaluation
	 * @param Player			The player that is responsible for playing back the sequence
	 */
	virtual void ComputeAndActuate(UObject* InObject, FMovieSceneBlendingAccumulator& Accumulator, FMovieSceneBlendingActuatorID InActuatorType, const FMovieSceneContext& Context, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) override final;

	/**
	 * Implemented by typed stacks to interrogate the final blended value for its data.
	 *
	 * @param AnimatedObject	The object that the value should be interrogated for
	 * @param InterrogationData The interrogation data to populate
	 * @param Accumulator		The accumulator that was used to gather and consolidate the blendable tokens
	 * @param ActuatorTypeID	A unique identifier to the actuator that should be used to apply the result to the object
	 * @param Context			Root-level movie scene evaluation context for the current frame
	 */
	virtual void Interrogate(UObject* AnimatedObject, FMovieSceneInterrogationData& InterrogationData, FMovieSceneBlendingAccumulator& Accumulator, FMovieSceneBlendingActuatorID ActuatorTypeID, const FMovieSceneContext& Context) override final;

	/**
	 * Helper function for saving pre-animated state for all entites that contributed to this stack, regardless of whether they want restore state or not
	 *
	 * @param Player 			The movie scene player that's playing back the sequence
	 * @param Args 				Variable arguments that are forwarded to IMovieScenePlayer::SavePreAnimatedState
	 */
	template<typename...T>
	void SavePreAnimatedStateForAllEntities(IMovieScenePlayer& Player, T&&... Args) const
	{
		SavePreAnimatedStateImpl(Player, EMovieSceneCompletionMode::RestoreState, Args...);
	}

	/**
	 * Helper function for saving pre-animated state for all entites that want RestoreState and relate to the current token stack.
	 *
	 * @param Player 			The movie scene player that's playing back the sequence
	 * @param Args 				Variable arguments that are forwarded to IMovieScenePlayer::SavePreAnimatedState
	 */
	template<typename...T>
	void SavePreAnimatedState(IMovieScenePlayer& Player, T&&... Args) const
	{
		SavePreAnimatedStateImpl(Player, TOptional<EMovieSceneCompletionMode>(), Args...);
	}
	
private:

	template<typename...T>
	void SavePreAnimatedStateImpl(IMovieScenePlayer& Player, TOptional<EMovieSceneCompletionMode> CompletionModeOverride, T&&... Args) const
	{
		bool bSavedState = false;
		for (const TBlendableToken<DataType>* Token : Tokens)
		{
			if (CompletionModeOverride.Get(Token->AnimatingScope.CompletionMode) == EMovieSceneCompletionMode::RestoreState)
			{
				FScopedPreAnimatedCaptureSource CaptureSource(&Player.PreAnimatedState, Token->AnimatingScope.Key, true);
				Player.SavePreAnimatedState(Forward<T>(Args)...);
				bSavedState = true;
			}
		}

		// Save global state if nothing else did
		if (!bSavedState)
		{
			Player.SavePreAnimatedState(Forward<T>(Args)...);
		}
	}
};
