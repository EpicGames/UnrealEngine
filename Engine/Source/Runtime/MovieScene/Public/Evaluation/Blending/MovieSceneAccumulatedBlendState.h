// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Evaluation/Blending/BlendableToken.h"
#include "Evaluation/Blending/BlendableTokenStack.h"
#include "Evaluation/Blending/MovieSceneBlendingActuatorID.h"
#include "Evaluation/MovieSceneAnimTypeID.h"
#include "Evaluation/MovieSceneEvaluationScope.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Misc/InlineValue.h"
#include "MovieSceneFwd.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/ObjectKey.h"

class IMovieScenePlayer;
class UObject;
struct FMovieSceneEvaluationOperand;

/**
 * Container struct that houses all tokens of a single data type that are to be applied using a specific actuator
 */
typedef TInlineValue<IBlendableTokenStack, 64> FActuatorTokenStackPtr;

struct FMovieSceneBlendingKey
{
	/** The object that the animation applies to */
	UObject* ObjectPtr;

	/** The type of the animation being applied to the object above */
	FMovieSceneBlendingActuatorID ActuatorType;

	friend uint32 GetTypeHash(const FMovieSceneBlendingKey& InKey)
	{
		return GetTypeHash(InKey.ObjectPtr) ^ GetTypeHash(InKey.ActuatorType);
	}

	friend bool operator==(const FMovieSceneBlendingKey& A, const FMovieSceneBlendingKey& B)
	{
		return A.ObjectPtr == B.ObjectPtr && A.ActuatorType == B.ActuatorType;
	}
};

/**
 * Blendable token state that's accumulated per-operand
 */
struct FMovieSceneAccumulatedBlendState
{
public:
	FMovieSceneAccumulatedBlendState() = default;

	FMovieSceneAccumulatedBlendState(const FMovieSceneAccumulatedBlendState&) = delete;
	FMovieSceneAccumulatedBlendState& operator=(const FMovieSceneAccumulatedBlendState&) = delete;

	FMovieSceneAccumulatedBlendState(FMovieSceneAccumulatedBlendState&&) = default;
	FMovieSceneAccumulatedBlendState& operator=(FMovieSceneAccumulatedBlendState&&) = default;

	/**
	 * Add a new token to this accumulator to be applied using the specified actuator type
	 *
	 * @param InScope			The currently evaluating scope. Used to track which entities require pre-animated state restoration when the final blended token is applied
	 * @param InActuatorType	Unique identifier for the actuator that should be used to apply this token once blended with all other tokens of the same type and actuator
	 * @param InToken			The token to blend
	 */
	template<typename DataType>
	void Add(FMovieSceneBlendingActuatorID InActuatorType, TBlendableToken<DataType>&& InToken)
	{
		TokensToBlend.Emplace(TTokenEntry<DataType>(InActuatorType, MoveTemp(InToken)));
	}

	/**
	 * Consolidate all tokens currently accumulated into the specified container by resolving the specified operand. Used for runtime evaluation.
	 * 
	 * @param InOutBlendState 	Map from object to a map of token stacks to populate. One FActuatorTokenStackPtr per actuator type ID.
	 * @param InOperand			The operand that is being animated
	 * @param Player			The movie scene player that is currently being animated
	 */
	MOVIESCENE_API void Consolidate(TMap<FMovieSceneBlendingKey, FActuatorTokenStackPtr>& InOutBlendState, FMovieSceneEvaluationOperand InOperand, IMovieScenePlayer& Player);

	/**
	 * Consolidate all tokens currently accumulated into the specified container. Used for offline interrogation.
	 * 
	 * @param InOutBlendState 	Map of token stacks to populate. One FActuatorTokenStackPtr per actuator type ID.
	 */
	MOVIESCENE_API void Consolidate(TMap<FMovieSceneBlendingKey, FActuatorTokenStackPtr>& InOutBlendState);

	/**
	 * Reset this container
	 */
	void Reset()
	{
		TokensToBlend.Reset();
	}

private:

	/** Base entry that is used to temporarily store token data for later consolidation */
	struct FTokenEntry
	{
		virtual ~FTokenEntry() = default;

		/**
		 * Consolidate this token into the specified array of stacks, one per actuator type
		 * @param Stacks 		Array of token stacks to populate, one per actuator type
		 */
		virtual void Consolidate(FActuatorTokenStackPtr& OutStack) = 0;

		FMovieSceneBlendingActuatorID GetActuatorID() const
		{
			return ActuatorTypeID;
		}

		/**
		 * Access the result of GetBlendingDataType<DataType>() for the data that this token contains
		 */
		FMovieSceneAnimTypeID GetDataTypeID() const
		{
			return DataTypeID;
		}

	protected:

		/**
		 * Constructor
		 * @param InActuatorTypeID			Type identifier that uniquely represents the actuator that should be used to apply this token
		 */
		FTokenEntry(FMovieSceneBlendingActuatorID InActuatorTypeID, FMovieSceneAnimTypeID InDataTypeID)
			: ActuatorTypeID(InActuatorTypeID), DataTypeID(InDataTypeID)
		{}
		/** Type ID for the actuator that is to be used to apply this token */
		FMovieSceneBlendingActuatorID ActuatorTypeID;
		/** Type ID for data that is contained within the token entry */
		FMovieSceneAnimTypeID DataTypeID;
	};

	/** Templated entry that is used to temporarily store token data for later consolidation */
	template<typename DataType>
	struct TTokenEntry : FTokenEntry
	{
		TTokenEntry(FMovieSceneBlendingActuatorID InActuatorTypeID, TBlendableToken<DataType>&& InToken)
			: FTokenEntry(InActuatorTypeID, GetBlendingDataType<DataType>())
			, Token(MoveTemp(InToken))
		{}

		/**
		 * Consolidate this token into the specified array of stacks, one per actuator type
		 * @param Stacks 		Array of token stacks to populate, one per actuator type
		 */
		virtual void Consolidate(FActuatorTokenStackPtr& OutStack) override final
		{
			// Attempt to find an existing stack for this actuator
			if (!OutStack.IsValid())
			{
				OutStack = TBlendableTokenStack<DataType>();
			}

			// Ensure that the stack is operating on the same data type.
			// If this ensure fires, something has added a token with an actuator ID where that actuator doesn't operate the data type that was addded
			// For example:
			//		// MyActuator knows how to apply a float
			//		struct FMyActuator : TMovieSceneBlendingActuator<float>
			//		{...};
			//
			//		Accumulator.DefineActuator(FMyActuator::GetActuatorTypeID(), TUniquePtr<IMovieSceneBlendingActuator>(new FMyActuator));
			//		// OK - constructing a float token for a float actuator
			//		ExecutionTokens.BlendToken(FMyActuator::GetActuatorTypeID(), TBlendableToken<float>(1.f, ...));
			//		// OK - constructing a float token out of another type for a float actuator
			//		ExecutionTokens.BlendToken(FMyActuator::GetActuatorTypeID(), TBlendableToken<float>(FCompoundType(), ...));
			//		// Assert - data type mismatch - cannot associate a TBlendableToken<int32> with an actuator ID that operates on a float
			//		ExecutionTokens.BlendToken(FMyActuator::GetActuatorTypeID(), TBlendableToken<int32>(1.f, ...));

			if (ensureMsgf(OutStack->DataTypeID == DataTypeID, TEXT("Data type mismatch between actuators of the same ID")))
			{
				// static_cast should be safe now that we've verified the data type matches
				TBlendableTokenStack<DataType>& TypedStack = static_cast<TBlendableTokenStack<DataType>&>(OutStack.GetValue());

				// Add a pointer to this token to the stack to blend
				TypedStack.AddToken(&Token);
			}
		}

		/** The actual token data - defunct after Consolidate is called */
		TBlendableToken<DataType> Token;
	};

	/** Array of all tokens that have been added this frame */
	TArray<TInlineValue<FTokenEntry, 128>> TokensToBlend;
};
