// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Stats/Stats.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"

#include "MovieSceneBlendType.generated.h"

struct FMovieSceneBlendTypeFieldIterator;

/** Movie scene blend type enumeration */
UENUM(BlueprintType)
enum class EMovieSceneBlendType : uint8
{
	Invalid UMETA(Hidden),
	/** Blends all other weighted values together as an average of the total weight */
	Absolute			= 0x1,
	/** Applies this value as a sum total of all other additives */
	Additive			= 0x2,
	/** Applies this value as a sum total of all other additives and the initial value before the animation */
	Relative			= 0x4,
	/** Applies this value as an additive equal to the difference between the current value and the first value */
	AdditiveFromBase    = 0x8,
	/** The value will override the current value */
	Override			= 0x10,

	// If you add new blend types, don't forget to change:
	// - All()
	// - Num()
};

/** Optional blend type structure */
USTRUCT(BlueprintType)
struct FOptionalMovieSceneBlendType
{
	GENERATED_BODY()

	FOptionalMovieSceneBlendType()
		: BlendType(EMovieSceneBlendType::Absolute)
		, bIsValid(false)
	{}

	explicit FOptionalMovieSceneBlendType(EMovieSceneBlendType InBlendType)
		: BlendType(InBlendType)
		, bIsValid(true)
	{}

	/**
	 * Assignment from a EMovieSceneBlendType enumeration
	 */
	FOptionalMovieSceneBlendType& operator=(EMovieSceneBlendType InBlendType)
	{
		bIsValid = true;
		BlendType = InBlendType;
		return *this;
	}

	/**
	 * Check if this blend type has been set
	 */
	bool IsValid() const
	{
		return bIsValid;
	}

	/**
	 * Get this blend type. Must have been set to a valid enumeration.
	 */
	EMovieSceneBlendType Get() const
	{
		check(bIsValid);
		return BlendType;
	}

	friend bool operator==(FOptionalMovieSceneBlendType A, EMovieSceneBlendType B)
	{
		return A.IsValid() && A.BlendType == B;
	}

	/** The actual blend type */
	UPROPERTY(BlueprintReadOnly, Category="Blend Type")
	EMovieSceneBlendType BlendType;

	/** Boolean indicating whether BlendType is valid */
	UPROPERTY(BlueprintReadOnly, Category="Blend Type")
	bool bIsValid;
};

/** Type that specifies a set of blend types that are supported for a particular section. Implemented in this way to avoid direct use of EMovieSceneBlendType as bit flags. */
struct FMovieSceneBlendTypeField
{
	/**
	 * Constructor
	 */
	MOVIESCENE_API FMovieSceneBlendTypeField();

	/**
	 * Retrieve a bit field representing all blend types
	 */
	MOVIESCENE_API static FMovieSceneBlendTypeField All();

	/**
	 * Retrieve a bit field representing No blend types
	 */
	MOVIESCENE_API static FMovieSceneBlendTypeField None();

	/**
	 * Add or remove the specified blend types
	 */
	template<typename... E> void Add(E... Types)	{ for (EMovieSceneBlendType BlendType : {Types...}) { Add(BlendType); } 			}
	template<typename... E> void Remove(E... Types)	{ for (EMovieSceneBlendType BlendType : {Types...}) { Remove(BlendType); } 			}

	/**
	 * Add the specified blend type to this field
	 */
	MOVIESCENE_API void Add(EMovieSceneBlendType Type);
	MOVIESCENE_API void Add(FMovieSceneBlendTypeField Field);

	/**
	 * Remove the specified blend type from this field
	 */
	MOVIESCENE_API void Remove(EMovieSceneBlendType Type);
	MOVIESCENE_API void Remove(FMovieSceneBlendTypeField Field);

	/**
	 * Invert this type field
	 */
	MOVIESCENE_API FMovieSceneBlendTypeField Invert() const;

	/**
	 * Check whether this field contains the specified blend type
	 */
	MOVIESCENE_API bool Contains(EMovieSceneBlendType InBlendType) const;

	/**
	 * Count how many blend types are contained within this field
	 */
	MOVIESCENE_API int32 Num() const;

	friend bool operator==(const FMovieSceneBlendTypeField& A, const FMovieSceneBlendTypeField& B)
	{
		return A.BlendTypeField == B.BlendTypeField;
	}

	friend bool operator!=(const FMovieSceneBlendTypeField& A, const FMovieSceneBlendTypeField& B)
	{
		return A.BlendTypeField != B.BlendTypeField;
	}

	friend MOVIESCENE_API FMovieSceneBlendTypeFieldIterator begin(const FMovieSceneBlendTypeField& Field);
	friend MOVIESCENE_API FMovieSceneBlendTypeFieldIterator end(const FMovieSceneBlendTypeField& Field);

private:
	explicit FMovieSceneBlendTypeField(EMovieSceneBlendType InBlendTypeField) : BlendTypeField(InBlendTypeField) {}

	/** The actual enumeration value representing all valid blend types */
	EMovieSceneBlendType BlendTypeField;
};

struct FMovieSceneBlendTypeFieldIterator
{
public:
	static FMovieSceneBlendTypeFieldIterator Begin(FMovieSceneBlendTypeField InField);
	static FMovieSceneBlendTypeFieldIterator End(FMovieSceneBlendTypeField InField);

	inline explicit operator bool() const 	{ return Offset >= 0 && Offset <= MaxValidOffset(); }
	inline bool operator!() const 			{ return Offset < 0 || Offset > MaxValidOffset(); }
	inline void operator++() 				{ IterateToNext(); }
	inline bool operator==(const FMovieSceneBlendTypeFieldIterator& RHS) const { return Field == RHS.Field && Offset == RHS.Offset; }
	inline bool operator!=(const FMovieSceneBlendTypeFieldIterator& RHS) const { return Field != RHS.Field || Offset != RHS.Offset; }

	MOVIESCENE_API EMovieSceneBlendType operator*();

private:
	friend MOVIESCENE_API FMovieSceneBlendTypeFieldIterator begin(const FMovieSceneBlendTypeField&);
	friend MOVIESCENE_API FMovieSceneBlendTypeFieldIterator end(const FMovieSceneBlendTypeField&);

	MOVIESCENE_API void IterateToNext();

	inline static int8 MaxValidOffset() { return (int8)(StaticEnum<EMovieSceneBlendType>()->NumEnums() - 2); }

	FMovieSceneBlendTypeField Field;
	int8 Offset;
};

