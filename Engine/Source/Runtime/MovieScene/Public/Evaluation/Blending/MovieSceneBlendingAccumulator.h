// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Evaluation/Blending/BlendableToken.h"
#include "Evaluation/Blending/BlendableTokenStack.h"
#include "Evaluation/Blending/MovieSceneAccumulatedBlendState.h"
#include "Evaluation/Blending/MovieSceneBlendType.h"
#include "Evaluation/Blending/MovieSceneBlendingActuator.h"
#include "Evaluation/Blending/MovieSceneBlendingActuatorID.h"
#include "Evaluation/MovieSceneEvaluationScope.h"
#include "Evaluation/MovieScenePlayback.h"
#include "Misc/AssertionMacros.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class IMovieScenePlayer;
class UObject;
struct FMovieSceneEvaluationOperand;
struct FMovieSceneEvaluationScope;
struct FMovieSceneInterrogationData;
struct FPersistentEvaluationData;

struct FMovieSceneBlendingAccumulator
{
	/**
	 * Add a global token (not bound to a particular object) that will be blended together with other tokens of the same type and actuator this frame
	 * Such tokens are collated actuator, meaning the same data type can be blended in the same manner but applied differently depending on context (ie, multiple floats that relate to slomo and fade tracks)
	 * 
	 * @see FindActuator, DefineActuator
	 *
	 * @param InScope 			The currently evaluating scope. Used to save preanimated state for the section
	 * @param InActuatorTypeID	Unique ID that identifies the actuator to use to apply the final blended state (must map to a valid actuator)
	 * @param InToken			The actual token data to blend
	 */
	template<typename ActuatorDataType>
	void BlendToken(FMovieSceneBlendingActuatorID InActuatorTypeID, const FMovieSceneEvaluationScope& InScope, const FMovieSceneContext& InContext, TBlendableToken<ActuatorDataType>&& InToken)
	{
		check(FindActuator<ActuatorDataType>(InActuatorTypeID));

		InToken.AnimatingScope = InScope;
		InToken.HierarchicalBias = InContext.GetHierarchicalBias();
		UnboundBlendState.Add(InActuatorTypeID, MoveTemp(InToken));
	}
	template<typename ActuatorDataType>
	void BlendToken(FMovieSceneBlendingActuatorID InActuatorTypeID, const FMovieSceneEvaluationScope& InScope, const FMovieSceneContext& InContext, ActuatorDataType&& InInputValue, EMovieSceneBlendType InBlendType, float InWeight = 1.f, int32 InBlendingPriority = INDEX_NONE)
	{
		check(FindActuator<ActuatorDataType>(InActuatorTypeID));
		UnboundBlendState.Add(InActuatorTypeID, TBlendableToken<ActuatorDataType>(Forward<ActuatorDataType>(InInputValue), InScope, InContext, InBlendType, InWeight, InBlendingPriority));
	}

	/**
	 * Add a token that will be blended together with other tokens of the same type and actuator this frame, and applied to all objects relating to the specified operand
	 * Such tokens are collated per-object and actuator, meaning the same data type can be blended in the same manner, but applied differently depending on context (ie, multiple float properties on the same object)
	 * 
	 * @see FindActuator, DefineActuator
	 *
	 * @param InOperand			The operand to which this token should be applied
	 * @param InActuatorTypeID	Unique ID that identifies the actuator to use to apply the final blended state (must map to a valid actuator)
	 * @param InToken			The actual token data to blend
	 */
	template<typename ActuatorDataType>
	void BlendToken(const FMovieSceneEvaluationOperand& InOperand, FMovieSceneBlendingActuatorID InActuatorTypeID, const FMovieSceneEvaluationScope& InScope, const FMovieSceneContext& InContext, TBlendableToken<ActuatorDataType>&& InToken)
	{
		check(FindActuator<ActuatorDataType>(InActuatorTypeID));
		InToken.AnimatingScope = InScope;
		InToken.HierarchicalBias = InContext.GetHierarchicalBias();
		OperandToBlendState.FindOrAdd(InOperand).Add(InActuatorTypeID, MoveTemp(InToken));
	}
	template<typename ActuatorDataType>
	void BlendToken(const FMovieSceneEvaluationOperand& InOperand, FMovieSceneBlendingActuatorID InActuatorTypeID, const FMovieSceneEvaluationScope& InScope, const FMovieSceneContext& InContext, ActuatorDataType&& InInputValue, EMovieSceneBlendType InBlendType, float InWeight = 1.f, int32 InBlendingPriority = INDEX_NONE)
	{
		check(FindActuator<ActuatorDataType>(InActuatorTypeID));
		OperandToBlendState.FindOrAdd(InOperand).Add(InScope, InActuatorTypeID, TBlendableToken<ActuatorDataType>(Forward<ActuatorDataType>(InInputValue), InScope, InContext, InBlendType, InWeight, InBlendingPriority));
	}

	/**
	 * Find an existing actuator with the specified ID that operates on a specific data type
	 *
	 * @param InActuatorTypeID	The ID that uniquely identifies the actuator to lookup
	 * @return A pointer to an actuator if one already exists, nullptr otherwise
	 */
	template<typename DataType>
	TMovieSceneBlendingActuator<DataType>* FindActuator(FMovieSceneBlendingActuatorID InActuatorTypeID) const
	{
		const TSharedRef<IMovieSceneBlendingActuator>* Actuator = Actuators.Find(InActuatorTypeID);
		if (Actuator)
		{
			if ((*Actuator)->GetDataTypeID() == GetBlendingDataType<DataType>())
			{
				return &static_cast<TMovieSceneBlendingActuator<DataType>&>(Actuator->Get());
			}
		}

		return nullptr;
	}

	/**
	 * Define an actuator with the specified unique ID that operates on a specific data type
	 *
	 * @param InActuatorTypeID	The ID that uniquely identifies the actuator to lookup
	 * @return A pointer to an actuator if one already exists, nullptr otherwise
	 */
	void DefineActuator(FMovieSceneBlendingActuatorID InActuatorTypeID, TSharedRef<IMovieSceneBlendingActuator> InActuator)
	{
		Actuators.Add(InActuatorTypeID, InActuator);
	}

	/**
	 * Apply all currently accumulated blends
	 */
	MOVIESCENE_API void Apply(const FMovieSceneContext& Context, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player);

	/**
	 * Interrogate the final blended result when applied to the specified object
	 */
	MOVIESCENE_API void Interrogate(const FMovieSceneContext& Context, FMovieSceneInterrogationData& InterrogationData, UObject* AnimatedObject);

	/** Map of actuator ID to actuator */
	TMap<FMovieSceneBlendingActuatorID, TSharedRef<IMovieSceneBlendingActuator>> Actuators;

private:

	TMap<FMovieSceneEvaluationOperand, FMovieSceneAccumulatedBlendState> OperandToBlendState;
	FMovieSceneAccumulatedBlendState UnboundBlendState;
};



#include "MovieSceneBlendingAccumulator.inl" // IWYU pragma: export
