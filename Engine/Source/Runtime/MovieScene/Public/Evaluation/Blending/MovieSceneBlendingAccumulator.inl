// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compilation/MovieSceneTemplateInterrogation.h"

MOVIESCENE_API FMovieSceneAnimTypeID GetInitialValueTypeID();

/** Definition needs to exist after the definition of FMovieSceneBlendingAccumulator */
template<typename DataType>
void TBlendableTokenStack<DataType>::ComputeAndActuate(UObject* InObject, FMovieSceneBlendingAccumulator& Accumulator, FMovieSceneBlendingActuatorID InActuatorType, const FMovieSceneContext& Context, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player)
{
	TMovieSceneBlendingActuator<DataType>* Actuator = Accumulator.FindActuator<DataType>(InActuatorType);
	if (!ensure(Actuator))
	{
		return;
	}

	TMovieSceneInitialValueStore<DataType> InitialValues(*Actuator, *this, InObject, &Player);

	typename TBlendableTokenTraits<DataType>::WorkingDataType WorkingTotal{};
	Tokens.Sort();
	for (const TBlendableToken<DataType>* Token : Tokens)
	{
		Token->AddTo(WorkingTotal, InitialValues);
	}

	DataType FinalResult = WorkingTotal.Resolve(InitialValues);
	Actuator->Actuate(InObject, FinalResult, *this, Context, PersistentData, Player);

	if (Actuator->HasInitialValue(InObject))
	{
		FMovieSceneAnimTypeID TypeID = FMovieSceneAnimTypeID::Combine(Actuator->GetActuatorID(), GetInitialValueTypeID());
		if (InObject)
		{
			SavePreAnimatedStateForAllEntities(Player, *InObject, TypeID, FMovieSceneRemoveInitialValueTokenProducer(Actuator->AsShared()));
		}
		else
		{
			SavePreAnimatedStateForAllEntities(Player, TypeID, FMovieSceneRemoveInitialGlobalValueTokenProducer(Actuator->AsShared()));
		}
	}
}


template<typename DataType>
void TBlendableTokenStack<DataType>::Interrogate(UObject* AnimatedObject, FMovieSceneInterrogationData& InterrogationData, FMovieSceneBlendingAccumulator& Accumulator, FMovieSceneBlendingActuatorID InActuatorType, const FMovieSceneContext& Context)
{
	TMovieSceneBlendingActuator<DataType>* Actuator = Accumulator.FindActuator<DataType>(InActuatorType);
	if (!ensure(Actuator))
	{
		return;
	}

	TMovieSceneInitialValueStore<DataType> InitialValues(*Actuator, *this, AnimatedObject, nullptr);

	typename TBlendableTokenTraits<DataType>::WorkingDataType WorkingTotal{};
	for (const TBlendableToken<DataType>* Token : Tokens)
	{
		Token->AddTo(WorkingTotal, InitialValues);
	}

	DataType FinalResult = WorkingTotal.Resolve(InitialValues);
	Actuator->Actuate(InterrogationData, FinalResult, *this, Context);
}
