// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MovieSceneFwd.h"
#include "UObject/ObjectKey.h"
#include "IMovieScenePlayer.h"
#include "Evaluation/Blending/BlendableTokenStack.h"
#include "Evaluation/Blending/MovieSceneInitialValueStore.h"

class UObject;

template<typename DataType> struct TBlendableTokenStack;

/**
 * Base class for all blending actuators
 */
struct IMovieSceneBlendingActuator : TSharedFromThis<IMovieSceneBlendingActuator>
{
	virtual ~IMovieSceneBlendingActuator()= default;

	/**
	 * Access the result of GetBlendingDataType<DataType>() for the data that this actuator can apply
	 */
	virtual FMovieSceneAnimTypeID GetDataTypeID() const = 0;

	/**
	 * Remove the initial value for the specified object from this actuator causing it to be re-computed next time it is required.
	 * @param InObject 		The object to remove the initial value for.
	 */
	virtual void RemoveInitialValueForObject(FObjectKey InObject) = 0;
};

/**
 * Templated blending actuator that knows how to apply a specific data type to an object
 */
template<typename DataType>
struct TMovieSceneBlendingActuator : IMovieSceneBlendingActuator
{
public:

	/**
	 * Apply the the specified value to an object.
	 *
	 * @param InObject 		The object to apply the value to. nullptr where this actuator is being used for a root track.
	 * @param InValue 		The value to apply to the object.
	 * @param OriginalStack	Reference to the original stack from which the final result was derived.
	 * @param Context		Movie scene context structure from the root level
	 * @param PersistentData	Persistent data store for the evaluation
	 * @param Player		The movie scene player currently running the sequence
	 */
	virtual void Actuate(UObject* InObject, typename TCallTraits<DataType>::ParamType InValue, const TBlendableTokenStack<DataType>& OriginalStack, const FMovieSceneContext& Context, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) = 0;

	/**
	 * Add the final result of a blending operation to the interrogation data
	 *
	 * @param InterrogationData The interrogation data to populate
	 * @param InValue 		The value to apply to the object.
	 * @param OriginalStack	Reference to the original stack from which the final result was derived.
	 * @param Context		Movie scene context structure from the root level
	 */
	virtual void Actuate(FMovieSceneInterrogationData& InterrogationData, typename TCallTraits<DataType>::ParamType InValue, const TBlendableTokenStack<DataType>& OriginalStack, const FMovieSceneContext& Context) const {}

	/**
	 * Retrieve the current value of the specified object
	 *
	 * @param InObject 		The object to retrieve the value from. nullptr where this actuator is being used for a root track.
	 * @param Player		The movie scene player currently running the sequence. Null when this actuator is used in an interrogation context.
	 * @return The current value of the object
	 */
	virtual DataType RetrieveCurrentValue(UObject* InObject, IMovieScenePlayer* Player) const = 0;

	/**
	 * Remove the initial value for the specified object from this actuator causing it to be re-computed next time it is required.
	 * @param InObject 		The object to remove the initial value for.
	 */
	virtual void RemoveInitialValueForObject(FObjectKey ObjectToRemove) override
	{
		InitialValues.RemoveAll([=](const FInitialValue& In) { return In.Object == ObjectToRemove; });
	}

	/**
	 * Access this actuator's unique identifier
	 */
	FMovieSceneBlendingActuatorID GetActuatorID() const
	{
		return ActuatorID;
	}

	/**
	 * Check whether this actuator has an initial value for the specified animated object
	 */
	bool HasInitialValue(FObjectKey InObject) const
	{
		for (const auto& InitialValue : InitialValues)
		{
			if (InitialValue.Object == InObject)
			{
				return true;
			}
		}
		return false;
	}

protected:
	
	/** Constructor */
	TMovieSceneBlendingActuator(FMovieSceneBlendingActuatorID InActuatorID)
		: ActuatorID(InActuatorID)
	{}

private:

	/**
	 * Access an identifier for the data type that this actuator can use
	 */
	virtual FMovieSceneAnimTypeID GetDataTypeID() const final
	{
		return GetBlendingDataType<DataType>();
	}

	friend struct TMovieSceneInitialValueStore<DataType>;

	struct FInitialValue
	{
		FInitialValue(FObjectKey InObject, const DataType& InValue) : Object(InObject), Value(InValue) {}

		FObjectKey Object;
		DataType Value;
	};

	/** Member that stores initial values for this actuator */
	TArray<FInitialValue> InitialValues;

	/** This actuator's unique identifier */
	FMovieSceneBlendingActuatorID ActuatorID;
};

