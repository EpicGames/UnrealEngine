// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Evaluation/MovieSceneAnimTypeID.h"

struct FMovieSceneBlendingActuatorID : FMovieSceneAnimTypeID
{
	explicit FMovieSceneBlendingActuatorID(FMovieSceneAnimTypeID InTypeID) : FMovieSceneAnimTypeID(InTypeID) {}

	friend bool operator==(FMovieSceneBlendingActuatorID A, FMovieSceneBlendingActuatorID B)
	{
		return A.ID == B.ID;
	}

	friend bool operator!=(FMovieSceneBlendingActuatorID A, FMovieSceneBlendingActuatorID B)
	{
		return A.ID != B.ID;
	}

	friend uint32 GetTypeHash(FMovieSceneBlendingActuatorID In)
	{
		return GetTypeHash(In.ID);
	}
};
