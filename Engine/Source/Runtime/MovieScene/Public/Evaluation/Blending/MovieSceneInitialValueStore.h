// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IMovieScenePlayer.h"
#include "MovieSceneExecutionToken.h"
#include "MovieSceneFwd.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectKey.h"

class IMovieScenePlayer;
class UObject;
struct IMovieSceneBlendingActuator;
template<typename DataType> struct TBlendableTokenStack;
template<typename DataType> struct TMovieSceneBlendingActuator;

/** Pre animated token producer that reverts the object's initial value from the actuator when its state is restored */
struct FMovieSceneRemoveInitialValueTokenProducer : IMovieScenePreAnimatedTokenProducer
{
	/** Construction from the object whose initial value to remove, and the actuator to remove it from */
	MOVIESCENE_API FMovieSceneRemoveInitialValueTokenProducer(TWeakPtr<IMovieSceneBlendingActuator> InWeakActuator);
	MOVIESCENE_API ~FMovieSceneRemoveInitialValueTokenProducer();

	MOVIESCENE_API virtual IMovieScenePreAnimatedTokenPtr CacheExistingState(UObject& InObject) const override;

private:
	/** The actuator to remove the initial value from */
	TWeakPtr<IMovieSceneBlendingActuator> WeakActuator;
};

/** Pre animated token producer that reverts a global initial value from the actuator when its state is restored */
struct FMovieSceneRemoveInitialGlobalValueTokenProducer : IMovieScenePreAnimatedGlobalTokenProducer
{
	/** Construction from the object whose initial value to remove, and the actuator to remove it from */
	MOVIESCENE_API FMovieSceneRemoveInitialGlobalValueTokenProducer(TWeakPtr<IMovieSceneBlendingActuator> InWeakActuator);
	MOVIESCENE_API ~FMovieSceneRemoveInitialGlobalValueTokenProducer();

	MOVIESCENE_API virtual IMovieScenePreAnimatedGlobalTokenPtr CacheExistingState() const override;

private:
	/** The actuator to remove the initial value from */
	TWeakPtr<IMovieSceneBlendingActuator> WeakActuator;
};

/** Temporary data structure used as a proxy to retrieve cached initial values for the currently animating object */
template<typename DataType>
struct TMovieSceneInitialValueStore
{
	/**
	 * Constructor
	 * @param InActuator 			The actuator that will apply the final value
	 * @param InStack 				The stack of tokens that are currently being blended (used for saving pre-animated state)
	 * @param InAnimatingObject 	(Optional) The object that is currently being animated (nullptr for root tracks)
	 * @param InPlayer 				(Optional) The movie scene player currently playing the sequence (nullptr for data interrogation)
	 */
	TMovieSceneInitialValueStore(TMovieSceneBlendingActuator<DataType>& InActuator, const TBlendableTokenStack<DataType>& InStack, UObject* InAnimatingObject, IMovieScenePlayer* InPlayer)
		: Actuator(InActuator)
		, Stack(InStack)
		, AnimatingObject(InAnimatingObject)
		, Player(InPlayer)
	{}

	/**
	 * Access the current (uncached) value from the object that relates to the current actuator's ID
	 */
	DataType RetrieveCurrentValue() const
	{
		return Actuator.RetrieveCurrentValue(AnimatingObject, Player);
	}

	/**
	 * Access the initial (cached) value from the object that relates to the current actuator's ID, before it was animated by this actuator
	 */
	DataType GetInitialValue() const
	{
		FObjectKey ThisObjectKey(AnimatingObject);
		for (const auto& InitialValue : Actuator.InitialValues)
		{
			if (InitialValue.Object == ThisObjectKey)
			{
				return InitialValue.Value;
			}
		}

		DataType NewInitialValue = Actuator.RetrieveCurrentValue(AnimatingObject, Player);
		if (Player)
		{
			Actuator.InitialValues.Emplace(ThisObjectKey, NewInitialValue);
		}

		return NewInitialValue;
	}

	/**
	 * Access the object that this store's initial values relate to
	 */
	UObject* GetAnimatingObject() const
	{
		return AnimatingObject;
	}

private:
	/** The actuator responsible for storing initial values. */
	TMovieSceneBlendingActuator<DataType>& Actuator;
	/** The stack of tokens that are being applied */
	const TBlendableTokenStack<DataType>& Stack;
	/** The object that is being animated (nullptr for root track animation) */
	UObject* AnimatingObject;
	/** Player that's playing back the sequence. Can be null. Potentially used when accessing values for root tracks. */
	IMovieScenePlayer* Player;
};
