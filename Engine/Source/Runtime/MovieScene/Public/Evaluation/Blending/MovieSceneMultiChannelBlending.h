// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Evaluation/Blending/MovieSceneBlendType.h"
#include "Evaluation/Blending/BlendableTokenStack.h"
#include "EulerTransform.h"

/**~
 * Generic multi-channel blending support for sequencer.
 * Works in conjunction with TMovieSceneBlendingActuator<Type> where Type has been set up with the correct traits to allow multi-channel blending:
 * Example:
 *
 * // My struct contains 3 floats, so is represented as a TMultiChannelValue<float, 3>
 * struct FMyStruct { float X, Y, Z; ... };
 * 
 * namespace MovieScene
 * {
 * 		// Marshall my struct into a multi-channel value
 * 		inline void MultiChannelFromData(FMyStruct In, TMultiChannelValue<float, 3>& Out)
 * 		{
 * 			Out = { In.X, In.Y, In.Z };
 * 		}
 * 		// Marshall my struct out of a static float array
 * 		inline void ResolveChannelsToData(float (&Channels)[3], FMyStruct& Out)
 * 		{
 * 			Out = FMyStruct(Channels[0], Channels[1], Channels[2]);
 * 		}
 * }
 * 
 * // Access a unique runtime type identifier for FMyStruct. Implemented in cpp to ensure there is only ever 1. This is required to support actuators operating on FMyStructs.
 * template<> MYMODULE_API FMovieSceneAnimTypeID GetBlendingDataType<FMyStruct>();
 *
 * // Inform the blending code to use a maksed blendable of 3 floats for my struct
 * template<> struct TBlendableTokenTraits<FMyStruct> 	{ typedef UE::MovieScene::TMaskedBlendable<float, 3> WorkingDataType; };
 *
 * // To inject a blendable token into the accumulator for an FMyStruct:
 * UE::MovieScene::TMultiChannelValue<float, 3> AnimatedData;
 * 
 * // Set the X and Z channels
 * AnimatedData.Set(0, 100.f);
 * AnimatedData.Set(2, 250.f);
 * 
 * // Ensure the accumulator knows how to apply an FMyStruct (usually this will be on a property, and happen through EnsureActuator<>)
 * FMovieSceneBlendingActuatorID ActuatorTypeID = FMyActuatorType::GetActuatorTypeID();
 * if (!ExecutionTokens.GetAccumulator().FindActuator<FMyStruct>(ActuatorTypeID))
 * {
 * 	ExecutionTokens.GetAccumulator().DefineActuator(ActuatorTypeID, MakeShared<FMyActuatorType>());
 * }
 * 
 * // Add the blendable to the accumulator
 * float Weight = EvaluateEasing(Context.GetTime());
 * // Constructing a TBlendableToken from any type other than TMultiChannelValue<float, 3> requires a supporting BlendValue function
 * ExecutionTokens.BlendToken(ActuatorTypeID, TBlendableToken<FMyStruct>(AnimatedData, EMovieSceneBlendType::Absolute, Weight));
 */

namespace UE
{
namespace MovieScene
{
	/**
	 * Generic value type that supports a specific number of channels, optionally masked out.
	 * Used for blending types that can be represented as a contiguous set of numeric values.
	 * Relative, Weighted and Additive blending occurs on a per-channel basis
	 */
	template<typename T, uint8 N>
	struct TMultiChannelValue
	{
		static_assert(N > 0 && N <= 32, "Cannot store more than 32 channels in this type.");

		/** Default Constructor */
		TMultiChannelValue()
			: Mask(0)
		{}

		/** Construction from a set of values. List size must match the number of channels. */
		template<typename OtherType>
		TMultiChannelValue(std::initializer_list<OtherType> InitList)
			: Mask((1 << N) - 1)
		{
			check(InitList.size() == N);
			T* CurrChannel = &Channels[0];
			for (auto Val : InitList)
			{
				(*CurrChannel++) = Val;
			}
		}

		/** Array indexing operator - returns a channel value */
		T operator[](uint8 Index) const
		{
			checkf(Index < N && IsSet(Index), TEXT("Attempting to access an invalid channel."));
			return Channels[Index];
		}

		/** Access a value with a default */
		T Get(uint8 Index, T Default) const
		{
			return IsSet(Index) ? Channels[Index] : Default;
		}

		/** Check if this value is empty */
		bool IsEmpty() const
		{
			return Mask == 0;
		}

		/** Check if every channel in this value is valid */
		bool IsFull() const
		{
			static const uint32 FullChannelMask = N == 32 ? 0xFFFFFFFF : (1 << N) - 1;
			return Mask == FullChannelMask;
		}

		/** Check whether the specified channel index is enabled */
		bool IsSet(uint8 Index) const
		{
			check(Index < N);
			return (Mask & (1 << Index)) != 0;
		}

		/** Enable and apply a value to the specified channel */
		void Set(uint8 Index, T Value)
		{
			check(Index < N);

			Channels[Index] = Value;
			Mask |= (1 << Index);
		}

		/** Increment the channel at the specified index by the specified amount */
		void Increment(uint8 Index, T Value)
		{
			check(Index < N);

			Channels[Index] = IsSet(Index) ? Channels[Index] + Value : Value;
			Mask |= (1 << Index);
		}

		/** Do weighted blend with current value, this is an override*/
		void WeightedBlend(uint8 Index, T Value, float Weight)
		{
			check(Index < N);

			Channels[Index] = IsSet(Index) ? ((1.0 - Weight)*Channels[Index]) + (Value * Weight) : Value;
			Mask |= (1 << Index);
		}

	private:

		/** Channel data */
		T Channels[N];

		/** Mask signifying which indices within Channels are valid */
		uint32 Mask;
	};

	/** Declaration of a function used to generate multi channel data from a source type. Overload in the MovieScene namespace for type-specific functionality. */
	template<typename T, typename SourceData, uint8 N>
	void MultiChannelFromData(SourceData InSourceData, TMultiChannelValue<T, N>& OutChannelData)
	{
		static_assert(std::is_same_v<T, void>, "MultiChannelFromData must be implemented to blend SourceData with multi-channel data.");
	}

	/**
	 * Declaration of a function used to popupate a specific type with generic channel data after blending has occurred. Overload in the MovieScene namespace for type-specific functionality.
	 * For example, to support multi channel blending
	 */
	template<typename T, typename SourceData, uint8 N>
	void ResolveChannelsToData(const TMultiChannelValue<T, N>& OutChannelData, SourceData& OutData)
	{
		static_assert(std::is_same_v<T, void>, "ResolveChannelsToData must be implemented to blend SourceData with multi-channel data.");
	}

	/** Working data type used to blend multi-channel values */
	template<typename DataType, uint8 N>
	struct TMaskedBlendable
	{
		TMaskedBlendable()
		{
			// All weights start at 0
			FMemory::Memzero(&AbsoluteWeights, sizeof(AbsoluteWeights));
		}

		/** Per-channel absolute values to apply, pre-multiplied by their weight */
		TMultiChannelValue<DataType, N> Absolute;
		/** Cumulative absolute weights for each channel */
		float AbsoluteWeights[N];

		/** Per-channel additive values to apply, pre-multiplied by their weight */
		TMultiChannelValue<DataType, N> Additive;

		/** Per-channel override values to apply, when set this will override the default absolute/additive set */
		TMultiChannelValue<DataType, N> Override;

		/** Cached initial value for this blendable in multi-channel form */
		TOptional<TMultiChannelValue<DataType, N>> InitialValue;

		/** Resolve this structure's data into a final value to pass to the actuator */
		template<typename ActualDataType>
		ActualDataType Resolve(TMovieSceneInitialValueStore<ActualDataType>& InitialValueStore)
		{
			TOptional<TMultiChannelValue<DataType, N>> CurrentValue;

			TMultiChannelValue<DataType, N> Result;

			// Iterate through each channel
			for (uint8 Channel = 0; Channel < N; ++Channel)
			{
				// Any animated channels with a weight of 0 should match the object's *initial* position. Exclusively additive channels are based implicitly off the initial value
				if (Override.IsSet(Channel)) //if override then we override
				{
					Result.Set(Channel, Override[Channel]);
				}
				else
				{
					const bool bUseInitialValue = (Absolute.IsSet(Channel) && AbsoluteWeights[Channel] == 0.f) || (!Absolute.IsSet(Channel) && Additive.IsSet(Channel));
					if (bUseInitialValue)
					{
						if (!InitialValue.IsSet())
						{
							InitialValue.Emplace();
							MultiChannelFromData(InitialValueStore.GetInitialValue(), InitialValue.GetValue());
						}

						Result.Set(Channel, InitialValue.GetValue()[Channel]);
					}
					else if (Absolute.IsSet(Channel))
					{
						// If it has a non-zero weight, divide by it, and apply the absolute total to the result
						Result.Set(Channel, Absolute[Channel] / AbsoluteWeights[Channel]);
					}

					// If it has any additive values in the channel, add those on
					if (Additive.IsSet(Channel))
					{
						Result.Increment(Channel, Additive[Channel]);
					}
				}

				// If the channel has not been animated at all, set it to the *current* value
				if (!Result.IsSet(Channel))
				{
					if (!CurrentValue.IsSet())
					{
						CurrentValue.Emplace();
						MultiChannelFromData(InitialValueStore.RetrieveCurrentValue(), CurrentValue.GetValue());
					}
					Result.Set(Channel, CurrentValue.GetValue()[Channel]);
				}
			}

			ensureMsgf(Result.IsFull(), TEXT("Attempting to apply a compound data type with some channels uninitialized."));

			// Resolve the final channel data into the correct data type for the actuator
			ActualDataType FinalResult;
			ResolveChannelsToData(Result, FinalResult);
			return FinalResult;
		}
	};

	template<typename OutputType, typename InputType, typename ActualValueType, uint8 ChannelSize>
	void BlendValue(TMaskedBlendable<OutputType, ChannelSize>& OutBlend, InputType InValue, int32 ChannelIndex, float Weight, EMovieSceneBlendType BlendType, int32 BlendingOrder, TMovieSceneInitialValueStore<ActualValueType>& InitialValueStore)
	{
		if (BlendType == EMovieSceneBlendType::Absolute || BlendType == EMovieSceneBlendType::Relative)
		{
			if (BlendType == EMovieSceneBlendType::Relative)
			{
				// If it's relative, we need to add our value onto the channel's initial value
				if (!OutBlend.InitialValue.IsSet())
				{
					OutBlend.InitialValue.Emplace();
					MultiChannelFromData(InitialValueStore.GetInitialValue(), OutBlend.InitialValue.GetValue());
				}
				OutBlend.Absolute.Increment(ChannelIndex, (OutBlend.InitialValue.GetValue()[ChannelIndex] + InValue) * Weight);
			}
			else
			{
				// Coerce to the output type before applying a weight to ensure that we're operating with the working data type throughout.
				// This allows us to blend integers as doubles without losing precision at extreme numeric ranges
				OutBlend.Absolute.Increment(ChannelIndex, OutputType(InValue) * Weight);
			}

			// Accumulate total weights
			OutBlend.AbsoluteWeights[ChannelIndex] += Weight;
		}
		else if (BlendingOrder != INDEX_NONE)
		{
			//If there is a blending order set we use the additive to contain
			//the full blended value consiting of absolute + overrides + additives
			//This will already be sorted so we just need to accumulate the values
			if (OutBlend.Override.IsSet(ChannelIndex) == false)
			{
				if (OutBlend.Absolute.IsSet(ChannelIndex))
				{
					// If it has a non-zero weight, divide by it, and apply the absolute total to the result
					OutputType AbsoluteWeight = (OutBlend.AbsoluteWeights[ChannelIndex] != 0.f) ? OutBlend.AbsoluteWeights[ChannelIndex] : 1.0;
					OutBlend.Override.Set(ChannelIndex, OutBlend.Absolute[ChannelIndex] / AbsoluteWeight);
				}
			}
			if (BlendType == EMovieSceneBlendType::Additive)
			{
				OutBlend.Override.Increment(ChannelIndex, OutputType(InValue) * Weight);
			}
			else if (BlendType == EMovieSceneBlendType::Override)
			{
				OutBlend.Override.WeightedBlend(ChannelIndex, OutputType(InValue), Weight);
			}
		}
		else if (BlendType == EMovieSceneBlendType::Additive)
		{
			// Additive animation just increments the additive channel
			// Coerce to the output type before applying a weight to ensure that we're operating with the working data type throughout.
			// This allows us to blend integers as doubles without losing precision at extreme numeric ranges
			OutBlend.Additive.Increment(ChannelIndex, OutputType(InValue) * Weight);
		}
	}

	template<typename OutputType, typename InputType, typename ActualValueType>
	void BlendValue(TMaskedBlendable<OutputType, 1>& OutBlend, InputType InValue, float Weight, EMovieSceneBlendType BlendType, int32 BlendingOrder, TMovieSceneInitialValueStore<ActualValueType>& InitialValueStore)
	{
		BlendValue(OutBlend, InValue, 0, Weight, BlendType, BlendingOrder, InitialValueStore);
	}

	template<typename OutputType, typename ActualValueType, uint8 ChannelSize>
	void BlendValue(TMaskedBlendable<OutputType, ChannelSize>& OutBlend, const TMultiChannelValue<OutputType, ChannelSize>& InValue, float Weight, EMovieSceneBlendType BlendType, int32 BlendingOrder, TMovieSceneInitialValueStore<ActualValueType>& InitialValueStore)
	{
		for (int32 Index = 0; Index < ChannelSize; ++Index)
		{
			if (InValue.IsSet(Index))
			{
				BlendValue(OutBlend, InValue[Index], Index, Weight, BlendType, BlendingOrder, InitialValueStore);
			}
		}
	}

	inline void MultiChannelFromData(int32 In, 				TMultiChannelValue<double, 1>& Out)	{ Out = { In }; }
	inline void MultiChannelFromData(float In, 				TMultiChannelValue<float, 1>& Out)	{ Out = { In }; }
	inline void MultiChannelFromData(FVector2D In, 			TMultiChannelValue<float, 2>& Out)	{ Out = { In.X, In.Y }; }
	inline void MultiChannelFromData(FVector In, 			TMultiChannelValue<float, 3>& Out)	{ Out = { In.X, In.Y, In.Z }; }
	inline void MultiChannelFromData(const FVector4& In, 	TMultiChannelValue<float, 4>& Out)	{ Out = { In.X, In.Y, In.Z, In.W }; }
	inline void MultiChannelFromData(const FTransform& In, 	TMultiChannelValue<float, 9>& Out)
	{
		FVector Translation = In.GetTranslation();
		FVector Rotation = In.GetRotation().Rotator().Euler();
		FVector Scale = In.GetScale3D();
		Out = { Translation.X, Translation.Y, Translation.Z, Rotation.X, Rotation.Y, Rotation.Z, Scale.X, Scale.Y, Scale.Z };
	}
	inline void MultiChannelFromData(const FEulerTransform& In, 	TMultiChannelValue<float, 9>& Out)
	{
		FVector Translation = In.Location;
		FVector Scale = In.Scale;
		FRotator3d Rotation = FRotator3d(In.Rotation);
		Out = { Translation.X, Translation.Y, Translation.Z, Rotation.Roll, Rotation.Pitch, Rotation.Yaw, Scale.X, Scale.Y, Scale.Z };
	}
	inline void MultiChannelFromData(const FLinearColor& In, TMultiChannelValue<float, 4>& Out)
	{
		Out = { In.R, In.G, In.B, In.A };
	}

	inline void ResolveChannelsToData(const TMultiChannelValue<double, 1>& In, 	int32& Out)			{ Out = In[0]; }
	inline void ResolveChannelsToData(const TMultiChannelValue<float, 1>& In, 	float& Out)			{ Out = In[0]; }
	inline void ResolveChannelsToData(const TMultiChannelValue<float, 2>& In, 	FVector2D& Out)		{ Out = FVector2D(In[0], In[1]); }
	inline void ResolveChannelsToData(const TMultiChannelValue<float, 3>& In, 	FVector& Out)		{ Out = FVector(In[0], In[1], In[2]); }
	inline void ResolveChannelsToData(const TMultiChannelValue<float, 4>& In, 	FVector4& Out)		{ Out = FVector4(In[0], In[1], In[2], In[3]); }
	inline void ResolveChannelsToData(const TMultiChannelValue<float, 9>& In, 	FTransform& Out)
	{
		Out = FTransform(
			FRotator::MakeFromEuler(FVector(In[3], In[4], In[5])),
			FVector(In[0], In[1], In[2]),
			FVector(In[6], In[7], In[8])
			);
	}
	inline void ResolveChannelsToData(const TMultiChannelValue<float, 9>& In, 	FEulerTransform& Out)
	{
		Out = FEulerTransform(
			FVector(In[0], In[1], In[2]),
			FRotator::MakeFromEuler(FVector(In[3], In[4], In[5])),
			FVector(In[6], In[7], In[8])
			);
	}
	inline void ResolveChannelsToData(const TMultiChannelValue<float, 4>& In, FLinearColor& Out)
	{
		Out = FLinearColor(In[0], In[1], In[2], In[3]);
	}
} // namespace MovieScene
} // namespace UE

/** Define runtime type identifiers for the built-in C++ data types */
template<> MOVIESCENE_API FMovieSceneAnimTypeID GetBlendingDataType<int32>();
template<> MOVIESCENE_API FMovieSceneAnimTypeID GetBlendingDataType<float>();
template<> MOVIESCENE_API FMovieSceneAnimTypeID GetBlendingDataType<FVector2D>();
template<> MOVIESCENE_API FMovieSceneAnimTypeID GetBlendingDataType<FVector>();
template<> MOVIESCENE_API FMovieSceneAnimTypeID GetBlendingDataType<FVector4>();
template<> MOVIESCENE_API FMovieSceneAnimTypeID GetBlendingDataType<FTransform>();
template<> MOVIESCENE_API FMovieSceneAnimTypeID GetBlendingDataType<FEulerTransform>();
template<> MOVIESCENE_API FMovieSceneAnimTypeID GetBlendingDataType<FLinearColor>();

/** Define working data types for blending calculations */
template<> struct TBlendableTokenTraits<int32> 			{ typedef UE::MovieScene::TMaskedBlendable<double,1> 	WorkingDataType; };
template<> struct TBlendableTokenTraits<float> 			{ typedef UE::MovieScene::TMaskedBlendable<float, 1> 	WorkingDataType; };
template<> struct TBlendableTokenTraits<FVector2D> 		{ typedef UE::MovieScene::TMaskedBlendable<float, 2> 	WorkingDataType; };
template<> struct TBlendableTokenTraits<FVector> 		{ typedef UE::MovieScene::TMaskedBlendable<float, 3> 	WorkingDataType; };
template<> struct TBlendableTokenTraits<FVector4> 		{ typedef UE::MovieScene::TMaskedBlendable<float, 4> 	WorkingDataType; };
template<> struct TBlendableTokenTraits<FTransform> 	{ typedef UE::MovieScene::TMaskedBlendable<float, 9> 	WorkingDataType; };
template<> struct TBlendableTokenTraits<FEulerTransform>{ typedef UE::MovieScene::TMaskedBlendable<float, 9> 	WorkingDataType; };
template<> struct TBlendableTokenTraits<FLinearColor>   { typedef UE::MovieScene::TMaskedBlendable<float, 4>    WorkingDataType; };