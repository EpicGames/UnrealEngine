// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Evaluation/MovieScenePlaybackCapabilities.h"

class UObject;

namespace UE::MovieScene
{

/** Playback capability for controlling how events are triggered */
struct IEventContextsPlaybackCapability
{
	UE_DECLARE_MOVIESCENE_PLAYBACK_CAPABILITY_API(MOVIESCENE_API, IEventContextsPlaybackCapability)

	/** Virtual destructor */
	virtual ~IEventContextsPlaybackCapability() = default;

	/** Get the contexts used for triggering events */
	virtual TArray<UObject*> GetEventContexts() const = 0;
};

}  // namespace UE::MovieScene

