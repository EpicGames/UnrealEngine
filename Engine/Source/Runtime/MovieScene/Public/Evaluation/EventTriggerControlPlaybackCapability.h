// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Evaluation/MovieScenePlaybackCapabilities.h"
#include "Misc/FrameTime.h"
#include "Misc/Optional.h"

namespace UE::MovieScene
{

/** Playback capability for controlling how events are triggered */
struct FEventTriggerControlPlaybackCapability
{
	UE_DECLARE_MOVIESCENE_PLAYBACK_CAPABILITY_API(MOVIESCENE_API, FEventTriggerControlPlaybackCapability)

	/**
	 * Returns whether triggering events should be temporarily disabled.
	 *
	 * @param OutDisabledUntilTime  The time until which to disable triggering events
	 */
	MOVIESCENE_API bool IsDisablingEventTriggers(FFrameTime& OutDisabledUntilTime) const;

	TOptional<FFrameTime> DisableEventTriggersUntilTime;
};

}  // namespace UE::MovieScene

