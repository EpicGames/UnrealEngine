// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameTime.h"
#include "MovieSceneTimeController.h"
#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "IMovieSceneCustomClockSource.generated.h"

class UObject;
struct FFrame;
struct FQualifiedFrameTime;

UINTERFACE(MinimalAPI)
class UMovieSceneCustomClockSource : public UInterface
{
	GENERATED_BODY()
};

/**
 * 
 */
class IMovieSceneCustomClockSource
{
public:

	GENERATED_BODY()

	UFUNCTION()
	virtual void OnTick(float DeltaSeconds, float InPlayRate) {}

	UFUNCTION()
	virtual void OnStartPlaying(const FQualifiedFrameTime& InStartTime) {}

	UFUNCTION()
	virtual void OnStopPlaying(const FQualifiedFrameTime& InStopTime) {}

	UFUNCTION()
	virtual FFrameTime OnRequestCurrentTime(const FQualifiedFrameTime& InCurrentTime, float InPlayRate) { return 0; }
};

struct FMovieSceneTimeController_Custom : FMovieSceneTimeController
{
	MOVIESCENE_API explicit FMovieSceneTimeController_Custom(const FSoftObjectPath& InObjectPath, TWeakObjectPtr<> PlaybackContext);

private:

	MOVIESCENE_API virtual void OnTick(float DeltaSeconds, float InPlayRate) override final;
	MOVIESCENE_API virtual void OnStartPlaying(const FQualifiedFrameTime& InStartTime) override final;
	MOVIESCENE_API virtual void OnStopPlaying(const FQualifiedFrameTime& InStopTime) override final;
	MOVIESCENE_API virtual FFrameTime OnRequestCurrentTime(const FQualifiedFrameTime& InCurrentTime, float InPlayRate) override final;

	void ResolveInterfacePtr();

	TWeakObjectPtr<> WeakPlaybackContext;

	TWeakObjectPtr<> WeakObject;

	IMovieSceneCustomClockSource* InterfacePtr;

	FSoftObjectPath ObjectPath;
};
