// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "MovieSceneSequenceID.h"
#include "Misc/Guid.h"
#include "Evaluation/MovieScenePlayback.h"
#include "IMovieSceneEvaluationHook.generated.h"

class IMovieScenePlayer;
class UMovieSceneEntitySystemLinker;

namespace UE
{
namespace MovieScene
{

struct FSharedPlaybackState;

enum class EEvaluationHookEvent
{
	Begin,
	Update,
	End,

	Trigger,
};

struct FEvaluationHookParams
{
	/** The object binding ID for the hook */
	FGuid ObjectBindingID;

	/** Evaluation context */
	FMovieSceneContext Context;

	/** The sequence ID for the hook */
	FMovieSceneSequenceID SequenceID = MovieSceneSequenceID::Root;

	int32 TriggerIndex = INDEX_NONE;
};


} // namespace MovieScene
} // namespace UE


UINTERFACE(MinimalAPI)
class UMovieSceneEvaluationHook : public UInterface
{
public:
	GENERATED_BODY()
};


/**
 * All evaluation hooks are executed at the end of the frame (at a time when re-entrancy is permitted), and cannot have any component dependencies
 */
class IMovieSceneEvaluationHook
{
public:

	using FSharedPlaybackState = UE::MovieScene::FSharedPlaybackState;

	GENERATED_BODY()

	MOVIESCENE_API virtual void Begin(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const;
	MOVIESCENE_API virtual void Update(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const;
	MOVIESCENE_API virtual void End(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const;

	MOVIESCENE_API virtual void Trigger(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const;

protected:

	UE_DEPRECATED(5.5, "Please implement the version that takes a SharedPlaybackState")
	virtual void Begin(IMovieScenePlayer* Player, const UE::MovieScene::FEvaluationHookParams& Params) const {}
	UE_DEPRECATED(5.5, "Please implement the version that takes a SharedPlaybackState")
	virtual void Update(IMovieScenePlayer* Player, const UE::MovieScene::FEvaluationHookParams& Params) const {}
	UE_DEPRECATED(5.5, "Please implement the version that takes a SharedPlaybackState")
	virtual void End(IMovieScenePlayer* Player, const UE::MovieScene::FEvaluationHookParams& Params) const {}

	UE_DEPRECATED(5.5, "Please implement the version that takes a SharedPlaybackState")
	virtual void Trigger(IMovieScenePlayer* Player, const UE::MovieScene::FEvaluationHookParams& Params) const {}
};

