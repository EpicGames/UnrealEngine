// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/SharedPointer.h"

class IMovieScenePlayer;
class UMovieSceneEntitySystemLinker;

namespace UE::MovieScene
{

struct FInstanceHandle;
struct FSharedPlaybackState;

/**
 * An identifier for a playback capability.
 */
struct FPlaybackCapabilityID
{
	int32 Index = INDEX_NONE;

	bool IsValid() const
	{
		return Index != INDEX_NONE;
	}

protected:
	static MOVIESCENE_API FPlaybackCapabilityID Register(const TCHAR* InDebugName);
};

/**
 * A strongly-typed identifier for a specific playback capability class.
 *
 * The base capablity class must create a static ID member that returns its own typed ID. We will use
 * this as a convention to quickly get the ID of a base capability class in the capabilities container.
 */
template<typename T>
struct TPlaybackCapabilityID : FPlaybackCapabilityID
{
public:
	using CapabilityType = T;

private:
	// Only T should construct this (to ensure safe construction over DLL boundaries)
	friend T;

	TPlaybackCapabilityID()
	{}

	explicit TPlaybackCapabilityID(int32 InIndex)
		: FPlaybackCapabilityID{ InIndex }
	{}

	UE_DEPRECATED(5.6, "Use the version that takes a debug name. If defining a static ID field, please upgrade to UE_DECLARE_MOVIESCENE_PLAYBACK_CAPABILITY_API and UE_DEFINE_MOVIESCENE_PLAYBACK_CAPABILITY instead.")
	static TPlaybackCapabilityID<T> Register()
	{
		return TPlaybackCapabilityID<T>::Register(TEXT("Unknown"));
	}

	static TPlaybackCapabilityID<T> Register(const TCHAR* InDebugName)
	{
		FPlaybackCapabilityID StaticID = FPlaybackCapabilityID::Register(InDebugName);
		return TPlaybackCapabilityID<T>(StaticID.Index);
	}
};

/**
 * Interface for playback capabilities that want to be notified of various operations.
 */
struct IPlaybackCapability
{
	virtual ~IPlaybackCapability() = default;

	/** Called after this capability has been added to a shared playback state */
	virtual void Initialize(TSharedRef<const FSharedPlaybackState> Owner) {}
	/** Called when a new sequence instance has been created and added to the sequence hierarchy */
	virtual void OnSubInstanceCreated(TSharedRef<const FSharedPlaybackState> Owner, const FInstanceHandle InstanceHandle) {}
	/** Called when the root sequence is cleaning cached data */
	virtual void InvalidateCachedData(UMovieSceneEntitySystemLinker* Linker) {}
};

/**
 * Structure providing basic information on a playback capability type.
 */
struct FPlaybackCapabilityIDInfo
{
#if UE_MOVIESCENE_ENTITY_DEBUG
	/** Display name for debugging. */
	FString DebugName;
#endif
};

/**
 * A registry for all known playback capability types.
 */
class FPlaybackCapabilityIDRegistry
{
public:

	/** Gets the registry. */
	static MOVIESCENE_API FPlaybackCapabilityIDRegistry* Get();

	/** Registers a new playback capability type with the given display name. */
	MOVIESCENE_API FPlaybackCapabilityID RegisterNewID(const TCHAR* InDebugName);

private:

	TArray<FPlaybackCapabilityIDInfo> Infos;
};

extern MOVIESCENE_API FPlaybackCapabilityIDRegistry* GPlaybackCapabilityIDRegistryForDebuggingVisualizers;

#if UE_MOVIESCENE_ENTITY_DEBUG

/**
 * Interface to a playback capability pointer that has a valid v-table for the debugger
 * to be able to show what it is.
 */
struct IPlaybackCapabilityDebuggingTypedPtr
{
	virtual ~IPlaybackCapabilityDebuggingTypedPtr() = default;
	void* Ptr = nullptr;
};

/**
 * Actual typed version of the pointer wrapper above, for debugging.
 */
template<typename T>
struct TPlaybackCapabilityDebuggingTypedPtr : IPlaybackCapabilityDebuggingTypedPtr
{
	TPlaybackCapabilityDebuggingTypedPtr(T* InPtr)
	{
		static_assert(sizeof(TPlaybackCapabilityDebuggingTypedPtr) == sizeof(IPlaybackCapabilityDebuggingTypedPtr), "Size must match");
		Ptr = InPtr;
	}
};

#endif

#define UE_DECLARE_MOVIESCENE_PLAYBACK_CAPABILITY_API(ApiDeclSpec, ClassName)\
	static ApiDeclSpec UE::MovieScene::TPlaybackCapabilityID<ClassName> GetPlaybackCapabilityID();\

#define UE_DECLARE_MOVIESCENE_PLAYBACK_CAPABILITY(ClassName)\
	static UE::MovieScene::TPlaybackCapabilityID<ClassName> GetPlaybackCapabilityID();\

#define UE_DEFINE_MOVIESCENE_PLAYBACK_CAPABILITY(ClassName)\
	UE::MovieScene::TPlaybackCapabilityID<ClassName> ClassName::GetPlaybackCapabilityID()\
	{\
		static UE::MovieScene::TPlaybackCapabilityID<ClassName> StaticID = \
			UE::MovieScene::TPlaybackCapabilityID<ClassName>::Register(TEXT(#ClassName));\
		return StaticID;\
	}

} // namespace UE::MovieScene

