// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TransactionallySafeMutex.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Misc/ScopeLock.h"
#include "Templates/TypeHash.h"

class FString;
struct FMovieSceneAnimTypeID;

/**
 * Animation type ID that uniquely identifies the type of a change made
 * (ie changing a transform, spawning an object, etc) as part of a sequence evaluation
 */
struct FMovieSceneAnimTypeID
{
	friend bool operator==(FMovieSceneAnimTypeID A, FMovieSceneAnimTypeID B)
	{
		return A.ID == B.ID;
	}

	friend bool operator!=(FMovieSceneAnimTypeID A, FMovieSceneAnimTypeID B)
	{
		return A.ID != B.ID;
	}

	friend uint32 GetTypeHash(FMovieSceneAnimTypeID In)
	{
		return GetTypeHash(In.ID);
	}

	MOVIESCENE_API static FMovieSceneAnimTypeID Unique();

	MOVIESCENE_API static FMovieSceneAnimTypeID Combine(FMovieSceneAnimTypeID A, FMovieSceneAnimTypeID B);

protected:

	friend MOVIESCENE_API FString LexToString(const FMovieSceneAnimTypeID&);

	FMovieSceneAnimTypeID(uint64* StaticPtr, uint32 Seed = 0)
		: ID(*StaticPtr ? *StaticPtr : Initialize(StaticPtr, Seed))
	{}
	FMovieSceneAnimTypeID()
		: ID(0)
	{}

	/**
	 * Initialize this structure by storing the hash result in the static ptr itself
	 * This ensures subsequent constructions with the same pointer are fast
	 */
	MOVIESCENE_API static uint64 Initialize(uint64* StaticPtr, uint32 Seed);

	/**
	 * Generate a hash from the specified static ptr
	 */
	MOVIESCENE_API static uint64 GenerateHash(void* StaticPtr, uint32 Seed);

	uint64 ID;
};

/**
 * Templated class that initializes a unique ID for the templated type (normally an execution token)
 * Care should be taken here not to expose the type in any way across a DLL boundary, as this will break
 * the uniqueness of the identifier. If it's necessary to expose the ID across DLL boundaries, use the following pattern:
 * 
 * 	MyAnimationToken.h:
 * 	
 * 	struct FMyAnimationToken : IMovieSceneExecutionToken
 * 	{
 * 		MYMODULE_API static FMovieSceneAnimTypeID GetTypeID();
 * 	};
 * 	
 * 	MyAnimationToken.cpp:
 * 	
 * 	FMovieSceneAnimTypeID FMyAnimationToken::GetTypeID()
 * 	{
 * 		return TMovieSceneAnimTypeID<FMyAnimationToken>();
 * 	}
 */
template<typename T, uint8 Seed = 0>
struct TMovieSceneAnimTypeID : FMovieSceneAnimTypeID
{
private:
	// Only T should construct this (to ensure safe construction over DLL boundaries)
	friend T;

	TMovieSceneAnimTypeID()
		: FMovieSceneAnimTypeID(&CachedID, Seed)
	{
	}
	static uint64 CachedID;
};
template<typename T, uint8 Seed> uint64 TMovieSceneAnimTypeID<T, Seed>::CachedID = 0;


/** Anim type ID container that uniquely identifies types of animated data based on a predicate data structure */
template<typename DataType>
struct TMovieSceneAnimTypeIDContainer
{
	/** Get the type unique animation type identifier for the specified predicate */
	FMovieSceneAnimTypeID GetAnimTypeID(const DataType& InPredicate)
	{
		UE::TScopeLock ScopeLock(Mutex);

		int32 Index = Data.IndexOfByKey(InPredicate);
		if (Index != INDEX_NONE)
		{
			FMovieSceneAnimTypeID Value = TypeIDs[Index];
			return Value;
		}

		Data.Add(InPredicate);

		FTypeID NewID(this, TypeIDs.Num());
		TypeIDs.Add(NewID);

		return NewID;
	}

private:

	struct FTypeID : FMovieSceneAnimTypeID
	{
		FTypeID(void* Base, uint32 InIndex)
		{
			ID = GenerateHash(Base, InIndex);
		}
	};

	UE::FTransactionallySafeMutex Mutex;

	/** Array of existing type identifiers */
	TArray<FMovieSceneAnimTypeID> TypeIDs;
	
	/** Array of data type predicates whose indices map to TypeIDs */
	TArray<DataType> Data;
};
