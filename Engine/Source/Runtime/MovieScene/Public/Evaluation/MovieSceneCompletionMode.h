// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneCompletionMode.generated.h"

/** Enumeration specifying how to handle state when this section is no longer evaluated */
UENUM(BlueprintType)
enum class EMovieSceneCompletionMode : uint8
{
	KeepState,

	RestoreState,

	ProjectDefault,
};