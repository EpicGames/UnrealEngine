// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Evaluation/MovieSceneCompletionMode.h"
#include "Evaluation/MovieSceneEvalTemplateBase.h"
#include "Evaluation/MovieSceneExecutionTokens.h"
#include "Evaluation/MovieScenePlayback.h"
#include "Evaluation/PersistentEvaluationData.h"
#include "Math/Range.h"
#include "Misc/AssertionMacros.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameTime.h"
#include "Misc/InlineValue.h"
#include "MovieSceneSection.h"
#include "Templates/Decay.h"
#include "Templates/EnableIf.h"
#include "Templates/PointerIsConvertibleFromTo.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "MovieSceneEvalTemplate.generated.h"

class FArchive;
class IMovieScenePlayer;
class UObject;
struct FMovieSceneEvaluationOperand;
struct FMovieSceneExecutionTokens;
struct FMovieSceneInterrogationData;

/**
 * Structure used for movie scene evaluation templates contained within a track. Typically these are defined as one per-section.
 * Serialized into a FMovieSceneEvaluationTemplate contained within the sequence itself (for fast initialization at runtime).
 * Templates are executed in a 3-phase algorithm:
 *		1) Initialize: (opt-in) Called at the start of the frame. Able to access mutable state from the playback context. Used to initialize any persistent state required for the evaluation pass.
 *		2) Evaluate: Potentially called on a thread. Should (where possible) perform all costly evaluation logic, accumulating into execution tokens which will be executed at a later time on the game thread.
 *		3) Execute: Called on all previously submitted execution tokens to apply the evaluated state to the movie scene player
 */
USTRUCT()
struct FMovieSceneEvalTemplate : public FMovieSceneEvalTemplateBase
{
public:
	GENERATED_BODY()

	/**
	 * Default Constructor
	 */
	FMovieSceneEvalTemplate()
	{
		CompletionMode = EMovieSceneCompletionMode::KeepState;
	}

	/**
	 * Check whether this template mandates Initialize being called.
	 * Defines whether a pointer to this track will be added to the initialization section of template evaluation.
	 * @return Boolean representing whether this template mandates Initialize being called
	 */
	bool RequiresInitialization() const
	{
		return (OverrideMask & RequiresInitializeFlag) != 0;
	}

	/**
	 * Check whether we should restore any pre-animated state that was supplied by this template when it is no longer evaluated
	 * @note					Pre-animated state bound to evaluation templates is reference counted across all similar animation types for a given object.
	 *							This ensures that pre-animated state restores correctly for overlapping templates.
	 */
	EMovieSceneCompletionMode GetCompletionMode() const
	{
		return CompletionMode;
	}

	/**
	 * Set this template's completion mode
	 * @note					Pre-animated state bound to evaluation templates is reference counted across all similar animation types for a given object.
	 *							This ensures that pre-animated state restores correctly for overlapping templates.
	 */
	void SetCompletionMode(EMovieSceneCompletionMode InCompletionMode)
	{
		CompletionMode = InCompletionMode;
	}

	/**
	 * Initialize this template, copying any data required for evaluation into the specified state block.
	 * @note					This function is intended to allow pre-frame set up, and should avoid mutating any state.
	 *							Only called if EnableOverrides(RequiresInitializeFlag) has been called.
	 *
	 * @param Operand			Unique handle to the operand on which we are to operate. May represent multiple objects. Resolve through IMovieScenePlayer::FindBoundObjects(Operand)
	 * @param Context			Evaluation context specifying the current evaluation time, sub sequence transform and other relevant information.
	 * @param PersistentData	Persistent data store which can be used to store arbitrary data pertaining to the current template that may be required in Evaluate(Swept)
	 * @param Player			The movie scene player currently playing back this sequence
	 */
	virtual void Initialize(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) const
	{
		ensureMsgf(false, TEXT("FMovieSceneEvalTemplate::Initialize has not been implemented. Verify EnableOverrides() usage is correct or implement this function."));
	}

	/**
	 * Evaluate this template, adding any execution tokens to the specified list
	 * @note					Only called when the containing template has an evaluation method of EEvaluationMethod::Static
	 *							This function should perform any expensive or costly evaluation logic required to calculate the final animated state.
	 *							Potentially called on a thread, and as such has no access to the current evaluation environment.
	 *
	 * @param Operand			Unique handle to the operand on which we are to operate. Only to be used as a reference, or forwarded throgh to an execution token.
	 * @param Context			Evaluation context specifying the current evaluation time, sub sequence transform and other relevant information.
	 * @param PersistentData	Persistent data store which can be used to access arbitrary data pertaining to the current template that should have been set up in initialize.
	 * @param ExecutionTokens	Stack of execution tokens that will be used to apply animated state to the environment at a later time.
	 */
	virtual void Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const
	{
		ensureMsgf(false, TEXT("FMovieSceneEvalTemplate::Evaluate has not been implemented. Verify that this template's evaluation track has correct evaluation method (usually set in UMovieSceneTrack::PostCompile), or implement this function."));
	}

	/**
	 * Evaluate this template over the given swept range, adding any execution tokens to the specified list.
	 * @note 					Only called when the containing template has an evaluation method of EEvaluationMethod::Swept
	 *							This function should perform any expensive or costly evaluation logic required to calculate the final animated state.
	 * 							Potentially called on a thread, and as such has no access to the current evaluation environment.
	 *
	 * @param Operand			Unique handle to the operand on which we are to operate. Only to be used as a reference, or forwarded throgh to an execution token.
	 * @param Context			Evaluation context specifying the current evaluation time, sub sequence transform and other relevant information.
	 * @param SweptRange		The range this is to be swept in this evaluation - always fully contained by the context's range
	 * @param PersistentData	Persistent data store which can be used to access arbitrary data pertaining to the current template that should have been set up in initialize.
	 * @param ExecutionTokens	Stack of execution tokens that will be used to apply animated state to the environment at a later time.
	*/
	virtual void EvaluateSwept(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const TRange<FFrameNumber>& SweptRange, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const
	{
		ensureMsgf(false, TEXT("FMovieSceneEvalTemplate::EvaluateSwept has not been implemented. Verify that this template's evaluation track has correct evaluation method (usually set in UMovieSceneTrack::PostCompile), or implement this function."));
	}

	/**
	 * Interrogate this template for its output. Should not have any side effects.
	 *
	 * @param Context				Evaluation context specifying the current evaluation time, sub sequence transform and other relevant information.
	 * @param Container				Container to populate with the desired output from this track
	 * @param BindingOverride		Optional binding to specify the object that is being animated by this track
	 */
	virtual void Interrogate(const FMovieSceneContext& Context, FMovieSceneInterrogationData& Container, UObject* BindingOverride) const
	{
	}

	/**
	 * Interrogate this template for its output. Should not have any side effects.
	 *
	 * @param Context				Evaluation context specifying the current evaluation time, sub sequence transform and other relevant information.
	 * @param SweptRange			The range to sweep, where this template evaluates with 'swept' evaluation
	 * @param Container				Container to populate with the desired output from this track
	 * @param BindingOverride		Optional binding to specify the object that is being animated by this track
	 */
	virtual void Interrogate(const FMovieSceneContext& Context, TRange<FFrameNumber> SweptRange, FMovieSceneInterrogationData& Container, UObject* BindingOverride) const
	{
	}

public:

	/**
	 * Set the source section from which this template originated
	 *
	 * @param SourceSection 		The source section
	 */
	void SetSourceSection(const UMovieSceneSection* InSourceSection)
	{
		SourceSectionPtr = InSourceSection;
	}

	/**
	 * Get the source section from which this template originated
	 *
	 * @return The source section from which this template originated
	 */
	const UMovieSceneSection* GetSourceSection() const
	{
		return SourceSectionPtr.Get();
	}

protected:

	/**
	 * Evaluate this template's easing functions based on the specified time
	 */
	MOVIESCENE_API float EvaluateEasing(FFrameTime CurrentTime) const;

	/**
	 * Enum evaluation flag structure defining which functions are to be called in implementations of this struct
	 */
	enum EOverrideMask
	{
		RequiresInitializeFlag		= 0x004,
	};

	/** Enumeration value signifying whether we should restore any animated state stored by this entity when this eval tempalte is no longer evaluated */
	UPROPERTY()
	EMovieSceneCompletionMode CompletionMode;

	/** The section from which this template originates */
	UPROPERTY()
	TWeakObjectPtr<const UMovieSceneSection> SourceSectionPtr;
};

/**
 * Custom serialized type that allows serializing structs derived from FMovieSceneEvalTemplate, and attempts to store an evaluation template in inline memory if possible
 */
USTRUCT()
struct FMovieSceneEvalTemplatePtr
#if CPP
	: TInlineValue<FMovieSceneEvalTemplate, 128>
#endif
{
	GENERATED_BODY()

	/** Default construction to an empty container */
	FMovieSceneEvalTemplatePtr(){}

	/** Construction from any FMovieSceneEvalTemplate derivative */
	template<
		typename T,
		typename = typename TEnableIf<TPointerIsConvertibleFromTo<typename TDecay<T>::Type, FMovieSceneEvalTemplate>::Value>::Type
	>
	FMovieSceneEvalTemplatePtr(T&& In)
		: TInlineValue(Forward<T>(In))
	{
		static_assert(!std::is_same_v<typename TDecay<T>::Type, FMovieSceneEvalTemplate>, "Direct usage of FMovieSceneEvalTemplate is prohibited.");
		
#if WITH_EDITOR
		checkf(T::StaticStruct() == &In.GetScriptStruct() && T::StaticStruct() != FMovieSceneEvalTemplate::StaticStruct(), TEXT("%s does not correctly override GetScriptStructImpl. Template will not serialize correctly."), *T::StaticStruct()->GetName());
#endif
	}

	/** Copy construction/assignment */
	FMovieSceneEvalTemplatePtr(const FMovieSceneEvalTemplatePtr& RHS)
	{
		*this = RHS;
	}
	FMovieSceneEvalTemplatePtr& operator=(const FMovieSceneEvalTemplatePtr& RHS)
	{
		if (RHS.IsValid())
		{
			UScriptStruct::ICppStructOps& StructOps = *RHS->GetScriptStruct().GetCppStructOps();

			void* Allocation = Reserve(StructOps.GetSize(), StructOps.GetAlignment());
			StructOps.Construct(Allocation);
			StructOps.Copy(Allocation, &RHS.GetValue(), 1);
		}
		else
		{
			Reset();
		}

		return *this;
	}

	/** Templates are moveable */
	FMovieSceneEvalTemplatePtr(FMovieSceneEvalTemplatePtr&&) = default;
	FMovieSceneEvalTemplatePtr& operator=(FMovieSceneEvalTemplatePtr&&) = default;

	/** Serialize the template */
	MOVIESCENE_API bool Serialize(FArchive& Ar);
};

template<> struct TStructOpsTypeTraits<FMovieSceneEvalTemplatePtr> : public TStructOpsTypeTraitsBase2<FMovieSceneEvalTemplatePtr>
{
	enum { WithSerializer = true, WithCopy = true };
};
