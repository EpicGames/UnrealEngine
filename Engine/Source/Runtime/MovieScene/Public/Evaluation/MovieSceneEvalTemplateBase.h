// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Evaluation/MovieScenePlayback.h"
#include "MovieSceneEvalTemplateBase.generated.h"


struct FPersistentEvaluationData;
class IMovieScenePlayer;


/** Empty struct used for serialization */
USTRUCT()
struct FMovieSceneEmptyStruct
{
	GENERATED_BODY()
};

/**
 * Base structure used for all movie scene evaluation templates
 */
USTRUCT()
struct FMovieSceneEvalTemplateBase
{
public:

	GENERATED_BODY()

	/**
	 * Default constructor
	 */
	FMovieSceneEvalTemplateBase()
		: OverrideMask(0)
	{
	}

	/**
	 * Virtual destruction
	 */
	virtual ~FMovieSceneEvalTemplateBase() = default;

	/**
	 * Access the most derived script struct type of this instance for serialization purposes
	 */
	UScriptStruct& GetScriptStruct() const { return GetScriptStructImpl(); }

	/**
	 * Check whether this entity requires set up when it is first evaluated
	 */
	inline bool RequiresSetup() const { return (OverrideMask & RequiresSetupFlag) != 0; }

	/**
	 * Check whether this entity requires tear up when it no longer being evaluated
	 */
	inline bool RequiresTearDown() const { return (OverrideMask & RequiresTearDownFlag) != 0; }

	/**
	 * Called before this template is evaluated for the first time, or since OnEndEvaluation has been called
	 *
	 * @param PersistentData		Persistent data proxy that may contain data pertaining to this entity
	 * @param Player				The player that is responsible for playing back this template
	 */
	inline void OnBeginEvaluation(FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) const
	{
		if (RequiresSetup())
		{
			Setup(PersistentData, Player);
		}
	}

	/**
	 * Called after this template is no longer being evaluated
	 *
	 * @param PersistentData		Persistent data proxy that may contain data pertaining to this entity
	 * @param Player				The player that is responsible for playing back this template
	 */
	inline void OnEndEvaluation(FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) const
	{
		if (RequiresTearDown())
		{
			TearDown(PersistentData, Player);
		}
	}

	/**
	 * Called after construction to enable overridden functions required by this type.
	 * @note 		Should call EnableOverrides() in derived structs with the appropriate flag mask.
	 * 				This is implemented as a virtual function to ensure consistency between serialized data and code.
	 *				Overriden function flags are not serialized to allow for future changes without breaking serialized data.
	 */
	virtual void SetupOverrides()
	{
	}

protected:

	/**
	 * Called before this template is evaluated for the first time, or since OnEndEvaluation has been called. Implement in derived classes.
	 * @note 						Only called when a derived type sets EnableOverrides(RequiresSetupFlag)
	 *
	 * @param PersistentData		Persistent data proxy that may contain data pertaining to this entity
	 * @param Player				The player that is responsible for playing back this template
	 */
	virtual void Setup(FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) const
	{
		ensureMsgf(false, TEXT("FMovieSceneEvalTemplateBase::Setup has not been implemented. Did you erroneously call EnableOverrides(RequiresSetupFlag)?"));
	}

	/**
	 * Called after this template is no longer being evaluated. Implement in derived classes.
	 * @note 						Only called when a derived type sets EnableOverrides(RequiresTearDownFlag)
	 *
	 * @param PersistentData		Persistent data proxy that may contain data pertaining to this entity
	 * @param Player				The player that is responsible for playing back this template
	 */
	virtual void TearDown(FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) const
	{
		ensureMsgf(false, TEXT("FMovieSceneEvalTemplateBase::TearDown has not been implemented. Did you erroneously call EnableOverrides(RequiresTearDownFlag)?"));
	}
	
	/**
	 * Retrieve the script struct pertaining to the most-derived type of this instance. Must be implemented in all derived classes for serialization to work correctly.
	 */
	virtual UScriptStruct& GetScriptStructImpl() const
	{
		ensureMsgf(false, TEXT("GetScriptStructImpl has not been implemented. This type will not serialize correctly."));
		return *StaticStruct();
	}

protected:

	/**
	 * Enable the overrides referred to by the specified flag mask
	 */
	void EnableOverrides(uint8 OverrideFlag)
	{
		OverrideMask |= OverrideFlag;
	}

	/** Base class flag mask that should be considered by any implementations of FMovieSceneEvalTemplateBase */
	enum EOverrideMask
	{
		RequiresSetupFlag		= 0x001,
		RequiresTearDownFlag	= 0x002,
	};

	/** Mask of overridden properties - not serialized, but setup in SetupOverrides */
	uint8 OverrideMask;
};
