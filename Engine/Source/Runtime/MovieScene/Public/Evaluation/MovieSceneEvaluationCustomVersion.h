// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"

// Custom version for movie scene evaluation serialization
namespace FMovieSceneEvaluationCustomVersion
{
	enum Type
	{
		Initial = 0,
		// Changed EvalOptions.bEvaluateNearestSection to be false by default
		ChangeEvaluateNearestSectionDefault,

		// Added support for blending
		AddBlendingSupport,

		// Added ECS evaluation
		EntityManager,

		// Removed GUID references to blueprint graphs and replaced with direct weak ptrs
		DeprecateEventGUIDs,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version
	const static FGuid GUID = FGuid(0xA462B7EA, 0xF4994E3A, 0x99C1EC1F, 0x8224E1B2);
}
