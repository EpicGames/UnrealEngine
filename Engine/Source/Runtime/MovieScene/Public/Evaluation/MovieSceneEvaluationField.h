// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "Evaluation/MovieSceneEvaluationKey.h"
#include "Evaluation/MovieSceneEvaluationTree.h"
#include "Evaluation/MovieSceneSegment.h"
#include "Evaluation/MovieSceneTrackIdentifier.h"
#include "Math/NumericLimits.h"
#include "Math/Range.h"
#include "Misc/FrameNumber.h"
#include "Misc/Guid.h"
#include "MovieSceneFrameMigration.h"
#include "MovieSceneSequenceID.h"
#include "Serialization/Archive.h"
#include "Templates/Function.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "MovieSceneEvaluationField.generated.h"

class UMovieSceneSequence;
class UObject;
struct FFrameNumber;
struct FMovieSceneSequenceHierarchy;
struct IMovieSceneSequenceTemplateStore;
class UMovieSceneCondition;


/**
 * A key that uniquely identifies an entity by its owner and ID
 */
USTRUCT()
struct FMovieSceneEvaluationFieldEntityKey
{
	GENERATED_BODY()

	friend bool operator==(FMovieSceneEvaluationFieldEntityKey A, FMovieSceneEvaluationFieldEntityKey B)
	{
		return A.EntityOwner == B.EntityOwner && A.EntityID == B.EntityID;
	}
	friend bool operator!=(FMovieSceneEvaluationFieldEntityKey A, FMovieSceneEvaluationFieldEntityKey B)
	{
		return !(A == B);
	}
	friend uint32 GetTypeHash(FMovieSceneEvaluationFieldEntityKey In)
	{
		return GetTypeHash(In.EntityOwner) ^ In.EntityID;
	}

	/** The entity owner - either a UMovieSceneSection or perhaps a UMovieSceneTrack. Must implement the IMovieSceneEntityProvider interface */
	UPROPERTY()
	TWeakObjectPtr<UObject> EntityOwner;

	/** The identifer for the entity within the owner. Normally this will be irrelevant (and 0), but may be used to identifier unique entities by index, or flags */
	UPROPERTY()
	uint32 EntityID = 0;
};



USTRUCT()
struct FMovieSceneEvaluationFieldEntity
{
public:

	GENERATED_BODY()

	FMovieSceneEvaluationFieldEntity()
		: SharedMetaDataIndex(INDEX_NONE)
	{}

	FMovieSceneEvaluationFieldEntity(const FMovieSceneEvaluationFieldEntityKey& InKey, int32 InSharedMetaDataIndex = INDEX_NONE)
		: Key(InKey)
		, SharedMetaDataIndex(InSharedMetaDataIndex)
	{}

	UPROPERTY()
	FMovieSceneEvaluationFieldEntityKey Key;

	UPROPERTY()
	int32 SharedMetaDataIndex;
};

USTRUCT()
struct FMovieSceneEvaluationFieldSharedEntityMetaData
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid ObjectBindingID;
};

USTRUCT()
struct FMovieSceneEvaluationFieldEntityMetaData
{
	GENERATED_BODY()

	FMovieSceneEvaluationFieldEntityMetaData()
		: ForcedTime(TNumericLimits<int32>::Lowest())
		, Flags(ESectionEvaluationFlags::None)
		, bEvaluateInSequencePreRoll(false)
		, bEvaluateInSequencePostRoll(false)
	{}

	bool IsRedundant() const
	{
		return *this == FMovieSceneEvaluationFieldEntityMetaData();
	}

	friend bool operator==(const FMovieSceneEvaluationFieldEntityMetaData& A, const FMovieSceneEvaluationFieldEntityMetaData& B)
	{
		return A.ForcedTime == B.ForcedTime && 
			A.Flags == B.Flags && 
			A.bEvaluateInSequencePreRoll == B.bEvaluateInSequencePreRoll && 
			A.bEvaluateInSequencePostRoll == B.bEvaluateInSequencePostRoll &&
			A.OverrideBoundPropertyPath == B.OverrideBoundPropertyPath &&
			A.NotifyFunctionName == B.NotifyFunctionName &&
			A.Condition == B.Condition;
	}

	/* Optional condition that the entity can provide to be checked at runtime before import. If the condition fails, the entity will not be imported. 
	*/
	UPROPERTY()
	TSoftObjectPtr<const UMovieSceneCondition> Condition;

	/** Opt-in - when this value is set, the entity should use this property path instead of the one defined on its generating section */
	UPROPERTY()
	FString OverrideBoundPropertyPath;

	/** Opt-in - when this value is set, the entity should call the specified function on the bound object (if it exists) */
	UPROPERTY()
	FName NotifyFunctionName;

	/** Opt-in - when this value is not MIN_int32, the entity should use this time to evaluate */
	UPROPERTY()
	FFrameNumber ForcedTime;

	UPROPERTY()
	ESectionEvaluationFlags Flags;

	/** Opt-in - when no meta-data is present, or this value is false, this entity cannot be evaluated as part of sub-sequence preroll */
	UPROPERTY()
	uint8 bEvaluateInSequencePreRoll : 1;

	/** Opt-in - when no meta-data is present, or this value is false, this entity cannot be evaluated as part of sub-sequence postroll */
	UPROPERTY()
	uint8 bEvaluateInSequencePostRoll : 1;
};

struct FMovieSceneEvaluationFieldEntityQuery
{
	FMovieSceneEvaluationFieldEntity Entity;
	int32 MetaDataIndex;
};

struct FMovieSceneEvaluationFieldEntityKeyFuncs : BaseKeyFuncs<FMovieSceneEvaluationFieldEntityQuery,FMovieSceneEvaluationFieldEntityKey,false>
{
	static inline FMovieSceneEvaluationFieldEntityKey GetSetKey(const FMovieSceneEvaluationFieldEntityQuery& Element)        { return Element.Entity.Key; }
	static inline bool Matches(const FMovieSceneEvaluationFieldEntityKey& A, const FMovieSceneEvaluationFieldEntityKey& B)   { return A == B;      }
	static inline uint32 GetKeyHash(const FMovieSceneEvaluationFieldEntityKey& Key)                                          { return GetTypeHash(Key); }
};
using FMovieSceneEvaluationFieldEntitySet = TSet<FMovieSceneEvaluationFieldEntityQuery, FMovieSceneEvaluationFieldEntityKeyFuncs>;

USTRUCT()
struct FMovieSceneEvaluationFieldEntityTree
{
	GENERATED_BODY()

	bool Serialize(FArchive& Ar)
	{
		Ar << SerializedData;
		return true;
	}

	bool Identical(const FMovieSceneEvaluationFieldEntityTree* Other, uint32 PortFlags) const
	{
		return Other->SerializedData == SerializedData;
	}

	struct FEntityAndMetaDataIndex
	{
		int32 EntityIndex;
		int32 MetaDataIndex;

		friend FArchive& operator<<(FArchive& Ar, FEntityAndMetaDataIndex& In)
		{
			return Ar << In.EntityIndex << In.MetaDataIndex;
		}
		friend bool operator==(const FEntityAndMetaDataIndex& A, const FEntityAndMetaDataIndex& B)
		{
			return A.EntityIndex == B.EntityIndex && A.MetaDataIndex == B.MetaDataIndex;
		}
	};
	TMovieSceneEvaluationTree<FEntityAndMetaDataIndex> SerializedData;
};
template<> struct TStructOpsTypeTraits<FMovieSceneEvaluationFieldEntityTree> : public TStructOpsTypeTraitsBase2<FMovieSceneEvaluationFieldEntityTree>
{
	enum { WithSerializer = true, WithIdentical = true, WithCopy = false };
};



/**
 * High-level container which acts as a look-up-table for IMovieSceneEntityProviders and their entities and meta-data in a sequence
 *
 * Generally generated and accessed through UMovieSceneCompiledDataManager, but can also be used independently.
 * Entity fields are constructed using FMovieSceneEntityComponentFieldBuilder which ensures the invariants of this class are 
 * maintained along with guaranteeing no redundant entries exist.
 */
USTRUCT()
struct FMovieSceneEntityComponentField
{
	GENERATED_BODY()

	/**
	 * Check if this field is empty
	 */
	bool IsEmpty() const
	{
		return Entities.Num() == 0;
	}

	/**
	 * Retrieve an entity from its index
	 */
	const FMovieSceneEvaluationFieldEntity& GetEntity(int32 EntityIndex) const
	{
		return Entities[EntityIndex];
	}

	/**
	 * Retrieve the meta-data for an entity
	 * @return The meta-data for an entity or nullptr if it has none associated with it
	 */
	const FMovieSceneEvaluationFieldEntityMetaData* FindMetaData(const FMovieSceneEvaluationFieldEntityQuery& InQuery) const
	{
		return InQuery.MetaDataIndex != INDEX_NONE ? &EntityMetaData[InQuery.MetaDataIndex] : nullptr;
	}

	/**
	 * Retrieve the shared meta-data for an entity
	 * @return The shared meta-data for an entity or nullptr if it has none associated with it
	 */
	const FMovieSceneEvaluationFieldSharedEntityMetaData* FindSharedMetaData(const FMovieSceneEvaluationFieldEntityQuery& InQuery) const
	{
		return InQuery.Entity.SharedMetaDataIndex != INDEX_NONE ? &SharedMetaData[InQuery.Entity.SharedMetaDataIndex] : nullptr;
	}

	/**
	 * Query the persistent entities for any given time within a sequence.
	 * @note: Persistent entities should remain alive until they are no longer present at the current time.
	 *
	 * @param QueryTime   The time at which to query the field (in the TickResolution of the sequence this was generated from)
	 * @param OutRange    Will receive the hull of the range that was intersected for which the resulting OutEntities remains constant
	 * @param OutEntities A set that will be populated with all the entities that exist at the specified time
	 */
	MOVIESCENE_API void QueryPersistentEntities(FFrameNumber QueryTime, TRange<FFrameNumber>& OutRange, FMovieSceneEvaluationFieldEntitySet& OutEntities) const;

	/**
	 * Query the persistent entities for any given time within a sequence.
	 * @note: Persistent entities should remain alive until they are no longer present at the current time.
	 *
	 * @param QueryTime     The time at which to query the field (in the TickResolution of the sequence this was generated from)
	 * @param QueryCallback A handler for dealing with the resulting entities
	 * @param OutRange      Will receive the hull of the range that was intersected for which the resulting OutEntities remains constant
	 */
	MOVIESCENE_API void QueryPersistentEntities(FFrameNumber QueryTime, TFunctionRef<bool(const FMovieSceneEvaluationFieldEntityQuery&)> QueryCallback, TRange<FFrameNumber>& OutRange) const;

	/**
	 * Check whether this field contains any one-shot entities
	 */
	MOVIESCENE_API bool HasAnyOneShotEntities() const;

	/**
	 * Query the one-shot entities that overlap with the specified query range.
	 * @note: One-shot entities only ever live for a single frame of evaluation.
	 *
	 * @param QueryRange  The ranger over which to query the field (in the TickResolution of the sequence this was generated from)
	 * @param OutEntities A set that will be populated with all the entities that overlapped at all with the specified range
	 */
	MOVIESCENE_API void QueryOneShotEntities(const TRange<FFrameNumber>& QueryRange, FMovieSceneEvaluationFieldEntitySet& OutEntityIndices) const;

private:

	friend struct FMovieSceneEntityComponentFieldBuilder;

	/** A hierarchical tree specifiying indices into the Entities array for any given time such persistent entities are active */
	UPROPERTY()
	FMovieSceneEvaluationFieldEntityTree PersistentEntityTree;

	/** A hierarchical tree specifiying indices into the Entities array for any given time such one-shot entities are active */
	UPROPERTY()
	FMovieSceneEvaluationFieldEntityTree OneShotEntityTree;

	/** 16 bytes - Flat array of unique entities. The 2 tree types specify indices into this array */
	UPROPERTY()
	TArray<FMovieSceneEvaluationFieldEntity> Entities;

	/** 16 bytes - Optional meta-data for specific entities. Specified in FMovieSceneEvaluationFieldEntity::MetaDataIndex. */
	UPROPERTY()
	TArray<FMovieSceneEvaluationFieldEntityMetaData> EntityMetaData;

	/** 16 bytes - Optional shared meta-data for groups of entities. Specified in FMovieSceneEvaluationFieldEntity::SharedMetaDataIndex. */
	UPROPERTY()
	TArray<FMovieSceneEvaluationFieldSharedEntityMetaData> SharedMetaData;
};


/**
 * Builder class used for populating an FMovieSceneEntityComponentField with data.
 * Ensures that null or redundant entities or meta-data are not added to the field, and that all indices are valid and correct.
 */
struct FMovieSceneEntityComponentFieldBuilder
{
	static constexpr uint32 InvalidEntityID = ~0u;

	/**
	 * Construction from a field to populate
	 */
	MOVIESCENE_API FMovieSceneEntityComponentFieldBuilder(FMovieSceneEntityComponentField* InField);

	/**
	 * Destructor that cleans up redundant data if necessary
	 */
	MOVIESCENE_API ~FMovieSceneEntityComponentFieldBuilder();

	/**
	 * Access the shared meta-data for all the entities created by this builder.
	 */
	MOVIESCENE_API FMovieSceneEvaluationFieldSharedEntityMetaData& GetSharedMetaData();

	/**
	 * Access the index of the shared meta-data of this builder.
	 */
	MOVIESCENE_API int32 GetSharedMetaDataIndex() const;

	/**
	 * Add meta-data to this tree returning its index within this builder
	 *
	 * @param InMetaData     The meta-data to add.
	 * @return A unique index for this meta-data within this builder, or INDEX_NONE if the meta-data is redundant
	 */
	MOVIESCENE_API int32 AddMetaData(const FMovieSceneEvaluationFieldEntityMetaData& InMetaData);

	/**
	 * Retrieve an index for the entity that is identified by the specified owner and ID
	 *
	 * @param EntityOwner    The owner that produces the entity at runtime. Must implement the IMovieSceneEntityProvider interface
	 * @param EntityID       (Optional) An identifier used to identify the entity inside IMovieSceneEntityProvider::ImportEntityImpl. Could be an index within an array or a set of flags.
	 * @return An index into this builder used to uniquely identify this entity.
	 */
	MOVIESCENE_API int32 FindOrAddEntity(UObject* EntityOwner, uint32 EntityID = 0);

	/**
	 * Add a persistent entity to the field for a given range. Equivalent to AddPersistentEntity(Range, FindOrAddEntity(EntityOwner, EntityID)).
	 * @note: Persistent entities remain alive for the entire duration of their applicable ranges.
	 *
	 * @param Range          The range within which this entity should be alive
	 * @param EntityOwner    The owner that produces the entity at runtime. Must implement the IMovieSceneEntityProvider interface
	 * @param EntityID       (Optional) An identifier used to identify the entity inside IMovieSceneEntityProvider::ImportEntityImpl. Could be an index within an array or a set of flags.
	 * @param MetaDataIndex  (Optional) Meta-data to use for this entitiy within this range. See AddMetaData.
	 */
	MOVIESCENE_API void AddPersistentEntity(const TRange<FFrameNumber>& Range, UObject* EntityOwner, uint32 EntityID = 0, int32 InMetaDataIndex = INDEX_NONE);

	/**
	 * Add a persistent entity to the field for a given range
	 * @note: Persistent entities remain alive for the entire duration of their applicable ranges.
	 *
	 * @param Range          The range within which this entity should be alive
	 * @param LocalIndex     The index to the entity retrieved from FindOrAddEntity.
	 * @param MetaDataIndex  (Optional) Meta-data to use for this entitiy within this range. See AddMetaData.
	 */
	MOVIESCENE_API void AddPersistentEntity(const TRange<FFrameNumber>& Range, int32 LocalIndex, int32 InMetaDataIndex = INDEX_NONE);

	/**
	 * Add a one-shot entity to the field for a given range. Equivalent to AddOneShotEntity(Range, FindOrAddEntity(EntityOwner, EntityID)).
	 * @note: One-shot entities are only ever alive for a single evaluation, regardless of the range within the field. This makes them ideal for events or triggers.
	 *
	 * @param Range          The range within which this entity should be alive
	 * @param EntityOwner    The owner that produces the entity at runtime. Must implement the IMovieSceneEntityProvider interface
	 * @param EntityID       (Optional) An identifier used to identify the entity inside IMovieSceneEntityProvider::ImportEntityImpl. Could be an index within an array or a set of flags.
	 * @param MetaDataIndex  (Optional) Meta-data to use for this entitiy within this range. See AddMetaData.
	 */
	MOVIESCENE_API void AddOneShotEntity(const TRange<FFrameNumber>& OneShotRange, UObject* EntityOwner, uint32 EntityID = 0, int32 InMetaDataIndex = INDEX_NONE);

	/**
	 * Add a one-shot entity to the field for a given range
	 * @note: One-shot entities are only ever alive for a single evaluation, regardless of the range within the field. This makes them ideal for events or triggers.
	 *
	 * @param Range          The range within which this entity should be alive
	 * @param LocalIndex     The index to the entity retrieved from FindOrAddEntity.
	 * @param MetaDataIndex  (Optional) Meta-data to use for this entitiy within this range. See AddMetaData.
	 */
	MOVIESCENE_API void AddOneShotEntity(const TRange<FFrameNumber>& OneShotRange, int32 LocalIndex, int32 InMetaDataIndex = INDEX_NONE);

private:

	/** Convert a user-facing local index into KeyToFieldIndex, into an index within FMovieSceneEntityComponentField::Entities */
	MOVIESCENE_API int32 LocalEntityIndexToFieldIndex(int32 LocalIndex);

	/** Convert a user-facing local meta-data index into MetaDataToFieldIndex, into an index within FMovieSceneEntityComponentField::EntityMetaData */
	MOVIESCENE_API int32 LocalMetaDataIndexToFieldIndex(int32 LocalIndex);

	/** Array of entity keys and their field index within FMovieSceneEntityComponentField::Entities */
	struct FKeyToIndex
	{
		FMovieSceneEvaluationFieldEntityKey Key;
		int32 FieldIndex;
	};
	TArray<FKeyToIndex, TInlineAllocator<4>> KeyToFieldIndex;

	struct FMetaDataToIndex
	{
		FMovieSceneEvaluationFieldEntityMetaData MetaData;
		int32 FieldIndex;
	};
	TArray<FMetaDataToIndex, TInlineAllocator<2>> MetaDataToFieldIndex;

	/** The field that we are building */
	FMovieSceneEntityComponentField* Field;

	/** (default: INDEX_NONE) The index into FMovieSceneEntityComponentField::SharedMetaData for all entities built by this builder, if it has been defined. */
	int32 SharedMetaDataIndex;
};


//---------------------------------------------------------
// Legacy track template field structures begin here
//---------------------------------------------------------


/** A pointer to a track held within an evaluation template */
USTRUCT()
struct FMovieSceneEvaluationFieldTrackPtr
{
	GENERATED_BODY()

	/**
	 * Default constructor
	 */
	FMovieSceneEvaluationFieldTrackPtr(){}

	/**
	 * Construction from a sequence ID, and the index of the track within that sequence's track list
	 */
	FMovieSceneEvaluationFieldTrackPtr(FMovieSceneSequenceIDRef InSequenceID, FMovieSceneTrackIdentifier InTrackIdentifier)
		: SequenceID(InSequenceID)
		, TrackIdentifier(InTrackIdentifier)
	{}

	/**
	 * Check for equality
	 */
	friend bool operator==(FMovieSceneEvaluationFieldTrackPtr A, FMovieSceneEvaluationFieldTrackPtr B)
	{
		return A.TrackIdentifier == B.TrackIdentifier && A.SequenceID == B.SequenceID;
	}

	/**
	 * Get a hashed representation of this type
	 */
	friend uint32 GetTypeHash(FMovieSceneEvaluationFieldTrackPtr LHS)
	{
		return HashCombine(GetTypeHash(LHS.TrackIdentifier), GetTypeHash(LHS.SequenceID));
	}

	/** The sequence ID that identifies to which sequence the track belongs */
	UPROPERTY()
	FMovieSceneSequenceID SequenceID;

	/** The Identifier of the track inside the track map (see FMovieSceneEvaluationTemplate::Tracks) */
	UPROPERTY()
	FMovieSceneTrackIdentifier TrackIdentifier;
};

/** A pointer to a particular segment of a track held within an evaluation template */
USTRUCT()
struct FMovieSceneEvaluationFieldSegmentPtr : public FMovieSceneEvaluationFieldTrackPtr
{
	GENERATED_BODY()

	/**
	 * Default constructor
	 */
	FMovieSceneEvaluationFieldSegmentPtr(){}

	/**
	 * Construction from a sequence ID, and the index of the track within that sequence's track list
	 */
	FMovieSceneEvaluationFieldSegmentPtr(FMovieSceneSequenceIDRef InSequenceID, FMovieSceneTrackIdentifier InTrackIdentifier, FMovieSceneSegmentIdentifier InSegmentID)
		: FMovieSceneEvaluationFieldTrackPtr(InSequenceID, InTrackIdentifier)
		, SegmentID(InSegmentID)
	{}

	/**
	 * Check for equality
	 */
	friend bool operator==(FMovieSceneEvaluationFieldSegmentPtr A, FMovieSceneEvaluationFieldSegmentPtr B)
	{
		return A.SegmentID == B.SegmentID && A.TrackIdentifier == B.TrackIdentifier && A.SequenceID == B.SequenceID;
	}

	/**
	 * Get a hashed representation of this type
	 */
	friend uint32 GetTypeHash(FMovieSceneEvaluationFieldSegmentPtr LHS)
	{
		return HashCombine(GetTypeHash(LHS.SegmentID), GetTypeHash(static_cast<FMovieSceneEvaluationFieldTrackPtr&>(LHS)));
	}

	/** The identifier of the segment within the track (see FMovieSceneEvaluationTrack::Segments) */
	UPROPERTY()
	FMovieSceneSegmentIdentifier SegmentID;
};

USTRUCT()
struct FMovieSceneFieldEntry_EvaluationTrack
{
	GENERATED_BODY()

	UPROPERTY()
	FMovieSceneEvaluationFieldTrackPtr TrackPtr;

	UPROPERTY()
	uint16 NumChildren = 0;
};

USTRUCT()
struct FMovieSceneFieldEntry_ChildTemplate
{
	GENERATED_BODY()

	FMovieSceneFieldEntry_ChildTemplate()
		: ChildIndex(-1)
		, Flags(ESectionEvaluationFlags::None)
		, ForcedTime(TNumericLimits<int32>::Lowest())
	{}

	FMovieSceneFieldEntry_ChildTemplate(uint16 InChildIndex, ESectionEvaluationFlags InFlags, FFrameNumber InForcedTime)
		: ChildIndex(InChildIndex)
		, Flags(InFlags)
		, ForcedTime(InForcedTime)
	{}

	UPROPERTY()
	uint16 ChildIndex;

	UPROPERTY()
	ESectionEvaluationFlags Flags;

	UPROPERTY()
	FFrameNumber ForcedTime;
};

/** Lookup table index for a group of evaluation templates */
USTRUCT()
struct FMovieSceneEvaluationGroupLUTIndex
{
	GENERATED_BODY()

	FMovieSceneEvaluationGroupLUTIndex()
		: NumInitPtrs(0)
		, NumEvalPtrs(0)
	{}

	/** The number of initialization pointers are stored after &FMovieSceneEvaluationGroup::SegmentPtrLUT[0] + LUTOffset. */
	UPROPERTY()
	int32 NumInitPtrs;

	/** The number of evaluation pointers are stored after &FMovieSceneEvaluationGroup::SegmentPtrLUT[0] + LUTOffset + NumInitPtrs. */
	UPROPERTY()
	int32 NumEvalPtrs;
};

/** Holds segment pointers for all segments that are active for a given range of the sequence */
USTRUCT()
struct FMovieSceneEvaluationGroup
{
	GENERATED_BODY()

	/** Array of indices that define all the flush groups in the range. */
	UPROPERTY()
	TArray<FMovieSceneEvaluationGroupLUTIndex> LUTIndices;

	/** */
	UPROPERTY()
	TArray<FMovieSceneFieldEntry_EvaluationTrack> TrackLUT;

	/** */
	UPROPERTY()
	TArray<FMovieSceneFieldEntry_ChildTemplate> SectionLUT;
};

/** Struct that stores the key for an evaluated entity, and the index at which it was (or is to be) evaluated */
USTRUCT()
struct FMovieSceneOrderedEvaluationKey
{
	GENERATED_BODY()

	UPROPERTY()
	FMovieSceneEvaluationKey Key;

	UPROPERTY()
	uint16 SetupIndex = 0;

	UPROPERTY()
	uint16 TearDownIndex = 0;
};

/** Informational meta-data that applies to a given time range */
USTRUCT()
struct FMovieSceneEvaluationMetaData
{
	GENERATED_BODY()

	/**
	 * Reset this meta-data
	 */
	void Reset()
	{
		ActiveSequences.Reset();
		ActiveEntities.Reset();
	}

	/**
	 * Diff the active sequences this frame, with the specified previous frame's meta-data
	 *
	 * @param LastFrame				Meta-data pertaining to the last frame
	 * @param NewSequences			(Optional) Ptr to an array that will be populated with sequences that are new this frame
	 * @param ExpiredSequences		(Optional) Ptr to an array that will be populated with sequences that are no longer being evaluated
	 */
	void DiffSequences(const FMovieSceneEvaluationMetaData& LastFrame, TArray<FMovieSceneSequenceID>* NewSequences, TArray<FMovieSceneSequenceID>* ExpiredSequences) const;

	/**
	 * Diff the active entities (tracks and sections) this frame, with the specified previous frame's meta-data
	 *
	 * @param LastFrame				Meta-data pertaining to the last frame
	 * @param NewKeys				(Optional) Ptr to an array that will be populated with entities that are new this frame
	 * @param ExpiredKeys			(Optional) Ptr to an array that will be populated with entities that are no longer being evaluated
	 */
	void DiffEntities(const FMovieSceneEvaluationMetaData& LastFrame, TArray<FMovieSceneOrderedEvaluationKey>* NewKeys, TArray<FMovieSceneOrderedEvaluationKey>* ExpiredKeys) const;

	/** Array of sequences that are active in this time range. */
	UPROPERTY()
	TArray<FMovieSceneSequenceID> ActiveSequences;

	/** Array of entities (tracks and/or sections) that are active in this time range. */
	UPROPERTY()
	TArray<FMovieSceneOrderedEvaluationKey> ActiveEntities;
};

/**
 * Memory layout optimized primarily for speed of searching the applicable ranges
 */
USTRUCT()
struct FMovieSceneEvaluationField
{
	GENERATED_BODY()

	/**
	 * Efficiently find the entry that exists at the specified time, if any
	 *
	 * @param Time			The time at which to seach
	 * @return The index within Ranges, Groups and MetaData that the current time resides, or INDEX_NONE if there is nothing to do at the requested time
	 */
	MOVIESCENE_API int32 GetSegmentFromTime(FFrameNumber Time) const;

	/**
	 * Deduce the indices into Ranges and Groups that overlap with the specified time range
	 *
	 * @param Range			The range to overlap with our field
	 * @return A range of indices for which GetRange() overlaps the specified Range, of the form [First, First+Num)
	 */
	MOVIESCENE_API TRange<int32> OverlapRange(const TRange<FFrameNumber>& Range) const;

	/**
	 * Invalidate a range in this field
	 *
	 * @param Range			The range to overlap with our field
	 * @return A range of indices into Ranges and Groups that overlap with the requested range
	 */
	MOVIESCENE_API void Invalidate(const TRange<FFrameNumber>& Range);

	/**
	 * Insert a new range into this field
	 *
	 * @param InRange		The total range to insert to the field. Will potentially be intersected with preexisting adjacent ranges
	 * @param InGroup		The group defining what should happen at this time
	 * @param InMetaData	The meta-data defining efficient access to what happens in this frame
	 * @return The index the entries were inserted at
	 */
	MOVIESCENE_API int32 Insert(const TRange<FFrameNumber>& InRange, FMovieSceneEvaluationGroup&& InGroup, FMovieSceneEvaluationMetaData&& InMetaData);

	/**
	 * Add the specified data to this field, assuming the specified range lies after any other entries
	 *
	 * @param InRange		The range to add
	 * @param InGroup		The group defining what should happen at this time
	 * @param InMetaData	The meta-data defining efficient access to what happens in this frame
	 */
	MOVIESCENE_API void Add(const TRange<FFrameNumber>& InRange, FMovieSceneEvaluationGroup&& InGroup, FMovieSceneEvaluationMetaData&& InMetaData);

	/**
	 * Access this field's signature
	 */
#if WITH_EDITORONLY_DATA
	const FGuid& GetSignature() const
	{
		return Signature;
	}
#endif

	/**
	 * Access this field's size
	 */
	int32 Size() const
	{
		return Ranges.Num();
	}

	/**
	 * Access this entire field's set of ranges
	 */
	TArrayView<const FMovieSceneFrameRange> GetRanges() const
	{
		return Ranges;
	}

	/**
	 * Lookup a valid range by index
	 * @param Index 	The valid index within the ranges to lookup
	 * @return The range
	 */
	const TRange<FFrameNumber>& GetRange(int32 Index) const
	{
		return Ranges[Index].Value;
	}

	/**
	 * Lookup a valid evaluation group by entry index
	 * @param Index 	The valid index within the evaluation group array to lookup
	 * @return The group
	 */
	const FMovieSceneEvaluationGroup& GetGroup(int32 Index) const
	{
		return Groups[Index];
	}

	/**
	 * Lookup valid meta-data by entry index
	 * @param Index 	The valid index within the meta-data array to lookup
	 * @return The meta-data
	 */
	const FMovieSceneEvaluationMetaData& GetMetaData(int32 Index) const
	{
		return MetaData[Index];
	}

private:
#if WITH_EDITORONLY_DATA
	/** Signature that uniquely identifies any state this field can be in - regenerated on mutation */
	UPROPERTY()
	FGuid Signature;
#endif

	/** Ranges stored separately for fast (cache efficient) lookup. Each index has a corresponding entry in FMovieSceneEvaluationField::Groups. */
	UPROPERTY()
	TArray<FMovieSceneFrameRange> Ranges;

	/** Groups that store segment pointers for each of the above ranges. Each index has a corresponding entry in FMovieSceneEvaluationField::Ranges. */
	UPROPERTY()
	TArray<FMovieSceneEvaluationGroup> Groups;

	/** Meta data that maps to entries in the 'Ranges' array. */
	UPROPERTY()
	TArray<FMovieSceneEvaluationMetaData> MetaData;
};
