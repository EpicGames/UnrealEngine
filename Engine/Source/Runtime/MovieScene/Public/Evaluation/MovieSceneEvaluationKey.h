// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MovieSceneSequenceID.h"
#include "Evaluation/MovieSceneTrackIdentifier.h"
#include "MovieSceneEvaluationKey.generated.h"

/** Keyable struct that represents a particular entity within an evaluation template (either a section/template or a track) */
USTRUCT()
struct FMovieSceneEvaluationKey
{
	GENERATED_BODY()

	/**
	 * Default construction to an invalid key
	 */
	FMovieSceneEvaluationKey()
		: SequenceID(MovieSceneSequenceID::Invalid)
		, TrackIdentifier(FMovieSceneTrackIdentifier::Invalid())
		, SectionIndex(-1)
	{}

	/**
	 * User construction
	 */
	FMovieSceneEvaluationKey(FMovieSceneSequenceIDRef InSequenceID, FMovieSceneTrackIdentifier InTrackIdentifier, uint32 InSectionIndex = uint32(-1))
		: SequenceID(InSequenceID)
		, TrackIdentifier(InTrackIdentifier)
		, SectionIndex(InSectionIndex)
	{}

	/**
	 * Check whether this key is valid
	 */
	bool IsValid() const
	{
		return SequenceID != MovieSceneSequenceID::Invalid && TrackIdentifier != FMovieSceneTrackIdentifier::Invalid();
	}

	/**
	 * Derive a new key from this one using the specified section identifier
	 */
	FMovieSceneEvaluationKey AsSection(uint32 InSectionIdentifier) const
	{
		FMovieSceneEvaluationKey NewKey = *this;
		NewKey.SectionIndex = InSectionIdentifier;
		return NewKey;
	}

	/**
	 * Convert this key into a track key
	 */
	FMovieSceneEvaluationKey AsTrack() const
	{
		FMovieSceneEvaluationKey NewKey = *this;
		NewKey.SectionIndex = uint32(-1);
		return NewKey;
	}

	friend bool operator==(const FMovieSceneEvaluationKey& A, const FMovieSceneEvaluationKey& B)
	{
		return A.TrackIdentifier == B.TrackIdentifier && A.SequenceID == B.SequenceID && A.SectionIndex == B.SectionIndex;
	}

	friend bool operator<(const FMovieSceneEvaluationKey& A, const FMovieSceneEvaluationKey& B)
	{
		if (A.SequenceID < B.SequenceID)
		{
			return true;
		}
		else if (A.SequenceID > B.SequenceID)
		{
			return false;
		}
		else if (A.TrackIdentifier < B.TrackIdentifier)
		{
			return true;
		}
		else
		{
			return A.TrackIdentifier == B.TrackIdentifier && A.SectionIndex < B.SectionIndex;
		}
	}

	friend uint32 GetTypeHash(const FMovieSceneEvaluationKey& In)
	{
		return GetTypeHash(In.SequenceID) ^ (~GetTypeHash(In.TrackIdentifier)) ^ In.SectionIndex;
	}

	/** Custom serialized to reduce memory footprint */
	bool Serialize(FArchive& Ar)
	{
		Ar << SequenceID;
		Ar << TrackIdentifier;
		Ar << SectionIndex;
		return true;
	}

	friend FArchive& operator<<(FArchive& Ar, FMovieSceneEvaluationKey& Key)
	{
		Key.Serialize(Ar);
		return Ar;
	}

	/** ID of the sequence that the entity is contained within */
	UPROPERTY()
	FMovieSceneSequenceID SequenceID;

	/** ID of the track this key relates to */
	UPROPERTY()
	FMovieSceneTrackIdentifier TrackIdentifier;

	/** Index of the section template within the track this key relates to (or -1 where this key relates to a track) */
	UPROPERTY()
	uint32 SectionIndex;
};

template<>
struct TStructOpsTypeTraits<FMovieSceneEvaluationKey> : public TStructOpsTypeTraitsBase2<FMovieSceneEvaluationKey>
{
	enum
	{
		WithSerializer = true, WithIdenticalViaEquality = true
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};
