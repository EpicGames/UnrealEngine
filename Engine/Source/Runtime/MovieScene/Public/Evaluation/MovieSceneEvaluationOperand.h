// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneSequenceID.h"
#include "Misc/Guid.h"

#include "MovieSceneEvaluationOperand.generated.h"

/**
 * Structure that describes an object that is to be animated. Used as an abstraction of the actual objects bound to object bindings.
 */
USTRUCT()
struct FMovieSceneEvaluationOperand
{
	GENERATED_BODY()

	/**
	 * Default Construction
	 */
	FMovieSceneEvaluationOperand()
		: SequenceID(0)
	{
	}

	/**
	 * Construction from a sequence ID and an object binding ID
	 */
	FMovieSceneEvaluationOperand(FMovieSceneSequenceIDRef InSequenceID, const FGuid& InObjectBindingID)
		: ObjectBindingID(InObjectBindingID)
		, SequenceID(InSequenceID)
	{
	}

	/**
	 * Check if this operand actually references anything in the sequence
	 */
	bool IsValid() const
	{
		return SequenceID != MovieSceneSequenceID::Invalid;
	}

	friend bool operator==(const FMovieSceneEvaluationOperand& A, const FMovieSceneEvaluationOperand& B)
	{
		return A.SequenceID == B.SequenceID && A.ObjectBindingID == B.ObjectBindingID;
	}

	friend uint32 GetTypeHash(const FMovieSceneEvaluationOperand& In)
	{
		return HashCombine(GetTypeHash(In.SequenceID), GetTypeHash(In.ObjectBindingID));
	}

	/** A GUID relating to either a posessable, or a spawnable binding */
	UPROPERTY()
	FGuid ObjectBindingID;

	/** The ID of the sequence within which the object binding resides */
	UPROPERTY()
	FMovieSceneSequenceID SequenceID;
};
