// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MovieSceneSection.h"
#include "Evaluation/MovieSceneEvaluationKey.h"

struct FMovieSceneEvaluationScope
{
	FMovieSceneEvaluationScope()
		: Key()
		, CompletionMode(EMovieSceneCompletionMode::KeepState)
	{
	}

	FMovieSceneEvaluationScope(const FMovieSceneEvaluationKey& InKey, EMovieSceneCompletionMode InCompletionMode)
		: Key(InKey)
		, CompletionMode(InCompletionMode)
	{
	}

	FMovieSceneEvaluationKey Key;
	EMovieSceneCompletionMode CompletionMode;
};