// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Delegates/MulticastDelegateBase.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "Evaluation/IMovieScenePlaybackCapability.h"
#include "Evaluation/MovieSceneEvaluationKey.h"
#include "Evaluation/PersistentEvaluationData.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Misc/Guid.h"
#include "MovieSceneSequence.h"
#include "MovieSceneSequenceID.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API MOVIESCENE_API

class IMovieScenePlayer;
class UMovieSceneSequence;
class UObject;
struct FMovieSceneEvaluationKey;
struct FMovieSceneObjectBindingID;
struct FSharedPersistentDataKey;
struct IPersistentEvaluationData;

namespace UE::MovieScene
{

	struct FInstanceHandle;
	struct FSharedPlaybackState;

	/**
	 * Playback capability for being notified of object bindings changing.
	 */
	struct IObjectBindingNotifyPlaybackCapability
	{
		UE_DECLARE_MOVIESCENE_PLAYBACK_CAPABILITY_API(MOVIESCENE_API, IObjectBindingNotifyPlaybackCapability)

		/** Called when multiple object bindings have changed. */
		virtual void NotifyBindingsChanged() {}

		/** Called when a specific object binding has changed. */
		virtual void NotifyBindingUpdate(const FGuid& InBindingId, FMovieSceneSequenceIDRef InSequenceID, TArrayView<TWeakObjectPtr<>> BoundObjects) {}
	};

	/**
	 * Playback capability for storing static object binding overrides.
	 */
	struct IStaticBindingOverridesPlaybackCapability
	{
		UE_DECLARE_MOVIESCENE_PLAYBACK_CAPABILITY_API(MOVIESCENE_API, IStaticBindingOverridesPlaybackCapability)

		virtual ~IStaticBindingOverridesPlaybackCapability() = default;

		/** Retrieves any override for the given operand */
		virtual FMovieSceneEvaluationOperand* GetBindingOverride(const FMovieSceneEvaluationOperand& InOperand) = 0;
		/** Adds an override for the given operand */
		virtual void AddBindingOverride(const FMovieSceneEvaluationOperand& InOperand, const FMovieSceneEvaluationOperand& InOverrideOperand) = 0;
		/** Removes any override set for the given operand */
		virtual void RemoveBindingOverride(const FMovieSceneEvaluationOperand& InOperand) = 0;
	};

	/**
	 * A re-usable implementation of IStaticBindingOverridesPlaybackCapability.
	 */
	struct FStaticBindingOverrides : public IStaticBindingOverridesPlaybackCapability
	{
		UE_API virtual FMovieSceneEvaluationOperand* GetBindingOverride(const FMovieSceneEvaluationOperand& InOperand) override;
		UE_API virtual void AddBindingOverride(const FMovieSceneEvaluationOperand& InOperand, const FMovieSceneEvaluationOperand& InOverrideOperand) override;
		UE_API virtual void RemoveBindingOverride(const FMovieSceneEvaluationOperand& InOperand) override;

	private:
		TMap<FMovieSceneEvaluationOperand, FMovieSceneEvaluationOperand> BindingOverrides;

		friend class ::IMovieScenePlayer;
	};

}  // namespace UE::MovieScene

/**
 * Object cache that looks up, resolves, and caches object bindings for a specific sequence
 */
struct FMovieSceneObjectCache
{
	using FSharedPlaybackState = UE::MovieScene::FSharedPlaybackState;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnBindingInvalidated, const FGuid&);
	/** Invoked when a binding is either explicitly invalidated (as a result of a spawnable being spawned, or a binding override being added)
	 *  or when a previously resolved binding becomes invalid */
	FOnBindingInvalidated OnBindingInvalidated;

	/**
	 * Find all objects that are bound to the specified binding ID
	 * @note Will look up, and cache any objects if the cache has been invalidated
	 *
	 * @param InBindingID		The object binding GUID for a spawnable or posessable in a UMovieScene
	 * @param Player			The movie scene player that is playing back the sequence
	 * @return An iterable type of all objects bound to the specified ID.
	 */
	MOVIESCENE_API TArrayView<TWeakObjectPtr<>> FindBoundObjects(const FGuid& InBindingID, TSharedRef<const FSharedPlaybackState> InSharedPlaybackState);

	/**
	 * Find all objects that are bound to the specified binding ID
	 * @note Does not update bindings if they are out of date, or invalid
	 *
	 * @param InBindingID		The object binding GUID for a spawnable or posessable in a UMovieScene
	 * @return An iterable type of all objects bound to the specified ID.
	 */
	MOVIESCENE_API TArrayView<const TWeakObjectPtr<>> IterateBoundObjects(const FGuid& InBindingID) const;

	/**
	 * Set the sequence that this cache applies to
	 *
	 * @param InSequence		The sequence that this cache applies to
	 * @param InSequenceID		The ID of the sequence within the root sequence
	 */
	MOVIESCENE_API void SetSequence(UMovieSceneSequence& InSequence, FMovieSceneSequenceIDRef InSequenceID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState);

	/**
	 * Attempt deduce the posessable or spawnable that relates to the specified object
	 * @note Will forcably resolve any out of date bindings in the entire sequence
	 *
	 * @param InObject			The object whose binding ID is to be find
	 * @param Player			The movie scene player that is playing back the sequence
	 * @return The object's spawnable or possessable GUID, or a zero GUID if it was not found
	 */
	MOVIESCENE_API FGuid FindObjectId(UObject& InObject, TSharedRef<const FSharedPlaybackState> SharedPlaybackState);

	/**
	* Attempt deduce the posessable or spawnable that relates to the specified object
	* @note Does not clear the existing cache
	*
	* @param InObject			The object whose binding ID is to be find
	* @param Player			The movie scene player that is playing back the sequence
	* @return The object's spawnable or possessable GUID, or a zero GUID if it was not found
	*/
	MOVIESCENE_API FGuid FindCachedObjectId(UObject& InObject, TSharedRef<const FSharedPlaybackState> SharedPlaybackState);

	/**
	 * Invalidate any object bindings for objects that have been destroyed
	 */
	MOVIESCENE_API void InvalidateExpiredObjects();

	/**
	 * Invalidate the object bindings for a specific object binding ID in this sequence
	 *
	 * @param InGuid			The object binding ID to invalidate bindings for
	 */
	MOVIESCENE_API void Invalidate(const FGuid& InGuid);

	/**
	 * Invalidate the object bindings for a specific object binding ID in the specified sequence ID.
	 * If the sequence ID matches this one, then we will look for the guid in this sequence. 
	 * If it does not, then we will see if any of our object bindings reference that one, and if so, they will be invalidated.
	 * 
	 * @param InGuid			The object binding ID to invalidate bindings for
	 */
	MOVIESCENE_API void Invalidate(const FGuid& InGuid, FMovieSceneSequenceIDRef InSequenceID);

	/**
	 * Invalidate the object bindings for a specific object binding ID if they are not already invalidated
	 *
	 * @param InGuid			The object binding ID to invalidate bindings for
	 */
	MOVIESCENE_API void InvalidateIfValid(const FGuid& InGuid);


	/* Gets the current activation on the provided binding. If no binding lifetime track is present, true will be returned.
	 * @param InGuid			The object binding ID
	 */
	MOVIESCENE_API bool GetBindingActivation(const FGuid& InGuid) const;

	/* Sets the binding to either active or inactive. Inactive bindings will invalidate and not resolve while inactive.
	 * @param InGuid			The object binding ID
	 * @param bActive		    Where to activate or deactivate the binding.
	 */
	MOVIESCENE_API void SetBindingActivation(const FGuid& InGuid, bool bActive);

	/**
	 * Completely erase all knowledge of, anc caches for all object bindings
	 */
	void Clear(TSharedRef<const FSharedPlaybackState> SharedPlaybackState);

	/**
	 * Get the sequence that this cache relates to
	 */
	UMovieSceneSequence* GetSequence() const { return WeakSequence.Get(); }

	/**
	 * Get the current serial number of this cache
	 */
	uint32 GetSerialNumber() const { return SerialNumber; }

	/**
	 * Filter all the object bindings in this object cache that contain the specified predicate object
	 *
	 * @param PredicateObject		The object to filter by. Any bindings referencing this object will be added to the output array.
	 * @param Player				The movie scene player that is playing back the sequence
	 * @param OutBindings			(mandatory) Array to populate with bindings that relate to the object
	 */
	void FilterObjectBindings(UObject* PredicateObject, TSharedRef<const FSharedPlaybackState> SharedPlaybackState, TArray<FMovieSceneObjectBindingID>* OutBindings);

public:

	// Backwards compatible API, to be deprecated later

	MOVIESCENE_API TArrayView<TWeakObjectPtr<>> FindBoundObjects(const FGuid& InBindingID, IMovieScenePlayer& Player);
	MOVIESCENE_API void SetSequence(UMovieSceneSequence& InSequence, FMovieSceneSequenceIDRef InSequenceID, IMovieScenePlayer& Player);
	MOVIESCENE_API FGuid FindObjectId(UObject& InObject, IMovieScenePlayer& Player);
	MOVIESCENE_API FGuid FindCachedObjectId(UObject& InObject, IMovieScenePlayer& Player);
	MOVIESCENE_API void Clear(IMovieScenePlayer& Player);
	MOVIESCENE_API void FilterObjectBindings(UObject* PredicateObject, IMovieScenePlayer& Player, TArray<FMovieSceneObjectBindingID>* OutBindings);

private:
	/**
	 * Update the bindings for the specified GUID
	 *
	 * @param InGuid			The object binding ID to update bindings for
	 * @param SharedPlaybackState  The playback state for the sequence
	 */
	void UpdateBindings(const FGuid& InGuid, TSharedRef<const FSharedPlaybackState> SharedPlaybackState);

	/**
	 * Invalidate the object bindings for a specific object binding ID
	 */
	bool InvalidateInternal(const FGuid& InGuid);

	/**
	 * Invalidate the object bindings for a specific object binding ID if they are not already invalidated
	 */
	bool InvalidateIfValidInternal(const FGuid& InGuid);

	/**
	 * Update the serial number of this instance.
	 */
	void UpdateSerialNumber();

	struct FBoundObjects
	{
		bool bUpToDate;
		TArray<TWeakObjectPtr<>, TInlineAllocator<1>> Objects;
	};

private:

	/** The sequence that we're caching objects for */
	TWeakObjectPtr<UMovieSceneSequence> WeakSequence;

	/** The sequence ID of the sequence within the root sequence */
	FMovieSceneSequenceID SequenceID;

	template<typename ValueType>
	struct TFastGuidKeyFuncs : BaseKeyFuncs<TPair<FGuid,ValueType>,FGuid,false>
	{
		typedef typename TTypeTraits<FGuid>::ConstPointerType KeyInitType;
		typedef const TPairInitializer<typename TTypeTraits<FGuid>::ConstInitType, typename TTypeTraits<ValueType>::ConstInitType>& ElementInitType;

		static inline KeyInitType GetSetKey(ElementInitType Element)
		{
			return Element.Key;
		}
		static inline bool Matches(KeyInitType A,KeyInitType B)
		{
			return A == B;
		}
		static inline uint32 GetKeyHash(KeyInitType Key)
		{
			return Key.A ^ Key.B ^ Key.C ^ Key.D;
		}
	};

	/** A map of bound objects */
	TMap<FGuid, FBoundObjects, FDefaultSetAllocator, TFastGuidKeyFuncs<FBoundObjects>> BoundObjects;

	/** Map of child bindings for any given object binding */
	typedef TArray<FGuid, TInlineAllocator<4>> FGuidArray;
	TMap<FGuid, FGuidArray, FDefaultSetAllocator, TFastGuidKeyFuncs<FGuidArray>> ChildBindings;
	
	/** For possessables in this scene that map to a binding in another scene (for example to a spawnable in another scene).
	  * Stored as reverse-lookup for speed of invalidation.
	  */
	TMap<FMovieSceneObjectBindingID, FGuidArray, FDefaultSetAllocator> ReverseMappedBindings;

	/* A set of inactive binding ids based on Binding Lifetime track. While inactive, these will be prevented from resolving.*/
	TSet<FGuid> InactiveBindingIds;

	/** Serial number for this cache */
	uint32 SerialNumber = 0;

	bool bReentrantUpdate = false;
};

/**
 * Provides runtime evaluation functions with the ability to look up state from the main game environment
 */
struct FMovieSceneEvaluationState : public UE::MovieScene::IPlaybackCapability
{
	using FSharedPlaybackState = UE::MovieScene::FSharedPlaybackState;

	UE_DECLARE_MOVIESCENE_PLAYBACK_CAPABILITY_API(MOVIESCENE_API, FMovieSceneEvaluationState)

	FMovieSceneEvaluationState() = default;
	FMovieSceneEvaluationState(const FMovieSceneEvaluationState&) = delete;
	FMovieSceneEvaluationState& operator=(const FMovieSceneEvaluationState&) = delete;

	/**
	 * Assign a sequence to a specific ID
	 *
	 * @param InSequenceID		The sequence ID to assign to
	 * @param InSequence		The sequence to assign
	 */
	MOVIESCENE_API void AssignSequence(FMovieSceneSequenceIDRef InSequenceID, UMovieSceneSequence& InSequence, TSharedRef<const FSharedPlaybackState> SharedPlaybackState);

	/**
	 * Attempt to locate a sequence from its ID
	 *
	 * @param InSequenceID		The sequence ID to lookup
	 */
	MOVIESCENE_API UMovieSceneSequence* FindSequence(FMovieSceneSequenceIDRef InSequenceID) const;

	/**
	 * Attempt to locate a sequence ID from a sequence
	 *
	 * @param InSequence		The sequence to look up
	 */
	MOVIESCENE_API FMovieSceneSequenceID FindSequenceId(UMovieSceneSequence* InSequence) const;
	MOVIESCENE_API FMovieSceneSequenceID FindSequenceId(const UMovieSceneSequence* InSequence) const;

	/**
	 * Attempt deduce the posessable or spawnable that relates to the specified object
	 * @note Will forcably resolve any out of date bindings in the entire sequence
	 *
	 * @param InObject			The object whose binding ID is to be find
	 * @param Player			The movie scene player that is playing back the sequence
	 * @return The object's spawnable or possessable GUID, or a zero GUID if it was not found
	 */
	MOVIESCENE_API FGuid FindObjectId(UObject& Object, FMovieSceneSequenceIDRef InSequenceID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState);

	/**
	* Attempt deduce the posessable or spawnable that relates to the specified object
	* @note Does not clear the existing cache
	*
	* @param InObject			The object whose binding ID is to be find
	* @param Player			The movie scene player that is playing back the sequence
	* @return The object's spawnable or possessable GUID, or a zero GUID if it was not found
	*/
	MOVIESCENE_API FGuid FindCachedObjectId(UObject& Object, FMovieSceneSequenceIDRef InSequenceID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState);

	/**
	 * Filter all the object bindings in this object cache that contain the specified predicate object
	 *
	 * @param PredicateObject		The object to filter by. Any bindings referencing this object will be added to the output array.
	 * @param Player				The movie scene player that is playing back the sequence
	 * @param OutBindings			(mandatory) Array to populate with bindings that relate to the object
	 */
	MOVIESCENE_API void FilterObjectBindings(UObject* PredicateObject, TSharedRef<const FSharedPlaybackState> SharedPlaybackState, TArray<FMovieSceneObjectBindingID>* OutBindings);

	/**
	 * Find an object cache pertaining to the specified sequence
	 *
	 * @param InSequenceID		The sequence ID to lookup
	 */
	inline FMovieSceneObjectCache* FindObjectCache(FMovieSceneSequenceIDRef SequenceID)
	{
		if (FVersionedObjectCache* Cache = ObjectCaches.Find(SequenceID))
		{
			return &Cache->ObjectCache;
		}
		return nullptr;
	}

	/**
	 * Find an object cache pertaining to the specified sequence
	 *
	 * @param InSequenceID		The sequence ID to lookup
	 */
	inline const FMovieSceneObjectCache* FindObjectCache(FMovieSceneSequenceIDRef SequenceID) const
	{
		if (const FVersionedObjectCache* Cache = ObjectCaches.Find(SequenceID))
		{
			return &Cache->ObjectCache;
		}
		return nullptr;
	}

	/**
	 * Get an object cache pertaining to the specified sequence
	 *
	 * @param InSequenceID		The sequence ID to lookup
	 */
	inline FMovieSceneObjectCache& GetObjectCache(FMovieSceneSequenceIDRef SequenceID)
	{
		FVersionedObjectCache* Cache = ObjectCaches.Find(SequenceID);
		if (!Cache)
		{
			Cache = &ObjectCaches.Add(SequenceID, FVersionedObjectCache());
		}
		return Cache->ObjectCache;
	}

	/**
	 * Locate objects bound to the specified object guid, in the specified sequence
	 * @note: Objects lists are cached internally until they are invalidate.
	 *
	 * @param ObjectBindingID 		The object to resolve
	 * @param SequenceID 			ID of the sequence to resolve for
	 *
	 * @return Iterable list of weak object pointers pertaining to the specified GUID
	 */
	TArrayView<TWeakObjectPtr<>> FindBoundObjects(const FGuid& ObjectBindingID, FMovieSceneSequenceIDRef SequenceID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
	{
		FMovieSceneObjectCache* Cache = FindObjectCache(SequenceID);
		if (Cache)
		{
			return Cache->FindBoundObjects(ObjectBindingID, SharedPlaybackState);
		}

		return TArrayView<TWeakObjectPtr<>>();
	}

	/**
	 * Locate objects bound to the specified sequence operand
	 * @note: Objects lists are cached internally until they are invalidate.
	 *
	 * @param Operand 			The movie scene operand to resolve
	 *
	 * @return Iterable list of weak object pointers pertaining to the specified GUID
	 */
	TArrayView<TWeakObjectPtr<>> FindBoundObjects(const FMovieSceneEvaluationOperand& Operand, TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
	{
		return FindBoundObjects(Operand.ObjectBindingID, Operand.SequenceID, SharedPlaybackState);
	}

	/**
	 * Invalidate any object caches that may now contain expired objects
	 */
	MOVIESCENE_API void InvalidateExpiredObjects();

	/**
	 * Forcably invalidate the specified object binding in the specified sequence
	 *
	 * @param InGuid			The object binding ID to invalidate
	 * @param InSequenceID		The sequence ID to which the object binding belongs
	 */
	MOVIESCENE_API void Invalidate(const FGuid& InGuid, FMovieSceneSequenceIDRef InSequenceID);


	/* Gets the current activation on the provided binding. If no binding lifetime track is present, true will be returned.
	 * @param InGuid			The object binding ID
	 * @param InSequenceID		The sequence ID to which the object binding belongs
	 */
	MOVIESCENE_API bool GetBindingActivation(const FGuid& InGuid, FMovieSceneSequenceIDRef InSequenceID) const;

	/* Sets the binding to either active or inactive. Inactive bindings will invalidate and not resolve while inactive. 
	 * @param InGuid			The object binding ID
	 * @param InSequenceID		The sequence ID to which the object binding belongs
	 * @param bActive		    Where to activate or deactivate the binding.
	 */
	MOVIESCENE_API void SetBindingActivation(const FGuid& InGuid, FMovieSceneSequenceIDRef InSequenceID, bool bActive);

	/**
	 * Forcably clear all object caches
	 */
	MOVIESCENE_API void ClearObjectCaches(TSharedRef<const FSharedPlaybackState> SharedPlaybackState);

	/**
	 * Get the serial number for this state.
	 */
	MOVIESCENE_API uint32 GetSerialNumber();

	/** A map of persistent evaluation data mapped by movie scene evaluation entity (i.e, a given track or section) */
	TMap<FMovieSceneEvaluationKey, TUniquePtr<IPersistentEvaluationData>> PersistentEntityData;

	/** A map of persistent evaluation data mapped by shared evaluation key. Such data can be accessed from anywhere given an operand and a unique identifier. */
	TMap<FSharedPersistentDataKey, TUniquePtr<IPersistentEvaluationData>> PersistentSharedData;

public:

	/** IPlaybackCapability members */
	MOVIESCENE_API virtual void Initialize(TSharedRef<const FSharedPlaybackState> Owner) override;
	MOVIESCENE_API virtual void OnSubInstanceCreated(TSharedRef<const FSharedPlaybackState> Owner, const UE::MovieScene::FInstanceHandle InstanceHandle) override;


private:

	void RegisterObjectCacheEvents(UMovieSceneEntitySystemLinker* Linker, const UE::MovieScene::FInstanceHandle& InstanceHandle, const FMovieSceneSequenceID SequenceID);

public:

	// Backwards compatible API, to be deprecated later

	MOVIESCENE_API void AssignSequence(FMovieSceneSequenceIDRef InSequenceID, UMovieSceneSequence& InSequence, IMovieScenePlayer& Player);
	MOVIESCENE_API FGuid FindObjectId(UObject& Object, FMovieSceneSequenceIDRef InSequenceID, IMovieScenePlayer& Player);
	MOVIESCENE_API FGuid FindCachedObjectId(UObject& Object, FMovieSceneSequenceIDRef InSequenceID, IMovieScenePlayer& Player);
	MOVIESCENE_API void FilterObjectBindings(UObject* PredicateObject, IMovieScenePlayer& Player, TArray<FMovieSceneObjectBindingID>* OutBindings);
	MOVIESCENE_API void ClearObjectCaches(IMovieScenePlayer& Player);

private:

	/** Object cache with a last known serial of it */
	struct FVersionedObjectCache
	{
		FMovieSceneObjectCache ObjectCache;
		FDelegateHandle OnInvalidateObjectBindingHandle;
		uint32 LastKnownSerial = 0;
	};

	/** Maps of bound objects, arranged by template ID */
	TMap<FMovieSceneSequenceID, FVersionedObjectCache> ObjectCaches;

	/** Current serial number of this collection of caches */
	uint32 SerialNumber = 0;
};

#undef UE_API
