// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Evaluation/MovieSceneEvaluationTrack.h"
#include "Evaluation/MovieSceneEvaluationTree.h"
#include "Evaluation/MovieSceneSegment.h"
#include "Evaluation/MovieSceneTrackIdentifier.h"
#include "Misc/Guid.h"
#include "MovieSceneFrameMigration.h"
#include "MovieSceneTrack.h"
#include "Serialization/Archive.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "MovieSceneEvaluationTemplate.generated.h"

class UMovieSceneSequence;
class UMovieSceneSubSection;
struct FMovieSceneEvaluationTrack;
struct FMovieSceneFrameRange;



USTRUCT()
struct FMovieSceneTemplateGenerationLedger
{
	GENERATED_BODY()

public:

	/**
	 * Lookup a track identifier by its originating signature
	 */
	MOVIESCENE_API FMovieSceneTrackIdentifier FindTrackIdentifier(const FGuid& InSignature) const;

	/**
	 * Add a new track for the specified signature. Signature must not have already been used
	 */
	MOVIESCENE_API void AddTrack(const FGuid& InSignature, FMovieSceneTrackIdentifier Identifier);

	/**
	 * Check whether we already have a sub section for the specified signature
	 */
	bool ContainsSubSection(const FGuid& InSignature)
	{
		return SubSectionRanges.Contains(InSignature);
	}

public:

	UPROPERTY()
	FMovieSceneTrackIdentifier LastTrackIdentifier;

	/** Map of track signature to array of track identifiers that it created */
	UPROPERTY()
	TMap<FGuid, FMovieSceneTrackIdentifier> TrackSignatureToTrackIdentifier;

	/** Map of sub section ranges that exist in a template */
	UPROPERTY()
	TMap<FGuid, FMovieSceneFrameRange> SubSectionRanges;
};
template<> struct TStructOpsTypeTraits<FMovieSceneTemplateGenerationLedger> : public TStructOpsTypeTraitsBase2<FMovieSceneTemplateGenerationLedger> { enum { WithCopy = true }; };

/** Data that represents a single sub-section */
USTRUCT()
struct FMovieSceneSubSectionData
{
	GENERATED_BODY()

	FMovieSceneSubSectionData() : Flags(ESectionEvaluationFlags::None) {}

	FMovieSceneSubSectionData(UMovieSceneSubSection& InSubSection, const FGuid& InObjectBindingId, ESectionEvaluationFlags InFlags);

	friend FArchive& operator<<(FArchive& Ar, FMovieSceneSubSectionData& In)
	{
		return Ar << In.Section << In.ObjectBindingId << In.Flags;
	}

	friend bool operator==(const FMovieSceneSubSectionData& A, const FMovieSceneSubSectionData& B)
	{
		return A.Section == B.Section && A.ObjectBindingId == B.ObjectBindingId && A.Flags == B.Flags;
	}

	/** The sub section itself */
	UPROPERTY()
	TWeakObjectPtr<UMovieSceneSubSection> Section;

	/** The object binding that the sub section belongs to (usually zero) */
	UPROPERTY()
	FGuid ObjectBindingId;

	/** Evaluation flags for the section */
	UPROPERTY()
	ESectionEvaluationFlags Flags;
};

/**
 * Sereal number used to identify evaluation template state that can only ever increase over its lifetime.
 * Only to be stored internally on FMovieSceneEvaluationTemplate.
 */
USTRUCT()
struct FMovieSceneEvaluationTemplateSerialNumber
{
	GENERATED_BODY()

	FMovieSceneEvaluationTemplateSerialNumber()
		: Value(0)
	{}

	/**
	 * Access this serial number's value
	 */
	uint32 GetValue() const
	{
		return Value;
	}

	/**
	* Increment this serial number
	*/
	void Increment()
	{
		++Value;
	}

private:
	
	friend struct FMovieSceneEvaluationTemplate;

	/**
	 * Copy/move semantics only ever initialize to zero, or passthrough, to ensure that FMovieSceneEvaluationTemplate::TemplateSerialNumber cannot ever move backwards.
	 */
	FMovieSceneEvaluationTemplateSerialNumber(const FMovieSceneEvaluationTemplateSerialNumber&) : Value(0) {}
	FMovieSceneEvaluationTemplateSerialNumber(FMovieSceneEvaluationTemplateSerialNumber&&)      : Value(0) {}

	FMovieSceneEvaluationTemplateSerialNumber& operator=(const FMovieSceneEvaluationTemplateSerialNumber&) { return *this; }
	FMovieSceneEvaluationTemplateSerialNumber& operator=(FMovieSceneEvaluationTemplateSerialNumber&&)      { return *this; }

	/** The internal value of the serial number */
	UPROPERTY()
	uint32 Value;
};
template<> struct TStructOpsTypeTraits<FMovieSceneEvaluationTemplateSerialNumber> : public TStructOpsTypeTraitsBase2<FMovieSceneEvaluationTemplateSerialNumber> { enum { WithCopy = false }; };

/**
 * Template that is used for efficient runtime evaluation of a movie scene sequence. Potentially serialized into the asset.
 */
USTRUCT()
struct FMovieSceneEvaluationTemplate
{
	GENERATED_BODY()

public:

	/**
	 * Attempt to locate a track with the specified identifier
	 */
	FMovieSceneEvaluationTrack* FindTrack(FMovieSceneTrackIdentifier Identifier)
	{
		// Fast, most common path
		if (FMovieSceneEvaluationTrack* Track = Tracks.Find(Identifier))
		{
			return Track;
		}

		return StaleTracks.Find(Identifier);
	}

	/**
	 * Attempt to locate a track with the specified identifier
	 */
	const FMovieSceneEvaluationTrack* FindTrack(FMovieSceneTrackIdentifier Identifier) const
	{
		// Fast, most common path
		if (const FMovieSceneEvaluationTrack* Track = Tracks.Find(Identifier))
		{
			return Track;
		}

		return StaleTracks.Find(Identifier);
	}

	/**
	 * Find a track within this template that relates to the specified signature
	 */
	FMovieSceneEvaluationTrack* FindTrack(const FGuid& InSignature)
	{
		return FindTrack(TemplateLedger.FindTrackIdentifier(InSignature));
	}

	/**
	 * Find a track within this template that relates to the specified signature
	 */
	const FMovieSceneEvaluationTrack* FindTrack(const FGuid& InSignature) const
	{
		return FindTrack(TemplateLedger.FindTrackIdentifier(InSignature));
	}

	/**
	 * Test whether the specified track identifier relates to a stale track
	 */
	bool IsTrackStale(FMovieSceneTrackIdentifier Identifier) const
	{
		return StaleTracks.Contains(Identifier);
	}

	/**
	 * Add a new track for the specified identifier
	 */
	MOVIESCENE_API FMovieSceneTrackIdentifier AddTrack(const FGuid& InSignature, FMovieSceneEvaluationTrack&& InTrack);

	/**
	 * Remove any tracks that correspond to the specified signature
	 */
	MOVIESCENE_API void RemoveTrack(const FGuid& InSignature);

	/**
	 * Iterate this template's tracks.
	 */
	MOVIESCENE_API const TMap<FMovieSceneTrackIdentifier, FMovieSceneEvaluationTrack>& GetTracks() const;

	/**
	 * Iterate this template's tracks (non-const).
	 * NOTE that this is intended for use during the compilation phase in-editor. 
	 * Beware of using this to modify tracks afterwards as it will almost certainly break evaluation.
	 */
	MOVIESCENE_API TMap<FMovieSceneTrackIdentifier, FMovieSceneEvaluationTrack>& GetTracks();

	/**
	 * Access this template's stale tracks.
	 */
	MOVIESCENE_API const TMap<FMovieSceneTrackIdentifier, FMovieSceneEvaluationTrack>& GetStaleTracks() const;

	/**
	 * Called after this template has been serialized in some way
	 */
#if WITH_EDITORONLY_DATA
	MOVIESCENE_API void PostSerialize(const FArchive& Ar);
#endif

	/**
	 * Purge any stale tracks we may have
	 */
	void PurgeStaleTracks()
	{
		StaleTracks.Reset();
	}

public:

	/**
	 * Get this template's generation ledger
	 */
	const FMovieSceneTemplateGenerationLedger& GetLedger() const
	{
		return TemplateLedger;
	}

	/**
	 * Remove any data within this template that does not reside in the specified set of signatures
	 */
	void RemoveStaleData(const TSet<FGuid>& ActiveSignatures);
private:

	/** Map of evaluation tracks from identifier to track */
	UPROPERTY()
	TMap<FMovieSceneTrackIdentifier, FMovieSceneEvaluationTrack> Tracks;

	/** Transient map of stale tracks. */
	TMap<FMovieSceneTrackIdentifier, FMovieSceneEvaluationTrack> StaleTracks;

public:

	UPROPERTY()
	FGuid SequenceSignature;

	/** Serial number that is incremented every time this template is re-generated through FMovieSceneEvaluationTemplateGenerator */
	UPROPERTY()
	FMovieSceneEvaluationTemplateSerialNumber TemplateSerialNumber;

private:

	UPROPERTY()
	FMovieSceneTemplateGenerationLedger TemplateLedger;

};
#if WITH_EDITORONLY_DATA
template<> struct TStructOpsTypeTraits<FMovieSceneEvaluationTemplate> : public TStructOpsTypeTraitsBase2<FMovieSceneEvaluationTemplate> { enum { WithPostSerialize = true }; };
#endif
