// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Map.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneInstanceRegistry.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "Templates/UniquePtr.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/StrongObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "MovieSceneFwd.h"
#include "MovieSceneSequenceID.h"
#include "Evaluation/MovieSceneRootOverridePath.h"
#include "Compilation/MovieSceneCompiledDataID.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "MovieSceneEvaluationTemplateInstance.generated.h"

struct FMovieSceneContext;
struct FMovieSceneSubSequenceData;
struct FMovieSceneBlendingAccumulator;

class IMovieScenePlayer;
class UMovieSceneSequence;
class FMovieSceneEntitySystemRunner;
class UMovieSceneEntitySystemLinker;
class UMovieSceneCompiledDataManager;

namespace UE { namespace MovieScene { struct FSequenceInstance; } }

/**
 * Root evaluation template instance used to play back any sequence
 */
USTRUCT()
struct FMovieSceneRootEvaluationTemplateInstance
{
public:
	GENERATED_BODY()

	MOVIESCENE_API FMovieSceneRootEvaluationTemplateInstance();

	MOVIESCENE_API ~FMovieSceneRootEvaluationTemplateInstance();

	/**
	 * Initialize this template instance with the specified sequence
	 *
	 * @param RootSequence				The sequence play back
	 * @param Player					The player responsible for playback
	 * @param TemplateStore				Template store responsible for supplying templates for a given sequence
	 */
	MOVIESCENE_API void Initialize(UMovieSceneSequence& RootSequence, IMovieScenePlayer& Player, UMovieSceneCompiledDataManager* InCompiledDataManager);

	/**
	 * Initialize this template instance as a "dormant" wrapper around an already
	 * existing shared playback state.
	 */
	MOVIESCENE_API void Initialize(TSharedRef<UE::MovieScene::FSharedPlaybackState> InSharedPlaybackState);

	/**
	 * Evaluate this sequence in a synchronous way.
	 *
	 * @param Context				Evaluation context containing the time (or range) to evaluate
	 */
	MOVIESCENE_API void EvaluateSynchronousBlocking(FMovieSceneContext Context);

	UE_DEPRECATED(5.4, "Use the version without the player parameter")
	MOVIESCENE_API void EvaluateSynchronousBlocking(FMovieSceneContext Context, IMovieScenePlayer& Player);

	UE_DEPRECATED(5.4, "Director instances are now auto-managed via FSequenceDirectorPlaybackCapability")
	MOVIESCENE_API void ResetDirectorInstances();

	MOVIESCENE_API bool IsValid() const;

	MOVIESCENE_API TSharedPtr<UE::MovieScene::FSharedPlaybackState> GetSharedPlaybackState();

	MOVIESCENE_API TSharedPtr<const UE::MovieScene::FSharedPlaybackState> GetSharedPlaybackState() const;

	MOVIESCENE_API UE::MovieScene::FRootInstanceHandle GetRootInstanceHandle() const;

	MOVIESCENE_API UMovieSceneSequence* GetRootSequence() const;

	MOVIESCENE_API FMovieSceneCompiledDataID GetCompiledDataID() const;

	MOVIESCENE_API UMovieSceneCompiledDataManager* GetCompiledDataManager() const;

	MOVIESCENE_API TSharedPtr<FMovieSceneEntitySystemRunner> GetRunner() const;

	MOVIESCENE_API bool HasEverUpdated() const;

	MOVIESCENE_API UMovieSceneEntitySystemLinker* GetEntitySystemLinker() const;

	MOVIESCENE_API const FMovieSceneSequenceHierarchy* GetHierarchy() const;

	MOVIESCENE_API void GetSequenceParentage(const UE::MovieScene::FInstanceHandle InstanceHandle, TArray<UE::MovieScene::FInstanceHandle>& OutParentHandles) const;

	MOVIESCENE_API const UE::MovieScene::FSequenceInstance* GetRootInstance() const;

	MOVIESCENE_API UE::MovieScene::FSequenceInstance* FindInstance(FMovieSceneSequenceID SequenceID);

	MOVIESCENE_API const UE::MovieScene::FSequenceInstance* FindInstance(FMovieSceneSequenceID SequenceID) const;

	MOVIESCENE_API UE::MovieScene::FMovieSceneEntityID FindEntityFromOwner(UObject* Owner, uint32 EntityID, FMovieSceneSequenceID SequenceID) const;
	MOVIESCENE_API void FindEntitiesFromOwner(UObject* Owner, FMovieSceneSequenceID SequenceID, TArray<UE::MovieScene::FMovieSceneEntityID>& OutEntityIDs) const;

	MOVIESCENE_API UMovieSceneSequence* GetSequence(FMovieSceneSequenceIDRef SequenceID) const;

	UE_DEPRECATED(5.4, "Director instances are now auto-managed via FSequenceDirectorPlaybackCapability")
	MOVIESCENE_API UObject* GetOrCreateDirectorInstance(FMovieSceneSequenceIDRef SequenceID, IMovieScenePlayer& Player);

	MOVIESCENE_API void PlaybackContextChanged(IMovieScenePlayer& Player);

	MOVIESCENE_API const FMovieSceneSubSequenceData* FindSubData(FMovieSceneSequenceIDRef SequenceID) const;

	MOVIESCENE_API void CopyActuators(FMovieSceneBlendingAccumulator& Accumulator) const;

	MOVIESCENE_API void TearDown();

	MOVIESCENE_API void EnableGlobalPreAnimatedStateCapture();

#if WITH_EDITOR
	MOVIESCENE_API void SetEmulatedNetworkMask(EMovieSceneServerClientMask InNewMask);
	MOVIESCENE_API EMovieSceneServerClientMask GetEmulatedNetworkMask() const;

	UE_DEPRECATED(5.4, "Please use the version without the player parameter")
	MOVIESCENE_API void SetEmulatedNetworkMask(EMovieSceneServerClientMask InNewMask, IMovieScenePlayer& Player);
#endif

private:

	static UMovieSceneEntitySystemLinker* ConstructEntityLinker(IMovieScenePlayer& Player);

private:

	/** The linker that we are using, cached from SharedPlaybackState. */
	UPROPERTY()
	TObjectPtr<UMovieSceneEntitySystemLinker> EntitySystemLinker;

	/** The playback state for the hierarchy of sequence instances. */
	TSharedPtr<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState;

	/** Whether we should destroy the sequence hierarchy when we are destroyed. */
	bool bOwnsSharedPlaybackState;

#if WITH_EDITOR
	EMovieSceneServerClientMask EmulatedNetworkMask;
#endif
};

template<>
struct TStructOpsTypeTraits<FMovieSceneRootEvaluationTemplateInstance> : TStructOpsTypeTraitsBase2<FMovieSceneRootEvaluationTemplateInstance>
{
	enum { WithCopy = false };
};

