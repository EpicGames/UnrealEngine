// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compilation/MovieSceneSegmentCompiler.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Evaluation/MovieSceneEvalTemplate.h"
#include "Evaluation/MovieSceneEvaluationField.h"
#include "Evaluation/MovieScenePlayback.h"
#include "Evaluation/MovieSceneSegment.h"
#include "Evaluation/MovieSceneTrackImplementation.h"
#include "Evaluation/PersistentEvaluationData.h"
#include "HAL/Platform.h"
#include "Misc/Guid.h"
#include "Misc/InlineValue.h"
#include "MovieSceneTrack.h"
#include "Templates/EnableIf.h"
#include "Templates/PointerIsConvertibleFromTo.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "MovieSceneEvaluationTrack.generated.h"

class FArchive;
class IMovieScenePlayer;
class UObject;
struct FMovieSceneEvaluationOperand;
struct FMovieSceneExecutionTokens;
struct FMovieSceneFieldEntry_ChildTemplate;
struct FMovieSceneInterrogationData;

/** Enumeration to determine how a track should be evaluated */
UENUM()
enum class EEvaluationMethod : uint8
{
	/** Evaluation only ever occurs at a single time. Delta is irrelevant. Example: Basic curve eval, animation */
	Static,

	/** Evaluation from one frame to the next must consider the entire swept delta range on the track. Example: Events */
	Swept,
};

/**
 * Evaluation track that is stored within an evaluation template for a sequence.
 * Contains user-defined evaluation templates, and an optional track implementation
 */
USTRUCT()
struct FMovieSceneEvaluationTrack
{
	GENERATED_BODY()

	/** Default construction (only for serialization) */
	FMovieSceneEvaluationTrack();

	/**
	 * User construction, for initialization during compilation
	 */
	MOVIESCENE_API FMovieSceneEvaluationTrack(const FGuid& InObjectBindingID);

	/** Copy construction/assignment */
	FMovieSceneEvaluationTrack(const FMovieSceneEvaluationTrack&) = default;
	FMovieSceneEvaluationTrack& operator=(const FMovieSceneEvaluationTrack&) = default;

	/** Move construction/assignment */
	FMovieSceneEvaluationTrack(FMovieSceneEvaluationTrack&&) = default;
	FMovieSceneEvaluationTrack& operator=(FMovieSceneEvaluationTrack&&) = default;

public:

	/**
	 * Get the object binding ID that this track belongs to
	 */
	inline const FGuid& GetObjectBindingID() const
	{
		return ObjectBindingID;
	}

	/**
	 * Get this track's child templates
	 * NOTE that this is intended for use during the compilation phase in-editor.
	 * Beware of using this to modify templates afterwards as it will almost certainly break evaluation.
	 */
	inline TArrayView<FMovieSceneEvalTemplatePtr> GetChildTemplates()
	{
		return TArrayView<FMovieSceneEvalTemplatePtr>(ChildTemplates.GetData(), ChildTemplates.Num());
	}

	/**
	 * Get this track's child templates
	 */
	inline TArrayView<const FMovieSceneEvalTemplatePtr> GetChildTemplates() const
	{
		return TArrayView<const FMovieSceneEvalTemplatePtr>(ChildTemplates.GetData(), ChildTemplates.Num());
	}

	/**
	 * Get the template from the given template index
	 */
	inline const FMovieSceneEvalTemplate& GetChildTemplate(int32 TemplateIndex) const
	{
		return *ChildTemplates[TemplateIndex];
	}

	/**
	 * Check whether we have a valid child template for the specified index
	 */
	inline bool HasChildTemplate(int32 TemplateIndex) const
	{
		return ChildTemplates.IsValidIndex(TemplateIndex) && ChildTemplates[TemplateIndex].IsValid();
	}

public:

	/**
	 * Get this track's evaluation group name. Only used during compilation.
	 *
	 * @return The evaluation group
	 */
	FName GetEvaluationGroup() const
	{
		return EvaluationGroup;
	}

	/**
	 * Set this track's flush group name.
	 *
	 * @note 	When not 'None', setting an evaluation group indicates that all tracks with similar
	 *			groups and priorities should be grouped together at runtime. Named groups can
	 *			be optionally flushed immediately at runtime by calling IMovieSceneTemplateGenerator::FlushGroupImmediately
	 *			with the appropriate group.
	 *
	 * @param InEvaluationGroup 		The evaluation group to assign this track to
	 */
	void SetEvaluationGroup(FName InEvaluationGroup)
	{
		EvaluationGroup = InEvaluationGroup;
	}

public:

	/**
	 * Get the evaluation bias to apply to this track. Higher priority tracks will be evaluated first.
	 *
	 * @return The evaluation priority
	 */
	uint16 GetEvaluationPriority() const
	{
		return EvaluationPriority;
	}

	/**
	 * Get the evaluation bias to apply to this track. Higher priority tracks will be evaluated first.
	 *
	 * @param InEvaluationPriority			The new priority
	 */
	void SetEvaluationPriority(uint16 InEvaluationPriority = 1000)
	{
		EvaluationPriority = InEvaluationPriority;
	}

	/**
	 * Get the method we should use to evaluate this track
	 *
	 * @return The method to use when evaluating this track
	 */
	EEvaluationMethod GetEvaluationMethod() const
	{
		return EvaluationMethod;
	}

	/**
	 * Set the method we should use to evaluate this track
	 *
	 * @param InMethod				The method to use when evaluating this track
	 */
	void SetEvaluationMethod(EEvaluationMethod InMethod)
	{
		EvaluationMethod = InMethod;
	}

	/**
	 * Define how this track evaluates in pre and postroll
	 *
	 * @param bInEvaluateInPreroll	Whether this track should evaluate in preroll
	 * @param bInEvaluateInPostroll	Whether this track should evaluate in postroll
	 */
	void SetPreAndPostrollConditions(bool bInEvaluateInPreroll, bool bInEvaluateInPostroll)
	{
		bEvaluateInPreroll = bInEvaluateInPreroll;
		bEvaluateInPostroll = bInEvaluateInPostroll;
	}

	/**
	 * @return Whether this track should evaluate in preroll
	 */
	bool ShouldEvaluateInPreroll() const
	{
		return bEvaluateInPreroll;
	}

	/**
	 * @return Whether this track should evaluate in postroll
	 */
	bool ShouldEvaluateInPostroll() const
	{
		return bEvaluateInPostroll;
	}

	/**
	 * Tell this track to prioritize its tear down over other tracks, regardless of evaluation priority
	 */
	void PrioritizeTearDown()
	{
		bTearDownPriority = true;
	}

	/**
	 * Check whether this track has tear down priority or not
	 */
	bool HasTearDownPriority() const
	{
		return bTearDownPriority;
	}

public:

	/**
	 * Called to initialize the specified segment index
	 * 
	 * @param SegmentID				The segment we are evaluating
	 * @param Operand				Operand that relates to the thing we will animate
	 * @param Context				Current sequence context
	 * @param PersistentData		Persistent data store
	 * @param Player				The player that is responsible for playing back this template
	 */
	void Initialize(TArrayView<const FMovieSceneFieldEntry_ChildTemplate> Children, const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) const;

	/**
	 * Called to evaluate the specified segment index
	 * 
	 * @param SegmentID				The segment we are evaluating
	 * @param Operand				Operand that relates to the thing we will animate
	 * @param Context				Current sequence context
	 * @param PersistentData		Persistent data store
	 * @param ExecutionTokens		Token stack on which to add tokens that will be executed later
	 */
	void Evaluate(TArrayView<const FMovieSceneFieldEntry_ChildTemplate> Children, const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const;

	/**
	 * Default implementation of initialization of child templates for the specified segment
	 * 
	 * @param SegmentID				The segment we are evaluating
	 * @param Operand				Operand that relates to the thing we will animate
	 * @param Context				Current sequence context
	 * @param PersistentData		Persistent data store
	 * @param Player				The player that is responsible for playing back this template
	 */
	MOVIESCENE_API void DefaultInitialize(TArrayView<const FMovieSceneFieldEntry_ChildTemplate> Children, const FMovieSceneEvaluationOperand& Operand, FMovieSceneContext Context, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) const;

	/**
	 * Default implementation of evaluation of child templates for the specified segment
	 * 
	 * @param SegmentID				The segment we are evaluating
	 * @param Operand				Operand that relates to the thing we will animate
	 * @param Context				Current sequence context
	 * @param PersistentData		Persistent data store
	 * @param ExecutionTokens		Token stack on which to add tokens that will be executed later
	 */
	MOVIESCENE_API void DefaultEvaluate(TArrayView<const FMovieSceneFieldEntry_ChildTemplate> Children, const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const;

	/**
	 * Interrogate this template for its output. Should not have any side effects.
	 *
	 * @param Context				Evaluation context specifying the current evaluation time, sub sequence transform and other relevant information.
	 * @param Container				Container to populate with the desired output from this track
	 * @param BindingOverride		Optional binding to specify the object that is being animated by this track
	 */
	MOVIESCENE_API  void Interrogate(const FMovieSceneContext& Context, FMovieSceneInterrogationData& Container, UObject* BindingOverride = nullptr) const;

private:

	/**
	 * Implementation function for static evaluation
	 */
	void EvaluateStatic(TArrayView<const FMovieSceneFieldEntry_ChildTemplate> Children, const FMovieSceneEvaluationOperand& Operand, FMovieSceneContext Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const;

	/**
	 * Implementation function for swept evaluation
	 */
	void EvaluateSwept(TArrayView<const FMovieSceneFieldEntry_ChildTemplate> Children, const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const;

public:

	/**
	 * Assign a single eval template to this track, spanning the entire sequence
	 * 
	 * @param InTemplate		The template to insert
	 */
	MOVIESCENE_API void DefineAsSingleTemplate(FMovieSceneEvalTemplatePtr&& InTemplate);

	/**
	 * Add an evaluation template to this track with the given track index
	 * 
	 * @param InTemplate		The template to insert
	 * @return The index of the new template
	 */
	MOVIESCENE_API int32 AddChildTemplate(FMovieSceneEvalTemplatePtr&& InTemplate);

	/**
	 * Set the source track from which this track originates
	 */
	void SetSourceTrack(UMovieSceneTrack* InSourceTrack)
	{
		SourceTrack = InSourceTrack;
	}

	/**
	 * Get the source track from which this track originates
	 */
	UMovieSceneTrack* GetSourceTrack() const
	{
		return SourceTrack.Get();
	}

	/**
	 * Assign a track implementation template to this track
	 * @note Track implementations are evaluated once per frame before any segments.
	 *
	 * @param InImplementation	The track implementation to use
	 */
	template<typename T>
	typename TEnableIf<TPointerIsConvertibleFromTo<T, FMovieSceneTrackImplementation>::Value>::Type 
		SetTrackImplementation(T&& InImpl)
	{
		TrackTemplate = FMovieSceneTrackImplementationPtr(MoveTemp(InImpl));
		TrackTemplate->SetupOverrides();
	}

	/**
	 * Setup overrides for any contained templates
	 */
	MOVIESCENE_API void SetupOverrides();

	/**
	 * Post serialize function
	 */
	MOVIESCENE_API void PostSerialize(const FArchive& Ar);

	bool HasCustomInitialize() const
	{
		return TrackTemplate.IsValid() && TrackTemplate->HasCustomInitialize();
	}

public:

	/**
	 * Called before this track is evaluated for the first time, or since OnEndEvaluation has been called
	 *
	 * @param PersistentData		Persistent data proxy that may contain data pertaining to this entity
	 * @param Player				The player that is responsible for playing back this template
	 */
	inline void OnBeginEvaluation(FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) const
	{
		if (!TrackTemplate.IsValid())
		{
			return;
		}
		TrackTemplate->OnBeginEvaluation(PersistentData, Player);
	}

	/**
	 * Called after this track is no longer being evaluated
	 *
	 * @param PersistentData		Persistent data proxy that may contain data pertaining to this entity
	 * @param Player				The player that is responsible for playing back this template
	 */
	void OnEndEvaluation(FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) const
	{
		if (!TrackTemplate.IsValid())
		{
			return;
		}
		TrackTemplate->OnEndEvaluation(PersistentData, Player);
	}

private:

	/** ID of the possessable or spawnable within the UMovieScene this track belongs to, if any. Zero guid where this relates to a root track. */
	UPROPERTY()
	FGuid ObjectBindingID;

	/** Evaluation priority. Highest is evaluated first */
	UPROPERTY()
	uint16 EvaluationPriority;

	/** Evaluation method - static or swept */
	UPROPERTY()
	EEvaluationMethod EvaluationMethod;

	/** The movie scene track that created this evaluation track. */
	UPROPERTY()
	TWeakObjectPtr<UMovieSceneTrack> SourceTrack;

	/** Domain-specific evaluation templates (normally 1 per section) */
	UPROPERTY()
	TArray<FMovieSceneEvalTemplatePtr> ChildTemplates;

	/** Domain-specific track implementation override. */
	UPROPERTY()
	FMovieSceneTrackImplementationPtr TrackTemplate;

	/** Flush group that determines whether this track belongs to a group of tracks */
	UPROPERTY()
	FName EvaluationGroup;

	/** Whether this track is evaluated in preroll */
	UPROPERTY()
	uint32 bEvaluateInPreroll : 1;

	/** Whether this track is evaluated in postroll */
	UPROPERTY()
	uint32 bEvaluateInPostroll : 1;

	/** Whether track should be given priority when being torn down */
	UPROPERTY()
	uint32 bTearDownPriority : 1;
};

template<> struct TStructOpsTypeTraits<FMovieSceneEvaluationTrack> : public TStructOpsTypeTraitsBase2<FMovieSceneEvaluationTrack> { enum { WithPostSerialize = true, WithCopy = false }; };

#if WITH_DEV_AUTOMATION_TESTS

struct FScopedOverrideTrackSegmentBlender
{
	MOVIESCENE_API FScopedOverrideTrackSegmentBlender(FMovieSceneTrackSegmentBlenderPtr&& InTrackSegmentBlender);
	MOVIESCENE_API ~FScopedOverrideTrackSegmentBlender();

	FScopedOverrideTrackSegmentBlender(const FScopedOverrideTrackSegmentBlender&) = delete;
	FScopedOverrideTrackSegmentBlender& operator=(const FScopedOverrideTrackSegmentBlender&) = delete;
};

#endif
