// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Math/Range.h"
#include "Math/RangeBound.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/FrameNumber.h"
#include "Serialization/Archive.h"
#include "Templates/Function.h"
#include "Templates/MemoryOps.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/SequencerObjectVersion.h"

struct FMovieSceneEvaluationTree;
struct FMovieSceneEvaluationTreeNode;
struct FMovieSceneEvaluationTreeRangeIterator;
template<typename DataType> struct TMovieSceneEvaluationTreeDataIterator;
template<typename DataType> struct TMovieSceneEvaluationTreeFormatter;

/** A structure that uniquely identifies an entry within a TEvaluationTreeEntryContainer */
struct FEvaluationTreeEntryHandle
{
	/**
	 * Default construction to an invalid handle
	 */
	FEvaluationTreeEntryHandle()
		: EntryIndex(INDEX_NONE)
	{}

	/**
	 * Check whether this identifier has been initialized. Caution: does not check that the identifier is valid within a given TEvaluationTreeEntryContainer
	 */
	bool IsValid() const
	{
		return EntryIndex != INDEX_NONE;
	}

	/**
	 * Equality operator
	 */
	friend bool operator==(const FEvaluationTreeEntryHandle& A, const FEvaluationTreeEntryHandle& B) { return A.EntryIndex == B.EntryIndex; }

	/**
	 * Inequality operator
	 */
	friend bool operator!=(const FEvaluationTreeEntryHandle& A, const FEvaluationTreeEntryHandle& B) { return A.EntryIndex != B.EntryIndex; }

	/**
	 * Serialization operator
	 */
	friend FArchive& operator<<(FArchive& Ar, FEvaluationTreeEntryHandle& In) { return Ar << In.EntryIndex; }

private:
	template<typename T> friend struct TEvaluationTreeEntryContainer;
	template<typename T> friend struct TMovieSceneEvaluationTreeFormatter;
	FEvaluationTreeEntryHandle(int32 InEntryIndex) : EntryIndex(InEntryIndex) {}

	/** Specifies an index into TEvaluationTreeEntryContainer<T>::Entries */
	int32 EntryIndex;
};

/**
 * Sub-divided container type that allocates smaller 'buckets' of capacity within a single allocation.
 * New entries, or entries needing additional capacity are reallocated at the end of the array to avoid reallocation or shuffling.
 * Memory footprint can be compacted once the container is fully built using TEvaluationTreeEntryContainer::Compact
 * Designed specifically to solve allocation cost incurred when compiling very large trees of sequence data.
 * Restrictions: Container can only ever add entries, never remove
 */
template<typename ElementType>
struct TEvaluationTreeEntryContainer
{
	/**
	 * Allocate a new entry of the specified capacity. Will default-construct Capacity elements in the item array.
	 *
	 * @param InitialCapacity        The initial capacity of the entry
	 * @return The identifier to be used to access the entry
	 */
	FEvaluationTreeEntryHandle AllocateEntry(int32 InitialCapacity = 2);

	/**
	 * Access the entry contents corresponding to the specified ID
	 * @return An array view for the currently allocated elements, not including spare capacity
	 */
	TArrayView<ElementType> Get(FEvaluationTreeEntryHandle ID);

	/**
	 * Const access to the entry contents corresponding to the specified ID
	 * @return A const array view for the currently allocated elements, not including spare capacity
	 */
	TArrayView<const ElementType> Get(FEvaluationTreeEntryHandle ID) const;

	/**
	 * Add a new element to the entry contents with the specified identifier
	 *
	 * @param ID           The entry identifier, obtained through AllocateEntry
	 * @param Element      The element to add
	 */
	void Add(FEvaluationTreeEntryHandle ID, ElementType&& Element);

	/**
	 * Insert a new element to the entry with the specified identifier at a specific index within the entry
	 *
	 * @param ID           The entry identifier, obtained through AllocateEntry
	 * @param Index        The index within the entry at which to insert the new element
	 * @param Element      The element to add
	 */
	void Insert(FEvaluationTreeEntryHandle ID, int32 Index, ElementType&& Element);

	/**
	 * Compress the item array to remove any unused capacity.
	 */
	void Compact();


	/**
	 * Reset this container to its default state
	 */
	void Reset()
	{
		Entries.Reset();
		Items.Reset();
	}

	/**
	 * Serialize this container
	 */
	friend FArchive& operator<<(FArchive& Ar, TEvaluationTreeEntryContainer& In)
	{
		if (Ar.IsSaving())
		{
			In.Compact();
		}

		Ar.UsingCustomVersion(FSequencerObjectVersion::GUID);
		return Ar << In.Entries << In.Items;
	}

	/**
	 * Equality operator
	 */
	friend bool operator==(const TEvaluationTreeEntryContainer<ElementType>& A, const TEvaluationTreeEntryContainer<ElementType>& B)
	{
		return A.Entries == B.Entries && A.Items == B.Items;
	}

private:

	/**
	 * Reserves the entry with the specified identifier to a new capacity
	 *
	 * @param ID           The entry identifier, obtained through AllocateEntry
	 * @param NewCapacity  The new capacity to hold in the entry
	 */
	void ReserveEntry(FEvaluationTreeEntryHandle ID, int32 NewCapacity);

	struct FEntry
	{
		/** The index into Items of the first item */
		int32 StartIndex;
		/** The number of currently valid items */
		int32 Size;
		/** The total capacity of allowed items before reallocating */
		int32 Capacity;

		friend FArchive& operator<<(FArchive& Ar, FEntry& In)
		{
			Ar << In.StartIndex << In.Size << In.Capacity;
			return Ar;
		}

		/**
		 * Equality operator
		 */
		friend bool operator==(const FEntry& A, const FEntry& B)
		{
			return A.StartIndex == B.StartIndex && A.Size == B.Size && A.Capacity == B.Capacity;
		}
	};

	/** List of allocated entries for each allocated entry. Should only ever grow, never shrink. Shrinking would cause previously established handles to become invalid. */
	TArray<FEntry> Entries;
	/** Linear array of allocated entry contents. Once allocated, indices are never invalidated until Compact is called. Entries needing more capacity are re-allocated on the end of the array. */
	TArray<ElementType> Items;
};

/**
 * Interface used for performing an abstract operation on an FMovieSceneEvaluationTreeNode.
 */
struct IMovieSceneEvaluationTreeNodeOperator
{
	/**
	 * Virtual destruction
	 */
	virtual ~IMovieSceneEvaluationTreeNodeOperator() = default;

	/**
	 * Called to invoke the operator for the specified node
	 *
	 * @param Node 			The node on which the operator should be performed
	 */
	virtual void operator()(FMovieSceneEvaluationTreeNode& Node) const = 0;
};

/**
 * A handle to a node in an FMovieSceneEvaluationTree, defined in terms of an entry handle (corrsponding to FMovieSceneEvaluationTree::ChildNodes), and its child index
 */
struct FMovieSceneEvaluationTreeNodeHandle
{
	/** Construction from the node's parent's children entry handle, and this node's index within its parent's children */
	FMovieSceneEvaluationTreeNodeHandle(FEvaluationTreeEntryHandle InChildrenHandle, int32 InIndex)
		: ChildrenHandle(InChildrenHandle), Index(InIndex)
	{}

	/** Special handle that represents the root node */
	static FMovieSceneEvaluationTreeNodeHandle Root()
	{
		return FMovieSceneEvaluationTreeNodeHandle(FEvaluationTreeEntryHandle(), 0);
	}

	/** Special handle that represents an invalid node */
	static FMovieSceneEvaluationTreeNodeHandle Invalid()
	{
		return FMovieSceneEvaluationTreeNodeHandle(FEvaluationTreeEntryHandle(), INDEX_NONE);
	}

	/** Check whether this node is the root node or not */
	bool IsRoot() const
	{
		return *this == Root();
	}

	/** Check whether this node should correspond to a node. Does not check whether this handle is actually valid for any given tree */
	bool IsValid() const
	{
		return ChildrenHandle.IsValid() || Index == 0;
	}

	/** Access the children handle that this node is contained within */
	FEvaluationTreeEntryHandle GetHandle() const
	{
		return ChildrenHandle;
	}

	/** Access the index of this node amongst its siblings */
	int32 GetChildIndex() const
	{
		return Index;
	}

private:

	/** Test two node handles for equality */
	friend bool operator==(const FMovieSceneEvaluationTreeNodeHandle& A, const FMovieSceneEvaluationTreeNodeHandle& B) { return A.ChildrenHandle == B.ChildrenHandle && A.Index == B.Index; }
	/** Test two node handles for inequality */
	friend bool operator!=(const FMovieSceneEvaluationTreeNodeHandle& A, const FMovieSceneEvaluationTreeNodeHandle& B) { return A.ChildrenHandle != B.ChildrenHandle || A.Index != B.Index; }
	/** Serialization operator */
	friend FArchive& operator<<(FArchive& Ar, FMovieSceneEvaluationTreeNodeHandle& In) { return Ar << In.ChildrenHandle << In.Index; }

	/** Entry handle for the parent's children in FMovieSceneEvaluationTree::ChildNodes */
	FEvaluationTreeEntryHandle ChildrenHandle;
	/** The index of this child within its parent's children */
	int32 Index;
};

/**
 * A tree node that represents a unique time range in the evaluation tree (or a grouping of smaller sub-divisions)
 */
struct FMovieSceneEvaluationTreeNode
{
	/** Constructors */
	FMovieSceneEvaluationTreeNode()
		: Range(TRange<FFrameNumber>::Empty())
		, Parent(FMovieSceneEvaluationTreeNodeHandle::Invalid())
	{}

	FMovieSceneEvaluationTreeNode(TRange<FFrameNumber> InRange, FMovieSceneEvaluationTreeNodeHandle InParent)
		: Range(InRange)
		, Parent(InParent)
	{}

	/** Serialization operator */
	friend FArchive& operator<<(FArchive& Ar, FMovieSceneEvaluationTreeNode& In)
	{
		return Ar << In.Range << In.Parent << In.ChildrenID << In.DataID;
	}

	/**
	 * Equality operator
	 */
	friend bool operator==(const FMovieSceneEvaluationTreeNode& A, const FMovieSceneEvaluationTreeNode& B)
	{
		return A.Range == B.Range && A.Parent == B.Parent && A.ChildrenID == B.ChildrenID && A.DataID == B.DataID;
	}

	/** The time-range that this node represents */
	TRange<FFrameNumber> Range;
	/** Handle to the parent node */
	FMovieSceneEvaluationTreeNodeHandle Parent;
	/** Identifier for the child node entries associated with this node (FMovieSceneEvaluationTree::ChildNodes) */
	FEvaluationTreeEntryHandle ChildrenID;
	/** Identifier for externally stored data entries associated with this node */
	FEvaluationTreeEntryHandle DataID;
};

/**
 * A tree structure used to efficiently reference overlapping time ranges hierarchically.
 * Each node represents a unique combination of 'entities' overlapping for a given range.
 * Example structure (dependent on order of addition):
 *		Time				-inf				10							20				25					30							inf
 *												[============= 0 ===========]
 *																			 [============= 1 ==================]
 *												[========================== 2 ==================================]
 *							[================== 3 ==========================]
 *																							[================== 4 ==========================]
 *														[===== 5 ======]
 *	----------------------------------------------------------------------------------------------------------------------------------------
 * Where each time range is added in order, this is represented as:
 *							[======== 3 =======][========== 0,2,3 ==========][============= 1,2 ================][============ 4 ===========]
 *															|									\
 *														[===== 5 ====]						[======== 4 ========]
 *	----------------------------------------------------------------------------------------------------------------------------------------
 *		Unique Ranges		[ 		3			| 0,2,3 |  0,2,3,5   | 0,2,3 |		1,2		|		1,2,4		|			4				]
 */
struct FMovieSceneEvaluationTree
{
	FMovieSceneEvaluationTree()
		: RootNode(FMovieSceneEvaluationTreeNode(TRange<FFrameNumber>::All(), FMovieSceneEvaluationTreeNodeHandle::Invalid()))
	{}

	FMovieSceneEvaluationTree(const FMovieSceneEvaluationTree& RHS) = default;

	FMovieSceneEvaluationTree& operator=(const FMovieSceneEvaluationTree& RHS) = default;

	/** Move construction must ensure that the root node is correctly restored to its default to ensure that invariant */
	FMovieSceneEvaluationTree(FMovieSceneEvaluationTree&& RHS)
		: RootNode(RHS.RootNode)
		, ChildNodes(MoveTemp(RHS.ChildNodes))
	{
		RHS.RootNode = FMovieSceneEvaluationTreeNode(TRange<FFrameNumber>::All(), FMovieSceneEvaluationTreeNodeHandle::Invalid());
	}

	/** Move assignment must ensure that the root node is correctly restored to its default to ensure that invariant */
	FMovieSceneEvaluationTree& operator=(FMovieSceneEvaluationTree&& RHS)
	{
		RootNode = RHS.RootNode;
		ChildNodes = MoveTemp(RHS.ChildNodes);
		RHS.RootNode = FMovieSceneEvaluationTreeNode(TRange<FFrameNumber>::All(), FMovieSceneEvaluationTreeNodeHandle::Invalid());
		return *this;
	}

	/**
	 * Start iterating this tree from the specified time
	 * 
	 * @param Time The time at which we should start iterating
	 * @return A bi-directional iterator that is set to the unique range that overlaps the current time
	 */
	MOVIESCENE_API FMovieSceneEvaluationTreeRangeIterator IterateFromTime(FFrameNumber Time) const;

	/**
	 * Start iterating this tree from the specified lower boundary
	 * 
	 * @param InStartingLowerBound The lowerbound at which we should start iterating
	 * @return A bi-directional iterator that is set to the unique range that overlaps the specified lowerbound
	 */
	MOVIESCENE_API FMovieSceneEvaluationTreeRangeIterator IterateFromLowerBound(TRangeBound<FFrameNumber> InStartingLowerBound) const;

	/**
	 * Access this tree's root node (infinite range)
	 *
	 * @return The root node
	 */
	const FMovieSceneEvaluationTreeNode& GetRootNode() const
	{
		return RootNode;
	}

	/**
	 * Non-const access to a node from its handle. Handle must be valid.
	 * @return A reference to the node that is valid as long as the tree remains unmodified
	 */
	FMovieSceneEvaluationTreeNode& GetNode(FMovieSceneEvaluationTreeNodeHandle Handle)
	{
		check(IsValid(Handle));
		return GetNode(Handle.GetHandle(), Handle.GetChildIndex());
	}

	/**
	 * Const access to a node from its handle. Handle must be valid.
	 * @return A reference to the node that is valid as long as the tree remains unmodified
	 */
	const FMovieSceneEvaluationTreeNode& GetNode(FMovieSceneEvaluationTreeNodeHandle Handle) const
	{
		check(IsValid(Handle));
		return GetNode(Handle.GetHandle(), Handle.GetChildIndex());
	}

	/**
	 * Get the children associated with the specified node
	 * 
	 * @param InNode		The node to get children for
	 * @return Array view of the node's children
	 */
	MOVIESCENE_API TArrayView<const FMovieSceneEvaluationTreeNode> GetChildren(const FMovieSceneEvaluationTreeNode& InNode) const;

	/**
	 * Get the children associated with the specified node
	 * 
	 * @param InNode		The node to get children for
	 * @return Array view of the node's children
	 */
	MOVIESCENE_API TArrayView< FMovieSceneEvaluationTreeNode> GetChildren(const FMovieSceneEvaluationTreeNode& InNode);

	/**
	 * Check whether the specified handle corresponds to a node within this tree
	 * 
	 * @param InHandle		Node handle to test
	 * @return true if the handle is valid, false otherwise
	 */
	bool IsValid(FMovieSceneEvaluationTreeNodeHandle Handle) const
	{
		return Handle.GetHandle().IsValid() ? ChildNodes.Get(Handle.GetHandle()).IsValidIndex(Handle.GetChildIndex()) : Handle.GetChildIndex() == 0;
	}

	void Reset()
	{
		RootNode.DataID = RootNode.ChildrenID = FEvaluationTreeEntryHandle();
		ChildNodes.Reset();
	}

	/**
	 * Insert the specified time range into this tree
	 *
	 * @param InTimeRange 		The time range to add to the tree
	 */
	MOVIESCENE_API void AddTimeRange(TRange<FFrameNumber> InTimeRange);

	/**
	 * Serialize this evaluation tree
	 */
	friend FArchive& operator<<(FArchive& Ar, FMovieSceneEvaluationTree& In)
	{
		return Ar << In.RootNode << In.ChildNodes;
	}

protected:

	/**
	 * Non-const access to a node from its parent's ChildrenID and this node's index.
	 * @return A reference to the node that is valid as long as the tree remains unmodified
	 */
	FMovieSceneEvaluationTreeNode& GetNode(FEvaluationTreeEntryHandle ChildrenID, int32 Index)
	{
		return ChildrenID.IsValid() ? ChildNodes.Get(ChildrenID)[Index] : RootNode;
	}

	/**
	 * Const access to a node from its parent's ChildrenID and this node's index.
	 * @return A reference to the node that is valid as long as the tree remains unmodified
	 */
	const FMovieSceneEvaluationTreeNode& GetNode(FEvaluationTreeEntryHandle ChildrenID, int32 Index) const
	{
		return ChildrenID.IsValid() ? ChildNodes.Get(ChildrenID)[Index] : RootNode;
	}

	/**
	 * Insert the specified time range into this tree
	 *
	 * @param InTimeRange 		The time range to add to the tree
	 * @param InOperator 		Operator implementation to call for each node that is relevant to the time range
	 * @param InParent 			The current parent node
	 * @param InPredicate 		(Optional) Predicate that determines whether the time should be added for a particular point
	 */
	MOVIESCENE_API void AddTimeRange(TRange<FFrameNumber> InTimeRange, const IMovieSceneEvaluationTreeNodeOperator& InOperator, FMovieSceneEvaluationTreeNodeHandle InParent, const TFunctionRef<bool(FMovieSceneEvaluationTreeNodeHandle)>* Predicate);

	/**
	 * Helper function that creates a new child for the specified parent node
	 *
	 * @param InEffectiveRange 	The time range that this child should represent. Must not overlap any other child's time range.
	 * @param InOperator 		Operator implementation to call for the new child node
	 * @param InsertIndex		The index at which to insert the new child (children must be sorted)
	 * @param InParent 			The node to add the child to
	 */
	MOVIESCENE_API void InsertNewChild(TRange<FFrameNumber> InEffectiveRange, const IMovieSceneEvaluationTreeNodeOperator& InOperator, int32 InsertIndex, FMovieSceneEvaluationTreeNodeHandle InParent);

	/** This tree's root node */
	FMovieSceneEvaluationTreeNode RootNode;

	/** Segmented array of all child nodes within this tree (in no particular order) */
	TEvaluationTreeEntryContainer<FMovieSceneEvaluationTreeNode> ChildNodes;
};

/**
 * Type that iterates contiguous range/data combinations sequentially (including empty space between time ranges)
 */
struct FMovieSceneEvaluationTreeRangeIterator
{
	/** Iterate the tree from -infinity */
	MOVIESCENE_API FMovieSceneEvaluationTreeRangeIterator(const FMovieSceneEvaluationTree& InTree);

	/** Iterate the tree ranges starting at the range that overlaps the specified lower bound */
	MOVIESCENE_API FMovieSceneEvaluationTreeRangeIterator(const FMovieSceneEvaluationTree& InTree, TRangeBound<FFrameNumber> StartingBound);

	/** Move onto the next time range */
	FMovieSceneEvaluationTreeRangeIterator& operator++()
	{
		Iter(true);
		return *this;
	}

	/** Move onto the previous time range */
	FMovieSceneEvaluationTreeRangeIterator& operator--()
	{
		Iter(false);
		return *this;
	}

	/** Get an iterator pointing to the next range */
	FMovieSceneEvaluationTreeRangeIterator Next() const
	{
		FMovieSceneEvaluationTreeRangeIterator N = *this;
		return ++N;
	}

	/** Get an iterator pointing to the previous range */
	FMovieSceneEvaluationTreeRangeIterator Previous() const
	{
		FMovieSceneEvaluationTreeRangeIterator P = *this;
		return --P;
	}

	/** Get the current range */
	TRange<FFrameNumber> Range() const
	{
		return CurrentRange;
	}

	/** Get the current node */
	FMovieSceneEvaluationTreeNodeHandle Node() const
	{
		return CurrentNodeHandle;
	}

	/** conversion to "bool" returning true if the iterator has not reached the last element. */
	explicit operator bool() const
	{
		return CurrentNodeHandle.IsValid();
	}

private:

	/** (In)Equality operators */
	friend bool operator==(const FMovieSceneEvaluationTreeRangeIterator& Lhs, const FMovieSceneEvaluationTreeRangeIterator& Rhs)
	{
		return Lhs.Tree == Rhs.Tree && Lhs.CurrentNodeHandle == Rhs.CurrentNodeHandle && Lhs.CurrentRange == Rhs.CurrentRange;
	}
	friend bool operator!=(const FMovieSceneEvaluationTreeRangeIterator& Lhs, const FMovieSceneEvaluationTreeRangeIterator& Rhs)
	{
		return Lhs.Tree != Rhs.Tree || Lhs.CurrentNodeHandle != Rhs.CurrentNodeHandle || Lhs.CurrentRange != Rhs.CurrentRange;
	}

	/** Compare a bound with a range based on whether we're iterating forwards or backwards */
	static bool CompareBound(bool bForwards, TRange<FFrameNumber> Range, TRangeBound<FFrameNumber> Bound)
	{
		return bForwards ? Range.GetLowerBound() == Bound : Range.GetUpperBound() == Bound;
	}

	/** Access the 'leading' bound of a range (lowerbound if forwards, upperbound if backwards) */
	static TRangeBound<FFrameNumber> GetLeadingBound(bool bForwards, TRange<FFrameNumber> Range)
	{
		return bForwards ? Range.GetLowerBound() : Range.GetUpperBound();
	}

	/** Access the 'trailing' bound of a range (upperbound if forwards, lowerbound if backwards) */
	static TRangeBound<FFrameNumber> GetTrailingBound(bool bForwards, TRange<FFrameNumber> Range)
	{
		return bForwards ? Range.GetUpperBound() : Range.GetLowerBound();
	}

	/** Iterate onto the next range based on whether we're going forwards or backwards */
	MOVIESCENE_API void Iter(bool bForwards);

	/** Find the next child within the given parent that is >= the specified bound based on whether we're going forwards or backwards */
	MOVIESCENE_API FMovieSceneEvaluationTreeNodeHandle FindNextChild(FMovieSceneEvaluationTreeNodeHandle ParentNodeHandle, TRangeBound<FFrameNumber> PredicateBound, bool bForwards);

private:
	/** The unique time range that we're currently at */
	TRange<FFrameNumber> CurrentRange;
	/** Handle of the current node */
	FMovieSceneEvaluationTreeNodeHandle CurrentNodeHandle;
	/** The tree we're iterating */
	const FMovieSceneEvaluationTree* Tree;
};

/**
 * Templated version of FMovieSceneEvaluationTree that is also able to associate arbitrary data with nodes
 */
template<typename DataType>
struct TMovieSceneEvaluationTree : FMovieSceneEvaluationTree
{

	/**
	 * Check whether this tree is empty
	 */
	bool IsEmpty() const
	{
		return GetChildren(GetRootNode()).Num() == 0 &&	GetDataForSingleNode(GetRootNode()).Num() == 0;
	}

	/**
	 * Add a time range with no data associated
	 *
	 * @param InTimeRange 		The range with which this data should be associated
	 */
	void Add(TRange<FFrameNumber> InTimeRange)
	{
		struct FNullOperator : IMovieSceneEvaluationTreeNodeOperator
		{
			virtual void operator()(FMovieSceneEvaluationTreeNode& InNode) const override
			{}
		};

		AddTimeRange(InTimeRange, FNullOperator(), FMovieSceneEvaluationTreeNodeHandle::Root(), nullptr);
	}

	/**
	 * Add a new time range with the associated data to the tree. Ensures no duplicates.
	 *
	 * @param InTimeRange 		The range with which this data should be associated
	 * @param InData 			The data to assoicate with this time range
	 */
	void AddUnique(TRange<FFrameNumber> InTimeRange, DataType InData)
	{
		AddTimeRange(InTimeRange, FAddUniqueOperator(*this, MoveTemp(InData)), FMovieSceneEvaluationTreeNodeHandle::Root(), nullptr);
	}

	/**
	 * Adds a new time range with the associated data to the tree, only for segments where there's no data yet.
	 *
	 * @param InTimeRange 		The range with which this data should be associated
	 * @param InData 			The data to assoicate with this time range
	 */
	void AddIfEmpty(TRange<FFrameNumber> InTimeRange, DataType InData)
	{
		AddSelective(InTimeRange, MoveTemp(InData), [this](FMovieSceneEvaluationTreeNodeHandle InNodeHandle){
			return this->GetAllData(InNodeHandle).IsValid() == false;
		});
	}

	/**
	 * Adds a new time range with the associated data to the tree, only for segments where there's no data yet.
	 *
	 * @param InTimeRange 		The range with which this data should be associated
	 * @param InData 			The data to assoicate with this time range
	 */
	void AddIfEmptySelective(TRange<FFrameNumber> InTimeRange, DataType InData, TFunctionRef<bool(FMovieSceneEvaluationTreeNodeHandle)> Predicate)
	{
		AddSelective(InTimeRange, MoveTemp(InData), [this, &Predicate](FMovieSceneEvaluationTreeNodeHandle InNodeHandle){
			return this->GetAllData(InNodeHandle).IsValid() && Predicate(InNodeHandle);
		});
	}

	/**
	 * Add a new time range with the associated data to the tree.
	 *
	 * @param InTimeRange 		The range with which this data should be associated
	 * @param InData 			The data to assoicate with this time range
	 */
	void AddSelective(TRange<FFrameNumber> InTimeRange, DataType InData, TFunctionRef<bool(FMovieSceneEvaluationTreeNodeHandle)> Predicate)
	{
		AddTimeRange(InTimeRange, FAddOperator(*this, MoveTemp(InData)), FMovieSceneEvaluationTreeNodeHandle::Root(), &Predicate);
	}

	/**
	 * Add a new time range with the associated data to the tree.
	 *
	 * @param InTimeRange 		The range with which this data should be associated
	 * @param InData 			The data to assoicate with this time range
	 */
	void Add(TRange<FFrameNumber> InTimeRange, DataType InData)
	{
		AddTimeRange(InTimeRange, FAddOperator(*this, MoveTemp(InData)), FMovieSceneEvaluationTreeNodeHandle::Root(), nullptr);
	}

	/**
	 * Compact the memory used by this tree so it's using as little as possible
	 */
	void Compact()
	{
		ChildNodes.Compact();
		Data.Compact();
	}

	/**
	 * Reset this tree
	 */
	void Reset()
	{
		FMovieSceneEvaluationTree::Reset();
		Data.Reset();
	}

	/**
	 * Access the data associated with a single node in the tree.
	 */
	TArrayView<const DataType> GetDataForSingleNode(const FMovieSceneEvaluationTreeNode& InNode) const
	{
		return InNode.DataID.IsValid() ? Data.Get(InNode.DataID) : TArrayView<const DataType>();
	}

	/**
	 * Access the data associated with a single node in the tree.
	 */
	TArrayView<DataType> GetMutableDataForSingleNode(const FMovieSceneEvaluationTreeNode& InNode)
	{
		return InNode.DataID.IsValid() ? Data.Get(InNode.DataID) : TArrayView<DataType>();
	}

	/**
	 * Create an iterator that will iterate all data associated with the specified node
	 *
	 * @param NodeHandle 		The node to get all data for
	 */
	TMovieSceneEvaluationTreeDataIterator<DataType> GetAllData(FMovieSceneEvaluationTreeNodeHandle NodeHandle) const
	{
		return TMovieSceneEvaluationTreeDataIterator<DataType>(*this, NodeHandle);
	}

	/**
	 * Serialization operator
	 */
	friend FArchive& operator<<(FArchive& Ar, TMovieSceneEvaluationTree<DataType>& In)
	{
		auto& BaseTree = static_cast<FMovieSceneEvaluationTree&>(In);
		return Ar << BaseTree << In.Data;
	}

	/**
	 * Equality operator
	 */
	friend bool operator==(const TMovieSceneEvaluationTree<DataType>& A, const TMovieSceneEvaluationTree<DataType>& B)
	{
		return A.Data == B.Data && A.RootNode == B.RootNode && A.ChildNodes == B.ChildNodes;
	}

private:

	/** Tree data container that corresponds to FMovieSceneEvaluationTreeNode::DataID */
	TEvaluationTreeEntryContainer<DataType> Data;

	friend struct TMovieSceneEvaluationTreeFormatter<DataType>;

private:
	/** Operator that adds data to nodes provided it doesn't already exist on the node */
	struct FAddUniqueOperator : IMovieSceneEvaluationTreeNodeOperator
	{
		FAddUniqueOperator(TMovieSceneEvaluationTree<DataType>& InTree, DataType&& InDataToInsert) : Tree(InTree), DataToInsert(InDataToInsert) {}

		virtual void operator()(FMovieSceneEvaluationTreeNode& InNode) const override
		{
			// If we haven't allocated data for this node yet, do so now
			if (!InNode.DataID.IsValid())
			{
				InNode.DataID = Tree.Data.AllocateEntry(1);
			}

			TArrayView<const DataType> NodeData = Tree.Data.Get(InNode.DataID);
			if (!NodeData.Contains(DataToInsert))
			{
				Tree.Data.Add(InNode.DataID, CopyTemp(DataToInsert));
			}
		}
		TMovieSceneEvaluationTree<DataType>& Tree;
		DataType DataToInsert;
	};

	/** Operator that adds data to nodes */
	struct FAddOperator : IMovieSceneEvaluationTreeNodeOperator
	{
		FAddOperator(TMovieSceneEvaluationTree<DataType>& InTree, DataType&& InDataToInsert) : Tree(InTree), DataToInsert(InDataToInsert) {}

		virtual void operator()(FMovieSceneEvaluationTreeNode& InNode) const override
		{
			// If we haven't allocated data for this node yet, do so now
			if (!InNode.DataID.IsValid())
			{
				InNode.DataID = Tree.Data.AllocateEntry(1);
			}

			Tree.Data.Add(InNode.DataID, CopyTemp(DataToInsert));
		}
		TMovieSceneEvaluationTree<DataType>& Tree;
		DataType DataToInsert;
	};
};

/**
 * An iterator type that is iterates all the data associated with a given node and its parents
 */
template<typename DataType>
struct TMovieSceneEvaluationTreeDataIterator
{
	/** Construction from a tree and a node */
	TMovieSceneEvaluationTreeDataIterator(const TMovieSceneEvaluationTree<DataType>& InTree, FMovieSceneEvaluationTreeNodeHandle StartNode)
		: Tree(&InTree)
		, CurrentNode(StartNode.IsValid() ? &InTree.GetNode(StartNode) : nullptr)
		, DataIndex(0)
	{
		// Walk up the tree until we find some data. We reset the iterator so begin() == end() where there is no data
		while (CurrentNode && !InTree.GetDataForSingleNode(*CurrentNode).IsValidIndex(DataIndex))
		{
			CurrentNode = CurrentNode->Parent.IsValid() ? &InTree.GetNode(CurrentNode->Parent) : nullptr;
			DataIndex = 0;
		}
	}

	/** Dereference the data for the current iteration */
	const DataType& operator*() const
	{
		return Tree->GetDataForSingleNode(*CurrentNode)[DataIndex];
	}

	/** Access the data for the current iteration */
	const DataType* operator->() const
	{
		return &Tree->GetDataForSingleNode(*CurrentNode)[DataIndex];
	}

	/** Check the iterator for validity */
	bool IsValid() const
	{
		return CurrentNode && Tree->GetDataForSingleNode(*CurrentNode).IsValidIndex(DataIndex);
	}

	/** Check the iterator for validity */
	explicit operator bool() const
	{
		return IsValid();
	}

	/** Move on to the next piece of data */
	TMovieSceneEvaluationTreeDataIterator& operator++()
	{
		++DataIndex;

		// Skip up parents while the data index is invalid
		while (CurrentNode && !Tree->GetDataForSingleNode(*CurrentNode).IsValidIndex(DataIndex))
		{
			CurrentNode = CurrentNode->Parent.IsValid() ? &Tree->GetNode(CurrentNode->Parent) : nullptr;
			DataIndex = 0;
		}
		return *this;
	}

	/** Range-for support */
	friend TMovieSceneEvaluationTreeDataIterator<DataType> begin(const TMovieSceneEvaluationTreeDataIterator<DataType>& In)                      { return In; }
	friend TMovieSceneEvaluationTreeDataIterator<DataType> end(const TMovieSceneEvaluationTreeDataIterator<DataType>& In)                        { return TMovieSceneEvaluationTreeDataIterator<DataType>(*In.Tree, FMovieSceneEvaluationTreeNodeHandle::Invalid()); }
	/** (In)Equality operators */
	friend bool operator==(const TMovieSceneEvaluationTreeDataIterator<DataType>& A, const TMovieSceneEvaluationTreeDataIterator<DataType>& B)   { return A.Tree == B.Tree && A.CurrentNode == B.CurrentNode && A.DataIndex == B.DataIndex; }
	friend bool operator!=(const TMovieSceneEvaluationTreeDataIterator<DataType>& A, const TMovieSceneEvaluationTreeDataIterator<DataType>& B)   { return A.Tree != B.Tree || A.CurrentNode != B.CurrentNode || A.DataIndex != B.DataIndex; }

private:
	/** The tree we're reading */
	const TMovieSceneEvaluationTree<DataType>* Tree;
	/** The current node of the iteration (nullptr for invalid iterators) */
	const FMovieSceneEvaluationTreeNode* CurrentNode;
	/** The current data index within CurrentNode's data */
	int32 DataIndex;
};

template<typename ElementType>
FEvaluationTreeEntryHandle TEvaluationTreeEntryContainer<ElementType>::AllocateEntry(int32 InitialCapacity)
{
	FEvaluationTreeEntryHandle ID{ Entries.Num() };
	Entries.Add(FEntry{Items.Num(), 0, InitialCapacity});
	Items.SetNum(Items.Num() + InitialCapacity);
	return ID;
}

template<typename ElementType>
TArrayView<ElementType> TEvaluationTreeEntryContainer<ElementType>::Get(FEvaluationTreeEntryHandle ID)
{
	const FEntry& Entry = Entries[ID.EntryIndex];
	return TArrayView<ElementType>(&Items[Entry.StartIndex], Entry.Size);
}

template<typename ElementType>
TArrayView<const ElementType> TEvaluationTreeEntryContainer<ElementType>::Get(FEvaluationTreeEntryHandle ID) const
{
	const FEntry& Entry = Entries[ID.EntryIndex];
	return TArrayView<const ElementType>(&Items[Entry.StartIndex], Entry.Size);
}

template<typename ElementType>
void TEvaluationTreeEntryContainer<ElementType>::Add(FEvaluationTreeEntryHandle ID, ElementType&& Element)
{
	FEntry& Entry = Entries[ID.EntryIndex];
	if (Entry.Size+1 > Entry.Capacity)
	{
		ReserveEntry(ID, FMath::Max<int32>(Entry.Capacity*2, 2));
	}

	Items[Entry.StartIndex + Entry.Size] = MoveTemp(Element);
	++Entry.Size;
}

template<typename ElementType>
void TEvaluationTreeEntryContainer<ElementType>::Insert(FEvaluationTreeEntryHandle ID, int32 Index, ElementType&& Element)
{
	FEntry& Entry = Entries[ID.EntryIndex];
	if (Entry.Size+1 > Entry.Capacity)
	{
		ReserveEntry(ID, FMath::Max<int32>(Entry.Capacity*2, 2));
	}

	Index = FMath::Clamp(Index, 0, Entry.Capacity-1);

	// Shift anything >= the insert index up one
	if (Index <= Entry.Size-1)
	{
		ElementType* OldLocation = &Items[Entry.StartIndex + Index];
		void* NewLocation = &Items[Entry.StartIndex + Index + 1];

		RelocateConstructItems<ElementType>(NewLocation, OldLocation, Entry.Size - Index);
		DefaultConstructItems<ElementType>((void*)OldLocation, 1);
	}

	// Assign the new element
	++Entry.Size;
	Items[Entry.StartIndex + Index] = MoveTemp(Element);
}

template<typename ElementType>
void TEvaluationTreeEntryContainer<ElementType>::Compact()
{
	TArray<ElementType> NewItems;

	for (FEntry& Entry : Entries)
	{
		if (Entry.Size > 0)
		{
			const int32 OldIndex = Entry.StartIndex;
			Entry.StartIndex = NewItems.Num();

			NewItems.Append(&Items[OldIndex], Entry.Size);
		}
		else
		{
			Entry.StartIndex = INDEX_NONE;
		}
		Entry.Capacity = Entry.Size;
	}

	NewItems.Shrink();
	Items = MoveTemp(NewItems);
}

template<typename ElementType>
void TEvaluationTreeEntryContainer<ElementType>::ReserveEntry(FEvaluationTreeEntryHandle ID, int32 NewCapacity)
{
	FEntry& Entry = Entries[ID.EntryIndex];

	// Function should only be called to grow an entry
	check(NewCapacity > Entry.Capacity);
	Entry.Capacity = NewCapacity;

	int32 NewStartIndex = Items.Num();

	// Reallocate the items to the new entry location
	Items.SetNum(Items.Num() + Entry.Capacity);

	ElementType* OldLocation = &Items[Entry.StartIndex];
	void* NewLocation = &Items[NewStartIndex];
	RelocateConstructItems<ElementType>(NewLocation, OldLocation, Entry.Size);
	DefaultConstructItems<ElementType>((void*)OldLocation, Entry.Size);

	Entry.StartIndex = NewStartIndex;
}
