// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "Evaluation/Blending/BlendableToken.h"
#include "Evaluation/Blending/MovieSceneBlendingAccumulator.h"
#include "Evaluation/Blending/MovieSceneBlendingActuatorID.h"
#include "Evaluation/MovieSceneEvaluationKey.h"
#include "Evaluation/MovieSceneEvaluationOperand.h"
#include "Evaluation/MovieSceneEvaluationScope.h"
#include "Evaluation/MovieScenePlayback.h"
#include "Evaluation/PersistentEvaluationData.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "Misc/InlineValue.h"
#include "MovieSceneExecutionToken.h"
#include "MovieSceneFwd.h"
#include "Templates/EnableIf.h"
#include "Templates/PointerIsConvertibleFromTo.h"
#include "Templates/RemoveReference.h"
#include "Templates/UnrealTemplate.h"

class IMovieScenePlayer;


/**
 * Ordered execution token stack that accumulates tokens that will apply animated state to the sequence environment at a later time
 */
struct FMovieSceneExecutionTokens
{
	FMovieSceneExecutionTokens()
		: Context(FMovieSceneEvaluationRange(FFrameTime(0), FFrameRate()), EMovieScenePlayerStatus::Stopped)
	{
	}

	FMovieSceneExecutionTokens(const FMovieSceneExecutionTokens&) = delete;
	FMovieSceneExecutionTokens& operator=(const FMovieSceneExecutionTokens&) = delete;

	FMovieSceneExecutionTokens(FMovieSceneExecutionTokens&&) = default;
	FMovieSceneExecutionTokens& operator=(FMovieSceneExecutionTokens&&) = default;

	/**
	 * Add a new IMovieSceneExecutionToken derived token to the stack
	 */
	template<typename T>
	inline typename TEnableIf<TPointerIsConvertibleFromTo<typename TRemoveReference<T>::Type, const IMovieSceneExecutionToken>::Value>::Type Add(T&& InToken)
	{
		check(Scope.Key.IsValid() && Operand.IsValid());
		OrderedTokens.Add(FEntry(Operand, Scope, Context, Forward<T>(InToken)));
	}

	/**
	* Add a new shared execution token to the sorted array of tokens
	*/
	template<typename T>
	inline typename TEnableIf<TPointerIsConvertibleFromTo<typename TRemoveReference<T>::Type, const IMovieSceneSharedExecutionToken>::Value>::Type AddShared(FMovieSceneSharedDataId ID, T&& InToken)
	{
		checkf(!SharedTokens.Contains(ID), TEXT("Already added a shared token of this type"));
		SharedTokens.Add(ID, MoveTemp(InToken));
	}

	/**
	 * Attempt to locate an existing shared execution token by its ID
	 */
	IMovieSceneSharedExecutionToken* FindShared(FMovieSceneSharedDataId ID)
	{
		auto* Existing = SharedTokens.Find(ID);
		return Existing ? Existing->GetPtr() : nullptr;
	}

public:

	/**
	 * Access the execution stack's blending accumulator which is responsible for marshalling all blending operations for all animated objects
	 * @return Mutable reference to the blending accumulator
	 */
	FMovieSceneBlendingAccumulator& GetBlendingAccumulator()
	{
		return BlendingAccumulator;
	}

	/**
	 * Access the execution stack's blending accumulator which is responsible for marshalling all blending operations for all animated objects
	 * @return Mutable reference to the blending accumulator
	 */
	const FMovieSceneBlendingAccumulator& GetBlendingAccumulator() const
	{
		return BlendingAccumulator;
	}

	/**
	 * Blend the specified global token using the specified actuator ID
	 * @note: Actuator must already exist for this function to succeed
	 *
	 * @param InActuatorTypeID 		Type identifier that uniquely identifies the actuator to be used to apply the final blend
	 * @param InToken 				Token holding the data to blend
	 */
	template<typename ActuatorDataType>
	void BlendToken(FMovieSceneBlendingActuatorID InActuatorTypeID, TBlendableToken<ActuatorDataType>&& InToken)
	{
		BlendingAccumulator.BlendToken(Operand, InActuatorTypeID, Scope, Context, MoveTemp(InToken));
	}

	/**
	 * Apply all execution tokens in order, followed by blended tokens
	 */
	MOVIESCENE_API void Apply(const FMovieSceneContext& RootContext, IMovieScenePlayer& Player);

public:

	/**
	 * Internal: Set the operand we're currently operating on
	 */
	inline void SetOperand(const FMovieSceneEvaluationOperand& InOperand)
	{
		Operand = InOperand;
	}

	/**
	 * Internal: Set the current scope
	 */
	inline void SetCurrentScope(const FMovieSceneEvaluationScope& InScope)
	{
		Scope = InScope;
	}

	/**
	 * Internal: Set the current context
	 */
	inline void SetContext(const FMovieSceneContext& InContext)
	{
		Context = InContext;
	}

	/**
	 * Get the current evaluation scope
	 */
	inline FMovieSceneEvaluationScope GetCurrentScope() const
	{
		return Scope;
	}

private:

	struct FEntry
	{
		FEntry(const FMovieSceneEvaluationOperand& InOperand, const FMovieSceneEvaluationScope& InScope, const FMovieSceneContext& InContext, TInlineValue<IMovieSceneExecutionToken, 64>&& InToken)
			: Operand(InOperand)
			, Scope(InScope)
			, Context(InContext)
			, Token(MoveTemp(InToken))
		{
		}

		FEntry(FEntry&&) = default;
		FEntry& operator=(FEntry&&) = default;

		/** The operand we were operating on when this token was added */
		FMovieSceneEvaluationOperand Operand;
		/** The evaluation scope at the time this token was created */
		FMovieSceneEvaluationScope Scope;
		/** The context from when this token was added */
		FMovieSceneContext Context;
		/** The user-provided token */
		TInlineValue<IMovieSceneExecutionToken, 64> Token;
	};

	/** Ordered array of tokens */
	TArray<FEntry> OrderedTokens;

	/** Sortable, shared array of identifyable tokens */
	TMap<FMovieSceneSharedDataId, TInlineValue<IMovieSceneSharedExecutionToken, 32>> SharedTokens;

	/** Accumulator used to marshal blended animation data */
	FMovieSceneBlendingAccumulator BlendingAccumulator;

	/** The operand we're currently operating on */
	FMovieSceneEvaluationOperand Operand;
	/** The current evaluation scope */
	FMovieSceneEvaluationScope Scope;
	/** The current context */
	FMovieSceneContext Context;
};
