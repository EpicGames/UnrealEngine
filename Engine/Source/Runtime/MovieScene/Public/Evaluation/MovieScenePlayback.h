// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Evaluation/MovieSceneSequenceTransform.h"
#include "Evaluation/MovieSceneTimeTransform.h"
#include "Evaluation/MovieSceneTimeWarping.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Math/NumericLimits.h"
#include "Math/Range.h"
#include "Math/RangeBound.h"
#include "Misc/AssertionMacros.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "Misc/Optional.h"
#include "MovieSceneFwd.h"
#include "MovieSceneTimeHelpers.h"


/** Enumeration specifying whether we're playing forwards or backwards */
enum class EPlayDirection
{
	Forwards, Backwards
};


/** MovieScene evaluation context. Should remain bitwise copyable, and contain no external state since this has the potential to be used on a thread */
struct FMovieSceneEvaluationRange
{
	/**
	 * Construct this range from a single fixed time
	 */
	MOVIESCENE_API FMovieSceneEvaluationRange(FFrameTime InTime, FFrameRate InFrameRate);

	/**
	 * Construct this range from a raw range and a direction
	 */
	MOVIESCENE_API FMovieSceneEvaluationRange(TRange<FFrameTime> InRange, FFrameRate InFrameRate, EPlayDirection InDirection);

	/**
	 * Construct this range from 2 times, and whether the range should include the previous time or not
	 */
	MOVIESCENE_API FMovieSceneEvaluationRange(FFrameTime InCurrentTime, FFrameTime InPreviousTime, FFrameRate InFrameRate, bool bInclusivePreviousTime = false, EPlayDirection PreferredDirection = EPlayDirection::Forwards);

	/**
	 * Convert a frame time range to a frame number range comprising all the frame numbers traversed in the range
	 */
	static MOVIESCENE_API TRange<FFrameNumber> TimeRangeToNumberRange(const TRange<FFrameTime>& InFrameTimeRange);

	/**
	 * Convert a frame number range to a frame time range
	 */
	static MOVIESCENE_API TRange<FFrameTime> NumberRangeToTimeRange(const TRange<FFrameNumber>& InFrameTimeRange);

	/**
	 * Get the range that we should be evaluating
	 */
	inline TRange<FFrameTime> GetRange() const
	{
		return EvaluationRange;
	}

	/**
	 * Get the range of frame numbers traversed over this evaluation range, not including partial frames
	 */
	inline TRange<FFrameNumber> GetFrameNumberRange() const
	{
		return TimeRangeToNumberRange(EvaluationRange);
	}

	/**
	 * Get the range of frame numbers traversed over this evaluation range by flooring the lower bound, and ceiling the upper bound.
	 * For example: a time range of [1.5, 5.6] will yield the equivalent of [1, 6). A time range of (2.0, 2.9) will yield the equivalent of [2,3).
	 */
	MOVIESCENE_API TRange<FFrameNumber> GetTraversedFrameNumberRange() const;

	/**
	 * Get the direction to evaluate our range
	 */
	inline EPlayDirection GetDirection() const 
	{
		return Direction;
	}

	/**
	 * Get the current time of evaluation.
	 */
	inline FFrameTime GetTime() const
	{
		if (TimeOverride != TNumericLimits<int32>::Lowest())
		{
			return TimeOverride;
		}

		return Direction == EPlayDirection::Forwards ? EvaluationRange.GetUpperBoundValue() : EvaluationRange.GetLowerBoundValue();
	}

	/**
	 * Get the current time to use for looking up within an evaluation field.
	 * Subtly different from GetTime in that it returns the previous tick for exclusive boundaries
	 */
	inline FFrameNumber GetEvaluationFieldTime() const
	{
		TRange<FFrameNumber> Range = GetFrameNumberRange();

		if (Direction == EPlayDirection::Forwards)
		{
			return UE::MovieScene::DiscreteExclusiveUpper(Range)-1;
		}
		else
		{
			return UE::MovieScene::DiscreteInclusiveLower(Range);
		}
	}

	/**
	 * Get the absolute amount of time that has passed since the last update (will always be >= 0)
	 */
	inline FFrameTime GetDelta() const
	{
		return EvaluationRange.Size<FFrameTime>();
	}

	/**
	 * Get the previous time of evaluation. Should not generally be used. Prefer GetRange instead.
	 */
	inline FFrameTime GetPreviousTime() const
	{
		return Direction == EPlayDirection::Forwards ? EvaluationRange.GetLowerBoundValue() : EvaluationRange.GetUpperBoundValue();
	}

	/**
	 * Get the current time offset by the specified amount in the direction of play
	 */
	inline FFrameTime GetOffsetTime(FFrameTime InOffset) const
	{
		FFrameTime Now = GetTime();
		return Direction == EPlayDirection::Forwards ? Now + InOffset : Now - InOffset;
	}

	/**
	 * Override the time that we're actually evaluating at
	 */
	inline void OverrideTime(FFrameNumber InTimeOverride)
	{
		TimeOverride = InTimeOverride;
	}

	/**
	 * Get the framerate that this context's times are in
	 * @return The framerate that all times are relative to
	 */
	inline FFrameRate GetFrameRate() const
	{
		return CurrentFrameRate;
	}

	/**
	 * Reset this range to a new range while retaining the current framerate and direction
	 */
	MOVIESCENE_API void ResetRange(const TRange<FFrameTime>& NewRange);

protected:

	/** The range to evaluate */
	TRange<FFrameTime> EvaluationRange;

	/** The framerate of the current sequence. */
	FFrameRate CurrentFrameRate;

	/** Whether to evaluate the range forwards, or backwards */
	EPlayDirection Direction;

	/** Overridden current time (doesn't manipulate the actual evaluated range) */
	FFrameNumber TimeOverride;
};

/** MovieScene evaluation context. Should remain bitwise copyable, and contain no external state since this has the potential to be used on a thread */
struct FMovieSceneContext : FMovieSceneEvaluationRange
{
	/**
	 * Construction from an evaluation range, and a current status
	 */
	FMovieSceneContext()
		: FMovieSceneEvaluationRange(TRange<FFrameTime>::Empty(), FFrameRate(60, 1), EPlayDirection::Forwards)
		, Status(EMovieScenePlayerStatus::Stopped)
		, PrePostRollStartEndTime(TNumericLimits<int32>::Lowest())
		, HierarchicalBias(0)
		, bHasJumped(false)
		, bHasLooped(false)
		, bSilent(false)
		, bSectionPreRoll(false)
		, bSectionPostRoll(false)
		, bHasPreRollEndTime(false)
		, bHasPostRollStartTime(false)
	{}

	/**
	 * Construction from an evaluation range, and a current status
	 */
	FMovieSceneContext(FMovieSceneEvaluationRange InRange)
		: FMovieSceneEvaluationRange(InRange)
		, Status(EMovieScenePlayerStatus::Stopped)
		, PrePostRollStartEndTime(TNumericLimits<int32>::Lowest())
		, HierarchicalBias(0)
		, bHasJumped(false)
		, bHasLooped(false)
		, bSilent(false)
		, bSectionPreRoll(false)
		, bSectionPostRoll(false)
		, bHasPreRollEndTime(false)
		, bHasPostRollStartTime(false)
	{}

	/**
	 * Construction from an evaluation range, and a current status
	 */
	FMovieSceneContext(FMovieSceneEvaluationRange InRange, EMovieScenePlayerStatus::Type InStatus)
		: FMovieSceneEvaluationRange(InRange)
		, Status(InStatus)
		, PrePostRollStartEndTime(TNumericLimits<int32>::Lowest())
		, HierarchicalBias(0)
		, bHasJumped(false)
		, bHasLooped(false)
		, bSilent(false)
		, bSectionPreRoll(false)
		, bSectionPostRoll(false)
		, bHasPreRollEndTime(false)
		, bHasPostRollStartTime(false)
	{}

	/**
	 * Get the playback status
	 */
	inline EMovieScenePlayerStatus::Type GetStatus() const
	{
		return Status;
	}

	/**
	 * Check whether we've just jumped to a different time
	 */
	inline bool HasJumped() const
	{
		return bHasJumped;
	}

	/**
	 * Check whether we've just looped around the sequence
	 * If this returns true, HasJumped() usually also returns true
	 */
	inline bool HasLooped() const
	{
		return bHasLooped;
	}

	/**
	 * Check whether we're evaluating in silent mode (no audio or mutating eval)
	 */
	inline bool IsSilent() const
	{
		return bSilent;
	}

	/**
	 * Get the current root to sequence transform for the current sub sequence
	 */
	inline const FMovieSceneSequenceTransform& GetRootToSequenceTransform() const
	{
		return RootToSequenceTransform;
	}

	/**
	 * Get the current root to sequence warp counter for the current sub sequence
	 */
	inline const FMovieSceneWarpCounter& GetRootToSequenceWarpCounter() const
	{
		return RootToSequenceWarpCounter;
	}

	/**
	 * Get the inverse transform of the current sub sequence, to transform local times back to root times.
	 */
	UE_DEPRECATED(5.4, "Please use GetSequenceToRootSequenceTransform instead.")
	MOVIESCENE_API FMovieSceneTimeTransform GetSequenceToRootTransform() const;

	/**
	 * Get the inverse sequence transform of the current sub sequence, to transform local times back to root times.
	 */
	MOVIESCENE_API FMovieSceneInverseSequenceTransform GetSequenceToRootSequenceTransform() const;

	/**
	 * Apply section pre and post roll based on whether we're in the leading (preroll), or trailing (postroll) region for the section, and the current play direction
	 *
	 * @param bInLeadingRegion			Whether we are considered to be in the section's leading (aka preroll) region
	 * @param bInTrailingRegion			Whether we are considered to be in the section's trailing (aka postroll) region
	 */
	inline void ApplySectionPrePostRoll(bool bInLeadingRegion, bool bInTrailingRegion)
	{
		if (Direction == EPlayDirection::Forwards)
		{
			bSectionPreRoll = bInLeadingRegion;
			bSectionPostRoll = bInTrailingRegion;
		}
		else
		{
			bSectionPreRoll = bInTrailingRegion;
			bSectionPostRoll = bInLeadingRegion;
		}
	}

public:

	/**
	 * Indicate that we've just jumped to a different time
	 */
	FMovieSceneContext& SetHasJumped(bool bInHasJumped)
	{
		bHasJumped = bInHasJumped;
		return *this;
	}

	/**
	 * Indicate that we've just looped around the sequence
	 * The caller should also probably call SetHasJumped(true)
	 */
	FMovieSceneContext& SetHasLooped(bool bInHasLooped)
	{
		bHasLooped = bInHasLooped;
		return *this;
	}

	/**
	 * Set the context to silent mode
	 */
	FMovieSceneContext& SetIsSilent(bool bInIsSilent)
	{
		bSilent = bInIsSilent;
		return *this;
	}

	/**
	 * Clamp the current evaluation range to the specified range (in the current transform space)
	 */
	FMovieSceneContext Clamp(TRange<FFrameTime> NewRange) const
	{
		FMovieSceneContext NewContext = *this;
		NewContext.EvaluationRange = TRange<FFrameTime>::Intersection(NewRange, NewContext.EvaluationRange);
		return NewContext;
	}

	/**
	 * Transform this context to a different sub sequence space
	 */
	MOVIESCENE_API FMovieSceneContext Transform(const FMovieSceneSequenceTransform& InTransform, FFrameRate NewFrameRate) const;

	/**
	 * Get the hierarchical bias for the current context
	 * @param InHierarchicalBias		The current hierarchical bias
	 */
	void SetHierarchicalBias(int32 InHierarchicalBias)
	{
		HierarchicalBias = InHierarchicalBias;
	}

	/**
	 * Get the hierarchical bias for the current context
	 * @return The current hierarchical bias
	 */
	int32 GetHierarchicalBias() const
	{
		return HierarchicalBias;
	}

public:

	/**
	 * Check if we're in any kind of preroll (either prerolling section specifically, or as part of a sub-section)
	 * @note Play direction has already been considered in the calculation of this function, so needs no extra consideration.
	 */
	bool IsPreRoll() const
	{
		return bHasPreRollEndTime || bSectionPreRoll;
	}

	/**
	 * Check if we're in any kind of postroll (either postrolling section specifically, or as part of a sub-section)
	 * @note Play direction has already been considered in the calculation of this function, so needs no extra consideration.
	 */
	bool IsPostRoll() const
	{
		return bHasPostRollStartTime || bSectionPostRoll;
	}

	/**
	 * Check whether we have an externally supplied time at which preroll will end.
	 * @note When not set (and IsPreRoll() is true), preroll ends at either the start or end of section bounds, depending on play direction.
	 */
	bool HasPreRollEndTime() const
	{
		return bHasPreRollEndTime;
	}

	/**
	 * Check whether we have an externally supplied time at which postroll started.
	 * @note When not set (and IsPostRoll() is true), preroll ends at either the start or end of section bounds, depending on play direction.
	 */
	bool HasPostRollStartTime() const
	{
		return bHasPostRollStartTime;
	}

	/**
	 * Access the time at which preroll will stop, and evaluation will commence
	 * @note: Only valid to call when HasPreRollEndTime() is true
	 */
	FFrameNumber GetPreRollEndFrame() const
	{
		checkf(bHasPreRollEndTime, TEXT("It's invalid to call GetPreRollEndFrame() without first checking HasPreRollEndTime()"));
		return PrePostRollStartEndTime;
	}

	/**
	 * Access the time at which post roll started (or in other terms: when evaluation stopped)
	 * @note: Only valid to call when HasPostRollStartTime() is true
	 */
	FFrameNumber GetPostRollStartFrame() const
	{
		checkf(bHasPostRollStartTime, TEXT("It's invalid to call GetPostRollStartFrame() without first checking HasPostRollStartTime()"));
		return PrePostRollStartEndTime;
	}

	/**
	 * Report the outer section pre and post roll ranges for the current context
	 *
	 * @param InLeadingRange			The leading (preroll) range in front of the outer section, in the current transformation's time space
	 * @param InTrailingRange			The trailing (postroll) range at the end of the outer section, in the current transformation's time space
	 */
	void ReportOuterSectionRanges(TRange<FFrameNumber> InLeadingRange, TRange<FFrameNumber> InTrailingRange)
	{
		const FFrameNumber Now = GetTime().FrameNumber;
		if (InLeadingRange.Contains(Now) && InLeadingRange.HasUpperBound())
		{
			PrePostRollStartEndTime = InLeadingRange.GetUpperBoundValue();

			bHasPreRollEndTime = Direction == EPlayDirection::Forwards;
			bHasPostRollStartTime = !bHasPreRollEndTime;
		}
		else if (InTrailingRange.Contains(Now) && InTrailingRange.HasLowerBound())
		{
			PrePostRollStartEndTime = InTrailingRange.GetLowerBoundValue();

			bHasPreRollEndTime = Direction == EPlayDirection::Backwards;
			bHasPostRollStartTime = !bHasPreRollEndTime;
		}
		else
		{
			bHasPreRollEndTime = bHasPostRollStartTime = false;
			PrePostRollStartEndTime = FFrameNumber(TNumericLimits<int32>::Lowest());
		}
	}

protected:

	/** The transform from the root sequence to the current sequence space */
	FMovieSceneSequenceTransform RootToSequenceTransform;

	/** The current warp count for any sub-sequence in the current sequence's hierarchy */
	FMovieSceneWarpCounter RootToSequenceWarpCounter;

	/** The current playback status */
	EMovieScenePlayerStatus::Type Status;

	/** When bHasPreRollEndTime or bHasPostRollStartTime is true, this defines either the frame at which 'real' evaluation commences, or finished */
	FFrameNumber PrePostRollStartEndTime;

	/** Hierachical bias. Higher bias should take precedence. */
	int32 HierarchicalBias;

protected:

	/** Whether this evaluation frame is happening as part of a large jump */
	bool bHasJumped : 1;

	/** Whether this evaluation frame is happening as part of looping around the sequence */
	bool bHasLooped : 1;

	/** Whether this evaluation should happen silently */
	bool bSilent : 1;

	/** True if we should explicitly preroll the section. Already reconciled with play direction. */
	bool bSectionPreRoll : 1;

	/** True if we should explicitly postroll the section. Already reconciled with play direction. */
	bool bSectionPostRoll : 1;

	/** True if the value of PrePostRollStartEndTime has been set, and refers to the time at which preroll will end. Already reconciled with play direction. */
	bool bHasPreRollEndTime : 1;

	/** True if the value of PrePostRollStartEndTime has been set, and refers to the time at which postroll started. Already reconciled with play direction. */
	bool bHasPostRollStartTime : 1;
};

/** Helper class designed to abstract the complexity of calculating evaluation ranges for previous times and fixed time intervals */
struct FMovieScenePlaybackPosition
{
	FMovieScenePlaybackPosition()
		: InputRate(0,0), OutputRate(0,0), EvaluationType(EMovieSceneEvaluationType::WithSubFrames)
	{}

	/**
	 * @return Whether we are evaluating with sub frames, or frame-locked
	 */
	inline EMovieSceneEvaluationType GetEvaluationType() const
	{
		return EvaluationType;
	}

	/**
	 * @return The input frame rate that all frame times provided to this class will be interpreted as
	 */
	inline FFrameRate GetInputRate() const
	{
		return InputRate;
	}

	/**
	 * @return The output frame rate that all frame times returned from this class will be interpreted as
	 */
	inline FFrameRate GetOutputRate() const
	{
		return OutputRate;
	}

public:

	/**
	 * Assign the input and output rates that frame times should be interpreted as.
	 *
	 * @param InInputRate           The framerate to interpret any frame time provided to this class
	 * @param InOutputRate          The framerate to use when returning any frame range from this class
	 * @param InputEvaluationType   Whether we're using frame-locked or sub-frame evaluation
	 */
	MOVIESCENE_API void SetTimeBase(FFrameRate InInputRate, FFrameRate InOutputRate, EMovieSceneEvaluationType InputEvaluationType);

	/**
	 * Reset this position to the specified time.
	 * @note Future calls to 'PlayTo' will include this time in its resulting evaluation range
	 */
	MOVIESCENE_API void Reset(FFrameTime StartPos);

	/**
	 * Get the last position that was set, in InputRate space
	 */
	FFrameTime GetCurrentPosition() const { return CurrentPosition; }

	/**
	 * Get the last actual time that was evaluated during playback, in InputRate space.
	 */
	TOptional<FFrameTime> GetLastPlayEvalPostition() const { return PreviousPlayEvalPosition; }

public:

	/**
	 * Jump to the specified input time.
	 * @note Will reset previous play position. Any subsequent call to 'PlayTo' will include NewPosition.
	 *
	 * @param NewPosition         The new frame time to set, in InputRate space
	 * @return A range encompassing only the specified time, in OutputRate space.
	 */
	MOVIESCENE_API FMovieSceneEvaluationRange JumpTo(FFrameTime NewPosition, EPlayDirection PreferredDirection = EPlayDirection::Forwards);

	/**
	 * Play from the previously evaluated play time, to the specified time
	 *
	 * @param NewPosition         The new frame time to set, in InputRate space
	 * @return An evaluation range from the previously evaluated time to the specified time, in OutputRate space.
	 */
	MOVIESCENE_API FMovieSceneEvaluationRange PlayTo(FFrameTime NewPosition, EPlayDirection PreferredDirection = EPlayDirection::Forwards);

	/**
	 * Get a range that encompasses the last evaluated range in OutputRate space.
	 * @return An optional evaluation range in OutputRate space.
	 */
	MOVIESCENE_API TOptional<FMovieSceneEvaluationRange> GetLastRange() const;

	/**
	 * Get a range encompassing only the current time, if available (in OutputRate space)
	 * @return An optional evaluation range in OutputRate space.
	 */
	MOVIESCENE_API FMovieSceneEvaluationRange GetCurrentPositionAsRange() const;

private:

	/**
	 * Check this class's invariants
	 */
	MOVIESCENE_API void CheckInvariants() const;

private:

	/** The framerate to be used when interpreting frame time values provided to this class (i.e. moviescene display rate) */
	FFrameRate InputRate;

	/** The framerate to be used when returning frame time values from this class (i.e. moviescene tick resolution) */
	FFrameRate OutputRate;

	/** The type of evaluation to use */
	EMovieSceneEvaluationType EvaluationType;

	/** The current time position set, in 'InputRate' time-space. */
	FFrameTime CurrentPosition;

	/** The previously evaluated position when playing, in 'InputRate' time-space. */
	TOptional<FFrameTime> PreviousPlayEvalPosition;

	/** The previously evaluated range if available, in 'OutputRate' time-space */
	TOptional<FMovieSceneEvaluationRange> LastRange;
};
