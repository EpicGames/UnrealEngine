// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Evaluation/IMovieScenePlaybackCapability.h"
#include "EntitySystem/RelativePtr.h"
#include "MovieSceneFwd.h"
#include "Templates/AlignmentTemplates.h"
#include "Templates/PointerIsConvertibleFromTo.h"
#include <type_traits>

class IMovieScenePlayer;
class IMovieScenePlaybackClient;

namespace UE::MovieScene
{

enum class EPlaybackCapabilityStorageMode : uint8
{
	Inline = 0,
	RawPointer,
	SharedPointer
};

/**
 * Return value for accessing raw capabalities.
 */
struct FPlaybackCapabilityPtr
{
	void* Ptr = 0;
	EPlaybackCapabilityStorageMode StorageMode;

	template<typename T>
	T* ResolveOptional() const
	{
		if (Ptr)
		{
			switch (StorageMode)
			{
				case EPlaybackCapabilityStorageMode::Inline:
					return static_cast<T*>(Ptr);
				case EPlaybackCapabilityStorageMode::RawPointer:
					return *static_cast<T**>(Ptr);
				case EPlaybackCapabilityStorageMode::SharedPointer:
					return static_cast<TSharedPtr<T>*>(Ptr)->Get();
				default:
					checkf(false, TEXT("Unexpected playback capability storage mode"));
					return nullptr;
			}
		}
		return nullptr;
	}

	template<typename T>
	T& ResolveChecked() const
	{
		check(Ptr);
		switch (StorageMode)
		{
			case EPlaybackCapabilityStorageMode::Inline:
				return *static_cast<T*>(Ptr);
			case EPlaybackCapabilityStorageMode::RawPointer:
				return **static_cast<T**>(Ptr);
			case EPlaybackCapabilityStorageMode::SharedPointer:
				return *static_cast<TSharedPtr<T>*>(Ptr)->Get();
			default:
				checkf(false, TEXT("Unexpected playback capability storage mode"));
				// Nothing reasonable to do, let's just proceed as if it was inline
				return *static_cast<T*>(Ptr);
		}
	}
};

// Utility callbacks for the concrete capability objects we have in a container.
using FPlaybackCapabilityInterfaceCastHelper = IPlaybackCapability*(*)(void* Ptr);
using FPlaybackCapabilityDestructionHelper = void(*)(void* Ptr);

// Callback for casting a stored capabilty object to an IPlaybackCapability pointer if possible.
template<typename StorageType>
struct TPlaybackCapabilityInterfaceCast
{
	static IPlaybackCapability* InterfaceCast(void* Ptr)
	{
		if constexpr (TPointerIsConvertibleFromTo<StorageType, IPlaybackCapability>::Value)
		{
			return static_cast<IPlaybackCapability*>((StorageType*)Ptr);
		}
		else
		{
			return nullptr;
		}
	}
};
template<typename PointedType>
struct TPlaybackCapabilityInterfaceCast<PointedType*>
{
	static IPlaybackCapability* InterfaceCast(void* Ptr) 
	{
		if constexpr (TPointerIsConvertibleFromTo<PointedType, IPlaybackCapability>::Value)
		{
			PointedType* TypedPtr = *(PointedType**)Ptr;
			return static_cast<IPlaybackCapability*>(TypedPtr);
		}
		else
		{
			return nullptr;
		}
	};
};
template<typename PointedType>
struct TPlaybackCapabilityInterfaceCast<TSharedPtr<PointedType>>
{
	static IPlaybackCapability* InterfaceCast(void* Ptr) 
	{
		if constexpr (TPointerIsConvertibleFromTo<PointedType, IPlaybackCapability>::Value)
		{
			TSharedPtr<PointedType>& TypedPtr = *(TSharedPtr<PointedType>*)Ptr;
			return static_cast<IPlaybackCapability*>(TypedPtr.Get());
		}
		else
		{
			return nullptr;
		}
	};
};

// Callback for destroying the stored capability object, whether that's the capability itself
// (when stored inline), or a shared pointer, or whatever else.
template<typename StorageType>
struct TPlaybackCapabilityDestructor
{
	static void Destroy(void* Ptr)
	{
		StorageType* StoragePtr = (StorageType*)Ptr;
		StoragePtr->~StorageType();
	}
};

// Helper callbacks for managing the lifetime of a capability.
struct FPlaybackCapabilityHelpers
{
	FPlaybackCapabilityInterfaceCastHelper InterfaceCast = nullptr;
	FPlaybackCapabilityDestructionHelper Destructor = nullptr;
};

template<typename StorageType>
struct TPlaybackCapabilityHelpers
{
	static FPlaybackCapabilityHelpers GetHelpers()
	{
		FPlaybackCapabilityHelpers Helpers;
		Helpers.InterfaceCast = &TPlaybackCapabilityInterfaceCast<StorageType>::InterfaceCast;
		Helpers.Destructor = &TPlaybackCapabilityDestructor<StorageType>::Destroy;
		return Helpers;
	}
};

// Storage traits for playback capabilities, only for internal use.
template<typename StorageType, typename CapabilityType, typename=void>
struct TPlaybackCapabilityStorageTraits;

template<typename StorageType, typename CapabilityType>
struct TPlaybackCapabilityStorageTraits<StorageType, CapabilityType, typename TEnableIf<TPointerIsConvertibleFromTo<StorageType, CapabilityType>::Value>::Type>
{
	static EPlaybackCapabilityStorageMode GetStorageMode() { return EPlaybackCapabilityStorageMode::Inline; }

	static uint8 ComputePointerOffset(StorageType* StoragePtr)
	{
		const uint64 PointerOffset =
			reinterpret_cast<uint64>(static_cast<CapabilityType*>(StoragePtr))
			-
			reinterpret_cast<uint64>(StoragePtr);
		check(PointerOffset <= TNumericLimits<uint8>::Max());
		return static_cast<uint8>(PointerOffset);
	}
};

template<typename CapabilityType>
struct TPlaybackCapabilityStorageTraits<CapabilityType*, CapabilityType, void>
{
	static EPlaybackCapabilityStorageMode GetStorageMode() { return EPlaybackCapabilityStorageMode::RawPointer; }

	static uint8 ComputePointerOffset(CapabilityType** StoragePtr) { return 0; }
};

template<typename CapabilityType>
struct TPlaybackCapabilityStorageTraits<TSharedPtr<CapabilityType>, CapabilityType, void>
{
	static EPlaybackCapabilityStorageMode GetStorageMode() { return EPlaybackCapabilityStorageMode::SharedPointer; }

	static uint8 ComputePointerOffset(TSharedPtr<CapabilityType>* StoragePtr) { return 0; }
};

/**
 * Header that describes an entry in the capabilities buffer.
 */
struct FPlaybackCapabilityHeader
{
#if UE_MOVIESCENE_ENTITY_DEBUG
	// Extra debugging field.
	IPlaybackCapabilityDebuggingTypedPtr* DebugPtr = nullptr;
#endif

	// 2 Bytes
	TRelativePtr<void, uint16> Capability;

	// 2 Bytes
	uint16 Sizeof = 0;

	// 1 Byte
	uint8 Alignment = 0;

	// 1 Byte
	uint8 PointerOffset = 0;

	// 1 Byte
	EPlaybackCapabilityStorageMode StorageMode = EPlaybackCapabilityStorageMode::Inline;

	/** Resolve the given raw pointer into a capability pointer */
	FPlaybackCapabilityPtr Resolve(void* InMemory) const
	{
		void* DerivedPointer = Capability.Resolve(InMemory);
		return FPlaybackCapabilityPtr{ (void*)((uint8*)DerivedPointer + PointerOffset), StorageMode };
	}
};

/**
 * Basic, mostly untyped, implementation of the capabilities buffer.
 */
struct FPlaybackCapabilitiesImpl
{
protected:

	FPlaybackCapabilitiesImpl() = default;

	FPlaybackCapabilitiesImpl(const FPlaybackCapabilitiesImpl&) = delete;
	FPlaybackCapabilitiesImpl& operator=(const FPlaybackCapabilitiesImpl&) = delete;

	FPlaybackCapabilitiesImpl(FPlaybackCapabilitiesImpl&&) = delete;
	FPlaybackCapabilitiesImpl& operator=(FPlaybackCapabilitiesImpl&&) = delete;

	bool HasCapability(uint32 CapabilityBit) const
	{
		return (AllCapabilities & CapabilityBit) != 0;
	}

	FPlaybackCapabilityPtr FindCapability(uint32 CapabilityBit) const
	{
		if (HasCapability(CapabilityBit))
		{
			const int32 Index = GetCapabilityIndex(CapabilityBit);
			check(Index >= 0 && Index < 255);
			return GetHeader(static_cast<uint8>(Index)).Resolve(Memory);
		}

		return FPlaybackCapabilityPtr();
	}

	FPlaybackCapabilityPtr GetCapabilityChecked(uint32 CapabilityBit) const
	{
		const int32 Index = GetCapabilityIndex(CapabilityBit);
		check(Index >= 0 && Index < 255);
		return GetHeader(static_cast<uint8>(Index)).Resolve(Memory);
	}

	/**
	 * Creates and stores a new capability object at the given bit.
	 * 
	 * - StorageType is what is actually constructed and stored in our memory buffer. It can be the same as
	 *			CapabilityType, a subclass of it, a pointer to it, or a shared pointer to it.
	 * 
	 * - CapabilityType is the base capability type, used onlyfor inline storage when the StorageType is a subclass
	 *			and we need to compute the pointer offset.
	 */
	template<typename StorageType, typename CapabilityType, typename ...ArgTypes>
	FPlaybackCapabilityPtr AddCapability(uint32 CapabilityBit, ArgTypes&&... InArgs)
	{
		checkf((AllCapabilities & CapabilityBit) == 0, TEXT("Capability already exists!"));

		// Add the enum entry
		AllCapabilities |= CapabilityBit;

		// Find the index of the new capability by counting how many bits are set before it
		// For example, given CapabilityBit=0b00010000 and AllCapabilities=0b00011011:
		//                      (CapabilityBit-1) = 0b00001111
		//    AllCapabilities & (CapabilityBit-1) = 0b00001011
		//                  CountBits(0b00001011) = 3
		const int32 NewCapabilityIndex = static_cast<int32>(FMath::CountBits(AllCapabilities & (CapabilityBit-1u)));

		const FPlaybackCapabilityHeader* ExistingHeaders = reinterpret_cast<const FPlaybackCapabilityHeader*>(Memory);

		uint64 RequiredAlignment = FMath::Max(alignof(StorageType), alignof(FPlaybackCapabilityHeader));

		const int32 ExistingNum = static_cast<int32>(Num);

		// Compute our required alignment for the allocation
		{
			for (int32 Index = 0; Index < ExistingNum; ++Index)
			{
				RequiredAlignment = FMath::Max(RequiredAlignment, (uint64)ExistingHeaders[Index].Alignment);
			}
		}

		// We'll keep track of where all our new capabilities will go, relative to the buffer start
		TArray<uint16, TInlineAllocator<16>> NewCapabilityOffsets;
		NewCapabilityOffsets.SetNum(ExistingNum + 1);

		// Compute the required size of our allocation
		uint64 RequiredSizeof = 0u;

		{
			// Allocate space for headers
			RequiredSizeof += (ExistingNum + 1) * sizeof(FPlaybackCapabilityHeader);

			int32 Index = 0;

			// Count up the sizes and alignments of pre-existing capabilities that exist before this new entry
			for (; Index < NewCapabilityIndex; ++Index)
			{
				RequiredSizeof = Align(RequiredSizeof, (uint64)ExistingHeaders[Index].Alignment);
				NewCapabilityOffsets[Index] = static_cast<uint16>(RequiredSizeof);
				RequiredSizeof += ExistingHeaders[Index].Sizeof;
			}

			// Count up the size and alignment for the new capability
			RequiredSizeof = Align(RequiredSizeof, alignof(StorageType));
			NewCapabilityOffsets[Index] = static_cast<uint16>(RequiredSizeof);
			RequiredSizeof += sizeof(StorageType);
			++Index;

			// Now count up the sizes and alignments of pre-existing capabilities that exist after this new entry
			for (; Index < ExistingNum+1; ++Index)
			{
				RequiredSizeof = Align(RequiredSizeof, (uint64)ExistingHeaders[Index-1].Alignment);
				NewCapabilityOffsets[Index] = static_cast<uint16>(RequiredSizeof);
				RequiredSizeof += ExistingHeaders[Index-1].Sizeof;
			}
		}

		check( RequiredAlignment <= 0XFF );

		/// ----------------------------------------

		uint8* OldMemory = Memory;

		// Make a new allocation if necessary
		const bool bNeedsReallocation = RequiredAlignment > Alignment || RequiredSizeof > Capacity;
		if (bNeedsReallocation)
		{
			// Use the greater of the required size or double the current size to allow some additional capcity
			Capacity = static_cast<uint16>(FMath::Max(RequiredSizeof, uint64(Capacity)*2));
			Memory = reinterpret_cast<uint8*>(FMemory::Malloc(Capacity, RequiredAlignment));
		}

		// We now have an extra entry
		++Num;

		// We now need to re-arrange memory carefully, going back to front in order to avoid overlaps.
		// For instance we can only move headers at the end of the whole operation, otherwise they could overwrite
		// the memory of the first couple capabilities before we've had a chance to move them. So we do:
		// 
		// 1) Relocate capabilities from the last one up to where the new one will go
		// 2) Allocate the new capability
		// 3) Relocate capabilities from the one just before the new one, down to the first one
		// 4) Relocate headers from the last one up to where the new one will go
		// 5) Allocate the new header
		// 6) Relocate headers from the one just before the new header, down to the first one
		//
		// In order to set the new capability pointers (obtained in steps 1-3) on the new headers (obtained in
		// steps 4-6), we save these pointers in a temporary array.

		const FPlaybackCapabilityHeader* OldHeaders = ExistingHeaders; // Better named variable for the rest of the steps...

		auto RelocateCapability = [this, OldMemory, OldHeaders, &NewCapabilityOffsets](int32 OldIndex, int32 NewIndex)
		{
			void* NewCapabilityPtr = this->Memory + NewCapabilityOffsets[NewIndex];
			void* OldCapabilityPtr = OldHeaders[OldIndex].Capability.Resolve(OldMemory);
			FMemory::Memmove(NewCapabilityPtr, OldCapabilityPtr, OldHeaders[OldIndex].Sizeof);
		};

		// Step 1
		int32 Index = static_cast<int32>(Num) - 1;
		for (; Index > NewCapabilityIndex; --Index)
		{
			RelocateCapability(Index - 1, Index);
		}

		// Step 2
		{
			void* NewCapabilityPtr = this->Memory + NewCapabilityOffsets[Index];
			StorageType* NewCapabilityTypedPtr = reinterpret_cast<StorageType*>(NewCapabilityPtr);
			new (NewCapabilityTypedPtr) StorageType (Forward<ArgTypes>(InArgs)...);
		}
		--Index;
		
		// Step 3
		for (; Index >= 0; --Index)
		{
			RelocateCapability(Index, Index);
		}
		
		FPlaybackCapabilityHeader* NewHeaders = reinterpret_cast<FPlaybackCapabilityHeader*>(Memory);

		auto RelocateHeader = [this, OldHeaders, NewHeaders, &NewCapabilityOffsets](int32 OldIndex, int32 NewIndex)
		{
			const FPlaybackCapabilityHeader& OldHeader(OldHeaders[OldIndex]);
			FPlaybackCapabilityHeader* NewHeaderPtr = &NewHeaders[NewIndex];
			new (NewHeaderPtr) FPlaybackCapabilityHeader(OldHeader);

			// Re-create the relative capability pointer, since the offset has changed (it's one capability
			// further down), and the base pointer may have changed too (if we re-allocated the buffer)
			void* NewCapabilityPtr = this->Memory + NewCapabilityOffsets[NewIndex];
			NewHeaderPtr->Capability = TRelativePtr<void, uint16>(this->Memory, NewCapabilityPtr);

			// The new header will share its debugging pointer with the old header at this point. But the old
			// header will be gone when we free the entire old memory buffer after step 6.
		};

		// Step 4
		Index = static_cast<int32>(Num) - 1;
		for (; Index > NewCapabilityIndex; --Index)
		{
			RelocateHeader(Index - 1, Index);
		}

		// Step 5
		FPlaybackCapabilityHeader* NewHeaderPtr = &NewHeaders[Index];
		{
			using FStorageTraits = TPlaybackCapabilityStorageTraits<StorageType, CapabilityType>;
			static_assert(alignof(StorageType) < 0x7F, "Required alignment of capability must fit in 7 bytes");

			void* NewCapabilityPtr = Memory + NewCapabilityOffsets[Index];
			StorageType* NewCapabilityTypedPtr = reinterpret_cast<StorageType*>(NewCapabilityPtr);

			static_assert(sizeof(StorageType) <= MAX_uint16, "Can't store a playback capability with a size above 65535 bits. Store it as a raw or shared pointer.");
			static_assert(alignof(StorageType) <= MAX_uint8, "Can't store a playback capability with an alignment above 255 bits. Store it as a raw or shared pointer.");

			new (NewHeaderPtr) FPlaybackCapabilityHeader(); // Reset to default constructor

			NewHeaderPtr->Capability.Reset(Memory, NewCapabilityPtr);
			NewHeaderPtr->Sizeof = static_cast<uint16>(sizeof(StorageType));
			NewHeaderPtr->Alignment = static_cast<uint8>(alignof(StorageType));
			NewHeaderPtr->StorageMode = FStorageTraits::GetStorageMode();
			// The pointer offset is whatever we need to add to the capability pointer in order to get
			// a pointer to capability type itself (in case the storage type is a sub-class of it)
			// We only need it if the capability object is stored inline inside our memory buffer.
			NewHeaderPtr->PointerOffset = FStorageTraits::ComputePointerOffset(NewCapabilityTypedPtr);

#if UE_MOVIESCENE_ENTITY_DEBUG
			// Set the debug pointer.
			NewHeaderPtr->DebugPtr = new TPlaybackCapabilityDebuggingTypedPtr<StorageType>(NewCapabilityTypedPtr);
#endif
		}
		--Index;

		// Step 6
		for (; Index >= 0; --Index)
		{
			RelocateHeader(Index, Index);
		}

		/// ----------------------------------------

		// Tidy up the old allocation. We do not call destructors here because we relocated everything.
		if (bNeedsReallocation && OldMemory)
		{
			FMemory::Free(OldMemory);
		}

		// Insert the helpers for the new capability.
		Helpers.Insert(TPlaybackCapabilityHelpers<StorageType>::GetHelpers(), NewCapabilityIndex);

		// Return the new capability pointer. We call the header's Resolve method here because returning 
		// the pointer of the stored capability would return the derived type pointer. 
		// We want the base (capability) pointer.
		return NewHeaderPtr->Resolve(Memory);
	}

	template<typename StorageType, typename CapabilityType, typename ...ArgTypes>
	bool OverwriteCapability(uint32 CapabilityBit, ArgTypes&&... InArgs)
	{
		if (!ensureMsgf(HasCapability(CapabilityBit), TEXT("The given capability does not exist in this container")))
		{
			return false;
		}

		// Get the header for this capability
		const int32 Index = GetCapabilityIndex(CapabilityBit);
		FPlaybackCapabilityHeader& Header = GetHeader(static_cast<uint8>(Index));

		// Check that we are overwriting the same storage mode
		using FStorageTraits = TPlaybackCapabilityStorageTraits<StorageType, CapabilityType>;
		EPlaybackCapabilityStorageMode GivenStorageMode = FStorageTraits::GetStorageMode();
		if (!ensureMsgf(GivenStorageMode == Header.StorageMode, TEXT("The given capability storage mode does not match the existing one")))
		{
			return false;
		}

		// Check that the types and alignments match
		size_t GivenSizeof = sizeof(StorageType);
		uint16 GivenAlignment = alignof(StorageType);
		if (!ensureMsgf(
					(GivenSizeof == Header.Sizeof && GivenAlignment == Header.Alignment),
					TEXT("The given capability size and alignment do not match the existing one")))
		{
			return false;
		}

		// Don't use the header's Resolve method, we don't want to offset the pointer, we want
		// the actual pointer to the actual stored capability.
		void* CapabilityPtr = Header.Capability.Resolve(Memory);

		// Call the destructor on the previous capability, which is important if it was stored inline or as
		// a shared pointer.
		Helpers[Index].Destructor(CapabilityPtr);

		// Allocate the new capability.
		StorageType* TypedCapabilityPtr = reinterpret_cast<StorageType*>(CapabilityPtr);
		new (TypedCapabilityPtr) StorageType(Forward<ArgTypes>(InArgs)...);

		// If we have inline storage, we could potentially have changed from a stored base-class to
		// a stored child-class, or vice-versa, or from one child-class to another child-class. In all these
		// cases, the helper functions (destructor, interface cast, etc) have changed, so let's store the 
		// new helper functions on the header.
		Helpers[Index] = TPlaybackCapabilityHelpers<StorageType>::GetHelpers();

#if UE_MOVIESCENE_ENTITY_DEBUG
		// Update the debug pointer.
		delete Header.DebugPtr;
		Header.DebugPtr = new TPlaybackCapabilityDebuggingTypedPtr<StorageType>(TypedCapabilityPtr);
#endif

		return true;
	}

	const FPlaybackCapabilityHeader& GetHeader(uint8 Index) const
	{
		check(Index < Num);
		return reinterpret_cast<FPlaybackCapabilityHeader*>(Memory)[Index];
	}

	FPlaybackCapabilityHeader& GetHeader(uint8 Index)
	{
		check(Index < Num);
		return reinterpret_cast<FPlaybackCapabilityHeader*>(Memory)[Index];
	}

	TArrayView<const FPlaybackCapabilityHeader> GetHeaders() const
	{
		return MakeArrayView(reinterpret_cast<FPlaybackCapabilityHeader*>(Memory), Num);
	}

	int32 GetCapabilityIndex(uint32 CapabilityBit) const
	{
		if ( (AllCapabilities & CapabilityBit) == 0)
		{
			return INDEX_NONE;
		}
		return FMath::CountBits(AllCapabilities & (CapabilityBit-1u));
	}
	
	// Schema:
	// [header_1|...|header_n|entry_0|...|entry_n]
	uint8* Memory = nullptr;
	uint16 Alignment = 0u;
	uint16 Capacity = 0u;
	uint8 Num = 0u;
	uint32 AllCapabilities = 0u;

	// Function pointers for invalidating, destroying, etc. capabilities.
	TArray<FPlaybackCapabilityHelpers> Helpers;
};

namespace Internal
{

/** Gets the type of the playback capability ID in a backwards compatible way. */
template<typename T, typename = void>
struct TPlaybackCapabilityIDType {};

/** Getter for the new GetPlaybackCapabilityID method. */
template<typename T>
struct TPlaybackCapabilityIDType<
	T,
	std::enable_if_t<
		std::is_assignable_v<FPlaybackCapabilityID, decltype(T::GetPlaybackCapabilityID())>
		>>
{
	using Type = std::decay_t<decltype(T::GetPlaybackCapabilityID())>;

	static Type Get() { return T::GetPlaybackCapabilityID(); }
};

/** Getter for the old ID static field. */
template<typename T>
struct TPlaybackCapabilityIDType<
	T,
	std::enable_if_t<
		std::is_assignable_v<FPlaybackCapabilityID, decltype(T::ID)>
		>>
{
	using Type = std::decay_t<decltype(T::ID)>;

	static Type Get() { return T::ID; }
};

}  // namespace Internal

/**
 * Actual playback capabilities container.
 */
struct FPlaybackCapabilities : FPlaybackCapabilitiesImpl
{
	FPlaybackCapabilities() = default;

	FPlaybackCapabilities(const FPlaybackCapabilities&) = delete;
	FPlaybackCapabilities& operator=(const FPlaybackCapabilities&) = delete;

	MOVIESCENE_API FPlaybackCapabilities(FPlaybackCapabilities&&);
	MOVIESCENE_API FPlaybackCapabilities& operator=(FPlaybackCapabilities&&);

	MOVIESCENE_API ~FPlaybackCapabilities();

	/** Checks whether this container has the given capability */
	template<typename T>
	bool HasCapability() const
	{
		uint32 CapabilityBit = 1 << Internal::TPlaybackCapabilityIDType<T>::Get().Index;
		return FPlaybackCapabilitiesImpl::HasCapability(CapabilityBit);
	}

	/** Finds the specified capability within the container, if present */
	template<typename T>
	T* FindCapability() const
	{
		// T must be the base capability type, and not a sub-class, because we are returning a pointer to it,
		// and we can't check what sort of sub-class we have or not.
		using CapabilityIDType = typename Internal::TPlaybackCapabilityIDType<T>::Type;
		static_assert(std::is_same<T, typename CapabilityIDType::CapabilityType>::value, "You must pass the actual playback capability type as a template parameter, not a sub-class.");

		const TPlaybackCapabilityID<T> CapabilityID(Internal::TPlaybackCapabilityIDType<T>::Get());
		uint32 CapabilityBit = 1 << CapabilityID.Index;
		FPlaybackCapabilityPtr Ptr = FPlaybackCapabilitiesImpl::FindCapability(CapabilityBit);
		return Ptr.ResolveOptional<T>();
	}

	/** Returns the specified capability within the container, asserts if not found */
	template<typename T>
	T& GetCapabilityChecked() const
	{
		// T must be the base capability type, and not a sub-class, because we are returning a reference to it,
		// and we can't check what sort of sub-class we have or not.
		using CapabilityIDType = typename Internal::TPlaybackCapabilityIDType<T>::Type;
		static_assert(std::is_same<T, typename CapabilityIDType::CapabilityType>::value, "You must pass the actual playback capability type as a template parameter, not a sub-class.");

		const TPlaybackCapabilityID<T> CapabilityID(Internal::TPlaybackCapabilityIDType<T>::Get());
		uint32 CapabilityBit = 1 << CapabilityID.Index;
		FPlaybackCapabilityPtr Ptr = FPlaybackCapabilitiesImpl::GetCapabilityChecked(CapabilityBit);
		return Ptr.ResolveChecked<T>();
	}

	/**
	 * Adds the specified capability to the container, using the supplied arguments to construct it.
	 * The capability object will be stored inline and owned by this container. It will be destroyed when the
	 * container itself is destroyed.
	 * If the template parameter is a sub-class of the playback capability class, that sub-class will be
	 * created and stored inline.
	 */
	template<typename T, typename ...ArgTypes>
	T& AddCapability(ArgTypes&&... InArgs)
	{
		using CapabilityIDType = typename Internal::TPlaybackCapabilityIDType<T>::Type;
		using CapabilityType = typename CapabilityIDType::CapabilityType;

		CapabilityType& NewCapability = DoAddCapability<T>(Internal::TPlaybackCapabilityIDType<T>::Get(), Forward<ArgTypes>(InArgs)...);
		return static_cast<T&>(NewCapability);
	}

	/**
	 * As per AsCapability, but with an explicit capability ID. This is useful when the first template 
	 * parameter doesn't have a single static member called ID providing the capability ID.
	 */
	template<typename T, typename U, typename ...ArgTypes>
	T& AddCapability(const TPlaybackCapabilityID<U> CapabilityID, ArgTypes&&... InArgs)
	{
		U& NewCapability = DoAddCapability<T>(CapabilityID, Forward<ArgTypes>(InArgs)...);
		return static_cast<T&>(NewCapability);
	}

	/**
	 * Adds the specified capability to the container, as a simple raw pointer
	 * Ownership of the capability object being pointed to is the caller's responsability. This container will only
	 * store a raw pointer.
	 */
	template<typename T>
	T& AddCapabilityRaw(T* InPointer)
	{
		using CapabilityIDType = typename Internal::TPlaybackCapabilityIDType<T>::Type;
		using CapabilityType = typename CapabilityIDType::CapabilityType;

		CapabilityType& NewCapability = DoAddCapability<CapabilityType*>(Internal::TPlaybackCapabilityIDType<T>::Get(), static_cast<CapabilityType*>(InPointer));
		return static_cast<T&>(NewCapability);
	}
	
	/**
	 * Adds the specified capability to the container, as a shared pointer
	 * Ownership of the capability object being pointed to respects classic shared pointer semantics. This container
	 * only maintains one such shared pointer until the container is destroyed.
	 */
	template<typename T>
	T& AddCapabilityShared(TSharedRef<T> InSharedRef)
	{
		using CapabilityIDType = typename Internal::TPlaybackCapabilityIDType<T>::Type;
		using CapabilityType = typename CapabilityIDType::CapabilityType;

		CapabilityType& NewCapability = DoAddCapability<TSharedPtr<CapabilityType>>(Internal::TPlaybackCapabilityIDType<T>::Get(), StaticCastSharedRef<CapabilityType>(InSharedRef));
		return static_cast<T&>(NewCapability);
	}

	/**
	 * Overwrites an existing capability, stored inline and owned by this container.
	 * The previous storage mode of the capability must also be inline.
	 * If the template parameter is a sub-class of the playback capability, the previously stored playback capability
	 * must not only have been stored inline, but must have been of the same sub-class (or a sub-class with the exact
	 * same size and alignment).
	 */
	template<typename T, typename ...ArgTypes>
	T& OverwriteCapability(ArgTypes&&... InArgs)
	{
		using CapabilityIDType = typename Internal::TPlaybackCapabilityIDType<T>::Type;
		using CapabilityType = typename CapabilityIDType::CapabilityType;

		CapabilityType& NewCapability = DoOverwriteCapability<T>(Internal::TPlaybackCapabilityIDType<T>::Get(), Forward<ArgTypes>(InArgs)...);
		return static_cast<T&>(NewCapability);
	}

	/**
	 * Overwrites an existing capability, stored as a raw pointer on the container.
	 * The previous storage mode of the capability must also be a raw pointer.
	 */
	template<typename T>
	T& OverwriteCapabilityRaw(T* InPointer)
	{
		using CapabilityIDType = typename Internal::TPlaybackCapabilityIDType<T>::Type;
		using CapabilityType = typename CapabilityIDType::CapabilityType;

		CapabilityType& NewCapability = DoOverwriteCapability<CapabilityType*>(Internal::TPlaybackCapabilityIDType<T>::Get(), static_cast<CapabilityType*>(InPointer));
		return static_cast<T&>(NewCapability);
	}
	
	/**
	 * Overwrites an existing capability, stored as a shared pointer on the container.
	 * The previous storage mode of the capability must also be a shared pointer.
	 */
	template<typename T>
	T& OverwriteCapabilityShared(TSharedRef<T> InSharedRef)
	{
		using CapabilityIDType = typename Internal::TPlaybackCapabilityIDType<T>::Type;
		using CapabilityType = typename CapabilityIDType::CapabilityType;

		CapabilityType& NewCapability = DoOverwriteCapability<TSharedPtr<CapabilityType>>(Internal::TPlaybackCapabilityIDType<T>::Get(), StaticCastSharedRef<CapabilityType>(InSharedRef));
		return static_cast<T&>(NewCapability);
	}

public:

	/**
	 * Calls OnSubInstanceCreated on any capability that implements the IPlaybackCapability interface.
	 */
	void OnSubInstanceCreated(TSharedRef<const FSharedPlaybackState> Owner, const FInstanceHandle InstanceHandle);

	/**
	 * Calls InvalidateCacheData on any capability that implements the IPlaybackCapability interface.
	 */
	void InvalidateCachedData(UMovieSceneEntitySystemLinker* Linker);

private:

	template<typename Callback, typename ...ArgTypes>
	void ForEachCapabilityInterface(Callback&& InCallback, ArgTypes&&... InArgs)
	{
		TArrayView<const FPlaybackCapabilityHeader> Headers = GetHeaders();
		for (int32 Index = 0; Index < Headers.Num(); ++Index)
		{
			const FPlaybackCapabilityHeader& Header = Headers[Index];
			const FPlaybackCapabilityHelpers& ThisHelpers = Helpers[Index];
			check(ThisHelpers.InterfaceCast != nullptr);
			{
				void* Ptr = Header.Capability.Resolve(Memory);
				if (IPlaybackCapability* Interface = (*ThisHelpers.InterfaceCast)(Ptr))
				{
					InCallback(*Interface, Forward<ArgTypes>(InArgs)...);
				}
			}
		}
	}

	template<typename Impl, typename T, typename ...ArgTypes>
	T& DoAddCapability(TPlaybackCapabilityID<T> CapabilityID, ArgTypes&&... InArgs)
	{
		uint32 CapabilityBit = 1 << CapabilityID.Index;
		FPlaybackCapabilityPtr Ptr = FPlaybackCapabilitiesImpl::AddCapability<Impl, T>(CapabilityBit, Forward<ArgTypes>(InArgs)...);
		return Ptr.ResolveChecked<T>();
	}

	template<typename Impl, typename T, typename ...ArgTypes>
	T& DoOverwriteCapability(TPlaybackCapabilityID<T> CapabilityID, ArgTypes&&... InArgs)
	{
		uint32 CapabilityBit = 1 << CapabilityID.Index;
		FPlaybackCapabilitiesImpl::OverwriteCapability<Impl, T>(CapabilityBit, Forward<ArgTypes>(InArgs)...);
		return GetCapabilityChecked<T>();
	}

	void Destroy();
};

} // namespace UE::MovieScene
