// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "Evaluation/MovieScenePlayback.h"
#include "Evaluation/MovieSceneSequenceTransform.h"
#include "Misc/Optional.h"
#include "MovieSceneFwd.h"

class UMovieSceneSequence;

namespace UE::MovieScene { struct FSharedPlaybackState; }

/**
 * Describes whether to dissect looping when playing back a sequence.
 *
 * Looping dissection means that, when a sequence loops, more than one evaluation context
 * is returned by the FMovieScenePlaybackManager::Update() methods: one context for updating 
 * to the "last valid time", and one context for updating through the first few frames of 
 * the next loop.
 */
enum class EMovieSceneLoopDissection
{
	/** 
	 * Do no dissect loop. Jump directly to the current time of the next loop, using an
	 * evaluation range from the first frame to the current time.
	 * WARNING: this means that the last few frames of the previous loop won't be evaluated!
	 */
	None,

	/**
	 * Only dissect one loop. Emit an evaluation range for the last few frames of the 
	 * current loop, and another evaluation range for the first few frames of the next 
	 * loop.
	 */
	DissectOne,

	/**
	 * Dissect all loops. As per DissectOne, but also emit evaluation ranges for entire loops
	 * if the delta-time is large enough, and/or the sequence is short enough, that we might
	 * have gone through entire extra loops.
	 */
	DissectAll
};

/**
 * A utility class that can manage a playing sequence's status and current time, while also
 * handling looping, ping-ponging, and other playback modes.
 *
 * All public APIs take and return times in display rate (e.g. 30fps frames). Internally, 
 * everything is treated as ticks (e.g. 60000 ticks/sec).
 */
class FMovieScenePlaybackManager
{
public:

	using FContexts = TArray<FMovieSceneContext, TInlineAllocator<2>>;

	/** Creates a default playback manager. */
	MOVIESCENE_API FMovieScenePlaybackManager();

	/** Creates a playback manager immediately initialized with the given sequence. */
	MOVIESCENE_API FMovieScenePlaybackManager(UMovieSceneSequence* InSequence);

	/** 
	 * Initializes the playback manager for the given sequence.
	 * This resets all playback settings (play rate, start/end offsets, etc.) to their default
	 * values.
	 */
	MOVIESCENE_API void Initialize(UMovieSceneSequence* InSequence);

	/**
	 * Updates the playback state and returns the evaluation contexts to use for evaluating
	 * the sequence. More than one context may be returned if the sequence loops once or more,
	 * and "looping dissection" is enabled.
	 *
	 * @param InDeltaSeconds  The delta-time to update with
	 * @param OutContexts     One or more update contexts to evaluate the sequence
	 */
	MOVIESCENE_API void Update(float InDeltaSeconds, FContexts& OutContexts);

	/**
	 * As per the other Update method, but takes a time (in display rate) instead of seconds.
	 */
	MOVIESCENE_API void UpdateTo(const FFrameTime NextTime, FContexts& OutContexts);

	/**
	 * Updates the playback state over only the current tick. This generally does not advance the
	 * current time by one tick, but by one *tick bound*. That is, if the current time is tick 0,
	 * this will evaluate the [0, 0] range, advancing to an exclusive lower bound, so that the
	 * next update will be (0, XYZ).
	 */
	MOVIESCENE_API FMovieSceneContext UpdateToNextTick();

	/**
	 * Returns an evaluation context for the current time, i.e. using a zero-width evaluation
	 * range set around the current time.
	 */
	MOVIESCENE_API FMovieSceneContext UpdateAtCurrentTime() const;

	/** Gets the current playback position, in display rate. */
	MOVIESCENE_API FFrameTime GetCurrentTime() const;

	/** Sets the current playback position, in display rate. */
	MOVIESCENE_API void SetCurrentTime(const FFrameTime& InFrameTime);
	/** Sets the current playback position, in display rate, as an offset from the effective start time. */
	MOVIESCENE_API void SetCurrentTimeOffset(const FFrameTime& InFrameTimeOffset);

	/** Get the effective playback range, in display rate, taking into account start/end offsets. */
	MOVIESCENE_API TRange<FFrameTime> GetEffectivePlaybackRange() const;

	/** Get the effective playback start time, in display rate, taking into account any start offset. */
	MOVIESCENE_API FFrameTime GetEffectiveStartTime() const;
	/** Get the effective playback end time, in display rate, taking into account any end offset. */
	MOVIESCENE_API FFrameTime GetEffectiveEndTime() const;

public:

	/** Gets the display rate of the sequence. */
	FFrameRate GetDisplayRate() const { return DisplayRate; }
	/** Gets the tick resolution of the sequence. */
	FFrameRate GetTickResolution() const { return PlaybackPosition.GetOutputRate(); }

	/** Gets whether looping dissection is enabled (see EMovieSceneLoopDissection). */
	EMovieSceneLoopDissection GetDissectLooping() const { return DissectLooping; }
	/** Sets whether looping dissection is enabled (see EMovieSceneLoopDissection). */
	void SetDissectLooping(EMovieSceneLoopDissection InDissectLooping) { DissectLooping = InDissectLooping; }

	/** Get the playback status. */
	EMovieScenePlayerStatus::Type GetPlaybackStatus() const { return PlaybackStatus; }
	/** Set the playback status. */
	void SetPlaybackStatus(EMovieScenePlayerStatus::Type InPlaybackStatus) { PlaybackStatus = InPlaybackStatus; }

	/** Gets the number of loops to play before ending playback. */
	int32 GetNumLoopsToPlay() const { return NumLoopsToPlay; }
	/** Sets the number of loops to play before ending playback. */
	MOVIESCENE_API void SetNumLoopsToPlay(int32 InNumLoopsToPlay);

	/** Returns the current number of loops completed so far. */
	int32 GetNumLoopsCompleted() const { return NumLoopsCompleted; }
	/** Reset the number of completed loops to zero. */
	void ResetNumLoopsCompleted() { NumLoopsCompleted = 0; }

	/** Gets the play-rate. */
	double GetPlayRate() const { return PlayRate; }
	/** Sets the play-rate. */
	void SetPlayRate(double InPlayRate) { PlayRate = InPlayRate; }

	/** Gets whether the sequence should be playing forwards. */
	bool IsPlayingForward() const { return PlayDirection == EPlayDirection::Forwards; } 
	/** Gets whether the sequence should be playing in reverse. */
	bool IsPlayingBackward() const { return PlayDirection == EPlayDirection::Backwards; } 
	/** Gets the playback direction. */
	EPlayDirection GetPlayDirection() const { return PlayDirection; }
	/** Sets the playback direction. */
	void SetPlayDirection(EPlayDirection InPlayDirection) { PlayDirection = InPlayDirection; }
	/** Reverses the playback direction. */
	MOVIESCENE_API void ReversePlayDirection();

	/** Gets whether the sequence should ping-pong between forwards and backwards playback. */
	bool IsPingPongPlayback() const { return bPingPongPlayback; }
	/** 
	 * Sets whether the sequence should ping-pong between forwards and backwards playback.
	 * Note that a "loop" is still one play through of the sequence, whether forwards or 
	 * backwards, so odd numbers of loops corresponds to a "ping" without a "pong".
	 */
	MOVIESCENE_API void SetPingPongPlayback(bool bInPingPongPlayback);

	/** Gets the start offset of the playback in display rate. */
	MOVIESCENE_API FFrameTime GetStartOffset() const;
	/** Gets the end offset of the playback in display rate. */
	MOVIESCENE_API FFrameTime GetEndOffset() const;
	/** Sets the start offset of the playback in display rate. */
	MOVIESCENE_API void SetStartOffset(const FFrameTime& InStartOffset);
	/** Sets the end offset of the playback in display rate. */
	MOVIESCENE_API void SetEndOffset(const FFrameTime& InEndOffset);
	/** Sets the end time of the playback in display rate. */
	MOVIESCENE_API void SetEndOffsetAsTime(const FFrameTime& InEndTime);

	/** Gets the time to stop at on the last loop of playback. */
	MOVIESCENE_API TOptional<FFrameTime> GetPlaybackEndTime() const;
	/** Sets the time to stop at on the last loop of playback. */
	MOVIESCENE_API void SetPlaybackEndTime(const FFrameTime& InEndTime);
	/** Removes any previously specified time to stop at. */
	MOVIESCENE_API void ClearPlaybackEndTime();

	/** Gets whether the playback time transform should be applied to time updates. */
	bool ShouldTransformPlaybackTime() const { return bTransformPlaybackTime; }
	/** Sets whether the playback time transform should be applied to time updates. */
	void SetTransformPlaybackTime(bool bInTransformPlaybackTime) { bTransformPlaybackTime = bInTransformPlaybackTime; }

	/**
	 * Gets the time transform to apply to the current time when updating.
	 * Only used if ShouldTransformPlaybackTime is true.
	 */
	const FMovieSceneSequenceTransform& GetPlaybackTimeTransform() const { return TimeTransform; }

	/*
	 * Sets the time transform to apply to the current time when updating.
	 * Only used if ShouldTransformPlaybackTime is true.
	 */
	void SetPlaybackTimeTransform(const FMovieSceneSequenceTransform& InTimeTransform) { TimeTransform = InTimeTransform; }

private:

	void InternalUpdateToTick(const FFrameNumber NextTick, FContexts& OutContexts);

	void ResetPlaybackSettings();

	void SetStartAndEndOffsetTicks(FFrameNumber InStartOffsetTicks, FFrameNumber InEndOffsetTicks);

	FFrameNumber GetLastValidTick() const;

private:

	FMovieScenePlaybackPosition PlaybackPosition;

	FFrameRate DisplayRate;

	EMovieScenePlayerStatus::Type PlaybackStatus;
	EPlayDirection PlayDirection;

	FFrameNumber SequenceStartTick;
	FFrameNumber SequenceEndTick;

	FFrameNumber StartOffsetTicks;
	FFrameNumber EndOffsetTicks;

	TOptional<FFrameNumber> PlaybackEndTick;

	int32 NumLoopsToPlay = 1;
	int32 NumLoopsCompleted = 0;

	double PlayRate = 1.0;

	FMovieSceneSequenceTransform TimeTransform;

	EMovieSceneLoopDissection DissectLooping = EMovieSceneLoopDissection::DissectOne;

	bool bPingPongPlayback = false;
	bool bTransformPlaybackTime = false;
};

