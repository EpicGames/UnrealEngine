// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CoreTypes.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UClass;
class UMovieSceneEntitySystemLinker;
class UObject;
struct FMovieSceneAnimTypeID;
struct FMovieSceneEvaluationKey;
struct FMovieSceneSequenceID;
struct IMovieScenePreAnimatedGlobalTokenProducer;
struct IMovieScenePreAnimatedTokenProducer;
template <typename FuncType> class TFunctionRef;

/**
 * Class that caches pre-animated state for objects that were manipulated by sequencer
 */
class FMovieSceneInstancePreAnimatedState
{
public:

	FMovieSceneInstancePreAnimatedState(UMovieSceneEntitySystemLinker* InLinker, UE::MovieScene::FRootInstanceHandle InInstanceHandle);

	FMovieSceneInstancePreAnimatedState(const FMovieSceneInstancePreAnimatedState&) = delete;
	FMovieSceneInstancePreAnimatedState& operator=(const FMovieSceneInstancePreAnimatedState&) = delete;

	MOVIESCENE_API ~FMovieSceneInstancePreAnimatedState();

	/**
	 * Check whether this sequence instance is capturing any and all changes of state so they can be restored later
	 */
	MOVIESCENE_API bool IsCapturingGlobalPreAnimatedState() const;

	/**
	 * Check whether this sequence instance is capturing any and all changes of state so they can be restored later
	 */
	MOVIESCENE_API void EnableGlobalPreAnimatedStateCapture();

	/**
	 * Retrieve the linker this container is bound to
	 */
	MOVIESCENE_API UMovieSceneEntitySystemLinker* GetLinker() const;

public:

	/**
	 * Save the current state of an object as defined by the specified token producer, identified by a specific anim type ID
	 * This will use the currently evaluating track template, evaluation hook or track instance (and its 'When Finished' property) as the capture source
	 */
	MOVIESCENE_API void SavePreAnimatedState(UObject& InObject, FMovieSceneAnimTypeID InTokenType, const IMovieScenePreAnimatedTokenProducer& Producer);

	/**
	 * Save the current state of the environment as defined by the specified token producer, identified by a specific anim type ID
	 * This will use the currently evaluating track template, evaluation hook or track instance (and its 'When Finished' property) as the capture source
	 */
	MOVIESCENE_API void SavePreAnimatedState(FMovieSceneAnimTypeID InTokenType, const IMovieScenePreAnimatedGlobalTokenProducer& Producer);

public:

	MOVIESCENE_API void RestorePreAnimatedState();

	MOVIESCENE_API void RestorePreAnimatedState(UObject& Object);

	MOVIESCENE_API void RestorePreAnimatedState(UClass* GeneratedClass);

	MOVIESCENE_API void RestorePreAnimatedState(UObject& Object, TFunctionRef<bool(FMovieSceneAnimTypeID)> InFilter);

	// Discards all pre-animated state without restoring it.
	MOVIESCENE_API void DiscardPreAnimatedState();

	/**
	 * Discard any tokens that relate to entity animation (ie sections or tracks) without restoring the values.
	 * Any global pre-animated state tokens (that reset the animation when saving a map, for instance) will remain.
	 */
	MOVIESCENE_API void DiscardEntityTokens();

	/**
	 * Discard any tokens that relate to the requested object (ie sections or tracks) without restoring the values.
	 * Any global pre-animated state tokens for this object will be removed.
	 */
	MOVIESCENE_API void DiscardAndRemoveEntityTokensForObject(UObject& Object);

public:

	/**
	 * Search the global pre-animated state extension for any captured state that originated from this sequence
	 * WARNING: This is a linear search across all state, and so is potentially very slow
	 */
	MOVIESCENE_API bool ContainsAnyStateForSequence() const;

private:

	void ConditionalInitializeEntityStorage(bool bOverrideWantsRestoreState);

private:

	/** Weak pointer to the linker that we're associated with */
	TWeakObjectPtr<UMovieSceneEntitySystemLinker> WeakLinker;

	/** The instance handle for the root sequence instance */
	UE::MovieScene::FRootInstanceHandle InstanceHandle;

	/** Whether global pre-animated state capture has been requested on the linker extension */
	bool bCapturingGlobalPreAnimatedState;
};

/**
 * A class similar to FMovieSceneInstancePreAnimatedState (above), which doesn't manage global pre-animated state 
 * requests, deferring those to the other class. This class is mostly for IMovieScenePlayer, for backwards
 * compatibility.
 */
class FMovieScenePreAnimatedState
{
public:

	FMovieScenePreAnimatedState() = default;
	FMovieScenePreAnimatedState(const FMovieScenePreAnimatedState&) = delete;
	FMovieScenePreAnimatedState& operator=(const FMovieScenePreAnimatedState&) = delete;

	MOVIESCENE_API void Initialize(UMovieSceneEntitySystemLinker* Linker, UE::MovieScene::FRootInstanceHandle InstanceHandle);
	MOVIESCENE_API bool IsCapturingGlobalPreAnimatedState() const;
	MOVIESCENE_API void EnableGlobalPreAnimatedStateCapture();
	MOVIESCENE_API UMovieSceneEntitySystemLinker* GetLinker() const;
	MOVIESCENE_API void SavePreAnimatedState(UObject& InObject, FMovieSceneAnimTypeID InTokenType, const IMovieScenePreAnimatedTokenProducer& Producer);
	MOVIESCENE_API void SavePreAnimatedState(FMovieSceneAnimTypeID InTokenType, const IMovieScenePreAnimatedGlobalTokenProducer& Producer);
	MOVIESCENE_API void RestorePreAnimatedState();
	MOVIESCENE_API void RestorePreAnimatedState(UObject& Object);
	MOVIESCENE_API void RestorePreAnimatedState(UClass* GeneratedClass);
	MOVIESCENE_API void RestorePreAnimatedState(UObject& Object, TFunctionRef<bool(FMovieSceneAnimTypeID)> InFilter);
	MOVIESCENE_API void DiscardPreAnimatedState();
	MOVIESCENE_API void DiscardEntityTokens();
	MOVIESCENE_API void DiscardAndRemoveEntityTokensForObject(UObject& Object);
	MOVIESCENE_API bool ContainsAnyStateForSequence() const;

private:

	FMovieSceneInstancePreAnimatedState* GetState() const;

	TWeakObjectPtr<UMovieSceneEntitySystemLinker> WeakLinker;
	UE::MovieScene::FRootInstanceHandle InstanceHandle;

	friend struct FScopedPreAnimatedCaptureSource;
};

