// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "UObject/ObjectMacros.h"
#include "IMovieScenePlayer.h"
#include "MovieSceneFwd.h"
#include "MovieSceneCommonHelpers.h"
#include "Evaluation/MovieSceneAnimTypeID.h"
#include "MovieSceneExecutionToken.h"
#include "Evaluation/PersistentEvaluationData.h"
#include "Evaluation/MovieSceneEvalTemplate.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "MovieSceneSequence.h"
#include "Evaluation/Blending/MovieSceneBlendingActuator.h"
#include "Evaluation/Blending/MovieSceneMultiChannelBlending.h"
#include "Compilation/MovieSceneTemplateInterrogation.h"
#include "TransformData.h"
#include "MovieScenePropertyTemplate.generated.h"

class UMovieScenePropertyTrack;

DECLARE_CYCLE_STAT(TEXT("Property Track Token Execute"), MovieSceneEval_PropertyTrack_TokenExecute, STATGROUP_MovieSceneEval);

namespace PropertyTemplate
{
	/** Persistent section data for a property section */
	struct FSectionData : IPersistentEvaluationData
	{
		MOVIESCENE_API FSectionData();

		/** Initialize track data with the specified property name, path, optional setter function, and optional notify function */
		MOVIESCENE_API void Initialize(FName InPropertyName, FString InPropertyPath);

		/** Property bindings used to get and set the property */
		TSharedPtr<FTrackInstancePropertyBindings> PropertyBindings;

		/** Cached identifier of the property we're editing */
		FMovieSceneAnimTypeID PropertyID;
	};

	template<typename PropertyValueType, typename IntermediateType = PropertyValueType>
	struct TTemporarySetterType { typedef PropertyValueType Type; };

	/** Convert from an intermediate type to the type used for setting a property value. Called when resetting pre animated state */
	template<typename PropertyValueType, typename IntermediateType = PropertyValueType>
	typename TTemporarySetterType<PropertyValueType, IntermediateType>::Type
		ConvertFromIntermediateType(const IntermediateType& InIntermediateType, IMovieScenePlayer& Player)
	{
		return InIntermediateType;
	}

	/** Convert from an intermediate type to the type used for setting a property value. Called during token execution. */
	template<typename PropertyValueType, typename IntermediateType = PropertyValueType>
	typename TTemporarySetterType<PropertyValueType, IntermediateType>::Type
		ConvertFromIntermediateType(const IntermediateType& InIntermediateType, const FMovieSceneEvaluationOperand& Operand, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player)
	{
		return InIntermediateType;
	}
	
	template<typename PropertyValueType, typename IntermediateType = PropertyValueType>
	IntermediateType ConvertToIntermediateType(PropertyValueType&& NewValue)
	{
		return Forward<PropertyValueType>(NewValue);
	}

	template<typename T>
	bool IsValueValid(const T& InValue)
	{
		return true;
	}

	/** Cached preanimated state for a given property */
	template<typename PropertyValueType, typename IntermediateType = PropertyValueType>
	struct TCachedState : IMovieScenePreAnimatedToken
	{
		TCachedState(typename TCallTraits<IntermediateType>::ParamType InValue, const FTrackInstancePropertyBindings& InBindings)
			: Value(MoveTempIfPossible(InValue))
			, Bindings(InBindings)
		{
		}

		virtual void RestoreState(UObject& Object, const UE::MovieScene::FRestoreStateParams& Params) override
		{
			IMovieScenePlayer* Player = Params.GetTerminalPlayer();
			if (!ensure(Player))
			{
				return;
			}

			auto NewValue = PropertyTemplate::ConvertFromIntermediateType<PropertyValueType, IntermediateType>(Value, *Player);
			if (IsValueValid(NewValue))
			{
				Bindings.CallFunction<PropertyValueType>(Object, NewValue);
			}
		}

		IntermediateType Value;
		FTrackInstancePropertyBindings Bindings;
	};

	template<typename PropertyValueType, typename IntermediateType = PropertyValueType>
	static IMovieScenePreAnimatedTokenPtr CacheExistingState(UObject& Object, FTrackInstancePropertyBindings& PropertyBindings)
	{
		return TCachedState<PropertyValueType, IntermediateType>(PropertyTemplate::ConvertToIntermediateType(PropertyBindings.GetCurrentValue<PropertyValueType>(Object)), PropertyBindings);
	}


	template<typename PropertyValueType>
	struct FTokenProducer : IMovieScenePreAnimatedTokenProducer
	{
		FTokenProducer(FTrackInstancePropertyBindings& InPropertyBindings) : PropertyBindings(InPropertyBindings) {}

		virtual IMovieScenePreAnimatedTokenPtr CacheExistingState(UObject& Object) const
		{
			return PropertyTemplate::CacheExistingState<PropertyValueType>(Object, PropertyBindings);
		}

		FTrackInstancePropertyBindings& PropertyBindings;
	};

}

/** Execution token that simple stores a value, and sets it when executed */
template<typename PropertyValueType, typename IntermediateType = PropertyValueType>
struct TPropertyTrackExecutionToken : IMovieSceneExecutionToken
{
	TPropertyTrackExecutionToken(IntermediateType InValue)
		: Value(MoveTemp(InValue))
	{}

	/** Execute this token, operating on all objects referenced by 'Operand' */
	virtual void Execute(const FMovieSceneContext& Context, const FMovieSceneEvaluationOperand& Operand, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) override
	{
		using namespace PropertyTemplate;

		MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_PropertyTrack_TokenExecute)
		
		FSectionData& PropertyTrackData = PersistentData.GetSectionData<FSectionData>();
		FTrackInstancePropertyBindings* PropertyBindings = PropertyTrackData.PropertyBindings.Get();

		auto NewValue = PropertyTemplate::ConvertFromIntermediateType<PropertyValueType, IntermediateType>(Value, Operand, PersistentData, Player);
		if (!IsValueValid(NewValue))
		{
			return;
		}

		for (TWeakObjectPtr<> WeakObject : Player.FindBoundObjects(Operand))
		{
			if (UObject* ObjectPtr = WeakObject.Get())
			{
				Player.SavePreAnimatedState(*ObjectPtr, PropertyTrackData.PropertyID, FTokenProducer<PropertyValueType>(*PropertyBindings));
				
				PropertyBindings->CallFunction<PropertyValueType>(*ObjectPtr, NewValue);
			}
		}
	}

	IntermediateType Value;
};

/** Blending actuator type that knows how to apply values of type PropertyType */
template<typename PropertyType>
struct TPropertyActuator : TMovieSceneBlendingActuator<PropertyType>
{
	PropertyTemplate::FSectionData PropertyData;
	TPropertyActuator(const PropertyTemplate::FSectionData& InPropertyData)
		: TMovieSceneBlendingActuator<PropertyType>(FMovieSceneBlendingActuatorID(InPropertyData.PropertyID))
		, PropertyData(InPropertyData)
	{}

	virtual PropertyType RetrieveCurrentValue(UObject* InObject, IMovieScenePlayer* Player) const
	{
		return PropertyData.PropertyBindings->GetCurrentValue<PropertyType>(*InObject);
	}

	virtual void Actuate(UObject* InObject, typename TCallTraits<PropertyType>::ParamType InFinalValue, const TBlendableTokenStack<PropertyType>& OriginalStack, const FMovieSceneContext& Context, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) override
	{
		ensureMsgf(InObject, TEXT("Attempting to evaluate a Property track '%s' with a null object."), *PropertyData.PropertyBindings->GetPropertyName().ToString());

		if (InObject)
		{
			OriginalStack.SavePreAnimatedState(Player, *InObject, PropertyData.PropertyID, PropertyTemplate::FTokenProducer<PropertyType>(*PropertyData.PropertyBindings));
			PropertyData.PropertyBindings->CallFunction<PropertyType>(*InObject, InFinalValue);
		}
	}
	//This will be specialized at end of this file for floats, transforms,eulertransforms and vectors
	virtual void Actuate(FMovieSceneInterrogationData& InterrogationData, typename TCallTraits<PropertyType>::ParamType InValue, const TBlendableTokenStack<PropertyType>& OriginalStack, const FMovieSceneContext& Context) const override
	{
	}
};


USTRUCT()
struct FMovieScenePropertySectionData
{
	GENERATED_BODY()

	UPROPERTY()
	FName PropertyName;

	UPROPERTY()
	FString PropertyPath;

	FMovieScenePropertySectionData()
	{}

	FMovieScenePropertySectionData(FName InPropertyName, FString InPropertyPath)
		: PropertyName(InPropertyName), PropertyPath(MoveTemp(InPropertyPath))
	{}

	/** Helper function to create FSectionData for a property section */
	void SetupTrack(FPersistentEvaluationData& PersistentData) const
	{
		SetupTrack<PropertyTemplate::FSectionData>(PersistentData);
	}

	template<typename T>
	void SetupTrack(FPersistentEvaluationData& PersistentData) const
	{
		PersistentData.AddSectionData<T>().Initialize(PropertyName, PropertyPath);
	}
};

USTRUCT()
struct FMovieScenePropertySectionTemplate : public FMovieSceneEvalTemplate
{
	GENERATED_BODY()
	
	MOVIESCENE_API FMovieScenePropertySectionTemplate();
	MOVIESCENE_API FMovieScenePropertySectionTemplate(FName PropertyName, const FString& InPropertyPath);
	MOVIESCENE_API FMovieScenePropertySectionTemplate(const FMovieScenePropertySectionTemplate&);
	MOVIESCENE_API ~FMovieScenePropertySectionTemplate();
public:
	//use thse keys for setting and iterating the correct types.
	MOVIESCENE_API const static FMovieSceneInterrogationKey GetFloatInterrogationKey();
	MOVIESCENE_API const static FMovieSceneInterrogationKey GetInt32InterrogationKey();
	MOVIESCENE_API const static FMovieSceneInterrogationKey GetVector4InterrogationKey();
	MOVIESCENE_API const static FMovieSceneInterrogationKey GetVectorInterrogationKey();
	MOVIESCENE_API const static FMovieSceneInterrogationKey GetVector2DInterrogationKey();
	MOVIESCENE_API const static FMovieSceneInterrogationKey GetColorInterrogationKey();

protected:

	virtual UScriptStruct& GetScriptStructImpl() const override { return *StaticStruct(); }
	/** Setup is only called if derived classes enable RequiresSetupFlag */
	MOVIESCENE_API virtual void Setup(FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) const override;

	/** Access an animation type identifier that uniquely represents the property this section animates */
	MOVIESCENE_API FMovieSceneAnimTypeID GetPropertyTypeID() const;

	/** Ensure that an actutor for this property type exists */
	template<typename PropertyType> FMovieSceneBlendingActuatorID EnsureActuator(FMovieSceneBlendingAccumulator& Accumulator) const
	{
		// Actuator type ID for this property
		FMovieSceneAnimTypeID UniquePropertyID = GetPropertyTypeID();
		FMovieSceneBlendingActuatorID ActuatorTypeID = FMovieSceneBlendingActuatorID(UniquePropertyID);
		if (!Accumulator.FindActuator<PropertyType>(ActuatorTypeID))
		{
			PropertyTemplate::FSectionData SectionData;
			SectionData.Initialize(PropertyData.PropertyName, PropertyData.PropertyPath);

			Accumulator.DefineActuator(ActuatorTypeID, MakeShared<TPropertyActuator<PropertyType>>(SectionData));
		}
		return ActuatorTypeID;
	}

	UPROPERTY()
	FMovieScenePropertySectionData PropertyData;
};

//Specializations 
template<>
inline void TPropertyActuator<float>::Actuate(FMovieSceneInterrogationData& InterrogationData, typename TCallTraits<float>::ParamType InValue, const TBlendableTokenStack<float>& OriginalStack, const FMovieSceneContext& Context) const
{
	float Value = InValue;
	InterrogationData.Add(Value, FMovieScenePropertySectionTemplate::GetFloatInterrogationKey());
};

template<>
inline void TPropertyActuator<int32>::Actuate(FMovieSceneInterrogationData& InterrogationData, typename TCallTraits<int32>::ParamType InValue, const TBlendableTokenStack<int32>& OriginalStack, const FMovieSceneContext& Context) const
{
	int32 Value = InValue;
	InterrogationData.Add(Value, FMovieScenePropertySectionTemplate::GetInt32InterrogationKey());
};

template<>
inline void TPropertyActuator<FVector2D>::Actuate(FMovieSceneInterrogationData& InterrogationData, typename TCallTraits<FVector2D>::ParamType InValue, const TBlendableTokenStack<FVector2D>& OriginalStack, const FMovieSceneContext& Context) const
{
	FVector2D Value = InValue;
	InterrogationData.Add(Value, FMovieScenePropertySectionTemplate::GetVector2DInterrogationKey());
};

template<>
inline void TPropertyActuator<FEulerTransform>::Actuate(FMovieSceneInterrogationData& InterrogationData, typename TCallTraits<FEulerTransform>::ParamType InValue, const TBlendableTokenStack<FEulerTransform>& OriginalStack, const FMovieSceneContext& Context) const
{
	FTransformData Value;
	Value.Translation = InValue.Location;
	Value.Rotation = InValue.Rotation;
	Value.Scale = InValue.Scale;
	InterrogationData.Add(Value, FMovieSceneInterrogationKey::GetTransformInterrogationKey());
};

template<>
inline void TPropertyActuator<FTransform>::Actuate(FMovieSceneInterrogationData& InterrogationData, typename TCallTraits<FTransform>::ParamType InValue, const TBlendableTokenStack<FTransform>& OriginalStack, const FMovieSceneContext& Context) const
{
	FTransformData Value;
	Value.Translation = InValue.GetTranslation();
	Value.Rotation = InValue.GetRotation().Rotator();
	Value.Scale = InValue.GetScale3D();
	InterrogationData.Add(Value, FMovieSceneInterrogationKey::GetTransformInterrogationKey());
};

template<>
inline void TPropertyActuator<FVector4>::Actuate(FMovieSceneInterrogationData& InterrogationData, typename TCallTraits<FVector4>::ParamType InValue, const TBlendableTokenStack<FVector4>& OriginalStack, const FMovieSceneContext& Context) const
{
	FVector4 Value = InValue;
	InterrogationData.Add(Value, FMovieScenePropertySectionTemplate::GetVector4InterrogationKey());
};

template<>
inline void TPropertyActuator<FVector>::Actuate(FMovieSceneInterrogationData& InterrogationData, typename TCallTraits<FVector>::ParamType InValue, const TBlendableTokenStack<FVector>& OriginalStack, const FMovieSceneContext& Context) const
{
	FVector Value = InValue;
	InterrogationData.Add(Value, FMovieScenePropertySectionTemplate::GetVectorInterrogationKey());
};


