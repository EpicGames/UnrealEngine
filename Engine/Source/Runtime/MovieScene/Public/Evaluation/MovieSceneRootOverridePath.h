// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "HAL/Platform.h"
#include "MovieSceneSequenceID.h"

class IMovieScenePlayer;
struct FMovieSceneSequenceHierarchy;

namespace UE
{
namespace MovieScene
{

struct FSharedPlaybackState;

/**
 * A path of unaccumulated sequence IDs ordered from child->parent->grandparent that is used to generate unique sequenceIDs for inner sequences
 * Optimized for Remap rather than Push/Pop by keeping sequence IDs child-parent order (the order they are required for remapping)
 */
struct FSubSequencePath
{
	/**
	 * Default construction to a root path
	 */
	MOVIESCENE_API FSubSequencePath();

	/**
	 * Set up this path from a specific sequence ID that points to a particular sequence in the specified hierarchy
	 *
	 * @param LeafID 			ID of the child-most sequence to include in this path
	 * @param Player 			Player from which to retrieve the hierarchy
	 */
	MOVIESCENE_API explicit FSubSequencePath(FMovieSceneSequenceID LeafID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState);

	UE_DEPRECATED(5.5, "Please use the version that takes an FSharedPlaybackState")
	MOVIESCENE_API explicit FSubSequencePath(FMovieSceneSequenceID LeafID, IMovieScenePlayer& Player);

	/**
	 * Set up this path from a specific sequence ID that points to a particular sequence in the specified hierarchy
	 *
	 * @param LeafID 			ID of the child-most sequence to include in this path
	 * @param RootHierarchy 	Hierarchy to get sequence IDs from
	 */
	MOVIESCENE_API explicit FSubSequencePath(FMovieSceneSequenceID LeafID, const FMovieSceneSequenceHierarchy* RootHierarchy);


	/**
	 * Find the first parent sequence ID that is common to both A and B
	 *
	 * @param A					The first sequence path
	 * @param B					The second sequence path
	 * @return The leaf-most parent common to both paths, or MovieSceneSequenceID::Root
	 */
	static MOVIESCENE_API FMovieSceneSequenceID FindCommonParent(const FSubSequencePath& A, const FSubSequencePath& B);


	/**
	 * Remap the specified sequence ID based on the currently evaluating sequence path, to the Root
	 *
	 * @param SequenceID			The sequence ID to find a template for
	 * @return Pointer to a template instance, or nullptr if the ID was not found
	 */
	/*inline*/ FMovieSceneSequenceID ResolveChildSequenceID(FMovieSceneSequenceID SequenceID) const
	{
		if (LIKELY(PathToRoot.Num() == 0))
		{
			return SequenceID;
		}

		for (FSequenceIDPair Parent : PathToRoot)
		{
			SequenceID = SequenceID.AccumulateParentID(Parent.Unaccumulated);
		}
		return SequenceID;
	}

	/**
	 * Reset this path to its default state (pointing to the root sequence)
	 */
	MOVIESCENE_API void Reset();


	/**
	 * Set up this path from a specific sequence ID that points to a particular sequence in the specified hierarchy
	 *
	 * @param LeafID 			ID of the child-most sequence to include in this path
	 * @param RootHierarchy 	Hierarchy to get sequence IDs from
	 */
	MOVIESCENE_API void Reset(FMovieSceneSequenceID LeafID, const FMovieSceneSequenceHierarchy* RootHierarchy);


	/**
	 * Check whether this path contains the specified sequence ID
	 *
	 * @param SequenceID		ID of the sequence to check for
	 * @return true if this path contains the sequence ID (or SequenceID == MovieSceneSequenceID::Root), false otherwise
	 */
	MOVIESCENE_API bool Contains(FMovieSceneSequenceID SequenceID) const;


	/**
	 * Return the number of generations between this path's leaf node, and the specified sequence ID
	 *
	 * @param SequenceID		ID of the parent sequence to count generations to
	 * @return The number of generations between the two nodes. (ie, 0 where SequenceID == Leaf, 1 for Immediate parents, 2 for grandparents etc)
	 */
	MOVIESCENE_API int32 NumGenerationsFromLeaf(FMovieSceneSequenceID SequenceID) const;


	/**
	 * Return the number of generations between the root and the specified sequence ID
	 *
	 * @param SequenceID		ID of the child sequence to count generations to
	 * @return The number of generations between the two nodes. (ie, 0 where SequenceID == Root, 1 for Immediate children, 2 for grandchildren)
	 */
	MOVIESCENE_API int32 NumGenerationsFromRoot(FMovieSceneSequenceID SequenceID) const;

	/**
	 * 
	 * 
	 * @param 
	 * @return
	 */
	MOVIESCENE_API FMovieSceneSequenceID MakeLocalSequenceID(FMovieSceneSequenceID ParentSequenceID) const;

	MOVIESCENE_API FMovieSceneSequenceID MakeLocalSequenceID(FMovieSceneSequenceID ParentSequenceID, FMovieSceneSequenceID TargetSequenceID) const;

	MOVIESCENE_API void PushGeneration(FMovieSceneSequenceID AccumulatedSequenceID, FMovieSceneSequenceID UnaccumulatedSequenceID);

	MOVIESCENE_API void PopTo(FMovieSceneSequenceID ParentSequenceID);

	MOVIESCENE_API void PopGenerations(int32 NumGenerations);

private:

	struct FSequenceIDPair
	{
		FMovieSceneSequenceID Unaccumulated;
		FMovieSceneSequenceID Accumulated;
	};

	/** A reverse path of deterministic sequence IDs required to accumulate from local -> root */
	TArray<FSequenceIDPair, TInlineAllocator<8>> PathToRoot;
};


} // namespace MovieScene
} // namespace UE
