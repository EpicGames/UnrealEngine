// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/EnumClassFlags.h"
#include "Variants/MovieSceneTimeWarpVariant.h"
#include "MovieSceneSectionParameters.generated.h"

/**
 * Flag structure that can be applied to any sub-section allowing control over various
 * behaviors for the nested sub-sequence.
 */
UENUM(Flags, BlueprintType)
enum class EMovieSceneSubSectionFlags : uint8
{
	None = 0,

	/*~ Begin mutually exclusive */
	/** When set, everything within the sub-section (including further sub-sections) should be keep-state. Mutually exclusive with OverrideRestoreState. */
	OverrideKeepState = 1 << 0,
	/** When set, everything within the sub-section (including further sub-sections) should be restore-state. Mutually exclusive with OverrideKeepState. */
	OverrideRestoreState = 1 << 1,
	/*~ End mutually exclusive */

	/** Everything inside this sub-sequence should ignore hierarchical bias and always be relevant */
	IgnoreHierarchicalBias = 1 << 2,

	/** Blend this sub sequence's hierarchical bias level using a higher -> lower override. Values from higher biases will override those in lower biases until a combined weight of 1 is reached. */
	BlendHierarchicalBias  = 1 << 3,

	AnyRestoreStateOverride = OverrideKeepState | OverrideRestoreState,
};
ENUM_CLASS_FLAGS(EMovieSceneSubSectionFlags)

namespace UE::MovieScene
{
	/** Accumulate parent and chld sub-section flags ensuring that flags are inherited correctly. */
	inline EMovieSceneSubSectionFlags AccumulateChildSubSectionFlags(EMovieSceneSubSectionFlags ParentFlags, EMovieSceneSubSectionFlags ChildFlags)
	{
		if (EnumHasAnyFlags(ParentFlags, EMovieSceneSubSectionFlags::AnyRestoreStateOverride))
		{
			// If the parent has any uninheritable flags based on the parent, ensure the child has the parent's flags
			return (ChildFlags & ~EMovieSceneSubSectionFlags::AnyRestoreStateOverride) | ParentFlags;
		}
		else
		{
			return ChildFlags | ParentFlags;
		}
	}

} // namespace UE::MovieScene

USTRUCT(BlueprintType)
struct FMovieSceneSectionParameters
{
	GENERATED_BODY()

	/** Default constructor */
	FMovieSceneSectionParameters()
		: StartFrameOffset(0)
		, EndFrameOffset(0)
		, FirstLoopStartFrameOffset(0)
		, TimeScale(1.0f)
		, HierarchicalBias(100)
		, Flags(EMovieSceneSubSectionFlags::None)
		, StartOffset_DEPRECATED(0.f)
		, PrerollTime_DEPRECATED(0.0f)
		, PostrollTime_DEPRECATED(0.0f)
	{}

public:
	/** Number of frames (in display rate) to skip at the beginning of the sub-sequence. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Clipping", meta=(UIFrameDisplayAs=Duration))
	FFrameNumber StartFrameOffset;

	/** Whether this section supports looping the sub-sequence. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Clipping")
	bool bCanLoop = false;

	/** Number of frames (in display rate) to skip at the beginning of the sub-sequence. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Clipping", meta=(EditCondition="bCanLoop", UIFrameDisplayAs=Duration))
	FFrameNumber EndFrameOffset;

	/** Number of frames (in display rate) to offset the first loop of the sub-sequence. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Clipping", meta=(EditCondition="bCanLoop", UIFrameDisplayAs=Duration))
	FFrameNumber FirstLoopStartFrameOffset;

	/** Playback time scaling factor. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Timing")
	FMovieSceneTimeWarpVariant TimeScale;

	/** Hierachical bias. Higher bias will take precedence. */
	UPROPERTY(config, BlueprintReadWrite, EditAnywhere, Category="Sequence")
	int32 HierarchicalBias;

	/** Sub-section flags defining how to deal with this sub-sequence */
	UPROPERTY(config, BlueprintReadWrite, Category="Sequence")
	EMovieSceneSubSectionFlags Flags;

	UPROPERTY()
	float StartOffset_DEPRECATED;
	UPROPERTY()
	float PrerollTime_DEPRECATED;
	UPROPERTY()
	float PostrollTime_DEPRECATED;
};
