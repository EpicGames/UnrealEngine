// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/FrameTime.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "Containers/ArrayView.h"
#include "UObject/SequencerObjectVersion.h"
#include "MovieSceneFrameMigration.h"
#include "MovieSceneSegment.generated.h"

struct FMovieSceneEvaluationTrackSegments;

/** Enumeration specifying how to evaluate a particular section when inside a segment */
UENUM()
enum class ESectionEvaluationFlags : uint8
{
	/** No special flags - normal evaluation */
	None		= 0x00,
	/** Segment resides inside the 'pre-roll' time for the section */
	PreRoll		= 0x01,
	/** Segment resides inside the 'post-roll' time for the section */
	PostRoll	= 0x02,

	ForceKeepState = 0x04,
	ForceRestoreState = 0x08,
};
ENUM_CLASS_FLAGS(ESectionEvaluationFlags);

/** A unique identifier for a segment within a FMovieSceneEvaluationTrackSegments container */
USTRUCT()
struct FMovieSceneSegmentIdentifier
{
	GENERATED_BODY()

	FMovieSceneSegmentIdentifier() : IdentifierIndex(INDEX_NONE) {}
	FMovieSceneSegmentIdentifier(int32 InIndex) : IdentifierIndex(InIndex) {}

	friend bool operator==(FMovieSceneSegmentIdentifier A, FMovieSceneSegmentIdentifier B) { return A.IdentifierIndex == B.IdentifierIndex; }
	friend bool operator!=(FMovieSceneSegmentIdentifier A, FMovieSceneSegmentIdentifier B) { return A.IdentifierIndex != B.IdentifierIndex; }
	friend uint32 GetTypeHash(FMovieSceneSegmentIdentifier LHS) { return GetTypeHash(LHS.IdentifierIndex); }

	bool IsValid() const
	{
		return IdentifierIndex != INDEX_NONE;
	}

	/** Custom serializer as this is just a type-safe int */
	bool Serialize(FArchive& Ar)
	{
		Ar << IdentifierIndex;
		return true;
	}

	friend FArchive& operator<<(FArchive& Ar, FMovieSceneSegmentIdentifier& ID)
	{
		ID.Serialize(Ar);
		return Ar;
	}

	int32 GetIndex() const
	{
		return IdentifierIndex;
	}

	UPROPERTY()
	int32 IdentifierIndex;
};

template<> struct TStructOpsTypeTraits<FMovieSceneSegmentIdentifier> : public TStructOpsTypeTraitsBase2<FMovieSceneSegmentIdentifier>
{
	enum { WithSerializer = true, WithIdenticalViaEquality = true };
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

/**
 * Evaluation data that specifies information about what to evaluate for a given template
 */
USTRUCT()
struct FSectionEvaluationData
{
	GENERATED_BODY()

	/** Default constructor */
	FSectionEvaluationData() : ImplIndex(-1), ForcedTime(TNumericLimits<int32>::Lowest()), Flags(ESectionEvaluationFlags::None) {}

	/** Construction from an implementaiton index (probably a section) */
	explicit FSectionEvaluationData(int32 InImplIndex)
		: ImplIndex(InImplIndex), ForcedTime(TNumericLimits<int32>::Lowest()), Flags(ESectionEvaluationFlags::None)
	{}

	/** Construction from an implementaiton index and a time to force evaluation at */
	FSectionEvaluationData(int32 InImplIndex, FFrameNumber InForcedTime)
		: ImplIndex(InImplIndex), ForcedTime(InForcedTime), Flags(ESectionEvaluationFlags::None)
	{}

	/** Construction from an implementaiton index and custom eval flags */
	FSectionEvaluationData(int32 InImplIndex, ESectionEvaluationFlags InFlags)
		: ImplIndex(InImplIndex), ForcedTime(TNumericLimits<int32>::Lowest()), Flags(InFlags)
	{}

	friend bool operator==(FSectionEvaluationData A, FSectionEvaluationData B)
	{
		return A.ImplIndex == B.ImplIndex && A.ForcedTime == B.ForcedTime && A.Flags == B.Flags;
	}

	FFrameTime GetTime(FFrameTime InActualTime)
	{
		return ForcedTime == TNumericLimits<int32>::Lowest() ? InActualTime : FFrameTime(ForcedTime);
	}

	/** Check if this is a preroll eval */
	inline bool IsPreRoll() const { return (Flags & ESectionEvaluationFlags::PreRoll) != ESectionEvaluationFlags::None; }

	/** Check if this is a postroll eval */
	inline bool IsPostRoll() const { return (Flags & ESectionEvaluationFlags::PostRoll) != ESectionEvaluationFlags::None; }

	friend FArchive& operator<<(FArchive& Ar, FSectionEvaluationData& Data)
	{
		FSectionEvaluationData::StaticStruct()->SerializeItem(Ar, &Data, nullptr);
		return Ar;
	}

	/** The implementation index we should evaluate (index into FMovieSceneEvaluationTrack::ChildTemplates) */
	UPROPERTY()
	int32 ImplIndex;

	/** A forced time to evaluate this section at */
	UPROPERTY()
	FFrameNumber ForcedTime;

	/** Additional flags for evaluating this section */
	UPROPERTY()
	ESectionEvaluationFlags Flags;
};

/**
 * Information about a single segment of an evaluation track
 */
USTRUCT()
struct FMovieSceneSegment
{
	GENERATED_BODY()

	FMovieSceneSegment()
	{}

	FMovieSceneSegment(const TRange<FFrameNumber>& InRange)
		: Range(InRange)
		, bAllowEmpty(false)
	{}

	FMovieSceneSegment(const TRange<FFrameNumber>& InRange, TArrayView<const FSectionEvaluationData> InApplicationImpls)
		: Range(InRange)
		, bAllowEmpty(true)
	{
		Impls.Reserve(InApplicationImpls.Num());
		for (const FSectionEvaluationData& Impl : InApplicationImpls)
		{
			Impls.Add(Impl);
		}
	}

	FMovieSceneSegment(const TRange<FFrameNumber>& InRange, std::initializer_list<FSectionEvaluationData> InApplicationImpls)
		: FMovieSceneSegment(InRange, MakeArrayView(InApplicationImpls))
	{
	}

	friend bool operator==(const FMovieSceneSegment& A, const FMovieSceneSegment& B)
	{
		return A.Range == B.Range && A.ID == B.ID && A.bAllowEmpty == B.bAllowEmpty && A.Impls == B.Impls;
	}

	bool IsValid() const
	{
		return bAllowEmpty || Impls.Num() != 0;
	}

	bool CombineWith(const FMovieSceneSegment& OtherSegment)
	{
		if (Range.Adjoins(OtherSegment.Range) && Impls == OtherSegment.Impls && bAllowEmpty == OtherSegment.bAllowEmpty)
		{
			Range = TRange<FFrameNumber>::Hull(OtherSegment.Range, Range);
			return true;
		}
		return false;
	}

	/** Custom serializer to accomodate the inline allocator on our array */
	bool Serialize(FArchive& Ar)
	{
		Ar.UsingCustomVersion(FSequencerObjectVersion::GUID);

		if (Ar.IsLoading() && Ar.CustomVer(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::FloatToIntConversion)
		{
			TRange<float> OldFloatRange;
			Ar << OldFloatRange;
			Range = FMovieSceneFrameRange::FromFloatRange(OldFloatRange);
		}
		else
		{
			Ar << Range;
		}

		if (Ar.CustomVer(FSequencerObjectVersion::GUID) >= FSequencerObjectVersion::EvaluationTree)
		{
			Ar << ID;
			Ar << bAllowEmpty;
		}

		int32 NumStructs = Impls.Num();
		Ar << NumStructs;

		if (Ar.IsLoading())
		{
			for (int32 Index = 0; Index < NumStructs; ++Index)
			{
				FSectionEvaluationData Data;
				Ar << Data;
				Impls.Add(Data);
			}
		}
		else if (Ar.IsSaving())
		{
			for (FSectionEvaluationData& Data : Impls)
			{
				Ar << Data;
			}
		}
		return true;
	}

	/** The segment's range */
	TRange<FFrameNumber> Range;

	FMovieSceneSegmentIdentifier ID;

	/** Whether this segment has been generated yet or not */
	bool bAllowEmpty;

	/** Array of implementations that reside at the segment's range */
	TArray<FSectionEvaluationData, TInlineAllocator<4>> Impls;
};

template<> struct TStructOpsTypeTraits<FMovieSceneSegment> : public TStructOpsTypeTraitsBase2<FMovieSceneSegment>
{
	enum { WithSerializer = true, WithCopy = true, WithIdenticalViaEquality = true };
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};
