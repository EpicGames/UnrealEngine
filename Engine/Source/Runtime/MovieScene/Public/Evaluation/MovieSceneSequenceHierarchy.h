// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Evaluation/MovieSceneEvaluationTree.h"
#include "Evaluation/MovieSceneSectionParameters.h"
#include "Evaluation/MovieSceneSegment.h"
#include "Evaluation/MovieSceneSequenceInstanceData.h"
#include "Evaluation/MovieSceneSequenceTransform.h"
#include "HAL/PlatformCrt.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "Misc/Guid.h"
#include "MovieSceneFrameMigration.h"
#include "MovieSceneSequenceID.h"
#include "Serialization/Archive.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "MovieSceneSequenceHierarchy.generated.h"

class UMovieSceneSequence;
class UMovieSceneSubSection;
class UMovieSceneCondition;
struct FMovieSceneSequenceID;
template <typename ElementType> class TRange;

/**
 * Sub sequence data that is stored within an evaluation template as a backreference to the originating sequence, and section
 */
USTRUCT()
struct FMovieSceneSubSequenceData
{
	GENERATED_BODY()

	/**
	 * Default constructor for serialization
	 */
	MOVIESCENE_API FMovieSceneSubSequenceData();

	/**
	 * Construction from a movie scene sequence, and a sub section name, and its valid play range
	 */
	MOVIESCENE_API FMovieSceneSubSequenceData(const UMovieSceneSubSection& InSubSection);

	/*~ Defined constructors, operators and destructor to work around deprecation warnings on linux */
	MOVIESCENE_API FMovieSceneSubSequenceData(const FMovieSceneSubSequenceData&);
	MOVIESCENE_API FMovieSceneSubSequenceData& operator=(const FMovieSceneSubSequenceData&);
	MOVIESCENE_API FMovieSceneSubSequenceData(FMovieSceneSubSequenceData&&);
	MOVIESCENE_API FMovieSceneSubSequenceData& operator=(FMovieSceneSubSequenceData&&);
	MOVIESCENE_API ~FMovieSceneSubSequenceData();

	/**
	 * Get this sub sequence's sequence asset, potentially loading it through its soft object path
	 */
	MOVIESCENE_API UMovieSceneSequence* GetSequence() const;

	/**
	 * Get this sub sequence's sequence asset if it is already loaded, will not attempt to load the sequence if not
	 */
	MOVIESCENE_API UMovieSceneSequence* GetLoadedSequence() const;

	/**
	 * Check whether this structure is dirty and should be reconstructed
	 */
	MOVIESCENE_API bool IsDirty(const UMovieSceneSubSection& InSubSection) const;

	/**
	* Gets the signature of the sub-section this points to. 
	*/
	FGuid GetSubSectionSignature() const { return SubSectionSignature; }

	/**
	 * Re-creates a sub-section parameter struct.
	 */
	UE_DEPRECATED(5.5, "This function is no longer supported")
	MOVIESCENE_API FMovieSceneSectionParameters ToSubSectionParameters() const;

	/** The sequence that the sub section references */
	UPROPERTY(meta=(AllowedClasses="/Script/MovieScene.MovieSceneSequence"))
	FSoftObjectPath Sequence;

	/** The transform from this sub sequence's parent to its own play space. */
	UPROPERTY()
	FMovieSceneSequenceTransform OuterToInnerTransform;

	/** Transform that transforms a given time from the sequences outer space, to its authored space. */
	UPROPERTY()
	FMovieSceneSequenceTransform RootToSequenceTransform;

#if WITH_EDITORONLY_DATA

	/** Transform that turns local space into warped local space based on a timewarp track that exists in this sequence. */
	UPROPERTY()
	FMovieSceneSequenceTransform LocalToWarpedLocalTransform;

	/** The transform from root space to this sub-sequence's unwarped local space. */
	UPROPERTY()
	FMovieSceneSequenceTransform RootToUnwarpedLocalTransform;

	UPROPERTY()
	FMovieSceneTransformBreadcrumbs StartTimeBreadcrumbs;

	UPROPERTY()
	FMovieSceneTransformBreadcrumbs EndTimeBreadcrumbs;

#endif // WITH_EDITORONLY_DATA

	/** The tick resolution of the inner sequence. */
	UPROPERTY()
	FFrameRate TickResolution;

	/** This sequence's deterministic sequence ID. Used in editor to reduce the risk of collisions on recompilation. */ 
	UPROPERTY()
	FMovieSceneSequenceID DeterministicSequenceID;
	
	/** This sub sequence's playback range according to its parent sub section. Clamped recursively during template generation */
	UPROPERTY()
	FMovieSceneFrameRange PlayRange;

	/** The range of the sub-sequence in its parent space. Unclamped. */
	UPROPERTY()
	FMovieSceneFrameRange ParentPlayRange;

	/** The sequence preroll range considering the start offset */
	UPROPERTY()
	FMovieSceneFrameRange PreRollRange;

	/** The sequence postroll range considering the start offset */
	UPROPERTY()
	FMovieSceneFrameRange PostRollRange;

	/** The accumulated hierarchical bias of this sequence. Higher bias will take precedence */
	UPROPERTY()
	int16 HierarchicalBias;

	/** Flags accumulated from parent->child for each sub-section that led to the inclusion of this sub-sequence */
	UPROPERTY()
	EMovieSceneSubSectionFlags AccumulatedFlags;

	/** Whether this sub-sequence can loop */
	UPROPERTY()
	bool bCanLoop = false;

	/** Instance data that should be used for any tracks contained immediately within this sub sequence */
	UPROPERTY()
	FMovieSceneSequenceInstanceDataPtr InstanceData;

	/** This sequence's path within its movie scene */
	UPROPERTY()
	FName SectionPath;

#if WITH_EDITORONLY_DATA

	/** The sub-sequence's full playback range, in its own local time space. */
	UE_DEPRECATED(5.5, "Please use GetSequence()->GetMovieScene()->GetPlaybackRange().")
	FMovieSceneFrameRange FullPlayRange;
	UE_DEPRECATED(5.5, "This field is no longer supported.")
	FFrameNumber ParentStartFrameOffset;
	UE_DEPRECATED(5.5, "This field is no longer supported.")
	FFrameNumber ParentEndFrameOffset;
	UE_DEPRECATED(5.5, "This field is no longer supported.")
	FFrameNumber ParentFirstLoopStartFrameOffset;
	UE_DEPRECATED(5.5, "This field is no longer used")
	FMovieSceneFrameRange UnwarpedPlayRange;

#endif
	
	/* Optional condition pulled off of the UMovieSceneSubSection or UMovieSceneSubTrack */
	UPROPERTY()
	TSoftObjectPtr<const UMovieSceneCondition> WeakCondition;


private:

	/** Cached version of the sequence to avoid resolving it every time */
	mutable TWeakObjectPtr<UMovieSceneSequence> CachedSequence;

	/** The sub section's signature at the time this structure was populated. */
	UPROPERTY()
	FGuid SubSectionSignature;
};

/**
 * Simple structure specifying parent and child sequence IDs for any given sequences
 */
USTRUCT()
struct FMovieSceneSequenceHierarchyNode
{
	GENERATED_BODY()

	/**
	 * Default construction used by serialization
	 */
	FMovieSceneSequenceHierarchyNode()
	{}

	/**
	 * Construct this hierarchy node from the sequence's parent ID
	 */
	FMovieSceneSequenceHierarchyNode(FMovieSceneSequenceIDRef InParentID)
		: ParentID(InParentID)
	{}

	/** Movie scene sequence ID of this node's parent sequence */
	UPROPERTY()
	FMovieSceneSequenceID ParentID;

	/** Array of child sequences contained within this sequence */
	UPROPERTY()
	TArray<FMovieSceneSequenceID> Children;
};

USTRUCT()
struct FMovieSceneSubSequenceTreeEntry
{
	GENERATED_BODY()

	FMovieSceneSubSequenceTreeEntry(FMovieSceneSequenceID InSequenceID, ESectionEvaluationFlags InFlags)
		: SequenceID(InSequenceID)
		, Flags(InFlags)
	{}

	MOVIESCENE_API FMovieSceneSubSequenceTreeEntry();

	MOVIESCENE_API FMovieSceneSubSequenceTreeEntry(const FMovieSceneSubSequenceTreeEntry&);
	MOVIESCENE_API FMovieSceneSubSequenceTreeEntry& operator=(const FMovieSceneSubSequenceTreeEntry&);

	MOVIESCENE_API FMovieSceneSubSequenceTreeEntry(FMovieSceneSubSequenceTreeEntry&&);
	MOVIESCENE_API FMovieSceneSubSequenceTreeEntry& operator=(FMovieSceneSubSequenceTreeEntry&&);

	MOVIESCENE_API ~FMovieSceneSubSequenceTreeEntry();

	friend FArchive& operator<<(FArchive& Ar, FMovieSceneSubSequenceTreeEntry& InOutEntry);
	friend bool operator==(const FMovieSceneSubSequenceTreeEntry& A, const FMovieSceneSubSequenceTreeEntry& B);

	FMovieSceneSequenceID SequenceID;
	ESectionEvaluationFlags Flags;

	UE_DEPRECATED(5.5, "This member is no longer supported")
	FMovieSceneWarpCounter RootToSequenceWarpCounter;
};

USTRUCT()
struct FMovieSceneSubSequenceTree
{
	GENERATED_BODY()

	friend bool operator==(const FMovieSceneSubSequenceTree& A, const FMovieSceneSubSequenceTree& B)
	{
		return A.Data == B.Data;
	}

	bool Serialize(FArchive& Ar)
	{
		Ar << Data;
		return true;
	}

	TMovieSceneEvaluationTree<FMovieSceneSubSequenceTreeEntry> Data;
};
template<> struct TStructOpsTypeTraits<FMovieSceneSubSequenceTree> : public TStructOpsTypeTraitsBase2<FMovieSceneSubSequenceTree> { enum { WithSerializer = true, WithIdenticalViaEquality = true }; };

/**
 * Structure that stores hierarchical information pertaining to all sequences contained within a root sequence
 */
USTRUCT()
struct FMovieSceneSequenceHierarchy
{
	GENERATED_BODY()

	FMovieSceneSequenceHierarchy()
		: RootNode(MovieSceneSequenceID::Invalid)
	{}

	/**
	 * Find the structural information for the specified sequence ID
	 *
	 * @param SequenceID 			The ID of the sequence to lookup
	 * @return pointer to the structural information, or nullptr if the sequence ID does not exist in this hierarchy 
	 */
	const FMovieSceneSequenceHierarchyNode* FindNode(FMovieSceneSequenceIDRef SequenceID) const
	{
		return SequenceID == MovieSceneSequenceID::Root ? &RootNode : Hierarchy.Find(SequenceID);
	}

	/**
	 * Find the structural information for the specified sequence ID
	 *
	 * @param SequenceID 			The ID of the sequence to lookup
	 * @return pointer to the structural information, or nullptr if the sequence ID does not exist in this hierarchy 
	 */
	FMovieSceneSequenceHierarchyNode* FindNode(FMovieSceneSequenceIDRef SequenceID)
	{
		return SequenceID == MovieSceneSequenceID::Root ? &RootNode : Hierarchy.Find(SequenceID);
	}

	/**
	 * Find the sub sequence and section information for the specified sequence ID
	 *
	 * @param SequenceID 			The ID of the sequence to lookup
	 * @return pointer to the sequence/section information, or nullptr if the sequence ID does not exist in this hierarchy 
	 */
	const FMovieSceneSubSequenceData* FindSubData(FMovieSceneSequenceIDRef SequenceID) const
	{
		return SequenceID == MovieSceneSequenceID::Root ? nullptr : SubSequences.Find(SequenceID);
	}

	/**
	 * Find the sub sequence and section information for the specified sequence ID
	 *
	 * @param SequenceID 			The ID of the sequence to lookup
	 * @return pointer to the sequence/section information, or nullptr if the sequence ID does not exist in this hierarchy 
	 */
	FMovieSceneSubSequenceData* FindSubData(FMovieSceneSequenceIDRef SequenceID)
	{
		return SequenceID == MovieSceneSequenceID::Root ? nullptr : SubSequences.Find(SequenceID);
	}

	/**
	 * Find the sub sequence for a given sequence ID, or nullptr if it was not found
	 *
	 * @return pointer to the sequence, or nullptr if the sequence ID does not exist in this hierarchy
	 */
	UMovieSceneSequence* FindSubSequence(FMovieSceneSequenceIDRef SequenceID) const
	{
		const FMovieSceneSubSequenceData* SubSequenceData = FindSubData(SequenceID);
		return SubSequenceData ? SubSequenceData->GetSequence(): nullptr;
	}

	/**
	 * Add the specified sub sequence data to the hierarchy
	 *
	 * @param Data 					The data to add
	 * @param ThisSequenceID 		The sequence ID of the sequence the data relates to
	 * @param ParentID 				The parent ID of this sequence data
	 */
	void Add(const FMovieSceneSubSequenceData& Data, FMovieSceneSequenceIDRef ThisSequenceID, FMovieSceneSequenceIDRef ParentID);

	/**
	 * Remove the specified sub sequence datas from the hierarchy.
	 */
	void Remove(TArrayView<const FMovieSceneSequenceID> SequenceIDs);

	/**
	 * Add an entry for the given sub sequence with the given root time range
	 */
	void AddRange(const TRange<FFrameNumber>& RootSpaceRange, FMovieSceneSequenceIDRef InSequenceID, ESectionEvaluationFlags InFlags);

	UE_DEPRECATED(5.5, "The RootToSequenceWarpCounter parameter is no longer required or supported")
	void AddRange(const TRange<FFrameNumber>& RootSpaceRange, FMovieSceneSequenceIDRef InSequenceID, ESectionEvaluationFlags InFlags, FMovieSceneWarpCounter RootToSequenceWarpCounter);

	/**
	 * Mutate the sub-data for the specified sequence ID - only to be used during compilation
	 */
	FMovieSceneSubSequenceData* MutateSubData(FMovieSceneSequenceIDRef SequenceID)
	{
		return SequenceID == MovieSceneSequenceID::Root ? nullptr : SubSequences.Find(SequenceID);
	}

	void SetRootTransform(FMovieSceneSequenceTransform&& InTransform)
	{
		RootTransform = MoveTemp(InTransform);
	}

	/** Get all sub-sequence IDs */
	void AllSubSequenceIDs(TArray<FMovieSceneSequenceID>& OutSequenceIDs) const
	{
		Hierarchy.GetKeys(OutSequenceIDs);
	}

	/** Access to all the subsequence data */
	const TMap<FMovieSceneSequenceID, FMovieSceneSubSequenceData>& AllSubSequenceData() const
	{
		return SubSequences;
	}

	/** Access to all the sub sequence nodes */
	const TMap<FMovieSceneSequenceID, FMovieSceneSequenceHierarchyNode>& AllSubSequenceNodes() const
	{
		return Hierarchy;
	}

	const TMovieSceneEvaluationTree<FMovieSceneSubSequenceTreeEntry>& GetTree() const
	{
		return Tree.Data;
	}

	EMovieSceneServerClientMask GetAccumulatedNetworkMask() const
	{
		return AccumulatedNetworkMask;
	}

	void AccumulateNetworkMask(EMovieSceneServerClientMask Mask)
	{
		AccumulatedNetworkMask &= Mask;
	}

	const FMovieSceneSequenceTransform& GetRootTransform() const
	{
		return RootTransform;
	}

#if !NO_LOGGING
	void LogHierarchy() const;
	void LogSubSequenceTree() const;
#endif

private:


	UPROPERTY()
	FMovieSceneSequenceHierarchyNode RootNode;

	UPROPERTY()
	FMovieSceneSubSequenceTree Tree;

	UPROPERTY()
	FMovieSceneSequenceTransform RootTransform;

	/** Map of all (recursive) sub sequences found in this template, keyed on sequence ID */
	UPROPERTY()
	TMap<FMovieSceneSequenceID, FMovieSceneSubSequenceData> SubSequences;

	/** Structural information describing the structure of the sequence */
	UPROPERTY()
	TMap<FMovieSceneSequenceID, FMovieSceneSequenceHierarchyNode> Hierarchy;

	/** Holds the accumulated network mask from all included sub-sections. 
	* If client or server-only subsections are found and included based on the gather params network mask, other bits will be excluded.
	* If the gather param network mask excludes client or server-only sub-sections, these will be skipped, and so not accumulated.
	* If no client or server-only subsections are found and included, the mask will be All.
	* If both client and server-only subsections are found and included, the mask will be None as each would exclude the other.
	*/
	UPROPERTY()
	EMovieSceneServerClientMask AccumulatedNetworkMask = EMovieSceneServerClientMask::All;
};
