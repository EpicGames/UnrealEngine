// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Misc/InlineValue.h"
#include "Templates/Decay.h"
#include "Templates/EnableIf.h"
#include "Templates/PointerIsConvertibleFromTo.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"

#include "MovieSceneSequenceInstanceData.generated.h"

class FArchive;

/**
 * Abstract base class that defines instance data for sub sequences
 */
USTRUCT()
struct FMovieSceneSequenceInstanceData
{
	GENERATED_BODY()

	/**
	 * Virtual destruction
	 */
	virtual ~FMovieSceneSequenceInstanceData()= default;

	/**
	 * Access the UStruct type of this data for serialization purposes
	 */
	UScriptStruct& GetScriptStruct() const
	{
		return GetScriptStructImpl();
	}

private:

	/**
	 * Implemented in derived types to retrieve the type of this struct
	 */
	virtual UScriptStruct& GetScriptStructImpl() const
	{
		check(false);
		return *StaticStruct();
	}
};

/**  */
USTRUCT()
struct FMovieSceneSequenceInstanceDataPtr
#if CPP
	: TInlineValue<FMovieSceneSequenceInstanceData, 16>
#endif
{
	GENERATED_BODY()

	/**
	 * Default construction to an empty container
	 */
	FMovieSceneSequenceInstanceDataPtr()
	{}

	/**
	 * Construction from any FMovieSceneSequenceInstanceData derivative
	 */
	template<
		typename T,
		typename = typename TEnableIf<TPointerIsConvertibleFromTo<typename TDecay<T>::Type, FMovieSceneSequenceInstanceData>::Value>::Type
	>
	FMovieSceneSequenceInstanceDataPtr(T&& In)
		: TInlineValue(Forward<T>(In))
	{
		typedef typename TDecay<T>::Type ClientType;

		static_assert(!std::is_same_v<ClientType, FMovieSceneSequenceInstanceData>, "Direct usage of FMovieSceneSequenceInstanceData is prohibited.");

#if WITH_EDITOR && DO_CHECK
		const UStruct* ClientStruct = ClientType::StaticStruct();
		checkf(ClientStruct == &In.GetScriptStruct() && ClientStruct != FMovieSceneSequenceInstanceData::StaticStruct(), TEXT("%s type does not correctly override GetScriptStructImpl. Track will not serialize correctly."), *ClientStruct->GetName());
#endif
	}
	
	/** Copy construction/assignment */
	MOVIESCENE_API FMovieSceneSequenceInstanceDataPtr(const FMovieSceneSequenceInstanceDataPtr& RHS);
	MOVIESCENE_API FMovieSceneSequenceInstanceDataPtr& operator=(const FMovieSceneSequenceInstanceDataPtr& RHS);

	/** Templates are moveable */
	FMovieSceneSequenceInstanceDataPtr(FMovieSceneSequenceInstanceDataPtr&&) = default;
	FMovieSceneSequenceInstanceDataPtr& operator=(FMovieSceneSequenceInstanceDataPtr&&) = default;

	/** Serialize the template */
	MOVIESCENE_API bool Serialize(FArchive& Ar);
};

template<> struct TStructOpsTypeTraits<FMovieSceneSequenceInstanceDataPtr> : public TStructOpsTypeTraitsBase2<FMovieSceneSequenceInstanceDataPtr>
{
	enum { WithSerializer = true, WithCopy = true };
};
