// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Containers/Array.h"
#include "MovieSceneFwd.h"
#include "Misc/AssertionMacros.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameTime.h"
#include "MovieSceneTimeTransform.h"
#include "MovieSceneTimeWarping.h"
#include "Variants/MovieSceneTimeWarpVariant.h"

#include "MovieSceneSequenceTransform.generated.h"

template<typename> class TRange;

struct FMovieSceneTransformBreadcrumbs;
struct FMovieSceneNestedSequenceTransform;
struct FMovieSceneInverseNestedSequenceTransform;

namespace UE::MovieScene
{
	struct FInverseTransformTimeParams;
	struct FTransformTimeParams;

	enum class ETimeWarpChannelDomain : uint8;
}


/*~
 * This file contains the structures necessary for representing transformations of time between various different
 *      'spaces' within the Sequencer/MovieScene codebase. Typically these spaces denote root and sub-sequences,
 *      or sub-sub-suquences. 'Lossy' transformations (ie, transformations that are mathematically non-linear)
 *      are supported and typically referred to as 'warping' or 'non-linear' transforms. Such transforms are always
 *      stored as a stack of nested FMovieSceneNestedSequenceTransform structures and cannot be combined mathematically
 *      like FMovieSceneTimeTransform can.
 *
 * When dealing with time transformations, it is common to use the following nomenclature interchangably:
 *      - Outer, Untransformed, Unwarped space; and,
 *      - Inner, Transformed, Warped space.
 * 
 * Inverse transformations are also supported, but via a different, more restrictive API. Where outer->inner transforms
 *      must always map an outer time to an inner time, inverse transformations can fail. This is clearly the case when 
 *      attempting to inverse-transform a time of 20 from a loop that only spans [0, 10). The requested time is not present
 *      in the set of times that can result from the outer->inner transform, and so the operation fails.
 *      Similarly, inverse transformations may have zero or more real solutions. Consider the previous example, but with a
 *      time of 5; the operation has an infinite number of solutions (one for each loop, or every t where t%5==0).
 *
 *                         ________ << FMovieSceneInverseSequenceTransform << ___________
 *                        /                                                              \
 * ----------------------                                                                -----------------------
 * | Untransformed space |                                                                |  Transformed space  |
 * -----------------------                                                                -----------------------
 *                        \_____________ >> FMovieSceneSequenceTransform >> _____________/
 */


/**
 * Enumeration defining how to store breadcrumb trails
 */
UENUM()
enum class EMovieSceneBreadcrumbMode : uint8
{
	/** Default: Only store breadcrumbs for non-linear transformations */
	Sparse,
	/** Store breadcrumbs for every nested time transformation */
	Dense
};

/**
 * Struct that tracks a breadcumb trail when transformiung a time through FMovieSceneSequenceTransform
 */
USTRUCT()
struct FMovieSceneTransformBreadcrumbs
{
	GENERATED_BODY()

	/**
	 * Default constructor, optionally taking a mode
	 */
	FMovieSceneTransformBreadcrumbs(EMovieSceneBreadcrumbMode InMode = EMovieSceneBreadcrumbMode::Sparse)
		: Mode(InMode)
	{}


	/**
	 * Return the breadcrumb at the specified index
	 */
	FFrameTime operator[](int32 Index) const
	{
		return Breadcrumbs[Index];
	}


	/**
	 * Retrieve the length of this breadcrumb trail
	 */
	int32 Num() const
	{
		return Breadcrumbs.Num();
	}


	/**
	 * Check if the specified index is valid
	 */
	bool IsValidIndex(int32 Index) const
	{
		return Breadcrumbs.IsValidIndex(Index);
	}


	/**
	 * Check whether this breadcrumb trail only contains breadcrumbs for non-linear transformations (true) or for everything (false)
	 */
	bool IsSparse() const
	{
		return Mode == EMovieSceneBreadcrumbMode::Sparse;
	}


	/**
	 * Retrieve this breadcrumb trail's capture mode
	 */
	EMovieSceneBreadcrumbMode GetMode() const
	{
		return Mode;
	}


	/**
	 * Restore this trail to its original (empty) state
	 */
	void Reset()
	{
		Breadcrumbs.Reset();
	}


	/**
	 * Add a breadcrumb to this trail
	 */
	void AddBreadcrumb(const FFrameTime& Breadcrumb)
	{
		Breadcrumbs.Add(Breadcrumb);
	}


	/**
	 * Pop the most recently added breadcrumb off this trail
	 */
	FFrameTime PopBreadcrumb()
	{
		return Breadcrumbs.Pop();
	}


	/**
	 * Prepend the specified breadcrumb trail to this one, resulting in a path from the start of Outer to the end of this
	 */
	void CombineWithOuterBreadcrumbs(const FMovieSceneTransformBreadcrumbs& OuterBreadcrumbs)
	{
		// Append the outer breadcrumbs to the head of this list
		Breadcrumbs.Insert(OuterBreadcrumbs.Breadcrumbs, 0);
	}


	/**
	 * Create a new breadcrumb trail of the same size as this one, but with all times set to a specific time
	 */
	FMovieSceneTransformBreadcrumbs OverwriteWith(FFrameTime InTime) const
	{
		FMovieSceneTransformBreadcrumbs Result;
		Result.Breadcrumbs.SetNum(Breadcrumbs.Num());
		for (FFrameTime& X : Result.Breadcrumbs)
		{
			X = InTime;
		}
		return Result;
	}

public:

	TArray<FFrameTime>::RangedForConstIteratorType begin () const { return Breadcrumbs.begin(); }
	TArray<FFrameTime>::RangedForConstIteratorType end   () const { return Breadcrumbs.end();   }

	MOVIESCENE_API operator TArrayView<const FFrameTime>() const;

private:

	UPROPERTY()
	TArray<FFrameTime> Breadcrumbs;

	UPROPERTY()
	EMovieSceneBreadcrumbMode Mode;
};


USTRUCT()
struct FMovieSceneWarpCounter : public FMovieSceneTransformBreadcrumbs
{
	GENERATED_BODY()

public:

	MOVIESCENE_API FMovieSceneWarpCounter();

	MOVIESCENE_API FMovieSceneWarpCounter(const FMovieSceneWarpCounter&);
	MOVIESCENE_API FMovieSceneWarpCounter& operator=(const FMovieSceneWarpCounter&);

	MOVIESCENE_API FMovieSceneWarpCounter(FMovieSceneWarpCounter&&);
	MOVIESCENE_API FMovieSceneWarpCounter& operator=(FMovieSceneWarpCounter&&);

	MOVIESCENE_API ~FMovieSceneWarpCounter();

	UE_DEPRECATED(5.5, "This function is no longer used. Please update your code to use time-based breadcrumbs instead.")
	void AddWarpingLevel(uint32 WarpCount)
	{
	}

	UE_DEPRECATED(5.5, "This function is no longer used. Please update your code to use time-based breadcrumbs instead.")
	void AddNonWarpingLevel()
	{
	}

	UE_DEPRECATED(5.5, "This function is no longer used. Please update your code to use time-based breadcrumbs instead.")
	int32 NumWarpCounts() const
	{
		return 0;
	}

	UE_DEPRECATED(5.5, "This function is no longer used. Please update your code to use time-based breadcrumbs instead.")
	uint32 LastWarpCount() const
	{
		return (uint32)(-1);
	}

	UE_DEPRECATED(5.5, "Warp counts are no longer supported.")
	TArray<uint32> WarpCounts;
};


/**
 * Structure used to represent a specific inverse transformation (ie from transformed to untransformed space) that cannot be combined with another.
 * Stored as a stack inside FMovieSceneInverseSequenceTransform to represent a complete transformation
 * from inner time-space to outer time-space.
 */
USTRUCT()
struct FMovieSceneInverseNestedSequenceTransform
{
	GENERATED_BODY()

	/**
	 * Default construction to an identity linear transform. Should only be used by serialization.
	 */
	FMovieSceneInverseNestedSequenceTransform()
	{}


	/**
	 * Construction from a linear transform
	 */
	FMovieSceneInverseNestedSequenceTransform(const FMovieSceneTimeTransform& InLinearTransform)
		: TimeScale(InLinearTransform.TimeScale), Offset(InLinearTransform.Offset)
	{}



	/**
	 * Construction from a linear transform represented as an offset and scale (scale is applied first)
	 */
	FMovieSceneInverseNestedSequenceTransform(const FFrameTime& InOffset, double InTimeScale)
		: TimeScale(InTimeScale), Offset(InOffset)
	{}


public:


	/**
	 * Check whether this transform is linear (true) or not (false)
	 */
	bool IsLinear() const
	{
		return TimeScale.GetType() == EMovieSceneTimeWarpType::FixedPlayRate;
	}


	/**
	 * Check whether this transformation requires a breadcrumb trail (true) or not (false)
	 * @note Breadcrumbs may still be added for dense trails
	 */
	bool NeedsBreadcrumb() const
	{
		return !IsLinear();
	}


	/**
	 * Convert this transform to its linear form.
	 * @note This is only valid to call where IsLinear() is true
	 */
	MOVIESCENE_API FMovieSceneTimeTransform AsLinear() const;


	/**
	 * Convert this transform to its inverse (ie a transform that converts from untransformed space to transformed space)
	 */
	MOVIESCENE_API FMovieSceneNestedSequenceTransform Inverse() const;


	/**
	 * Attempt to transform the specified time by this inverse transform. The operation can fail if the time does not map to the un-transformed time space (for example, when looping).
	 * 
	 * @param Time        The time to transform
	 * @param Breadcrumb  A breadcrumb in the un-transformed space to assist in resolving ambiguous transformations (ie, to transform into the correct time based on the breadcrumb's loop)
	 *
	 * @return The transformed time if successful, or an epty optional otherwise
	 */
	MOVIESCENE_API TOptional<FFrameTime> TryTransformTime(FFrameTime Time, FFrameTime Breadcrumb) const;


	/**
	 * Attempt to transform the specified time by this inverse transform. The operation can fail if the time does not map to the un-transformed time space (for example, when looping).
	 * 
	 * @param Time        The time to transform
	 * @param Breadcrumb  A breadcrumb in the un-transformed space to assist in resolving ambiguous transformations (ie, to transform into the correct time based on the breadcrumb's loop)
	 * @param Params      Parameter structure that controls how to perform the inverse operation (ie to disallow cycling outside of the current loop; to ignore clamps etc)
	 * 
	 * @return The transformed time if successful, or an epty optional otherwise
	 */
	MOVIESCENE_API TOptional<FFrameTime> TryTransformTime(FFrameTime Time, FFrameTime Breadcrumb, const UE::MovieScene::FInverseTransformTimeParams& Params) const;


	/**
	 * Attempt to transform the specified time by this inverse transform within a specified un-transformed range, calling a functor for every instance of the specified time in the un-transformed space.
	 * 
	 * @param Time        The time to transform
	 * @param Visitor     A callback to invoke for every instance of Time in the untransformed range. The result of the functor determines whether to continue iteration (true) or not (false).
	 * @param UntransformedRangeStart      The inclusive start time in untransformed space within which to allow Visitor to be invoked
	 * @param UntransformedRangeEnd        The inclusive end time in untransformed space within which to allow Visitor to be invoked
	 * 
	 * @return True if every invocation of Visitor returned true, or there were no invocations; false if any invocation of Visitor returned false.
	 */
	MOVIESCENE_API bool TransformTimeWithinRange(FFrameTime Time, const TFunctionRef<bool(FFrameTime)>& Visitor, FFrameTime UntransformedRangeStart, FFrameTime UntransformedRangeEnd) const;

private:

	friend FMovieSceneNestedSequenceTransform;

	/** Time scale as either a fixed play rate, or as an external implementation */
	UPROPERTY()
	FMovieSceneTimeWarpVariant TimeScale;

	/** Constant time offset. Offset is applied differently for different internal implementations of TimeScale. */
	UPROPERTY()
	FFrameTime Offset;
};



/**
 * Structure used to represent a specific transformation (ie from untransformed to transformed space) that cannot be combined with another.
 *
 * Stored as a stack inside FMovieSceneSequenceTransform to represent a complete transformation from inner time-space to outer time-space.
 */
USTRUCT()
struct FMovieSceneNestedSequenceTransform
{
	GENERATED_BODY()


	/**
	 * Default construction to an identity linear transform
	 */
	MOVIESCENE_API FMovieSceneNestedSequenceTransform();

	MOVIESCENE_API FMovieSceneNestedSequenceTransform(const FMovieSceneNestedSequenceTransform&);
	MOVIESCENE_API FMovieSceneNestedSequenceTransform& operator=(const FMovieSceneNestedSequenceTransform&);

	MOVIESCENE_API FMovieSceneNestedSequenceTransform(FMovieSceneNestedSequenceTransform&&);
	MOVIESCENE_API FMovieSceneNestedSequenceTransform& operator=(FMovieSceneNestedSequenceTransform&&);

	MOVIESCENE_API ~FMovieSceneNestedSequenceTransform();

	/**
	 * Construction from a time warp variant
	 */
	FMovieSceneNestedSequenceTransform(FMovieSceneTimeWarpVariant&& InVariant)
		: TimeScale(MoveTemp(InVariant))
	{
		TimeScale.MakeWeakUnsafe();
	}


	/**
	 * Construction from a linear time transformation
	 */
	FMovieSceneNestedSequenceTransform(const FMovieSceneTimeTransform& InLinearTransform)
		: TimeScale(InLinearTransform.TimeScale), Offset(InLinearTransform.Offset)
	{}


	/**
	 * Construction from a linear time transformation (scale applies first)
	 */
	FMovieSceneNestedSequenceTransform(FFrameTime InOffset, double InTimeScale)
		: TimeScale(InTimeScale), Offset(InOffset)
	{}


	/**
	 * Default construction to an identity linear transform
	 */
	FMovieSceneNestedSequenceTransform(FFrameTime InOffset, FMovieSceneTimeWarpVariant&& InTimeScale)
		: TimeScale(MoveTemp(InTimeScale)), Offset(InOffset)
	{
		TimeScale.MakeWeakUnsafe();
	}


public:


	/**
	 * Check whether this transform is linear (true) or not (false)
	 */
	bool IsLinear() const
	{
		return TimeScale.GetType() == EMovieSceneTimeWarpType::FixedPlayRate;
	}


	/**
	 * Convert this transform to its linear form.
	 * @note This is only valid to call where IsLinear() is true
	 */
	FMovieSceneTimeTransform AsLinear() const
	{
		return FMovieSceneTimeTransform(Offset, TimeScale.AsFixedPlayRateFloat());
	}


	/**
	 * Check whether this transformation requires a breadcrumb trail (true) or not (false)
	 * @note Breadcrumbs may still be added for dense trails
	 */
	bool NeedsBreadcrumb() const
	{
		return !IsLinear();
	}


	/**
	 * Returns whether this transform is an identity transformation (ie, a*T = a)
	 */
	bool IsIdentity() const
	{
		return Offset == 0 && TimeScale.GetType() == EMovieSceneTimeWarpType::FixedPlayRate && TimeScale.AsFixedPlayRate() == 1.0;
	}


	/**
	 * Transform the specified time from untransformed to transformed space
	 * 
	 * @param Time    The untransformed time to transform
	 * 
	 * @return the transformed time
	 */
	MOVIESCENE_API FFrameTime TransformTime(FFrameTime Time) const;


	/**
	 * Transform the specified time from untransformed to transformed space
	 * 
	 * @param Time    The untransformed time to transform
	 * @param Params  Transformation parameters, allowing control over the transformation, and gathering of breadcrumbs
	 *
	 * @return the transformed time
	 */
	MOVIESCENE_API FFrameTime TransformTime(FFrameTime Time, const UE::MovieScene::FTransformTimeParams& Params) const;


	/**
	 * Generate the inverse of this transformation (ie a transform from transformed to untransformed space)
	 */
	MOVIESCENE_API FMovieSceneInverseNestedSequenceTransform Inverse() const;


	/**
	 * Given a range in untransformed space, compute the hull of times that this range encompass when transformed.
	 * For instance, if this transform represents a loop of [0, 10) an input range of:
	 *     - [15,19) would yield [5,9)
	 *     - [5,25) would yield [0,10)
	 * 
	 * @note Disjoint ranges are not supported by this function. An input of [5, 19) under the previous example would yield
	 *       [0, 9) because we can only return a single range.
	 */
	MOVIESCENE_API TRange<FFrameTime> ComputeTraversedHull(const TRange<FFrameTime>& Range) const;


	/**
	 * Check whether this transformation supports 'boundaries'. A boundary is implementation defined,
	 *     but is generally a loop or cycle point.
	 */
	MOVIESCENE_API bool SupportsBoundaries() const;


	/**
	 * Retrieve this nested transform's time-warp domain, if it has one
	 */
	MOVIESCENE_API TOptional<UE::MovieScene::ETimeWarpChannelDomain> GetWarpDomain() const;


	/**
	 * Extract all the boundaries for this transform within the specified untransformed start and end time,
	 * invoking Visitor for every boundary that is present within the range. Iteration will continue until
	 * Visitor returns false, at which point this function itself will return false.
	 *
	 * @param UntransformedRange       The range within which Visitor can be invoked in untransformed space
	 * @param Visitor                  A functor to invoke for every boundary that is found within the range. Return value signifies whether iteration continues (true) or not (false)
	 * 
	 * @return (true) if there were no boundaries, or Visitor returned true for all encountered boundaries; false otherwise
	 */
	MOVIESCENE_API bool ExtractBoundariesWithinRange(const TRange<FFrameTime>& UntransformedRange, const TFunctionRef<bool(FFrameTime)>& Visitor) const;


	/**
	 * Convert this transformation to a string representation
	 */
	MOVIESCENE_API void ToString(TStringBuilderBase<TCHAR>& OutBuilder) const;


	/**
	 * Equality comparison
	 */
	friend bool operator==(const FMovieSceneNestedSequenceTransform& A, const FMovieSceneNestedSequenceTransform& B)
	{
		return A.TimeScale == B.TimeScale && A.Offset == B.Offset;
	}


	/**
	 * Inequality comparison
	 */
	friend bool operator!=(const FMovieSceneNestedSequenceTransform& A, const FMovieSceneNestedSequenceTransform& B)
	{
		return A.TimeScale != B.TimeScale || A.Offset != B.Offset;
	}

	/*~ Begin StructOpsTypeTraits */
	void PostSerialize(const FArchive& Ar);
	/*~ End StructOpsTypeTraits */

private:

	/**
	 * Time scale implemented as an optionally-warping variant
	 */
	UPROPERTY()
	FMovieSceneTimeWarpVariant TimeScale;

	/**
	 * Linear time transform for this sub-sequence.
	 */
	UPROPERTY()
	FFrameTime Offset;

public:

	/*~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~ Begin API deprecation ~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~*/
	UE_DEPRECATED(5.5, "Please update your code to use a FMovieSceneTimeWarpVariant")
	FMovieSceneNestedSequenceTransform(const FMovieSceneTimeWarping& InWarping)
	{
	}

	UE_DEPRECATED(5.5, "Please update your code to use a FMovieSceneTimeWarpVariant")
	FMovieSceneNestedSequenceTransform(FMovieSceneTimeTransform InLinearTransform, const FMovieSceneTimeWarping& InWarping)
		: TimeScale(InLinearTransform.TimeScale), Offset(InLinearTransform.Offset)
	{
	}

	UE_DEPRECATED(5.5, "Please update your code to check for IsLinear()")
	bool IsLooping() const
	{
		return TimeScale.GetType() == EMovieSceneTimeWarpType::Loop;
	}

	UE_DEPRECATED(5.5, "Please use Inverse()")
	FMovieSceneNestedSequenceTransform InverseLinearOnly() const
	{
		return IsLinear() ? AsLinear().Inverse() : FMovieSceneNestedSequenceTransform();
	}

	UE_DEPRECATED(5.5, "Please use Inverse()")
	FMovieSceneNestedSequenceTransform InverseFromWarp(uint32 WarpCount) const
	{
		return IsLinear() ? AsLinear().Inverse() : FMovieSceneNestedSequenceTransform();
	}

	UE_DEPRECATED(5.5, "Warping is now implemented as a variant within TimeScale")
	FMovieSceneTimeWarping Warping;
	/*~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~  End API deprecation  ~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~*/
};
template<>
struct TStructOpsTypeTraits<FMovieSceneNestedSequenceTransform> : public TStructOpsTypeTraitsBase2<FMovieSceneNestedSequenceTransform>
{
	enum
	{
		WithPostSerialize = true,
	};
};




/**
 * Movie scene sequence transform class that transforms from one time-space to another, represented as
 * a linear transformation plus zero or more complex, non-linear transformations.
 */
USTRUCT()
struct FMovieSceneSequenceTransform
{
	GENERATED_BODY()


	/**
	 * Default construction to the identity transform
	 */
	FMovieSceneSequenceTransform()
	{}

	/**
	 * Construction from an offset, and a scale
	 *
	 * @param InOffset 			The offset to translate by
	 * @param InTimeScale 		The timescale. For instance, if a sequence is playing twice as fast, pass 2.f
	 */
	explicit FMovieSceneSequenceTransform(FFrameTime InOffset, float InTimeScale = 1.f)
		: LinearTransform(InOffset, InTimeScale)
	{}


	/**
	 * Construction from a linear time transform.
	 *
	 * @param InLinearTransform	The linear transform
	 */
	explicit FMovieSceneSequenceTransform(FMovieSceneTimeTransform InLinearTransform)
		: LinearTransform(InLinearTransform)
	{}


	/**
	 * Construction from a single nested sequence transform structure
	 *
	 * @param InNestedTransform	The transform
	 */
	explicit FMovieSceneSequenceTransform(FMovieSceneNestedSequenceTransform&& InNestedTransform)
	{
		if (InNestedTransform.IsLinear())
		{
			LinearTransform = InNestedTransform.AsLinear();
		}
		else
		{
			NestedTransforms.Emplace(InNestedTransform);
		}
	}


	/**
	 * Equality comparison
	 */
	friend bool operator==(const FMovieSceneSequenceTransform& A, const FMovieSceneSequenceTransform& B)
	{
		return A.LinearTransform == B.LinearTransform && A.NestedTransforms == B.NestedTransforms;
	}


	/**
	 * Inequality comparison
	 */
	friend bool operator!=(const FMovieSceneSequenceTransform& A, const FMovieSceneSequenceTransform& B)
	{
		return A.LinearTransform != B.LinearTransform || A.NestedTransforms != B.NestedTransforms;
	}


	/**
	 * Returns whether this sequence transform includes any time warping.
	 */
	bool NeedsBreadcrumbs() const
	{
		return !IsLinear();
	}


	/**
	 * Returns whether this sequence transform is purely linear (i.e. doesn't involve time warping).
	 */
	bool IsLinear() const
	{
		return NestedTransforms.Num() == 0;
	}


	/**
	 * Convert this transform to its linear representation.
	 * @note It is invalid to call this function unless IsLinear() is true
	 */
	FMovieSceneTimeTransform AsLinear() const
	{
		return LinearTransform;
	}


	/**
	 * Returns whether this sequence transform is an identity transform (i.e. it doesn't change anything).
	 */
	MOVIESCENE_API bool IsIdentity() const;


	/**
	 * Transform the specified time into the inner-most (transformed) space
	 * 
	 * @param Time       The input time to transform
	 * 
	 * @return The resulting time in transformed space
	 */
	MOVIESCENE_API FFrameTime TransformTime(FFrameTime Time) const;


	/**
	 * Transform the specified time into the inner-most (transformed) space
	 * 
	 * @param Time       The input time to transform
	 * @param Params     Parameters for controlling the transform operation, and harvesting breadcrumbs
	 * 
	 * @return The resulting time in transformed space
	 */
	MOVIESCENE_API FFrameTime TransformTime(FFrameTime InTime, const UE::MovieScene::FTransformTimeParams& Params) const;


	/**
	 * Given a range in untransformed space, compute the hull of times that this range encompass when transformed.
	 * For instance, if this transform represents a loop of [0, 10) an input range of:
	 *     - [15,19) would yield [5,9)
	 *     - [5,25) would yield [0,10)
	 * 
	 * @note Disjoint ranges are not supported by this function. An input of [5, 19) under the previous example would yield
	 *       [0, 9) because we can only return a single range.
	 */
	MOVIESCENE_API TRange<FFrameTime> ComputeTraversedHull(const TRange<FFrameTime>& Range) const;


	/**
	 * Given a range in untransformed space, compute the hull of times that this range encompass when transformed.
	 * See above overload for additional details.
	 */
	MOVIESCENE_API TRange<FFrameTime> ComputeTraversedHull(const TRange<FFrameNumber>& Range) const;


	/**
	 * Retrieve the first active timewarp domain that is present in this transform, if any is present at all
	 */
	MOVIESCENE_API TOptional<UE::MovieScene::ETimeWarpChannelDomain> FindFirstWarpDomain() const;


public:


	/**
	 * Add the specified linear transform to the end of this transform stack (ie, applying it last)
	 */
	MOVIESCENE_API void Add(FMovieSceneTimeTransform InTransform);



	/**
	 * Add the specified nested transform to the end of this transform stack (ie, applying it last). Does nothing if the supplied transform is an identity.
	 */
	MOVIESCENE_API void Add(FMovieSceneNestedSequenceTransform InTransform);


	/**
	 * Add the specified warping transform to the end of this transform stack with an offset (ie, applying it last)
	 */
	MOVIESCENE_API void Add(FFrameTime InOffset, FMovieSceneTimeWarpVariant&& InTimeWarp);


	/**
	 * Add an entry to this transform denoting it should loop between the specified start and end point
	 * @note Start and End are interpreted in the transformed space of the current stack of transforms in this class.
	 * 
	 * @param Start     The start frame number of the loop
	 * @param End       The ending frame number of the loop
	 */
	MOVIESCENE_API void AddLoop(FFrameNumber Start, FFrameNumber End);


	/**
	 * Append another transform to this one, resulting in a transform that effectively goes from this -> Tail
	 * 
	 * @param Tail      The transform to append
	 */
	MOVIESCENE_API void Append(const FMovieSceneSequenceTransform& Tail);


	/**
	 * Multiply this tranmsform with another transform, resulting in a single transform that gets from RHS parent to LHS space
	 * @note Transforms apply from right to left
	 */
	MOVIESCENE_API FMovieSceneSequenceTransform operator*(const FMovieSceneSequenceTransform& RHS) const;


	/**
	 * Compute the inverse of this transform, that is: the transform that goes from transformed to untransformed space.
	 */
	MOVIESCENE_API FMovieSceneInverseSequenceTransform Inverse() const;


	/**
	 * Extract all the boundaries for this transform within the specified untransformed start and end time,
	 * invoking Visitor for every boundary that is present within the range. Iteration will continue until
	 * Visitor returns false, at which point this function itself will return false.
	 *
	 * @param UntransformedStart       The inclusive start of the range within which Visitor can be invoked
	 * @param UntransformedEnd         The inclusive end of the range within which Visitor can be invoked
	 * @param Visitor                  A functor to invoke for every boundary that is found within the range. Return value signifies whether iteration continues (true) or not (false)
	 * 
	 * @return (true) if there were no boundaries, or Visitor returned true for all encountered boundaries; false otherwise
	 */
	MOVIESCENE_API bool ExtractBoundariesWithinRange(FFrameTime UntransformedStart, FFrameTime UntransformedEnd, const TFunctionRef<bool(FFrameTime)>& Visitor) const;

	/*~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~ Begin API deprecation ~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~*/
	UE_DEPRECATED(5.5, "This function is no longer supported. Please use !IsLinear()")
	bool IsLooping() const;
	UE_DEPRECATED(5.5, "Please upgrade your code to use TransformTime that takes breadcrumbs")
	MOVIESCENE_API void TransformTime(FFrameTime InTime, FFrameTime& OutTime, FMovieSceneWarpCounter& OutWarpCounter) const;
	UE_DEPRECATED(5.5, "Transforms no longer have a constant time scale.")
	MOVIESCENE_API float GetTimeScale() const;
	UE_DEPRECATED(5.5, "Please use ComputeTraversedHull or TransformTime")
	MOVIESCENE_API TRange<FFrameTime> TransformRangePure(const TRange<FFrameTime>& Range) const;
	UE_DEPRECATED(5.5, "Please use ComputeTraversedHull or TransformTime")
	MOVIESCENE_API TRange<FFrameTime> TransformRangeUnwarped(const TRange<FFrameTime>& Range) const;
	UE_DEPRECATED(5.5, "Please use ComputeTraversedHull or TransformTime")
	MOVIESCENE_API TRange<FFrameTime> TransformRangeConstrained(const TRange<FFrameTime>& Range) const;
	UE_DEPRECATED(5.5, "Please use ComputeTraversedHull or TransformTime")
	MOVIESCENE_API TRange<FFrameNumber> TransformRangePure(const TRange<FFrameNumber>& Range) const;
	UE_DEPRECATED(5.5, "Please use ComputeTraversedHull or TransformTime")
	MOVIESCENE_API TRange<FFrameNumber> TransformRangeUnwarped(const TRange<FFrameNumber>& Range) const;
	UE_DEPRECATED(5.5, "Please use ComputeTraversedHull or TransformTime")
	MOVIESCENE_API TRange<FFrameNumber> TransformRangeConstrained(const TRange<FFrameNumber>& Range) const;
	UE_DEPRECATED(5.5, "Please use Inverse()")
	MOVIESCENE_API FMovieSceneSequenceTransform InverseNoLooping() const;
	UE_DEPRECATED(5.4, "Please use InverseNoLooping instead.")
	MOVIESCENE_API FMovieSceneTimeTransform InverseLinearOnly() const;
	UE_DEPRECATED(5.4, "Please use InverseFromAllFirstLoops instead.")
	MOVIESCENE_API FMovieSceneTimeTransform InverseFromAllFirstWarps() const;
	UE_DEPRECATED(5.4, "Please use InverseFromLoop instead.")
	MOVIESCENE_API FMovieSceneTimeTransform InverseFromWarp(const FMovieSceneWarpCounter& WarpCounter) const;
	UE_DEPRECATED(5.4, "Please use InverseFromLoop instead.")
	MOVIESCENE_API FMovieSceneTimeTransform InverseFromWarp(const TArrayView<const uint32>& WarpCounts) const;
	UE_DEPRECATED(5.5, "Please use Inverse()")
	MOVIESCENE_API FMovieSceneSequenceTransform InverseFromAllFirstLoops() const;
	UE_DEPRECATED(5.5, "Please use Inverse()")
	MOVIESCENE_API FMovieSceneSequenceTransform InverseFromLoop(const FMovieSceneWarpCounter& LoopCounter) const;
	UE_DEPRECATED(5.5, "Please use Inverse()")
	FMovieSceneSequenceTransform InverseFromLoop(const TArrayView<const FFrameTime>& Breadcrumbs) const;
	UE_DEPRECATED(5.5, "Please use Inverse()")
	MOVIESCENE_API FMovieSceneSequenceTransform InverseFromLoop(const TArrayView<const uint32>& LoopCounts) const;
	/*~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~  End API deprecation  ~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~*/

	/**
	 * The initial linear transformation represented as a 2D matrix. Always applied first.
	 */
	UPROPERTY()
	FMovieSceneTimeTransform LinearTransform;

	/**
	 * Additional transformations that should be applied after LinearTransform.
	 * This array is populated whenever a non-linear transform is encountered.
	 * */
	UPROPERTY()
	TArray<FMovieSceneNestedSequenceTransform> NestedTransforms;
};



/**
 * The inverse of a FMovieSceneSequenceTransform representing a transformation from transformed, to untransformed space.
 * This uses a different class and API because the algorithms for computing the inverse of non-linear are different,
 * often more complex, and can fail. Whereas an FMovieSceneSequenceTransform can only represent a 1:1 mapping from outer
 * to inner space, its inverse is a (sometimes empty) many:many mapping.
 * 
 * Consider a looping transform with a duration of 10 frames: [0, 10). Every time in the outer space maps to a time in the
 * inner space, but the opposite is not true. Only frames 0-10 exist in the inner space, and each frame in that time maps
 * to an infinite number of solutions in the outer space. Conversely, any inner time outside the loop range, ie, [-inf, 0)..(10, +inf]
 * cannot be transformed into the outer space.
 * 
 * For this reason, the API only has functions for attempting such computations (TryTransformTime), and iterating the solutions
 * for any given time within a range.
 * 
 * The inverse of an inverse transform is the original transform such that T*(1/T)=I theoretically holds true, although transform
 * multiplication is not actually supported by the API.
 */
USTRUCT()
struct FMovieSceneInverseSequenceTransform
{
	GENERATED_BODY()


	/**
	 * Returns whether this is a linear transform involving no non-linear components
	 */
	bool IsLinear() const
	{
		return NestedTransforms.Num() == 0;
	}


	/**
	 * Cast this transform to a linear transformation, provided IsLinear() is true.
	 */
	FMovieSceneTimeTransform AsLinear() const
	{
		check(IsLinear());
		return LinearTransform;
	}


	/**
	 * Attempt to transform the specified transformed time into its untransformed space.
	 * This function can fail if the time does not map to any other times in the untransformed space.
	 * 
	 * @param Time         The transformed time to convert to untransformed space
	 * @param Breadcrumbs  A trail of breadcrumbs that lead us from untransformed to transformed space.
	 *                     The algorithm will attempt to find the closest solution to each breadcrumb as it goes.
	 * 
	 * @return The time in untransformed space if succesful, or an empty optional otherwise.
	 */
	MOVIESCENE_API TOptional<FFrameTime> TryTransformTime(FFrameTime Time, const FMovieSceneTransformBreadcrumbs& Breadcrumbs) const;


	/**
	 * Attempt to transform the specified transformed time into its untransformed space.
	 * This function can fail if the time does not map to any other times in the untransformed space.
	 * 
	 * @param Time         The transformed time to convert to untransformed space
	 * @param Breadcrumbs  A trail of breadcrumbs that lead us from untransformed to transformed space.
	 *                     The algorithm will attempt to find the closest solution to each breadcrumb as it goes.
	 * @param Params       Additional parameters that control the algotihm, such as whether it can look forwards, backwards, or across cycles/loops.
	 * 
	 * @return The time in untransformed space if succesful, or an empty optional otherwise.
	 */
	MOVIESCENE_API TOptional<FFrameTime> TryTransformTime(FFrameTime Time, const FMovieSceneTransformBreadcrumbs& Breadcrumbs, const UE::MovieScene::FInverseTransformTimeParams& Params) const;


	/**
	 * Fallback overload that does not require a breadcrumb trail if one is not available.
	 * 
	 * Attempt to transform the specified transformed time into its untransformed space.
	 * This function can fail if the time does not map to any other times in the untransformed space.
	 * 
	 * @note This overload may yield unexpected times due to the lack of breadcrumb context. Breadcrumbs should be used if possible.
	 *
	 * @param Time         The transformed time to convert to untransformed space
	 * 
	 * @return The time in untransformed space if succesful, or an empty optional otherwise.
	 */
	MOVIESCENE_API TOptional<FFrameTime> TryTransformTime(FFrameTime Time) const;


	/**
	 * Fallback overload that does not require a breadcrumb trail if one is not available.
	 * 
	 * Attempt to transform the specified transformed time into its untransformed space.
	 * This function can fail if the time does not map to any other times in the untransformed space.
	 * 
	 * @note This overload may yield unexpected times due to the lack of breadcrumb context. Breadcrumbs should be used if possible.
	 *
	 * @param Time         The transformed time to convert to untransformed space
	 * @param Params       Additional parameters that control the algotihm, such as whether it can look forwards, backwards, or across cycles/loops.
	 * 
	 * @return The time in untransformed space if succesful, or an empty optional otherwise.
	 */
	MOVIESCENE_API TOptional<FFrameTime> TryTransformTime(FFrameTime Time, const UE::MovieScene::FInverseTransformTimeParams& Params) const;

	/**
	 * Transforms a time from transformed to untransformed space within a finite range specified by two breadcrumb trails.
	 * For each solution that exists in the untransformed space, Visitor will be invoked and its result defines whether the
	 * algorithm continues (true) or terminates early (false).
	 *
	 * @param Time              The transformed time to convert to untransformed space
	 * @param Visitor           A functor to invoke for every solution that exists within the range.
	 *                          Returning true will allow the algorithm to continue, false will cause it to terminate early.
	 * @param StartBreadCrumbs  A breadcrumb trail for the start of the allowable range to solve within.
	 * @param EndBreadCrumbs    A breadcrumb trail for the end of the allowable range to solve within.
	 * 
	 * @return False if any invocation of Visitor returned false, true otherwise (if there were no solutions, or every invocation of Visitor returned true).
	 */
	MOVIESCENE_API bool TransformTimeWithinRange(FFrameTime Time, const TFunctionRef<bool(FFrameTime)>& Visitor, const FMovieSceneTransformBreadcrumbs& StartBreadcrumbs, const FMovieSceneTransformBreadcrumbs& EndBreadcrumbs) const;


	/**
	 * Transforms a finite range in the transformed space, to non-empty ranges in untransformed space.
	 * For each solution that exists in the untransformed space, Visitor will be invoked and its result defines whether the
	 * algorithm continues (true) or terminates early (false).
	 *
	 * @param Range             The range in transformed space to convert to untransformed space.
	 * @param Visitor           A functor to invoke for every solution that exists within the range.
	 *                          Returning true will allow the algorithm to continue, false will cause it to terminate early.
	 * @param StartBreadCrumbs  A breadcrumb trail for the start of the allowable range to solve within.
	 * @param EndBreadCrumbs    A breadcrumb trail for the end of the allowable range to solve within.
	 * 
	 * @return False if any invocation of Visitor returned false, true otherwise (if there were no solutions, or every invocation of Visitor returned true).
	 */
	MOVIESCENE_API bool TransformFiniteRangeWithinRange(const TRange<FFrameTime>& Range, TFunctionRef<bool(TRange<FFrameTime>)> Visitor, const FMovieSceneTransformBreadcrumbs& StartBreadcrumbs, const FMovieSceneTransformBreadcrumbs& EndBreadcrumbs) const;


	/**
	 * Legacy function that folds all linear transforms together. Does not account for non-linear transforms and should not be used any more.
	 */
	UE_DEPRECATED(5.5, "This function is no longer supported. Please use FMovieSceneInverseSequenceTransform directly.")
	MOVIESCENE_API FMovieSceneTimeTransform AsLegacyLinearTimeTransform() const;

private:


	/**
	 * Implementation function for TransformTimeWithinRange that recursively solves 1:many transformations
	 */
	bool RecursiveTransformTimeWithinRange(int32 NestingIndex, FFrameTime Time, const TFunctionRef<bool(FFrameTime)>& FinalVisitor, TArrayView<const FFrameTime> StartBreadcrumbs, TArrayView<const FFrameTime> EndBreadcrumbs) const;

private:

	friend FMovieSceneSequenceTransform;


	/**
	 * The final linear transformation represented as a 2D matrix. Always applied last.
	 */
	UPROPERTY()
	FMovieSceneTimeTransform LinearTransform;


	/**
	 * Additional transformations that should be applied before LinearTransform.
	 * This array is populated whenever a non-linear transform is encountered.
	 * */
	UPROPERTY()
	TArray<FMovieSceneInverseNestedSequenceTransform> NestedTransforms;
};

/**
 * Transform a time by a sequence transform
 *
 * @param InTime 			The time to transform
 * @param RHS 				The transform
 */
MOVIESCENE_API FFrameTime operator*(FFrameTime InTime, const FMovieSceneSequenceTransform& RHS);

/**
 * Transform a time by a sequence transform
 *
 * @param InTime 			The time to transform
 * @param RHS 				The transform
 */
inline FFrameTime& operator*=(FFrameTime& InTime, const FMovieSceneSequenceTransform& RHS)
{
	InTime = InTime * RHS;
	return InTime;
}

/** Convert a FMovieSceneSequenceTransform into a string */
FString LexToString(const FMovieSceneSequenceTransform& InTransform);

/** Convert a FMovieSceneWarpCounter into a string */
FString LexToString(const FMovieSceneWarpCounter& InCounter);