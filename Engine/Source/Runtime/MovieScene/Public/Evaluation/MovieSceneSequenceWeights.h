// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/SortedMap.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "MovieSceneSequenceID.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"

class UMovieSceneEntitySystemLinker;

namespace UE::MovieScene
{

/**
 * Container class for keeping track of externally created entities for providing dynamic sequence weights
 */
class FSequenceWeights
{
public:
	MOVIESCENE_API FSequenceWeights(UMovieSceneEntitySystemLinker* InLinker, FRootInstanceHandle InRootInstanceHandle);
	MOVIESCENE_API ~FSequenceWeights();

	/**
	 * Set the weight of the specifieid sequence ID by creating a new HierarchicalEasingProvider entity and assigning its weight
	 *
	 * @param InSequenceID    The sequence ID to assign a weight to
	 * @param Weight          The weight to assign. This is multiplied with all other weights contributing to this sequence
	 */
	MOVIESCENE_API void SetWeight(FMovieSceneSequenceID InSequenceID, double Weight = 1.0);


	/**
	 * Set the weights of the specifieid sequence IDs by creating new HierarchicalEasingProvider entities and assigning their weights
	 *
	 * @param Weights         Map containing all desired sequence weights organized by their sequence ID
	 */
	MOVIESCENE_API void SetWeights(const TSortedMap<FMovieSceneSequenceID, double>& Weights);


	/**
	 * Remove a previously allocated weight for the specified sequence ID
	 */
	MOVIESCENE_API void RemoveWeight(FMovieSceneSequenceID InSequenceID);

private:

	/** The linker that contains the playback data for this container */
	TWeakObjectPtr<UMovieSceneEntitySystemLinker> WeakLinker;

	/** Map of allocated entity IDs organized by sequence ID */
	TSortedMap<FMovieSceneSequenceID, FMovieSceneEntityID> WeightEntitiesBySequenceID;

	/** The root instance handle for our main sequence */
	FRootInstanceHandle RootInstanceHandle;
};


} // namespace UE::MovieScene
