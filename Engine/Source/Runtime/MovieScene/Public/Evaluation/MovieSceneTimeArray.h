// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/FrameTime.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"

/** Entry struct for the TMovieSceneTimeArray class */
template<typename DataType>
struct TMovieSceneTimeArrayEntry
{
	FFrameTime RootTime;
	DataType Datum;
};

/**
 * A utility class that lets you store a collection of timestamped data originating from various time bases.
 *
 * All of the data is stored in "root" time space. That is: as you add timestamped data, these timestamps are
 * converted back to "root times" using the inverse of the current time transform. Pushing and popping time
 * transforms, and incrementing loop counts, makes it possible to change what's considered "local time", 
 * which affects this inverse transformation.
 */
template<typename DataType>
struct TMovieSceneTimeArray
{
	void Add(FFrameTime RootTime, const DataType& Datum)
	{
		Entries.Emplace(FEntry{ RootTime, Datum });
	}

	/** Clears all entries */
	void Clear()
	{
		Entries.Clear();
	}

	/** Gets the current list of entries in the array */
	TArrayView<const TMovieSceneTimeArrayEntry<DataType>> GetEntries() const
	{
		return TArrayView<const FEntry>(Entries);
	}

private:

	using FEntry = TMovieSceneTimeArrayEntry<DataType>;

	TArray<FEntry> Entries;
};
