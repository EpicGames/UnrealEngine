// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/FrameTime.h"
#include "MovieSceneTimeTransform.generated.h"

/**
 * Movie scene sequence transform class that transforms from one time-space to another.
 *
 * @note The transform can be thought of as the top row of a 2x2 matrix, where the bottom row is the identity:
 * 			| TimeScale	Offset	|
 *			| 0			1		|
 *
 * As such, traditional matrix mathematics can be applied to transform between different sequence's time-spaces.
 *
 * Transforms apply time scale first, and then offset.
 */
USTRUCT()
struct FMovieSceneTimeTransform
{
	GENERATED_BODY()

	/**
	 * Default construction to the identity transform
	 */
	FMovieSceneTimeTransform()
		: TimeScale(1.f)
		, Offset(0)
	{}

	/**
	 * Construction from an offset, and a scale
	 *
	 * @param InOffset 			The offset to translate by
	 * @param InTimeScale 		The timescale. For instance, if a sequence is playing twice as fast, pass 2.f
	 */
	explicit FMovieSceneTimeTransform(FFrameTime InOffset, float InTimeScale = 1.f)
		: TimeScale(InTimeScale)
		, Offset(InOffset)
	{
		checkf(!FMath::IsNearlyZero(TimeScale), TEXT("It is invalid to create a linear transformation with a zero timescale. This case should be handled externally by FMovieSceneTimeWarpVariant and FLAG_Zero."))
	}

	friend bool operator==(const FMovieSceneTimeTransform& A, const FMovieSceneTimeTransform& B)
	{
		return A.TimeScale == B.TimeScale && A.Offset == B.Offset;
	}

	friend bool operator!=(const FMovieSceneTimeTransform& A, const FMovieSceneTimeTransform& B)
	{
		return A.TimeScale != B.TimeScale || A.Offset != B.Offset;
	}

	/**
	 * Returns whether this transform is an identity transform.
	 */
	bool IsIdentity() const
	{
		return Offset.FrameNumber.Value == 0 && FMath::IsNearlyZero(Offset.GetSubFrame()) && FMath::IsNearlyEqual(TimeScale, 1.f);
	}

	/**
	 * Retrieve the inverse of this transform
	 */
	FMovieSceneTimeTransform Inverse() const
	{
		const FFrameTime NewOffset = FMath::IsNearlyZero(TimeScale) ? -Offset : -Offset / TimeScale;
		return FMovieSceneTimeTransform(NewOffset, FMath::IsNearlyZero(TimeScale) ? std::numeric_limits<float>::infinity() : 1.f / TimeScale);
	}

	/** The sequence's time scale (or play rate) */
	UPROPERTY()
	float TimeScale;

	/** Scalar frame offset applied after the scale */
	UPROPERTY()
	FFrameTime Offset;
};

/**
 * Transform a time by a sequence transform
 *
 * @param InTime 			The time to transform
 * @param RHS 				The transform
 */
inline FFrameTime operator*(FFrameTime InTime, const FMovieSceneTimeTransform& RHS)
{
	// Avoid floating point conversion when in the same time-space
	if (RHS.TimeScale == 1.f)
	{
		return InTime + RHS.Offset;
	}
	else if (!FMath::IsFinite(RHS.TimeScale))
	{
		// Special case infinite timescale (inverse of a zero-timescale) and just return the RHS.Offset
		return RHS.Offset;
	}

	return InTime * RHS.TimeScale + RHS.Offset;
}

/**
 * Transform a time by a sequence transform
 *
 * @param InTime 			The time to transform
 * @param RHS 				The transform
 */
inline FFrameTime& operator*=(FFrameTime& InTime, const FMovieSceneTimeTransform& RHS)
{
	InTime = InTime * RHS;
	return InTime;
}

/**
 * Transform a time range by a sequence transform
 *
 * @param LHS 				The time range to transform
 * @param RHS 				The transform
 */
template<typename T>
TRange<T> operator*(const TRange<T>& LHS, const FMovieSceneTimeTransform& RHS)
{
	TRange<T> Result = LHS;

	if (!Result.GetLowerBound().IsOpen())
	{
		Result.SetLowerBoundValue(Result.GetLowerBoundValue() * RHS);
	}
	if (!Result.GetUpperBound().IsOpen())
	{
		Result.SetUpperBoundValue(Result.GetUpperBoundValue() * RHS);
	}

	return Result;
}

inline TRange<FFrameNumber> operator*(const TRange<FFrameNumber>& LHS, const FMovieSceneTimeTransform& RHS)
{
	TRange<FFrameNumber> Result = LHS;

	if (!Result.GetLowerBound().IsOpen())
	{
		Result.SetLowerBoundValue((Result.GetLowerBoundValue() * RHS).FloorToFrame());
	}
	if (!Result.GetUpperBound().IsOpen())
	{
		Result.SetUpperBoundValue((Result.GetUpperBoundValue() * RHS).FloorToFrame());
	}

	return Result;
}

/**
 * Transform a time range by a sequence transform
 *
 * @param InTime 			The time range to transform
 * @param RHS 				The transform
 */
template<typename T>
TRange<T>& operator*=(TRange<T>& LHS, const FMovieSceneTimeTransform& RHS)
{
	LHS = LHS * RHS;
	return LHS;
}

/**
 * Multiply 2 transforms together, resulting in a single transform that gets from RHS parent to LHS space
 * @note Transforms apply from right to left
 */
inline FMovieSceneTimeTransform operator*(const FMovieSceneTimeTransform& LHS, const FMovieSceneTimeTransform& RHS)
{
	// The matrix multiplication occurs as follows:
	//
	// | TimeScaleA	, OffsetA	|	.	| TimeScaleB, OffsetB	|
	// | 0			, 1			|		| 0			, 1			|
	//
	const FFrameTime ScaledOffsetRHS = (LHS.TimeScale == 1.f || !FMath::IsFinite(LHS.TimeScale)) ? RHS.Offset : RHS.Offset * LHS.TimeScale;
	return FMovieSceneTimeTransform(
		LHS.Offset + ScaledOffsetRHS,		// New Offset
		RHS.TimeScale * LHS.TimeScale		// New TimeScale
		);
}

/** Convert a FFrameTime into a string */
inline FString LexToString(const FMovieSceneTimeTransform& InTransform)
{
	if (InTransform.TimeScale == 1.f)
	{
		if (InTransform.Offset.GetSubFrame() == 0.f)
		{
			return *FString::Printf(TEXT("[ %+i ]"), InTransform.Offset.FrameNumber.Value);
		}
		else
		{
			return *FString::Printf(TEXT("[ %+i+%.3f ]"), 
					InTransform.Offset.FrameNumber.Value, InTransform.Offset.GetSubFrame());
		}
	}
	else
	{
		if (InTransform.Offset.GetSubFrame() == 0.f)
		{
			return *FString::Printf(TEXT("[ %+i x%.3f ]"), 
					InTransform.Offset.FrameNumber.Value, InTransform.TimeScale);
		}
		else
		{
			return *FString::Printf(TEXT("[ %+i+%.3f x%.3f ]"), 
					InTransform.Offset.FrameNumber.Value, InTransform.Offset.GetSubFrame(), InTransform.TimeScale);
		}
	}
}
