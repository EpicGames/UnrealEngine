// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Math/Range.h"
#include "Math/RangeBound.h"
#include "Misc/AssertionMacros.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameTime.h"
#include "MovieSceneTimeTransform.h"
#include "UObject/ObjectMacros.h"

#include "MovieSceneTimeWarping.generated.h"

/**
 * Transform time by warping it around from end to start. This is mostly useful for
 * looping things.
 */
USTRUCT()
struct FMovieSceneTimeWarping
{
	UE_DEPRECATED(5.5, "Please update your code to the new FMovieSceneSequenceTransform API. The closest analogue is FMovieSceneTimeWarpLoop.")
	MOVIESCENE_API static const uint32 InvalidWarpCount;

	GENERATED_BODY()

	FMovieSceneTimeWarping()
		: Start(0)
		, End(0)
	{}

	explicit FMovieSceneTimeWarping(FFrameNumber InStart, FFrameNumber InEnd)
		: Start(InStart)
		, End(InEnd)
	{
		check(Start <= End);
	}

	/**
	 * Returns the length of the warping.
	 */
	UE_DEPRECATED(5.5, "Please update your code to the new FMovieSceneSequenceTransform API. The closest analogue is FMovieSceneTimeWarpLoop.")
	FFrameNumber Length() const { return End - Start; }

	/**
	 * Returns whether this warping transform is doing anything.
	 */
	UE_DEPRECATED(5.5, "Please update your code to the new FMovieSceneSequenceTransform API. The closest analogue is FMovieSceneTimeWarpLoop.")
	bool IsValid() const { return End > Start; }

	/**
	 * Returns a range that encompasses the whole warping time span.
	 */
	UE_DEPRECATED(5.5, "Please update your code to the new FMovieSceneSequenceTransform API. The closest analogue is FMovieSceneTimeWarpLoop.")
	TRange<FFrameTime> GetRange() const
	{
		const TRangeBound<FFrameTime> OutLower(TRangeBound<FFrameTime>::Inclusive(Start));
		const TRangeBound<FFrameTime> OutUpper(TRangeBound<FFrameTime>::Exclusive(End));
		return TRange<FFrameTime>(OutLower, OutUpper);
	}

	/**
	 * Returns a transformation that takes us from a local time into a given loop back to the root time.
	 */
	UE_DEPRECATED(5.5, "Please update your code to the new FMovieSceneSequenceTransform API. The closest analogue is FMovieSceneTimeWarpLoop.")
	FMovieSceneTimeTransform InverseFromWarp(uint32 WarpCount) const;

	/**
	 * Transforms the given frame and returns the warped frame time along with the warp index we ended up in.
	 */
	UE_DEPRECATED(5.5, "Please update your code to the new FMovieSceneSequenceTransform API. The closest analogue is FMovieSceneTimeWarpLoop.")
	void TransformFrame(FFrameNumber InFrame, FFrameNumber& OutFrame, uint32& OutWarpIndex) const;

	/**
	 * Transforms the given time and returns the warped time along with the warp index we ended up in.
	 */
	UE_DEPRECATED(5.5, "Please update your code to the new FMovieSceneSequenceTransform API. The closest analogue is FMovieSceneTimeWarpLoop.")
	void TransformTime(FFrameTime InTime, FFrameTime& OutTime, uint32& OutWarpIndex) const;

	/**
	 * Transforms the given time by warping it by a specific warp count, regardless of how many warps are
	 * needed in theory to stay in the warping range.
	 */
	UE_DEPRECATED(5.5, "Please update your code to the new FMovieSceneSequenceTransform API. The closest analogue is FMovieSceneTimeWarpLoop.")
	void TransformTimeSpecific(FFrameTime InTime, uint32 WarpCount, FFrameTime& OutTime) const;

	/** 
	 * Transforms the given range in a "naive" way, i.e. its lower and upper bounds are transformed
	 * independently by this time warping. This means that the output range could be "inside-out"... that is,
	 * the output lower bound could be greater than the output upper bound, like, for instance, in the case
	 * of the input range starting near the end of a loop, and ending near the beginning of another loop.
	 */
	UE_DEPRECATED(5.5, "Please update your code to the new FMovieSceneSequenceTransform API. The closest analogue is FMovieSceneTimeWarpLoop.")
	TRange<FFrameTime> TransformRangePure(const TRange<FFrameTime>& Range) const;

	/**
	 * Transforms the given range by time-warping its lower bound, and computing an upper bound so that the
	 * size of the input range is preserved. So if the input range starts in the middle of a loop and lasts
	 * 3 loops, the output range will start in the middle of the first loop, and last 3 times as long as
	 * the Length of this time warp.
	 */
	UE_DEPRECATED(5.5, "Please update your code to the new FMovieSceneSequenceTransform API. The closest analogue is FMovieSceneTimeWarpLoop.")
	TRange<FFrameTime> TransformRangeUnwarped(const TRange<FFrameTime>& Range) const;

	/**
	 * Transforms the given range by time-warping it and figuring out if it "covered" a full loop or not.
	 * If it covered a full loop, return a full-loop range, i.e. a range that goes from StartOffset to 
	 * (Length - EndOffset).
	 * If it didn't cover a full loop, return the transformed range, which should be a subset of the full
	 * loop range mentioned above.
	 */
	UE_DEPRECATED(5.5, "Please update your code to the new FMovieSceneSequenceTransform API. The closest analogue is FMovieSceneTimeWarpLoop.")
	TRange<FFrameTime> TransformRangeConstrained(const TRange<FFrameTime>& Range) const;

	UE_DEPRECATED(5.5, "Please update your code to the new FMovieSceneSequenceTransform API. The closest analogue is FMovieSceneTimeWarpLoop.")
	friend bool operator==(const FMovieSceneTimeWarping& A, const FMovieSceneTimeWarping& B)
	{
		return A.Start == B.Start && A.End == B.End;
	}

	UE_DEPRECATED(5.5, "Please update your code to the new FMovieSceneSequenceTransform API. The closest analogue is FMovieSceneTimeWarpLoop.")
	friend bool operator!=(const FMovieSceneTimeWarping& A, const FMovieSceneTimeWarping& B)
	{
		return A.Start != B.Start || A.End != B.End;
	}

	UPROPERTY()
	FFrameNumber Start;

	UPROPERTY()
	FFrameNumber End;
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS

inline FFrameNumber operator*(FFrameNumber InFrame, const FMovieSceneTimeWarping& RHS)
{
	checkSlow(RHS.IsValid());
	const FFrameNumber WarpLength = RHS.Length();
	return (InFrame - RHS.Start) % WarpLength + RHS.Start;
}

inline FFrameTime operator*(FFrameTime InTime, const FMovieSceneTimeWarping& RHS)
{
	checkSlow(RHS.IsValid());
	const FFrameNumber WarpLength = RHS.Length();
	return (InTime - RHS.Start) % WarpLength + RHS.Start;
}

inline FFrameNumber& operator*=(FFrameNumber& InFrame, const FMovieSceneTimeWarping& RHS)
{
	InFrame = InFrame * RHS;
	return InFrame;
}

inline FFrameTime& operator*=(FFrameTime& InTime, const FMovieSceneTimeWarping& RHS)
{
	InTime = InTime * RHS;
	return InTime;
}

inline FMovieSceneTimeTransform FMovieSceneTimeWarping::InverseFromWarp(uint32 WarpCount) const
{
	check(IsValid() || WarpCount == InvalidWarpCount);

	if (WarpCount == InvalidWarpCount || WarpCount == 0)
	{
		return FMovieSceneTimeTransform();
	}
	else
	{
		const FFrameNumber WarpLength = Length();
		const FFrameNumber WarpsOffset = WarpLength * (float)WarpCount;
		return FMovieSceneTimeTransform(WarpsOffset);
	}
}

inline void FMovieSceneTimeWarping::TransformFrame(FFrameNumber InFrame, FFrameNumber& OutFrame, uint32& OutWarpIndex) const
{
	checkSlow(IsValid());
	const FFrameNumber WarpLength = Length();
	OutWarpIndex = 0;
	FFrameNumber TempFrame = InFrame - Start;
	while (TempFrame >= WarpLength)
	{
		TempFrame -= WarpLength;
		++OutWarpIndex;
	}
	OutFrame = TempFrame + Start;
}

inline void FMovieSceneTimeWarping::TransformTime(FFrameTime InTime, FFrameTime& OutTime, uint32& OutWarpIndex) const
{
	checkSlow(IsValid());
	const FFrameTime WarpLength = Length();
	OutWarpIndex = 0;
	FFrameTime TempTime = InTime - Start;
	while (TempTime >= WarpLength)
	{
		TempTime = TempTime - WarpLength;
		++OutWarpIndex;
	}
	OutTime = TempTime + Start;
}

inline void FMovieSceneTimeWarping::TransformTimeSpecific(FFrameTime InTime, uint32 WarpCount, FFrameTime& OutTime) const
{
	checkSlow(IsValid());
	const FFrameTime WarpLength = Length();
	FFrameTime TempTime = InTime - Start;
	for (uint32 WarpIndex = 0; WarpIndex < WarpCount; ++WarpIndex)
	{
		TempTime = TempTime - WarpLength;
	}
	OutTime = TempTime + Start;
}

inline TRange<FFrameTime> FMovieSceneTimeWarping::TransformRangePure(const TRange<FFrameTime>& Range) const
{
	checkSlow(IsValid());

	const TRangeBound<FFrameTime> SourceLower = Range.GetLowerBound();
	const TRangeBound<FFrameTime> TransformedLower =
		SourceLower.IsOpen() ?
			TRangeBound<FFrameTime>() :
			SourceLower.IsInclusive() ?
				TRangeBound<FFrameTime>::Inclusive(SourceLower.GetValue() * (*this)) :
				TRangeBound<FFrameTime>::Exclusive(SourceLower.GetValue() * (*this));

	const TRangeBound<FFrameTime> SourceUpper = Range.GetUpperBound();
	const TRangeBound<FFrameTime> TransformedUpper =
		SourceUpper.IsOpen() ?
			TRangeBound<FFrameTime>() :
			SourceUpper.IsInclusive() ?
				TRangeBound<FFrameTime>::Inclusive(SourceUpper.GetValue() * (*this)) :
				TRangeBound<FFrameTime>::Exclusive(SourceUpper.GetValue() * (*this));

	return TRange<FFrameTime>(TransformedLower, TransformedUpper);
}

inline TRange<FFrameTime> FMovieSceneTimeWarping::TransformRangeUnwarped(const TRange<FFrameTime>& Range) const
{
	checkSlow(IsValid());

	if (!Range.GetLowerBound().IsOpen() && !Range.GetUpperBound().IsOpen())
	{
		const TRangeBound<FFrameTime> SourceLower = Range.GetLowerBound();
		const TRangeBound<FFrameTime> TransformedLower =
			SourceLower.IsInclusive() ?
				TRangeBound<FFrameTime>::Inclusive(SourceLower.GetValue() * (*this)) :
				TRangeBound<FFrameTime>::Exclusive(SourceLower.GetValue() * (*this));

		const FFrameTime RangeSize = Range.Size<FFrameTime>();
		const FFrameTime TransformedUpperValue = TransformedLower.GetValue() + RangeSize;
		const TRangeBound<FFrameTime> TransformedUpper =
			Range.GetUpperBound().IsInclusive() ?
				TRangeBound<FFrameTime>::Inclusive(TransformedUpperValue) :
				TRangeBound<FFrameTime>::Exclusive(TransformedUpperValue);

		return TRange<FFrameTime>(TransformedLower, TransformedUpper);
	}
	else
	{
		return TransformRangePure(Range);
	}
}

inline TRange<FFrameTime> FMovieSceneTimeWarping::TransformRangeConstrained(const TRange<FFrameTime>& Range) const
{
	checkSlow(IsValid());
	
	if (!Range.GetLowerBound().IsOpen() && !Range.GetUpperBound().IsOpen())
	{
		const FFrameNumber WarpLength = Length();
		const FFrameTime RangeSize = Range.Size<FFrameTime>();
		if (RangeSize >= WarpLength)
		{
			// If the range is longer than a loop, we could, in theory, end up with a disjointed range, like, for
			// example, a range that starts in the first loop and ends in the second loop. Regardless of whether
			// we actually finished a full loop or not, we need here to take the "hull" of that, which is always
			// the looping range inside our local space.
			return GetRange();
		}

		const TRange<FFrameTime> UnwarpedTransformedRange = TransformRangeUnwarped(Range);
		if (UnwarpedTransformedRange.GetUpperBoundValue() > End || UnwarpedTransformedRange.GetLowerBoundValue() < Start)
		{
			// The input range is going over a warp boundary, so we return the whole warping range.
			return GetRange();
		}

		// The input range fits inside a single loop, so we can return that.
		return UnwarpedTransformedRange;
	}
	else
	{
		// One of the bounds is open, so we are looping from or to infinitiy. That encompasses our whole
		// looping range.
		return GetRange();
	}
}

/** Convert a FMovieSceneTimeWarping into a string */
inline FString LexToString(const FMovieSceneTimeWarping& InWarping)
{
	return *FString::Printf(TEXT("[ %+i ⟳ %+i ]"), InWarping.Start.Value, InWarping.End.Value);
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS