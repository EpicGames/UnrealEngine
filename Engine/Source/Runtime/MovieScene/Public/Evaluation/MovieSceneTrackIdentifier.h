// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/EditorObjectVersion.h"
#include "MovieSceneTrackIdentifier.generated.h"

USTRUCT()
struct FMovieSceneTrackIdentifier
{
	GENERATED_BODY()

	FMovieSceneTrackIdentifier()
		: Value(-1)
	{}

	static FMovieSceneTrackIdentifier Invalid() { return FMovieSceneTrackIdentifier(); }

	explicit operator bool() const
	{
		return Value != -1;
	}

	FMovieSceneTrackIdentifier& operator++()
	{
		++Value;
		return *this;
	}

	inline friend bool operator==(FMovieSceneTrackIdentifier A, FMovieSceneTrackIdentifier B)
	{
		return A.Value == B.Value;
	}

	inline friend bool operator!=(FMovieSceneTrackIdentifier A, FMovieSceneTrackIdentifier B)
	{
		return A.Value != B.Value;
	}

	inline friend bool operator<(FMovieSceneTrackIdentifier A, FMovieSceneTrackIdentifier B)
	{
		return A.Value < B.Value;
	}

	inline friend bool operator>(FMovieSceneTrackIdentifier A, FMovieSceneTrackIdentifier B)
	{
		return A.Value > B.Value;
	}

	inline friend uint32 GetTypeHash(FMovieSceneTrackIdentifier In)
	{
		return In.Value;
	}

	/** Custom serialized to reduce memory footprint */
	bool Serialize(FArchive& Ar)
	{
		Ar.UsingCustomVersion(FEditorObjectVersion::GUID);
		if (Ar.CustomVer(FEditorObjectVersion::GUID) < FEditorObjectVersion::MovieSceneMetaDataSerialization)
		{
			return false;
		}

		Ar << Value;
		return true;
	}

	friend FArchive& operator<<(FArchive& Ar, FMovieSceneTrackIdentifier& Identifier)
	{
		Identifier.Serialize(Ar);
		return Ar;
	}

private:

	friend struct FMovieSceneEvaluationTemplate;
	
	UPROPERTY()
	uint32 Value;
};

template<>
struct TStructOpsTypeTraits<FMovieSceneTrackIdentifier> : public TStructOpsTypeTraitsBase2<FMovieSceneTrackIdentifier>
{
	enum
	{
		WithSerializer = true, WithIdenticalViaEquality = true
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};
