// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Evaluation/MovieSceneEvalTemplateBase.h"
#include "Evaluation/MovieScenePlayback.h"
#include "Evaluation/MovieSceneSegment.h"
#include "Evaluation/PersistentEvaluationData.h"
#include "Misc/AssertionMacros.h"
#include "Misc/InlineValue.h"
#include "Templates/Decay.h"
#include "Templates/EnableIf.h"
#include "Templates/PointerIsConvertibleFromTo.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"

#include "MovieSceneTrackImplementation.generated.h"

class FArchive;
class IMovieScenePlayer;
class UObject;
struct FMovieSceneEvaluationOperand;
struct FMovieSceneEvaluationTrack;
struct FMovieSceneExecutionTokens;
struct FMovieSceneFieldEntry_ChildTemplate;
struct FMovieSceneInterrogationData;

/**
 * Structure that allows the implementation of setup/teardown/initialization/evaluation logic at the track level.
 */
USTRUCT()
struct FMovieSceneTrackImplementation : public FMovieSceneEvalTemplateBase
{
public:
	GENERATED_BODY()

	/**
	 * Default constructor
	 */
	FMovieSceneTrackImplementation()
	{
	}

	/**
	 * Determine whether this track implementation has its own custom initialization override
	 */
	bool HasCustomInitialize() const
	{
		return (OverrideMask & CustomInitializeFlag) != 0;
	}

	/**
	 * Determine whether this track implementation has its own custom evaluation override
	 */
	bool HasCustomEvaluate() const
	{
		return (OverrideMask & CustomEvaluateFlag) != 0;
	}

	/**
	 * Perform pre frame initialization on the specified segment of the track. Will generally call Initialize on all child templates in the current segment as well.
	 * @note					This function is intended to allow pre-frame set up, and should avoid mutating any state.
	 *							Only called if EnableOverrides(CustomInitializeFlag) has been called (see SetupOverrides).
	 *
	 * @param Track				The parent evaluation track that has knowledge of all child tracks and segments
	 * @param SegmentID			The identifier of the segment at the current time
	 * @param Operand			Unique handle to the operand on which we are to operate. May represent multiple objects. Resolve through IMovieScenePlayer::FindBoundObjects(Operand)
	 * @param Context			Evaluation context specifying the current evaluation time, sub sequence transform and other relevant information.
	 * @param PersistentData	Persistent data store which can be used to store arbitrary data pertaining to the current template that may be required in Evaluate(Swept)
	 * @param Player			The movie scene player currently playing back this sequence
	 */
	virtual void Initialize(const FMovieSceneEvaluationTrack& Track, TArrayView<const FMovieSceneFieldEntry_ChildTemplate> Children, const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) const
	{
		ensureMsgf(false, TEXT("FMovieSceneTrackImplementation::Initialize has not been implemented. Did you erroneously call EnableOverrides(CustomInitializeFlag)?"));
	}

	/**
	 * Perform evaluation on the specified segment of the track. Will generally call Evaluate on all child templates in the current segment as well.
	 * @note					This function should perform any expensive or costly evaluation logic required to calculate the final animated state.
	 *							Potentially called on a thread, and as such has no access to the current evaluation environment.
	 *							Only called if EnableOverrides(CustomEvaluateFlag) has been called (see SetupOverrides).
	 *
	 * @param Track				The parent evaluation track that has knowledge of all child tracks and segments
	 * @param SegmentID			The identifier of the segment at the current time
	 * @param Operand			Unique handle to the operand on which we are to operate. Only to be used as a reference, or forwarded throgh to an execution token.
	 * @param Context			Evaluation context specifying the current evaluation time, sub sequence transform and other relevant information.
	 * @param PersistentData	Persistent data store which can be used to access arbitrary data pertaining to the current template that should have been set up in initialize.
	 * @param ExecutionTokens	Stack of execution tokens that will be used to apply animated state to the environment at a later time.
	 */
	virtual void Evaluate(const FMovieSceneEvaluationTrack& Track, TArrayView<const FMovieSceneFieldEntry_ChildTemplate> Children, const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const
	{
		ensureMsgf(false, TEXT("FMovieSceneTrackImplementation::Evaluate has not been implemented. Did you erroneously call EnableOverrides(CustomEvaluateFlag)?"));
	}

	/**
	 * Interrogate this template for its output. Should not have any side effects.
	 *
	 * @param Context				Evaluation context specifying the current evaluation time, sub sequence transform and other relevant information.
	 * @param Container				Container to populate with the desired output from this track
	 * @param BindingOverride		Optional binding to specify the object that is being animated by this track
	 */
	virtual bool Interrogate(const FMovieSceneContext& Context, FMovieSceneInterrogationData& Container, UObject* BindingOverride = nullptr) const
	{
		return false;
	}

protected:

	/**
	 * Enum evaluation flag structure defining which functions are to be called in implementations of this struct
	 */
	enum EOverrideMask
	{
		CustomInitializeFlag	= 0x004,
		CustomEvaluateFlag		= 0x008,
	};
};

/** Custom serialized type that attempts to store a track implementation template in inline memory if possible */
USTRUCT()
struct FMovieSceneTrackImplementationPtr
#if CPP
	: TInlineValue<FMovieSceneTrackImplementation, 48>
#endif
{
	GENERATED_BODY()

	/** Default construction to an empty container */
	FMovieSceneTrackImplementationPtr(){}

	/** Construction from any FMovieSceneTrackImplementation derivative */
	template<
		typename T,
		typename = typename TEnableIf<TPointerIsConvertibleFromTo<typename TDecay<T>::Type, FMovieSceneTrackImplementation>::Value>::Type
	>
	FMovieSceneTrackImplementationPtr(T&& In)
		: TInlineValue(Forward<T>(In))
	{
		static_assert(!std::is_same_v<typename TDecay<T>::Type, FMovieSceneTrackImplementation>, "Direct usage of FMovieSceneTrackImplementation is prohibited.");

#if WITH_EDITOR
		checkf(T::StaticStruct() == &In.GetScriptStruct() && T::StaticStruct() != FMovieSceneTrackImplementation::StaticStruct(), TEXT("%s type does not correctly override GetScriptStructImpl. Track will not serialize correctly."), *T::StaticStruct()->GetName());
#endif
	}
	
	/** Copy construction/assignment */
	FMovieSceneTrackImplementationPtr(const FMovieSceneTrackImplementationPtr& RHS)
	{
		*this = RHS;
	}
	FMovieSceneTrackImplementationPtr& operator=(const FMovieSceneTrackImplementationPtr& RHS)
	{
		if (RHS.IsValid())
		{
			UScriptStruct::ICppStructOps& StructOps = *RHS->GetScriptStruct().GetCppStructOps();

			void* Allocation = Reserve(StructOps.GetSize(), StructOps.GetAlignment());
			StructOps.Construct(Allocation);
			StructOps.Copy(Allocation, &RHS.GetValue(), 1);
		}
		else
		{
			Reset();
		}

		return *this;
	}

	/** Templates are moveable */
	FMovieSceneTrackImplementationPtr(FMovieSceneTrackImplementationPtr&&) = default;
	FMovieSceneTrackImplementationPtr& operator=(FMovieSceneTrackImplementationPtr&&) = default;

	/** Serialize the template */
	MOVIESCENE_API bool Serialize(FArchive& Ar);
};

template<> struct TStructOpsTypeTraits<FMovieSceneTrackImplementationPtr> : public TStructOpsTypeTraitsBase2<FMovieSceneTrackImplementationPtr>
{
	enum { WithSerializer = true, WithCopy = true };
};
