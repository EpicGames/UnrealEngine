// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "Evaluation/MovieSceneEvaluationKey.h"
#include "Evaluation/MovieSceneEvaluationOperand.h"
#include "Evaluation/MovieSceneSequenceInstanceData.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "MovieSceneFwd.h"
#include "Stats/Stats.h"
#include "Templates/TypeHash.h"
#include "Templates/UniquePtr.h"

class IMovieScenePlayer;

/**
 * Unique identifier for shared persistent data entries (see FSharedPersistentDataKey)
 */
struct FMovieSceneSharedDataId
{
	/**
	 * Allocate a new unique identifier
	 */
	static MOVIESCENE_API FMovieSceneSharedDataId Allocate();

	FMovieSceneSharedDataId(const FMovieSceneSharedDataId&) = default;
	FMovieSceneSharedDataId& operator=(const FMovieSceneSharedDataId&) = default;

	friend bool operator==(FMovieSceneSharedDataId A, FMovieSceneSharedDataId B) { return A.UniqueId == B.UniqueId; }

	friend uint32 GetTypeHash(FMovieSceneSharedDataId In) { return GetTypeHash(In.UniqueId); }

private:
	FMovieSceneSharedDataId() {}
	uint32 UniqueId;
};

/**
 * A key to a piece of data that is potentially shared between multiple tracks
 */
struct FSharedPersistentDataKey
{
	/**
	 * Construction from a shared data ID, and an operand
	 */
	FSharedPersistentDataKey(FMovieSceneSharedDataId InUniqueId, const FMovieSceneEvaluationOperand& InOperand)
		: UniqueId(InUniqueId)
		, Operand(InOperand)
	{}

	friend bool operator==(const FSharedPersistentDataKey& A, const FSharedPersistentDataKey& B)
	{
		return A.UniqueId == B.UniqueId && A.Operand == B.Operand;
	}

	friend uint32 GetTypeHash(const FSharedPersistentDataKey& In)
	{
		return HashCombine(GetTypeHash(In.Operand), GetTypeHash(In.UniqueId));
	}

	/** The actual shared ID */
	FMovieSceneSharedDataId UniqueId;

	/** The operand that this key relates to (may be invalid where the data pertains to root tracks) */
	FMovieSceneEvaluationOperand Operand;
};


/**
 * Interface that must be used for all persistent data objects
 */
struct IPersistentEvaluationData
{
	virtual ~IPersistentEvaluationData()= default;
};


DECLARE_CYCLE_STAT(TEXT("Persistent Data Access"), MovieSceneEval_PersistentData_Access, STATGROUP_MovieSceneEval);

/**
 * Structure that stores persistent data that track templates may need during evaluation.
 * Such data can be thought of as a cache which exists as long as the track is being evaluated.
 * The cache can store any abstract data provided it implements IPersistentEvaluationData.
 * Data is stored in buckets that is keyed on either the track (ie, accessible from all child templates/sections), or section (only accessible within the section)
 * Type-safety (through the templated methods) is the responsibility of the user. There should only ever be 1 type of data for each section/track association.
 */
struct FPersistentEvaluationData
{
	/**
	 * Proxy constructor from 2 externally owned maps for entity, and shared data
	 */
	MOVIESCENE_API FPersistentEvaluationData(IMovieScenePlayer& InPlayer);

	FPersistentEvaluationData(const FPersistentEvaluationData&) = delete;
	FPersistentEvaluationData& operator=(const FPersistentEvaluationData&) = delete;

public:
	/**
	 * User accessor functions for persistent data relating to the current track
	 */
	template<typename T>	   T& 	GetOrAddTrackData()		{ return GetOrAdd<T>(TrackKey); }
	template<typename T>	   T& 	AddTrackData()			{ return Add<T>(TrackKey); }
	template<typename T>	   T& 	GetTrackData()			{ return Get<T>(TrackKey); }
	template<typename T>	   T*	FindTrackData()			{ return Find<T>(TrackKey); }
	 /**~ Section data access is considered const as it can only ever be accessed from a single template (it can do whatever it likes with its own data) */
	template<typename T>	   T& 	GetTrackData() const	{ return Get<T>(TrackKey); }
	template<typename T>	   T*	FindTrackData() const	{ return Find<T>(TrackKey); }
	void 							ResetTrackData()		{ Reset(TrackKey); }

	/**
	 * User accessor functions for persistent data relating to the current section
	 */
	template<typename T>	   T& 	GetOrAddSectionData()	{ return GetOrAdd<T>(SectionKey); }
	template<typename T>	   T& 	AddSectionData()		{ return Add<T>(SectionKey); }
	/**~ Section data access is considered const as it can only ever be accessed from a single template (it can do whatever it likes with its own data) */
	template<typename T>	   T& 	GetSectionData() const	{ return Get<T>(SectionKey); }
	template<typename T>	   T*	FindSectionData() const	{ return Find<T>(SectionKey); }
	void 							ResetSectionData()		{ Reset(SectionKey); }

	/**
	 * Get the raw instance data for the current sequence
	 */
	MOVIESCENE_API const FMovieSceneSequenceInstanceData* GetInstanceData() const;

	/**
	 * Get the player
	 */
	IMovieScenePlayer& GetMovieScenePlayer() const { return Player; }

	/**
	 * Find the current sequence's instance data as the templated type, provided its type matches
	 */
	template<typename T> const T*   FindInstanceData() const
	{
		const FMovieSceneSequenceInstanceData* InstanceData = GetInstanceData();
		if (InstanceData && (&InstanceData->GetScriptStruct() == T::StaticStruct()))
		{
			return static_cast<const T*>(InstanceData);
		}
		return nullptr;
	}

public:
	
	/**
	 * Get the currently set track key (ie the track we're currently evaluating)
	 */
	const FMovieSceneEvaluationKey& GetTrackKey() const
	{
		return TrackKey;
	}

	/**
	 * Get the currently set section key (ie the section we're currently evaluating)
	 */
	const FMovieSceneEvaluationKey& GetSectionKey() const
	{
		return SectionKey;
	}

	/**
	 * Set the current track
	 */
	void SetTrackKey(const FMovieSceneEvaluationKey& Key) const
	{
		TrackKey = Key;
	}

	/**
	 * Set the current section
	 */
	void SetSectionKey(const FMovieSceneEvaluationKey& Key) const
	{
		SectionKey = Key;
	}

	/**
	 * Set the current section based off the current track with the specified section identifier
	 */
	const FMovieSceneEvaluationKey& DeriveSectionKey(uint32 InSectionIdentifier) const
	{
		SectionKey = TrackKey.AsSection(InSectionIdentifier);
		return SectionKey;
	}

public:

	/**
	 * User accessor functions for shared data keys
	 */
	template<typename T>
	T& GetOrAdd(const FSharedPersistentDataKey& InKey)
	{
		MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_PersistentData_Access)

		check(SharedData);

		if (TUniquePtr<IPersistentEvaluationData>* Existing = SharedData->Find(InKey))
		{
			return static_cast<T&>(*Existing->Get());
		}

		return Add<T>(InKey);
	}

	template<typename T>
	T& Add(const FSharedPersistentDataKey& InKey)
	{
		MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_PersistentData_Access)

		check(SharedData);

		T* Ptr = new T;
		SharedData->Add(InKey, TUniquePtr<IPersistentEvaluationData>(Ptr));
		return *Ptr;
	}

	template<typename T>
	T* Find(const FSharedPersistentDataKey& InKey)
	{
		MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_PersistentData_Access)

		if (ensure(SharedData))
		{
			TUniquePtr<IPersistentEvaluationData>* Existing = SharedData->Find(InKey);
			return Existing ? static_cast<T*>(Existing->Get()) : nullptr;
		}
		return nullptr;
	}

	template<typename T>
	const T* Find(const FSharedPersistentDataKey& InKey) const
	{
		MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_PersistentData_Access)

		if (ensure(SharedData))
		{
			const TUniquePtr<IPersistentEvaluationData>* Existing = SharedData->Find(InKey);
			return Existing ? static_cast<const T*>(Existing->Get()) : nullptr;
		}
		return nullptr;
	}

	template<typename T>
	T& Get(const FSharedPersistentDataKey& InKey)
	{
		MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_PersistentData_Access)

		T* Ptr = Find<T>(InKey);
		check(Ptr);
		return *Ptr;
	}

	template<typename T>
	const T& Get(const FSharedPersistentDataKey& InKey) const
	{
		MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_PersistentData_Access)

		const T* Ptr = Find<T>(InKey);
		check(Ptr);
		return *Ptr;
	}

	void Reset(const FSharedPersistentDataKey& InKey)
	{
		MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_PersistentData_Access)
		
		if (ensure(SharedData))
		{
			SharedData->Remove(InKey);
		}
	}

private:

	/** Implementation methods */
	template<typename T>
	T& GetOrAdd(const FMovieSceneEvaluationKey& InKey)
	{
		MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_PersistentData_Access)

		check(EntityData);

		if (TUniquePtr<IPersistentEvaluationData>* Existing = EntityData->Find(InKey))
		{
			return static_cast<T&>(*Existing->Get());
		}

		return Add<T>(InKey);
	}

	template<typename T>
	T& Add(const FMovieSceneEvaluationKey& InKey)
	{
		MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_PersistentData_Access)

		check(EntityData);

		T* Ptr = new T;
		EntityData->Add(InKey, TUniquePtr<IPersistentEvaluationData>(Ptr));
		return *Ptr;
	}

	template<typename T>
	T* Find(const FMovieSceneEvaluationKey& InKey) const
	{
		MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_PersistentData_Access)

		if (ensure(EntityData))
		{
			const TUniquePtr<IPersistentEvaluationData>* Existing = EntityData->Find(InKey);
			return Existing ? static_cast<T*>(Existing->Get()) : nullptr;
		}
		return nullptr;
	}

	template<typename T>
	T& Get(const FMovieSceneEvaluationKey& InKey) const
	{
		MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_PersistentData_Access)

		T* Ptr = Find<T>(InKey);
		check(Ptr);
		return *Ptr;
	}

	void Reset(const FMovieSceneEvaluationKey& InKey)
	{
		MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_PersistentData_Access)
		
		if (ensure(EntityData))
		{
			EntityData->Remove(InKey);
		}
	}

private:
	
	/** The movie scene player */
	IMovieScenePlayer& Player;

	/** Persistent data that's associated with a template entity (such as a track or a section) */
	TMap<FMovieSceneEvaluationKey, TUniquePtr<IPersistentEvaluationData>>* EntityData;

	/** Persistent data that's shared across multiple template entities */
	TMap<FSharedPersistentDataKey, TUniquePtr<IPersistentEvaluationData>>* SharedData;

	// The keys themselves are mutable since this a proxy representation of the data above.
	// For a const FPersistentEvaluationData& we can change the ptr to the persistent data, but we can't change the data itself (as with a const T*)
	mutable FMovieSceneEvaluationKey TrackKey;
	mutable FMovieSceneEvaluationKey SectionKey;
};
