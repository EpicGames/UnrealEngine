// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Map.h"
#include "Containers/Array.h"

namespace UE
{
namespace MovieScene
{

struct FRootInstanceHandle;
struct FRestoreStateParams;
struct FPreAnimatedStateMetaData;
struct FPreAnimatedStorageGroupHandle;
struct FPreAnimatedStorageID;
struct FPreAnimatedStorageIndex;

struct IPreAnimatedCaptureSource
{
	virtual ~IPreAnimatedCaptureSource()= default;

	virtual void Reset() = 0;
	virtual bool ContainsInstanceHandle(FRootInstanceHandle RootInstanceHandle) const = 0;
	virtual void GatherAndRemoveExpiredMetaData(const FRestoreStateParams& Params, TArray<FPreAnimatedStateMetaData>& OutExpiredMetaData) = 0;
	virtual void GatherAndRemoveMetaDataForGroup(FPreAnimatedStorageGroupHandle Group, TArray<FPreAnimatedStateMetaData>& OutExpiredMetaData) = 0;
	virtual void GatherAndRemoveMetaDataForStorage(FPreAnimatedStorageID StorageID, FPreAnimatedStorageIndex StorageIndex, TArray<FPreAnimatedStateMetaData>& OutExpiredMetaData) = 0;
	virtual void GatherAndRemoveMetaDataForRootInstance(FRootInstanceHandle InstanceHandle, TArray<FPreAnimatedStateMetaData>& OutExpiredMetaData) = 0;
};


} // namespace MovieScene
} // namespace UE
