// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "EntitySystem/EntityAllocationIterator.h"
#include "EntitySystem/MovieSceneComponentPtr.h"
#include "EntitySystem/MovieScenePropertyBinding.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateTypes.h"

struct FObjectKey;
struct FMovieSceneEvaluationKey;

class FReferenceCollector;
class FTrackInstancePropertyBindings;
class UMovieSceneEntitySystemLinker;

namespace UE
{
namespace MovieScene
{

struct FRestoreStateParams;
struct FCustomPropertyIndex;
struct FPreAnimatedStateExtension;

struct IPreAnimatedObjectEntityStorage;
struct IPreAnimatedObjectPropertyStorage;

struct IPreAnimatedStorage : TSharedFromThis<IPreAnimatedStorage>
{
	virtual ~IPreAnimatedStorage() = default;

	virtual FPreAnimatedStorageID GetStorageType() const = 0;

	virtual void Initialize(FPreAnimatedStorageID StorageID, FPreAnimatedStateExtension* ParentExtension) {}
	virtual void OnObjectReplaced(FPreAnimatedStorageIndex StorageIndex, const FObjectKey& OldObject, const FObjectKey& NewObject) {}

	virtual EPreAnimatedStorageRequirement RestorePreAnimatedStateStorage(FPreAnimatedStorageIndex StorageIndex, EPreAnimatedStorageRequirement SourceRequirement, EPreAnimatedStorageRequirement TargetRequirement, const FRestoreStateParams& Params) = 0;
	virtual EPreAnimatedStorageRequirement DiscardPreAnimatedStateStorage(FPreAnimatedStorageIndex StorageIndex, EPreAnimatedStorageRequirement SourceRequirement) = 0;

	virtual IPreAnimatedObjectPropertyStorage* AsPropertyStorage() { return nullptr; }
	virtual IPreAnimatedObjectEntityStorage*   AsObjectStorage()   { return nullptr; }

	virtual void AddReferencedObjects(FReferenceCollector& ReferenceCollector) {}
};



struct IPreAnimatedObjectEntityStorage
{
	virtual void BeginTrackingEntities(const FPreAnimatedTrackerParams& Params, TRead<FMovieSceneEntityID> EntityIDs, TRead<FRootInstanceHandle> InstanceHandles, TRead<UObject*> BoundObjects) = 0;
	virtual void BeginTrackingEntity(FMovieSceneEntityID EntityID, bool bWantsRestoreState, FRootInstanceHandle InstanceHandle, UObject* BoundObject) = 0;
	virtual void CachePreAnimatedValues(const FCachePreAnimatedValueParams& Params, TArrayView<UObject* const> BoundObjects) = 0;
};



struct IPreAnimatedStateTokenStorage
{
	virtual void RestoreState(UMovieSceneEntitySystemLinker* Linker, const FMovieSceneEvaluationKey& Key, FRootInstanceHandle InstanceHandle) = 0;
};



struct IPreAnimatedObjectPropertyStorage
{
	using FThreeWayAccessor = TMultiReadOptional<FCustomPropertyIndex, uint16, TSharedPtr<FTrackInstancePropertyBindings>>;

	virtual void BeginTrackingEntities(const FPreAnimatedTrackerParams& Params, TRead<FMovieSceneEntityID> EntityIDs, TRead<FRootInstanceHandle> InstanceHandles, TRead<UObject*> BoundObjects, TRead<FMovieScenePropertyBinding> PropertyBindings) = 0;
	virtual void CachePreAnimatedValues(const FCachePreAnimatedValueParams& Params, FEntityAllocationProxy Item, TRead<UObject*> Objects, TRead<FMovieScenePropertyBinding> PropertyBindings, FThreeWayAccessor Properties) = 0;
};



} // namespace MovieScene
} // namespace UE
