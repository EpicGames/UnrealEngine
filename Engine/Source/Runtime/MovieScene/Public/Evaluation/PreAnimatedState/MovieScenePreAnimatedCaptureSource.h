// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "EntitySystem/TrackInstance/MovieSceneTrackInstance.h"
#include "Evaluation/MovieSceneEvaluationKey.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateTypes.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedCaptureSources.h"
#include "Misc/TVariant.h"
#include "MovieSceneSequenceID.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FMovieScenePreAnimatedState;
class UMovieSceneEntitySystemLinker;
class UMovieSceneTrackInstance;
class UObject;
struct FMovieSceneEvaluationKey;
struct FMovieSceneSequenceID;
struct FMovieSceneTrackInstanceInput;

namespace UE
{
namespace MovieScene
{

struct FPreAnimatedStateExtension;
struct FPreAnimatedStateMetaData;
struct FSharedPlaybackState;

}
}


/**
 * Scoped structure that can be used to wrap a call to SavePreAnimatedState to associate the capture with a specific capture source.
 */
struct FScopedPreAnimatedCaptureSource
{
	using FRootInstanceHandle = UE::MovieScene::FRootInstanceHandle;
	using FSharedPlaybackState = UE::MovieScene::FSharedPlaybackState;

	/**
	 * Construct this capture source from a template (FMovieSceneEvalTemplate) evaluation key, and whether this should restore state when the template is finished
	 */
	MOVIESCENE_API explicit FScopedPreAnimatedCaptureSource(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, const FMovieSceneEvaluationKey& InEvalKey, bool bInWantsRestoreState);

	/**
	 * Construct this capture source from a template (FMovieSceneEvalTemplate) evaluation key, and whether this should restore state when the template is finished
	 */
	MOVIESCENE_API explicit FScopedPreAnimatedCaptureSource(FMovieScenePreAnimatedState* InPreAnimatedState, const FMovieSceneEvaluationKey& InEvalKey, bool bInWantsRestoreState);

	/**
	 * Construct this capture source from an evaluation hook (UMovieSceneEvaluationHookSection), its instance handle, and whether this should restore state when the template is finished
	 */
	MOVIESCENE_API explicit FScopedPreAnimatedCaptureSource(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, const UObject* InEvalHook, FMovieSceneSequenceID InSequenceID, bool bInWantsRestoreState);

	/**
	 * Construct this capture source from an evaluation hook (UMovieSceneEvaluationHookSection), its instance handle, and whether this should restore state when the template is finished
	 */
	MOVIESCENE_API explicit FScopedPreAnimatedCaptureSource(FMovieScenePreAnimatedState* InPreAnimatedState, const UObject* InEvalHook, FMovieSceneSequenceID InSequenceID, bool bInWantsRestoreState);

	/**
	 * Construct this capture source from a track instance (UMovieSceneTrackInstance) and whether this should restore state when the template is finished
	 */
	MOVIESCENE_API explicit FScopedPreAnimatedCaptureSource(UMovieSceneEntitySystemLinker* InLinker, UMovieSceneTrackInstance* InTrackInstance, bool bInWantsRestoreState);

	/**
	 * Construct this capture source from a track instance input (UMovieSceneTrackInstance + FMovieSceneTrackInstanceInput) and whether this should restore state when the template is finished
	 */
	MOVIESCENE_API explicit FScopedPreAnimatedCaptureSource(UMovieSceneEntitySystemLinker* InLinker, const FMovieSceneTrackInstanceInput& TrackInstanceInput);

	FScopedPreAnimatedCaptureSource(const FScopedPreAnimatedCaptureSource&) = delete;
	void operator=(const FScopedPreAnimatedCaptureSource&) = delete;

	FScopedPreAnimatedCaptureSource(FScopedPreAnimatedCaptureSource&&) = delete;
	void operator=(FScopedPreAnimatedCaptureSource&&) = delete;

	MOVIESCENE_API ~FScopedPreAnimatedCaptureSource();

	bool WantsRestoreState() const
	{
		return bWantsRestoreState;
	}

private:

	friend class FMovieSceneEntitySystemRunner;
	friend struct UE::MovieScene::FPreAnimatedStateExtension;

	static FScopedPreAnimatedCaptureSource*& GetCaptureSourcePtr();

	void BeginTracking(const UE::MovieScene::FPreAnimatedStateMetaData& MetaData, UMovieSceneEntitySystemLinker* Linker);
	FRootInstanceHandle GetRootInstanceHandle(UMovieSceneEntitySystemLinker* Linker) const;

	using FPreAnimatedEvaluationKeyType = UE::MovieScene::FPreAnimatedEvaluationKeyType;
	using FPreAnimatedEvalHookKeyType = UE::MovieScene::FPreAnimatedEvalHookKeyType;
	using CaptureSourceType = TVariant<FPreAnimatedEvaluationKeyType, FPreAnimatedEvalHookKeyType, UMovieSceneTrackInstance*, FMovieSceneTrackInstanceInput>;

	CaptureSourceType Variant;
	FRootInstanceHandle RootInstanceHandle;
	TWeakObjectPtr<UMovieSceneEntitySystemLinker> WeakLinker;
	FScopedPreAnimatedCaptureSource* PrevCaptureSource;
	bool bWantsRestoreState;
};
