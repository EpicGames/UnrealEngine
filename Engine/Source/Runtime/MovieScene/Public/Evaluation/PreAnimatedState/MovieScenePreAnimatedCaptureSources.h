// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "CoreTypes.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "EntitySystem/TrackInstance/MovieSceneTrackInstance.h"
#include "Evaluation/IMovieSceneEvaluationHook.h"
#include "Evaluation/MovieSceneEvaluationKey.h"
#include "Evaluation/PreAnimatedState/IMovieScenePreAnimatedCaptureSource.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateTypes.h"
#include "UObject/ObjectKey.h"

class UMovieSceneTrackInstance;
class UObject;

namespace UE
{
namespace MovieScene
{

struct FPreAnimatedStateExtension;
struct FRestoreStateParams;

enum class EPreAnimatedCaptureSourceState
{
	// This is a new capture source
	New,
	// This capture source needed to be updated due to its capture requirements changing
	Updated,
	// The capture source has already been encountered and does not need updating
	UpToDate,
};

/** Key type for pre-animated state associated to evaluation hooks */
struct FPreAnimatedEvalHookKeyType
{
	FObjectKey Hook;
	FRootInstanceHandle RootInstanceHandle;
	FMovieSceneSequenceID SequenceID;

	friend uint32 GetTypeHash(const FPreAnimatedEvalHookKeyType& In)
	{
		return GetTypeHash(In.Hook) ^ GetTypeHash(In.RootInstanceHandle) ^ GetTypeHash(In.SequenceID);
	}
	friend bool operator==(const FPreAnimatedEvalHookKeyType& A, const FPreAnimatedEvalHookKeyType& B)
	{
		return A.Hook == B.Hook && A.RootInstanceHandle == B.RootInstanceHandle && A.SequenceID == B.SequenceID;
	}
};

/** Key type for pre-animated state associated to evaluation templates */
struct FPreAnimatedEvaluationKeyType
{
	FMovieSceneEvaluationKey EvaluationKey;
	FRootInstanceHandle RootInstanceHandle;

	friend uint32 GetTypeHash(const FPreAnimatedEvaluationKeyType& In)
	{
		return GetTypeHash(In.EvaluationKey) ^ GetTypeHash(In.RootInstanceHandle);
	}
	friend bool operator==(const FPreAnimatedEvaluationKeyType& A, const FPreAnimatedEvaluationKeyType& B)
	{
		return A.EvaluationKey == B.EvaluationKey && A.RootInstanceHandle == B.RootInstanceHandle; 
	}
};

/**
 * Structure responsible for tracking contributions to pre-animated state entries that originate from track templates (ie, from an IMovieSceneExecutionToken::Execute)
 */
template<typename KeyType>
struct TPreAnimatedCaptureSources : IPreAnimatedCaptureSource
{
	TPreAnimatedCaptureSources(FPreAnimatedStateExtension* InOwner);

	/**
	 * Make an association for the specified evaluation key to the value specified by Entry,
	 * along with an indication of whether it should be restored on completion
	 */
	MOVIESCENE_API EPreAnimatedCaptureSourceState BeginTrackingCaptureSource(const KeyType& InKey, const FPreAnimatedStateMetaData& MetaData);

	/**
	 * Stop tracking the evaluation key in question, where associated with the given storage value ID
	 * This restores the value if it was captured with bWantsRestoreState
	 */
	MOVIESCENE_API void StopTrackingCaptureSource(const KeyType& InKey, FPreAnimatedStorageID InStorageID);

	/**
	 * Stop tracking the evaluation key in question, restoring the value if it was captured with bWantsRestoreState
	 */
	MOVIESCENE_API void StopTrackingCaptureSource(const KeyType& InKey);

	// IPreAnimatedCaptureSource members
	MOVIESCENE_API void Reset() override;
	MOVIESCENE_API bool ContainsInstanceHandle(FRootInstanceHandle RootInstanceHandle) const override;
	MOVIESCENE_API void GatherAndRemoveExpiredMetaData(const FRestoreStateParams& Params, TArray<FPreAnimatedStateMetaData>& OutExpiredMetaData) override;
	MOVIESCENE_API void GatherAndRemoveMetaDataForGroup(FPreAnimatedStorageGroupHandle Group, TArray<FPreAnimatedStateMetaData>& OutExpiredMetaData) override;
	MOVIESCENE_API void GatherAndRemoveMetaDataForStorage(FPreAnimatedStorageID StorageID, FPreAnimatedStorageIndex StorageIndex, TArray<FPreAnimatedStateMetaData>& OutExpiredMetaData) override;
	MOVIESCENE_API void GatherAndRemoveMetaDataForRootInstance(FRootInstanceHandle InstanceHandle, TArray<FPreAnimatedStateMetaData>& OutExpiredMetaData) override;

private:

	TMap<KeyType, FPreAnimatedStateMetaDataArray> KeyToMetaData;
	FPreAnimatedStateExtension* Owner;
};


/**
 * Structure responsible for tracking contributions to pre-animated state entries that originate from ECS data (ie, from FMovieSceneEntityIDs)
 */
struct FPreAnimatedEntityCaptureSource : TPreAnimatedCaptureSources<FMovieSceneEntityID>
{
	MOVIESCENE_API FPreAnimatedEntityCaptureSource(FPreAnimatedStateExtension* InOwner);

	MOVIESCENE_API void BeginTrackingEntity(const FPreAnimatedStateEntry& Entry, FMovieSceneEntityID EntityID, FRootInstanceHandle RootInstanceHandle, bool bWantsRestoreState);

	void StopTrackingEntity(FMovieSceneEntityID EntityID, FPreAnimatedStorageID StorageID)
	{
		StopTrackingCaptureSource(EntityID, StorageID);
	}

	void StopTrackingEntity(FMovieSceneEntityID EntityID)
	{
		StopTrackingCaptureSource(EntityID);
	}
};

/**
 * Structure responsible for tracking contributions to pre-eanimated state entries that originate from track templates (ie, from an IMovieSceneExecutionToken::Execute)
 */
struct FPreAnimatedTemplateCaptureSources : TPreAnimatedCaptureSources<FPreAnimatedEvaluationKeyType>
{
	MOVIESCENE_API FPreAnimatedTemplateCaptureSources(FPreAnimatedStateExtension* InOwner);

	/**
	 * Make an association for the specified evaluation key to the value specified by Entry,
	 * along with an indication of whether it should be restored on completion
	 */
	MOVIESCENE_API EPreAnimatedCaptureSourceState BeginTrackingCaptureSource(const FMovieSceneEvaluationKey& EvaluationKey, const FPreAnimatedStateMetaData& MetaData);

	/**
	 * Stop tracking the evaluation key in question, restoring the value if it was captured with bWantsRestoreState
	 */
	MOVIESCENE_API void StopTrackingCaptureSource(const FMovieSceneEvaluationKey& EvaluationKey, FRootInstanceHandle RootInstanceHandle);

	// Un-shadow base class methods (it's OK, they're not virtual)
	using TPreAnimatedCaptureSources<FPreAnimatedEvaluationKeyType>::BeginTrackingCaptureSource;
	using TPreAnimatedCaptureSources<FPreAnimatedEvaluationKeyType>::StopTrackingCaptureSource;
};

/**
 * Structure responsible for tracking contributions to pre-eanimated state entries that originate from track EvaluationHooks (ie, from an IMovieSceneExecutionToken::Execute)
 */
struct FPreAnimatedEvaluationHookCaptureSources : TPreAnimatedCaptureSources<FPreAnimatedEvalHookKeyType>
{
	MOVIESCENE_API FPreAnimatedEvaluationHookCaptureSources(FPreAnimatedStateExtension* InOwner);

	/**
	 * Make an association for the specified evaluation key to the value specified by Entry,
	 * along with an indication of whether it should be restored on completion
	 */
	MOVIESCENE_API EPreAnimatedCaptureSourceState BeginTrackingCaptureSource(const UObject* Hook, FMovieSceneSequenceID SequenceID, const FPreAnimatedStateMetaData& MetaData);

	/**
	 * Stop tracking the evaluation key in question, restoring the value if it was captured with bWantsRestoreState
	 */
	MOVIESCENE_API void StopTrackingCaptureSource(const UObject* Hook, FRootInstanceHandle RootInstanceHandle, FMovieSceneSequenceID SequenceID);

	// Un-shadow base class methods (it's OK, they're not virtual)
	using TPreAnimatedCaptureSources<FPreAnimatedEvalHookKeyType>::BeginTrackingCaptureSource;
	using TPreAnimatedCaptureSources<FPreAnimatedEvalHookKeyType>::StopTrackingCaptureSource;
};


/**
 * Structure responsible for tracking contributions to pre-eanimated state entries that originate from specific track template inputs
 */
struct FPreAnimatedTrackInstanceCaptureSources : TPreAnimatedCaptureSources<FObjectKey>
{
	MOVIESCENE_API FPreAnimatedTrackInstanceCaptureSources(FPreAnimatedStateExtension* InOwner);

	/**
	 * Make an association for the specified evaluation key to the value specified by Entry,
	 * along with an indication of whether it should be restored on completion
	 */
	MOVIESCENE_API EPreAnimatedCaptureSourceState BeginTrackingCaptureSource(UMovieSceneTrackInstance* TrackInstance, const FPreAnimatedStateMetaData& MetaData);

	/**
	 * Stop tracking the evaluation key in question, restoring the value if it was captured with bWantsRestoreState
	 */
	MOVIESCENE_API void StopTrackingCaptureSource(UMovieSceneTrackInstance* TrackInstance);
};


/**
 * Structure responsible for tracking contributions to pre-eanimated state entries that originate from track templates (ie, from an IMovieSceneExecutionToken::Execute)
 */
struct FPreAnimatedTrackInstanceInputCaptureSources : TPreAnimatedCaptureSources<FMovieSceneTrackInstanceInput>
{
	MOVIESCENE_API FPreAnimatedTrackInstanceInputCaptureSources(FPreAnimatedStateExtension* InOwner);
};



} // namespace MovieScene
} // namespace UE
