// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedCaptureSources.h"

UE_DEPRECATED_HEADER(5.4, "FPreAnimatedEntityCaptureSource has been moved to MovieScenePreAnimatedCaptureSources.h")

