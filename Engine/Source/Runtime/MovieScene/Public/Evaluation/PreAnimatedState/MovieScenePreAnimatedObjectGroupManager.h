// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreTypes.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateExtension.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.h"
#include "UObject/ObjectKey.h"

class UClass;
class UObject;

namespace UE
{
namespace MovieScene
{
struct FPreAnimatedStorageGroupHandle;
template <typename StorageType> struct TAutoRegisterPreAnimatedStorageID;

struct FPreAnimatedObjectGroupManager : TPreAnimatedStateGroupManager<FObjectKey>
{
	static MOVIESCENE_API TAutoRegisterPreAnimatedStorageID<FPreAnimatedObjectGroupManager> GroupManagerID;

	MOVIESCENE_API void OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap);

	MOVIESCENE_API void GetGroupsByClass(UClass* GeneratedClass, TArray<FPreAnimatedStorageGroupHandle>& OutGroupHandles);

	MOVIESCENE_API void GatherStaleStorageGroups(TArray<FPreAnimatedStorageGroupHandle>& StaleGroupStorage) const override;
};


} // namespace MovieScene
} // namespace UE
