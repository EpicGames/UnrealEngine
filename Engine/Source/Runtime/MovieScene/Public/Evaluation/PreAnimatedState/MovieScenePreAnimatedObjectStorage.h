// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Evaluation/PreAnimatedState/MovieSceneRestoreStateParams.h"
#include "Evaluation/PreAnimatedState/IMovieScenePreAnimatedStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedObjectGroupManager.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedCaptureSources.h"
#include "EntitySystem/BuiltInComponentTypes.h"


namespace UE
{
namespace MovieScene
{

/**
 * Group state class that groups pre-animated storage together by bound object
 *
 * Inherit from this class by implementing the following members:
 *
 *	 using KeyType     = FObjectKey;
 *	 using StorageType = <your storage type>;
 *	 StorageType CachePreAnimatedValue(const FObjectKey& Object);
 *	 void RestorePreAnimatedValue(const FObjectKey& Object, StorageType& InOutCachedValue, const FRestoreStateParams& Params);
 */
struct FBoundObjectPreAnimatedStateTraits : FPreAnimatedStateTraits
{
	enum { NeedsInitialize = true, SupportsGrouping = true, SupportsReplaceObject = true };

	MOVIESCENE_API void Initialize(FPreAnimatedStorageID InStorageID, FPreAnimatedStateExtension* InParentExtension);

	template<typename T>
	T ResolveComponent(T&& In)
	{
		return Forward<T>(In);
	}

	/* Defined as a template rather than a variadic function to prevent error C4840 */
	template<typename... T>
	FPreAnimatedStorageGroupHandle FindGroup(UObject* BoundObject, T&&... Unused)
	{
		return FindGroupImpl(BoundObject);
	}
	template<typename... T>
	FPreAnimatedStorageGroupHandle FindGroup(const FObjectComponent& BoundObject, T&&... Unused)
	{
		return FindGroupImpl(BoundObject);
	}

	/* Defined as a template rather than a variadic function to prevent error C4840 */
	template<typename... T>
	FPreAnimatedStorageGroupHandle MakeGroup(UObject* BoundObject, T&&... Unused)
	{
		return MakeGroupImpl(BoundObject);
	}
	template<typename... T>
	FPreAnimatedStorageGroupHandle MakeGroup(const FObjectComponent& BoundObject, T&&... Unused)
	{
		return MakeGroupImpl(BoundObject);
	}

	MOVIESCENE_API FPreAnimatedStorageGroupHandle FindGroupImpl(UObject* BoundObject);
	MOVIESCENE_API FPreAnimatedStorageGroupHandle FindGroupImpl(const FObjectComponent& BoundObject);

	MOVIESCENE_API FPreAnimatedStorageGroupHandle MakeGroupImpl(UObject* BoundObject);
	MOVIESCENE_API FPreAnimatedStorageGroupHandle MakeGroupImpl(const FObjectComponent& BoundObject);

	template<typename ...T>
	void ReplaceObject(TTuple<FObjectKey, T...>& InOutKey, const FObjectKey& NewObject)
	{
		InOutKey.template Get<0>() = NewObject;
	}
	template<typename KeyType>
	void ReplaceObject(KeyType& InOutKey, const FObjectKey& NewObject)
	{
		InOutKey.Object = NewObject;
	}
	template<typename ObjectType>
	void ReplaceObject(TObjectKey<ObjectType>& InOutKey, const FObjectKey& NewObject)
	{
		if (ObjectType* CastResult = Cast<ObjectType>(NewObject.ResolveObjectPtr()))
		{
			InOutKey = CastResult;
		}
	}
	void ReplaceObject(FObjectKey& InOutKey, const FObjectKey& NewObject)
	{
		InOutKey = NewObject;
	}
	TSharedPtr<FPreAnimatedObjectGroupManager> ObjectGroupManager;
};

template<typename ObjectTraits>
struct TPreAnimatedStateStorage_ObjectTraits
	: TPreAnimatedStateStorage<ObjectTraits>
	, IPreAnimatedObjectEntityStorage
{
	using KeyType = typename ObjectTraits::KeyType;
	using StorageType = typename ObjectTraits::StorageType;

	static_assert(ObjectTraits::SupportsGrouping, "Pre-animated object state storage should support grouping by object");

	TPreAnimatedStateStorage_ObjectTraits()
	{}

public:

	IPreAnimatedObjectEntityStorage* AsObjectStorage() override
	{
		return this;
	}

	void BeginTrackingEntities(const FPreAnimatedTrackerParams& Params, TRead<FMovieSceneEntityID> EntityIDs, TRead<FRootInstanceHandle> InstanceHandles, TRead<UObject*> BoundObjects) override
	{
		const int32 Num = Params.Num;
		const bool  bWantsRestore = Params.bWantsRestoreState;

		if (!this->ParentExtension->IsCapturingGlobalState() && !bWantsRestore)
		{
			return;
		}

		FPreAnimatedEntityCaptureSource* EntityMetaData = this->ParentExtension->GetOrCreateEntityMetaData();
		for (int32 Index = 0; Index < Num; ++Index)
		{
			UObject* BoundObject = BoundObjects[Index];
			KeyType Key{ BoundObject };

			FPreAnimatedStorageGroupHandle GroupHandle  = this->Traits.MakeGroup(BoundObject);
			FPreAnimatedStorageIndex       StorageIndex = this->GetOrCreateStorageIndex(Key);

			FPreAnimatedStateEntry Entry{ GroupHandle, FPreAnimatedStateCachedValueHandle{ this->StorageID, StorageIndex } };
			EntityMetaData->BeginTrackingEntity(Entry, EntityIDs[Index], InstanceHandles[Index], bWantsRestore);
		}
	}

	FPreAnimatedStateEntry MakeEntry(UObject* BoundObject)
	{
		KeyType Key{ BoundObject };
	
		FPreAnimatedStorageIndex       StorageIndex = this->GetOrCreateStorageIndex(Key);
		FPreAnimatedStorageGroupHandle GroupHandle  = this->Traits.MakeGroup(BoundObject);
		return FPreAnimatedStateEntry{ GroupHandle, FPreAnimatedStateCachedValueHandle{ this->StorageID, StorageIndex } };
	}

	void BeginTrackingEntity(FMovieSceneEntityID EntityID, bool bWantsRestoreState, FRootInstanceHandle RootInstanceHandle, UObject* BoundObject) override
	{
		if (!this->ParentExtension->IsCapturingGlobalState() && !bWantsRestoreState)
		{
			return;
		}

		FPreAnimatedEntityCaptureSource* EntityMetaData = this->ParentExtension->GetOrCreateEntityMetaData();

		FPreAnimatedStateEntry Entry = MakeEntry(BoundObject);
		EntityMetaData->BeginTrackingEntity(Entry, EntityID, RootInstanceHandle, bWantsRestoreState);
	}

	void CachePreAnimatedValues(const FCachePreAnimatedValueParams& Params, TArrayView<UObject* const> BoundObjects) override
	{
		for (UObject* BoundObject : BoundObjects)
		{
			if (BoundObject)
			{
				CachePreAnimatedValue(Params, BoundObject);
			}
		}
	}

	void CachePreAnimatedValue(const FCachePreAnimatedValueParams& Params, UObject* BoundObject, EPreAnimatedCaptureSourceTracking TrackingMode = EPreAnimatedCaptureSourceTracking::CacheIfTracked)
	{
		if (this->ShouldTrackCaptureSource(TrackingMode, BoundObject))
		{
			FPreAnimatedStateEntry Entry = MakeEntry(BoundObject);

			this->TrackCaptureSource(Entry, TrackingMode);

			CachePreAnimatedValue(Params, Entry, BoundObject);
		}
	}
	
	void CachePreAnimatedValue(const FCachePreAnimatedValueParams& Params, const FPreAnimatedStateEntry& Entry, UObject* BoundObject)
	{
		const FPreAnimatedStorageIndex StorageIndex = Entry.ValueHandle.StorageIndex;
	
		EPreAnimatedStorageRequirement StorageRequirement = this->ParentExtension->GetStorageRequirement(Entry);
		if (!this->IsStorageRequirementSatisfied(StorageIndex, StorageRequirement))
		{
			StorageType NewValue = this->Traits.CachePreAnimatedValue(BoundObject);
			this->AssignPreAnimatedValue(StorageIndex, StorageRequirement, MoveTemp(NewValue));
		}
	
		if (Params.bForcePersist)
		{
			this->ForciblyPersistStorage(StorageIndex);
		}
	}

	template<typename OnCacheValue /* StorageType(const KeyType&) */>
	void CachePreAnimatedValue(const FCachePreAnimatedValueParams& Params, UObject* BoundObject, OnCacheValue&& CacheCallback, EPreAnimatedCaptureSourceTracking TrackingMode = EPreAnimatedCaptureSourceTracking::CacheIfTracked)
	{
		if (this->ShouldTrackCaptureSource(TrackingMode, BoundObject))
		{
			FPreAnimatedStateEntry Entry = MakeEntry(BoundObject);

			this->TrackCaptureSource(Entry, TrackingMode);
			const FPreAnimatedStorageIndex StorageIndex = Entry.ValueHandle.StorageIndex;

			EPreAnimatedStorageRequirement StorageRequirement = this->ParentExtension->GetStorageRequirement(Entry);
			if (!this->IsStorageRequirementSatisfied(StorageIndex, StorageRequirement))
			{
				KeyType Key{ BoundObject };
				StorageType NewValue = CacheCallback(Key);
				this->AssignPreAnimatedValue(StorageIndex, StorageRequirement, MoveTemp(NewValue));
			}

			if (Params.bForcePersist)
			{
				this->ForciblyPersistStorage(StorageIndex);
			}
		}
	}
};

} // namespace MovieScene
} // namespace UE

