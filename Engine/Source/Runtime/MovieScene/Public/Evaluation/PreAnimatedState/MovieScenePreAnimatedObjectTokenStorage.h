// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "Evaluation/MovieSceneAnimTypeID.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedObjectGroupManager.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedObjectStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateExtension.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateTypes.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.h"
#include "Evaluation/PreAnimatedState/MovieSceneRestoreStateParams.h"
#include "Misc/InlineValue.h"
#include "MovieSceneExecutionToken.h"
#include "Templates/Tuple.h"
#include "UObject/ObjectKey.h"

class UObject;


namespace UE
{
namespace MovieScene
{
struct FRestoreStateParams;
template <typename StorageType> struct TAutoRegisterPreAnimatedStorageID;


struct FPreAnimatedObjectTokenTraits : FBoundObjectPreAnimatedStateTraits
{
	using KeyType     = TTuple<FObjectKey, FMovieSceneAnimTypeID>;
	using StorageType = IMovieScenePreAnimatedTokenPtr;

	static void RestorePreAnimatedValue(const KeyType& InKey, IMovieScenePreAnimatedTokenPtr& Token, const FRestoreStateParams& Params)
	{
		if (UObject* Object = InKey.Get<0>().ResolveObjectPtr())
		{
			Token->RestoreState(*Object, Params);
		}
	}
};


struct FAnimTypePreAnimatedStateObjectStorage : TPreAnimatedStateStorage<FPreAnimatedObjectTokenTraits>
{
	static MOVIESCENE_API TAutoRegisterPreAnimatedStorageID<FAnimTypePreAnimatedStateObjectStorage> StorageID;

	MOVIESCENE_API FPreAnimatedStateEntry FindEntry(UObject* Object, FMovieSceneAnimTypeID AnimTypeID);
	MOVIESCENE_API FPreAnimatedStateEntry MakeEntry(UObject* Object, FMovieSceneAnimTypeID AnimTypeID);
};


} // namespace MovieScene
} // namespace UE






