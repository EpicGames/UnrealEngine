// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Evaluation/MovieSceneEvaluationKey.h"
#include "UObject/ObjectKey.h"
#include "UObject/Object.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieScenePropertySystemTypes.h"
#include "EntitySystem/MovieScenePropertyRegistry.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieScenePropertyBinding.h"
#include "Evaluation/PreAnimatedState/MovieSceneRestoreStateParams.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.inl"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedObjectStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedCaptureSources.h"


namespace UE
{
namespace MovieScene
{


/**
 * Pre-animated property value, including cached meta-data
 */
template<typename StorageType, typename ...MetaDataTypes>
struct TPreAnimatedPropertyValue
{
	TPreAnimatedPropertyValue()
	{}

	TPreAnimatedPropertyValue(typename TCallTraits<MetaDataTypes>::ParamType... InMetaData)
		: MetaData(InMetaData...)
	{}

	StorageType Data;
	TVariant<const FCustomPropertyAccessor*, uint16, TSharedPtr<FTrackInstancePropertyBindings>> Binding;
	TTuple<MetaDataTypes...> MetaData;
};


/**
 * Pre-animated property value, specialized for no meta-data
 */
template<typename StorageType>
struct TPreAnimatedPropertyValue<StorageType>
{
	StorageType Data;
	TVariant<const FCustomPropertyAccessor*, uint16, TSharedPtr<FTrackInstancePropertyBindings>> Binding;
};


/**
 * Pre-Animated traits class that wraps a user-provided property trait that defines property accessors 
 */
template<typename PropertyTraits, typename MetaDataIndices, typename ...MetaDataTypes>
struct TPreAnimatedPropertyTraits;

template<typename PropertyTraits, int... MetaDataIndices, typename ...MetaDataTypes>
struct TPreAnimatedPropertyTraits<PropertyTraits, TIntegerSequence<int, MetaDataIndices...>, MetaDataTypes...> : FBoundObjectPreAnimatedStateTraits
{
	using KeyType = TTuple<FObjectKey, FName>;
	using StorageType = TPreAnimatedPropertyValue<typename PropertyTraits::StorageType, MetaDataTypes...>;

	const PropertyTraits* PropertyTraitsInstance;

	TPreAnimatedPropertyTraits(const PropertyTraits* InPropertyTraitsInstance)
		: PropertyTraitsInstance(InPropertyTraitsInstance)
	{}

	void RestorePreAnimatedValue(const KeyType& InKey, StorageType& CachedValue, const FRestoreStateParams& Params)
	{
		UObject* Object = InKey.Get<0>().ResolveObjectPtr();
		if (!Object)
		{
			return;
		}

		if (const uint16* FastOffset = CachedValue.Binding.template TryGet<uint16>())
		{
			PropertyTraitsInstance->SetObjectPropertyValue(Object, CachedValue.MetaData.template Get<MetaDataIndices>()..., *FastOffset, CachedValue.Data);
		}
		else  if (const TSharedPtr<FTrackInstancePropertyBindings>* Bindings = CachedValue.Binding.template TryGet<TSharedPtr<FTrackInstancePropertyBindings>>())
		{
			PropertyTraitsInstance->SetObjectPropertyValue(Object, CachedValue.MetaData.template Get<MetaDataIndices>()..., Bindings->Get(), CachedValue.Data);
		}
		else if (const FCustomPropertyAccessor* CustomAccessor = CachedValue.Binding.template Get<const FCustomPropertyAccessor*>())
		{
			PropertyTraitsInstance->SetObjectPropertyValue(Object, CachedValue.MetaData.template Get<MetaDataIndices>()..., *CustomAccessor, CachedValue.Data);
		}
	}
};



/**
 * Pre-Animated traits class that wraps a user-provided property trait that defines property accessors with no meta-data
 */
template<typename PropertyTraits>
struct TPreAnimatedPropertyTraits<PropertyTraits, TPropertyMetaData<>, TIntegerSequence<int>> : FBoundObjectPreAnimatedStateTraits
{
	using KeyType = TTuple<FObjectKey, FName>;
	using StorageType = TPreAnimatedPropertyValue<typename PropertyTraits::StorageType>;

	const PropertyTraits* PropertyTraitsInstance;

	TPreAnimatedPropertyTraits(const PropertyTraits* InPropertyTraitsInstance)
		: PropertyTraitsInstance(InPropertyTraitsInstance)
	{}

	void RestorePreAnimatedValue(const KeyType& InKey, StorageType& CachedValue, const FRestoreStateParams& Params)
	{
		UObject* Object = InKey.Get<0>().ResolveObjectPtr();
		if (!Object)
		{
			return;
		}

		if (const uint16* FastOffset = CachedValue.Binding.template TryGet<uint16>())
		{
			PropertyTraitsInstance->SetObjectPropertyValue(Object, *FastOffset, CachedValue.Data);
		}
		else  if (const TSharedPtr<FTrackInstancePropertyBindings>* Bindings = CachedValue.Binding.template TryGet<TSharedPtr<FTrackInstancePropertyBindings>>())
		{
			PropertyTraitsInstance->SetObjectPropertyValue(Object, Bindings->Get(), CachedValue.Data);
		}
		else if (const FCustomPropertyAccessor* CustomAccessor = CachedValue.Binding.template Get<const FCustomPropertyAccessor*>())
		{
			PropertyTraitsInstance->SetObjectPropertyValue(Object, *CustomAccessor, CachedValue.Data);
		}
	}
};


template<typename PropertyTraits, typename MetaDataTypes, typename MetaDataIndices>
struct TPreAnimatedPropertyStorageImpl;

template<typename PropertyTraits, typename ...MetaDataTypes, int ...MetaDataIndices>
struct TPreAnimatedPropertyStorageImpl<PropertyTraits, TPropertyMetaData<MetaDataTypes...>, TIntegerSequence<int, MetaDataIndices...>>
	: TPreAnimatedStateStorage<TPreAnimatedPropertyTraits<PropertyTraits, TIntegerSequence<int, MetaDataIndices...>, MetaDataTypes...>>
	, IPreAnimatedObjectPropertyStorage
{
	using StorageTraits = TPreAnimatedPropertyTraits<PropertyTraits, TIntegerSequence<int, MetaDataIndices...>, MetaDataTypes...>;
	using StorageType = typename StorageTraits::StorageType;

	static_assert(StorageTraits::SupportsGrouping, "Pre-animated storage for properties should support grouping by object");

	TPreAnimatedPropertyStorageImpl(const FPropertyDefinition& InPropertyDefinition)
		: TPreAnimatedStateStorage<StorageTraits>(static_cast<const PropertyTraits*>(InPropertyDefinition.TraitsInstance))
		, PropertyTraitsInstance(static_cast<const PropertyTraits*>(InPropertyDefinition.TraitsInstance))
		, MetaDataComponents(InPropertyDefinition.MetaDataTypes)
	{
		check(MetaDataComponents.Num() == sizeof...(MetaDataIndices));

		if (InPropertyDefinition.CustomPropertyRegistration)
		{
			CustomAccessors = InPropertyDefinition.CustomPropertyRegistration->GetAccessors();
		}
	}

	IPreAnimatedObjectPropertyStorage* AsPropertyStorage() override
	{
		return this;
	}

	void BeginTrackingEntities(const FPreAnimatedTrackerParams& Params, TRead<FMovieSceneEntityID> EntityIDs, TRead<FRootInstanceHandle> InstanceHandles, TRead<UObject*> BoundObjects, TRead<FMovieScenePropertyBinding> PropertyBindings) override
	{
		const int32 Num = Params.Num;
		const bool  bWantsRestore = Params.bWantsRestoreState;

		if (!this->ParentExtension->IsCapturingGlobalState() && !bWantsRestore)
		{
			return;
		}

		FPreAnimatedEntityCaptureSource* EntityMetaData = this->ParentExtension->GetOrCreateEntityMetaData();
		for (int32 Index = 0; Index < Num; ++Index)
		{
			UObject* BoundObject  = BoundObjects[Index];
			if (!BoundObject)
			{
				continue;
			}

			FPreAnimatedStateEntry Entry = this->MakeEntry(BoundObject, PropertyBindings[Index].PropertyPath);
			EntityMetaData->BeginTrackingEntity(Entry, EntityIDs[Index], InstanceHandles[Index], bWantsRestore);
		}
	}

	void CachePreAnimatedValues(const FCachePreAnimatedValueParams& Params, FEntityAllocationProxy Item, TRead<UObject*> BoundObjects, TRead<FMovieScenePropertyBinding> PropertyBindings, FThreeWayAccessor Properties) override
	{
		const FEntityAllocation* Allocation = Item.GetAllocation();

		TTuple< TComponentReader<MetaDataTypes>... > MetaData(
			Allocation->ReadComponents(MetaDataComponents[MetaDataIndices].template ReinterpretCast<MetaDataTypes>())...
			);

		const uint16* Fast = Properties.Get<1>();
		const FCustomPropertyIndex* Custom = Properties.Get<0>();
		const TSharedPtr<FTrackInstancePropertyBindings>* Slow = Properties.Get<2>();

		const int32 Num = Allocation->Num();
		for (int32 Index = 0; Index < Num; ++Index)
		{
			UObject* BoundObject  = BoundObjects[Index];
			if (!BoundObject)
			{
				continue;
			}

			if (!this->ShouldTrackCaptureSource(EPreAnimatedCaptureSourceTracking::CacheIfTracked, BoundObject, PropertyBindings[Index].PropertyPath))
			{
				continue;
			}

			FPreAnimatedStateEntry Entry = this->MakeEntry(BoundObject, PropertyBindings[Index].PropertyPath);

			this->TrackCaptureSource(Entry, EPreAnimatedCaptureSourceTracking::CacheIfTracked);

			EPreAnimatedStorageRequirement StorageRequirement = this->ParentExtension->GetStorageRequirement(Entry);
			if (!this->IsStorageRequirementSatisfied(Entry.ValueHandle.StorageIndex, StorageRequirement))
			{
				StorageType NewValue(MetaData.template Get<MetaDataIndices>()[Index]...);

				if (Fast)
				{
					NewValue.Binding.template Set<uint16>(Fast[Index]);
					PropertyTraitsInstance->GetObjectPropertyValue(BoundObjects[Index], MetaData.template Get<MetaDataIndices>()[Index]..., Fast[Index], NewValue.Data);
				}
				else if (Custom)
				{
					const FCustomPropertyAccessor& Accessor = this->CustomAccessors[Custom[Index].Value];

					NewValue.Binding.template Set<const FCustomPropertyAccessor*>(&Accessor);
					PropertyTraitsInstance->GetObjectPropertyValue(BoundObjects[Index], MetaData.template Get<MetaDataIndices>()[Index]..., Accessor, NewValue.Data);
				}
				else if (Slow)
				{
					const TSharedPtr<FTrackInstancePropertyBindings>& Bindings = Slow[Index];

					NewValue.Binding.template Set<TSharedPtr<FTrackInstancePropertyBindings>>(Bindings);
					PropertyTraitsInstance->GetObjectPropertyValue(BoundObjects[Index], MetaData.template Get<MetaDataIndices>()[Index]..., Bindings.Get(), NewValue.Data);
				}

				this->AssignPreAnimatedValue(Entry.ValueHandle.StorageIndex, StorageRequirement, MoveTemp(NewValue));
			}

			if (Params.bForcePersist)
			{
				this->ForciblyPersistStorage(Entry.ValueHandle.StorageIndex);
			}
		}
	}

protected:

	const PropertyTraits* PropertyTraitsInstance;
	TArrayView<const FComponentTypeID> MetaDataComponents;
	FCustomAccessorView CustomAccessors;
};


template<typename PropertyTraits>
using TPreAnimatedPropertyStorage = TPreAnimatedPropertyStorageImpl<PropertyTraits, typename PropertyTraits::MetaDataType, TMakeIntegerSequence<int, PropertyTraits::MetaDataType::Num>>;






} // namespace MovieScene
} // namespace UE






