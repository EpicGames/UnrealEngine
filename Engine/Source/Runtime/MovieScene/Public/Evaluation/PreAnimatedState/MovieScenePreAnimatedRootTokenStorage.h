// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "Evaluation/MovieSceneAnimTypeID.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateExtension.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateTypes.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.h"
#include "Evaluation/PreAnimatedState/MovieSceneRestoreStateParams.h"
#include "Misc/InlineValue.h"
#include "MovieSceneExecutionToken.h"


namespace UE
{
namespace MovieScene
{
struct FPreAnimatedStateExtension;
struct FRestoreStateParams;

struct FPreAnimatedRootTokenTraits : FPreAnimatedStateTraits
{
	using KeyType     = FMovieSceneAnimTypeID;
	using StorageType = IMovieScenePreAnimatedGlobalTokenPtr;

	static void RestorePreAnimatedValue(FMovieSceneAnimTypeID, IMovieScenePreAnimatedGlobalTokenPtr& Token, const FRestoreStateParams& Params)
	{
		Token->RestoreState(Params);
	}
};

struct FAnimTypePreAnimatedStateRootStorage : TPreAnimatedStateStorage<FPreAnimatedRootTokenTraits>
{
	static MOVIESCENE_API TAutoRegisterPreAnimatedStorageID<FAnimTypePreAnimatedStateRootStorage> StorageID;

	MOVIESCENE_API FPreAnimatedStateEntry FindEntry(FMovieSceneAnimTypeID AnimTypeID);
	MOVIESCENE_API FPreAnimatedStateEntry MakeEntry(FMovieSceneAnimTypeID AnimTypeID);

public:

	FPreAnimatedStorageID GetStorageType() const override { return StorageID; }
	MOVIESCENE_API void Initialize(FPreAnimatedStorageID InStorageID, FPreAnimatedStateExtension* ParentExtension) override;
};

} // namespace MovieScene
} // namespace UE
