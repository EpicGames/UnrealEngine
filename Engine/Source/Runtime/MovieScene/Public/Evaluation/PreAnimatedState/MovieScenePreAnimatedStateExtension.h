// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "Containers/SortedMap.h"
#include "Containers/SparseArray.h"
#include "CoreTypes.h"
#include "EntitySystem/MovieScenePropertySystemTypes.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "Evaluation/MovieSceneAnimTypeID.h"
#include "Evaluation/PreAnimatedState/IMovieScenePreAnimatedStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateTypes.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Less.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UObject/ObjectKey.h"

class FReferenceCollector;
class UMovieSceneEntitySystemLinker;
class UObject;
struct FMovieSceneAnimTypeID;
struct FScopedPreAnimatedCaptureSource;
struct IMovieScenePreAnimatedGlobalTokenProducer;
struct IMovieScenePreAnimatedTokenProducer;

namespace UE
{
namespace MovieScene
{

struct FAnimTypePreAnimatedStateObjectStorage;
struct FAnimTypePreAnimatedStateRootStorage;
struct FPreAnimatedEntityCaptureSource;
struct FPreAnimatedEvaluationHookCaptureSources;
struct FPreAnimatedStateEntry;
struct FPreAnimatedStateExtension;
struct FPreAnimatedTemplateCaptureSources;
struct FPreAnimatedTrackInstanceCaptureSources;
struct FPreAnimatedTrackInstanceInputCaptureSources;
struct FRestoreStateParams;
struct FSharedPlaybackState;
struct IPreAnimatedCaptureSource;
struct IPreAnimatedStorage;

/**
 * Interface required for any logical grouping of pre-animated state
 */
struct IPreAnimatedStateGroupManager
{
	virtual ~IPreAnimatedStateGroupManager()= default;
	virtual void InitializeGroupManager(FPreAnimatedStateExtension* Extension) = 0;
	virtual void OnGroupDestroyed(FPreAnimatedStorageGroupHandle Group) = 0;
	virtual void GatherStaleStorageGroups(TArray<FPreAnimatedStorageGroupHandle>& StaleGroupStorage) const = 0;
};


/**
 * Pre-animated state extension that houses all cached values for objects animated by a specific UMovieSceneEntitySystemLinker
 * 
 *     - The presence of this extension denotes that pre-animated state should be stored in one form or another.
 *          If the extension is not present, this implies that there are no IMovieScenePlayers that need global Restore State, and no tracks set to 'Restore State'.
 *          FPreAnimatedStateExtension::NumRequestsForGlobalState defines whether this extension needs to cache any and all changes or not
 *     - Each specific 'type' of pre-animated state is defined by a unique StorageID (TPreAnimatedStorageID), registered through RegisterStorage
 *     - Pre-animated state is grouped into buckets in order to preserve significant ordering constraints (for instance, each object must restore state in the reverse order it was cached)
 *     - Groups are managed by registered IPreAnimatedStateGroupManager instances. The main implementation is FPreAnimatedObjectGroupManager, which maintains an object to group mapping
 */
struct FPreAnimatedStateExtension
{
public:

	MOVIESCENE_API FPreAnimatedStateExtension();
	MOVIESCENE_API ~FPreAnimatedStateExtension();

public:

	MOVIESCENE_API void Initialize(UMovieSceneEntitySystemLinker* InLinker);

	/**
	 * Called from TAutoRegisterPreAnimatedStorageID to register a new application-wide storage type
	 */
	template<typename StorageType>
	static TPreAnimatedStorageID<StorageType> RegisterStorage()
	{
		FPreAnimatedStorageID StorageID = RegisterStorageInternal();
		return static_cast<const TPreAnimatedStorageID<StorageType>&>(StorageID);
	}

public:

	/**
	 * Find a specific storage type by its ID.
	 * @return The typed storage, or nullptr if it does not exist in this linker
	 */
	template<typename StorageType>
	TSharedPtr<StorageType> FindStorage(TPreAnimatedStorageID<StorageType> InStorageID) const
	{
		TSharedPtr<IPreAnimatedStorage> TypedStorage = StorageImplementations.FindRef(InStorageID);
		return StaticCastSharedPtr<StorageType>(TypedStorage);
	}


	/**
	 * Add a specific storage type to this extension
	 */
	template<typename StorageType>
	void AddStorage(TPreAnimatedStorageID<StorageType> InStorageID, TSharedPtr<StorageType> InStorage)
	{
		check(!StorageImplementations.Contains(InStorageID));
		StorageImplementations.Add(InStorageID, InStorage);
	}


	/**
	 * Get a specific type of storage, creating it if it does not already exist
	 * @note The template type must have a static StorageID of type TPreAnimatedStorageID<StorageType>.
	 */
	template<typename StorageType>
	TSharedPtr<StorageType> GetOrCreateStorage()
	{
		return GetOrCreateStorage(StorageType::StorageID);
	}


	/**
	 * Get a specific type of storage, creating it if it does not already exist
	 */
	template<typename StorageType>
	TSharedPtr<StorageType> GetOrCreateStorage(TPreAnimatedStorageID<StorageType> InStorageID)
	{
		TSharedPtr<StorageType> Existing = FindStorage(InStorageID);
		if (!Existing)
		{
			Existing = MakeShared<StorageType>();
			Existing->Initialize(InStorageID, this);
			AddStorage(InStorageID, Existing);
		}

		return Existing;
	}


	/**
	 * Get a genericly typed storage entry by its erased ID, failing an assertion if it does not exist
	 */
	TSharedPtr<IPreAnimatedStorage> GetStorageChecked(FPreAnimatedStorageID InStorageID) const
	{
		return StorageImplementations.FindChecked(InStorageID);
	}

public:


	/**
	 * Find a group manager by its type. Group managers enable logical grouping of entries to ensure the correct restoration order.
	 * @note The templated type must implement a static GroupManagerID member of type TPreAnimatedStorageID<GroupManagerType>
	 * @return A pointer to the group manager, or nullptr if it does not exist.
	 */
	template<typename GroupManagerType>
	TSharedPtr<GroupManagerType> FindGroupManager() const
	{
		return StaticCastSharedPtr<GroupManagerType>(GroupManagers.FindRef(GroupManagerType::GroupManagerID).Pin());
	}


	/**
	 * Get or create a group manager by its type. Group managers enable logical grouping of entries to ensure the correct restoration order.
	 * @note The templated type must implement a static GroupManagerID member of type TPreAnimatedStorageID<GroupManagerType>
	 * @return A pointer to the group manager, or nullptr if it does not exist.
	 */
	template<typename GroupManagerType>
	TSharedPtr<GroupManagerType> GetOrCreateGroupManager()
	{
		TSharedPtr<IPreAnimatedStateGroupManager> Existing = GroupManagers.FindRef(GroupManagerType::GroupManagerID).Pin();

		if (!Existing)
		{
			Existing = MakeShared<GroupManagerType>();
			Existing->InitializeGroupManager(this);
			GroupManagers.Add(GroupManagerType::GroupManagerID, Existing);
		}

		return StaticCastSharedPtr<GroupManagerType>(Existing);
	}

	/**
	 * Called by group managers to allocate a new group
	 */
	MOVIESCENE_API FPreAnimatedStorageGroupHandle AllocateGroup(TSharedPtr<IPreAnimatedStateGroupManager> GroupManager);

	/**
	 * Called by group managers to free an existing group
	 */
	MOVIESCENE_API void FreeGroup(FPreAnimatedStorageGroupHandle Index);

public:

	/**
	 * Check if this extension has any current requests to capture global (persistent) state entries
	 */
	bool IsCapturingGlobalState() const
	{
		return NumRequestsForGlobalState > 0;
	}

	/**
	 * Check whether any previously cached entries may have become invalid due to a recent call to RestoreGlobalState
	 * If this function returns true, clients should consider re-saving pre-animated state even if it already did so
	 */
	bool AreEntriesInvalidated() const
	{
		return bEntriesInvalidated;
	}

	/**
	 * Resets the entry invalidation flag
	 */
	void ResetEntryInvalidation()
	{
		bEntriesInvalidated = false;
	}

public:

	/**
	 * Restore any state for any persistent tokens captured during the course of evaluation
	 *
	 * @param Params    Parameters for restoration - if TerminalInstanceHandle is invalid then _all_ state will be restored, regardless of the instance it was cached from
	 */
	MOVIESCENE_API void RestoreGlobalState(const FRestoreStateParams& Params);

	/**
	 * Discards any state for any persistent tokens captured during the course of evaluation without restoring it.
	 *
	 * @param Params    Parameters for restoration - if TerminalInstanceHandle is invalid then _all_ state will be discarded, regardless of the instance it was cached from
	 */
	MOVIESCENE_API void DiscardGlobalState(const FRestoreStateParams& Params);

	/**
	 * Restore any state cached for the specified group
	 *
	 * @params GroupHandle  Handle to the group to restore
	 * @param  Params       Parameters for restoration
	 */
	MOVIESCENE_API void RestoreStateForGroup(FPreAnimatedStorageGroupHandle GroupHandle, const FRestoreStateParams& Params);

	/**
	* Called during Garbage Collection to clean up preanimated state on invalid bound objects. Does not restore state.
	*/
	void DiscardStaleObjectState();

	/**
	 * Called during blueprint re-instancing to replace the object bound to a specific group handle with another.
	 */
	MOVIESCENE_API void ReplaceObjectForGroup(FPreAnimatedStorageGroupHandle GroupHandle, const FObjectKey& OldObject, const FObjectKey& NewObject);


	/**
	 * Discard any transient state and all meta-data for any currently animating objects, whilst preserving the cached values internally.
	 * Calling this function will cause any currently animating 'RestoreState' sections to re-cache their values if they are re-evaluated
	 * Any 'RestoreState' sections which are deleted or subsequently not-evaluated will not cause their values to be restored
	 */
	MOVIESCENE_API void DiscardTransientState();

	/**
	 * Discard any and all cached values for the specified group without restoring them.
	 * @note This function should only be used to forcibly serialize animated values into a level
	 */
	MOVIESCENE_API void DiscardStateForGroup(FPreAnimatedStorageGroupHandle GroupHandle);

	/**
	 * Discard the specified cached value and any and all capture source tracking related to it.
	 */
	MOVIESCENE_API void DiscardStateForStorage(FPreAnimatedStorageID StorageID, FPreAnimatedStorageIndex StorageIndex);

	/**
	 * Search for any captured state that originated from the specified root instance handle
	 * WARNING: This is a linear search across all state, and so is potentially very slow
	 */
	MOVIESCENE_API bool ContainsAnyStateForInstanceHandle(FRootInstanceHandle RootInstanceHandle) const;

public:

	// These methods for globally saving pre-animated state shouldn't be used directly:
	//
	// - use FScopedPreAnimatedCaptureSource to capture from a specific source.
	//
	// - use the similar methods on SharedPlaybackState->GetPreAnimatedState() to save only when it's
	//   needed for that given sequence instance.
	//
	MOVIESCENE_API void SavePreAnimatedState(FMovieSceneAnimTypeID InTokenType, const IMovieScenePreAnimatedGlobalTokenProducer& Producer);
	MOVIESCENE_API void SavePreAnimatedState(UObject& InObject, FMovieSceneAnimTypeID InTokenType, const IMovieScenePreAnimatedTokenProducer& Producer);

	MOVIESCENE_API void SavePreAnimatedStateDirectly(FMovieSceneAnimTypeID InTokenType, const IMovieScenePreAnimatedGlobalTokenProducer& Producer);
	MOVIESCENE_API void SavePreAnimatedStateDirectly(UObject& InObject, FMovieSceneAnimTypeID InTokenType, const IMovieScenePreAnimatedTokenProducer& Producer);

public:

	MOVIESCENE_API FPreAnimatedEntityCaptureSource* GetEntityMetaData() const;
	MOVIESCENE_API FPreAnimatedEntityCaptureSource* GetOrCreateEntityMetaData();

	MOVIESCENE_API FPreAnimatedTrackInstanceCaptureSources* GetTrackInstanceMetaData() const;
	MOVIESCENE_API FPreAnimatedTrackInstanceCaptureSources* GetOrCreateTrackInstanceMetaData();

	MOVIESCENE_API FPreAnimatedTrackInstanceInputCaptureSources* GetTrackInstanceInputMetaData() const;
	MOVIESCENE_API FPreAnimatedTrackInstanceInputCaptureSources* GetOrCreateTrackInstanceInputMetaData();

	MOVIESCENE_API FPreAnimatedTemplateCaptureSources* GetTemplateMetaData() const;
	MOVIESCENE_API FPreAnimatedTemplateCaptureSources* GetOrCreateTemplateMetaData();

	MOVIESCENE_API FPreAnimatedEvaluationHookCaptureSources* GetEvaluationHookMetaData() const;
	MOVIESCENE_API FPreAnimatedEvaluationHookCaptureSources* GetOrCreateEvaluationHookMetaData();

	MOVIESCENE_API bool HasActiveCaptureSource() const;

	MOVIESCENE_API void AddWeakCaptureSource(TWeakPtr<IPreAnimatedCaptureSource> InWeakMetaData);
	MOVIESCENE_API void RemoveWeakCaptureSource(TWeakPtr<IPreAnimatedCaptureSource> InWeakMetaData);

	MOVIESCENE_API void EnsureMetaData(const FPreAnimatedStateEntry& Entry);
	MOVIESCENE_API void AddSourceMetaData(const FPreAnimatedStateEntry& Entry);
	MOVIESCENE_API bool MetaDataExists(const FPreAnimatedStateEntry& Entry) const;

	MOVIESCENE_API void AddMetaData(const FPreAnimatedStateMetaData& MetaData);
	MOVIESCENE_API void RemoveMetaData(const FPreAnimatedStateMetaData& MetaData);
	MOVIESCENE_API void UpdateMetaData(const FPreAnimatedStateMetaData& MetaData);

	MOVIESCENE_API EPreAnimatedStorageRequirement GetStorageRequirement(const FPreAnimatedStateEntry& Entry) const;

private:

	MOVIESCENE_API void FreeGroupInternal(FPreAnimatedStorageGroupHandle Handle);

	MOVIESCENE_API void AddReferencedObjects(UMovieSceneEntitySystemLinker*, FReferenceCollector& ReferenceCollector);

	using FContributionRemover = TFunctionRef<void(IPreAnimatedStorage&, FPreAnimatedStorageIndex)>;
	void HandleMetaDataToRemove(
			const FRestoreStateParams& Params, 
			TArrayView<FPreAnimatedStateMetaData> MetaDataToRemove, 
			FContributionRemover RemoveFunc);

public:

	/** Called to handle replaced objects */
	void OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap);

	/** The number of requests that have been made to capture global state - only one should exist per playing sequence */
	uint32 NumRequestsForGlobalState;

private:

	/** Meta-data pertaining to the entity IDs from which animated state originates */
	TUniquePtr<FPreAnimatedEntityCaptureSource> EntityCaptureSource;

	/** Meta-data pertaining to pre-animated state originating from track instances */
	TUniquePtr<FPreAnimatedTrackInstanceCaptureSources> TrackInstanceCaptureSource;

	/** Meta-data pertaining to pre-animated state originating from track instances from a specific input */
	TUniquePtr<FPreAnimatedTrackInstanceInputCaptureSources> TrackInstanceInputCaptureSource;

	/** Meta-data ledger for any pre-animated state that originates from track templates */
	TUniquePtr<UE::MovieScene::FPreAnimatedTemplateCaptureSources> TemplateCaptureSource;

	/** Meta-data ledger for any pre-animated state that originates from evaluation hooks */
	TUniquePtr<UE::MovieScene::FPreAnimatedEvaluationHookCaptureSources> EvaluationHookCaptureSource;

	/** Weakly held meta data provided by FMovieScenePreAnimatedState for various other origins */
	TArray<TWeakPtr<IPreAnimatedCaptureSource>> WeakExternalCaptureSources;

	/** Pointers to the storage for state bound to objects, organized by FMovieSceneAnimTypeID */
	TWeakPtr<FAnimTypePreAnimatedStateObjectStorage> WeakGenericObjectStorage;
	/** Pointers to the storage for state created from root tracks, or otherwise not bound to objects */
	TWeakPtr<FAnimTypePreAnimatedStateRootStorage> WeakGenericRootStorage;

private:

	friend struct ::FScopedPreAnimatedCaptureSource;

	struct FAggregatePreAnimatedStateMetaData
	{
		FAggregatePreAnimatedStateMetaData(FPreAnimatedStateCachedValueHandle InValueHandle)
			: ValueHandle(InValueHandle)
		{}

		FPreAnimatedStateCachedValueHandle ValueHandle;

		FInstanceHandle TerminalInstanceHandle;
		uint16 NumContributors = 0;
		uint16 NumRestoreContributors = 0;
		bool bWantedRestore = false;
		bool bEligibleForGlobalRestore = false;
	};

	MOVIESCENE_API FAggregatePreAnimatedStateMetaData* FindMetaData(const FPreAnimatedStateEntry& Entry);
	MOVIESCENE_API const FAggregatePreAnimatedStateMetaData* FindMetaData(const FPreAnimatedStateEntry& Entry) const;

	MOVIESCENE_API FAggregatePreAnimatedStateMetaData* GetOrAddMetaDataInternal(const FPreAnimatedStateEntry& Entry);

	struct FPreAnimatedGroupMetaData
	{
		TSharedPtr<IPreAnimatedStateGroupManager> GroupManagerPtr;
		TArray<FAggregatePreAnimatedStateMetaData, TInlineAllocator<4>> AggregateMetaData;
	};

	TSparseArray<FPreAnimatedGroupMetaData> GroupMetaData;

	TMap<FPreAnimatedStateCachedValueHandle, FAggregatePreAnimatedStateMetaData> UngroupedMetaData;

	TSortedMap<FPreAnimatedStorageID, TSharedPtr<IPreAnimatedStorage>> StorageImplementations;
	TSortedMap<FPreAnimatedStorageID, TWeakPtr<IPreAnimatedStateGroupManager>> GroupManagers;

	/** Linker is always valid because this extension is owned by the Linker */
	UMovieSceneEntitySystemLinker* Linker;

	bool bEntriesInvalidated;

private:

	static MOVIESCENE_API FPreAnimatedStorageID RegisterStorageInternal();
};


template<typename KeyType>
struct TPreAnimatedStateGroupManager : IPreAnimatedStateGroupManager, TSharedFromThis<TPreAnimatedStateGroupManager<KeyType>>
{
	void InitializeGroupManager(FPreAnimatedStateExtension* InExtension) override
	{
		Extension = InExtension;
	}

	void OnGroupDestroyed(FPreAnimatedStorageGroupHandle Group) override
	{
		KeyType Temp = StorageGroupsToKey.FindChecked(Group);

		StorageGroupsByKey.Remove(Temp);
		StorageGroupsToKey.Remove(Group);
	}

	
	virtual void GatherStaleStorageGroups(TArray<FPreAnimatedStorageGroupHandle>& StaleGroupStorage) const override
	{
		
	}

	FPreAnimatedStorageGroupHandle FindGroupForKey(const KeyType& InKey) const
	{
		return StorageGroupsByKey.FindRef(InKey);
	}

	FPreAnimatedStorageGroupHandle MakeGroupForKey(const KeyType& InKey)
	{
		FPreAnimatedStorageGroupHandle GroupHandle = StorageGroupsByKey.FindRef(InKey);
		if (GroupHandle)
		{
			return GroupHandle;
		}

		GroupHandle = Extension->AllocateGroup(this->AsShared());
		StorageGroupsByKey.Add(InKey, GroupHandle);
		StorageGroupsToKey.Add(GroupHandle, InKey);
		return GroupHandle;
	}

protected:

	TMap<KeyType, FPreAnimatedStorageGroupHandle> StorageGroupsByKey;
	TMap<FPreAnimatedStorageGroupHandle, KeyType> StorageGroupsToKey;

	FPreAnimatedStateExtension* Extension;
};

} // namespace MovieScene
} // namespace UE
