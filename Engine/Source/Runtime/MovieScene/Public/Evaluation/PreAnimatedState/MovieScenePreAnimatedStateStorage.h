// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/SortedMap.h"
#include "Containers/SparseArray.h"
#include "CoreTypes.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/EntityAllocationIterator.h"
#include "EntitySystem/MovieSceneComponentPtr.h"
#include "EntitySystem/MovieSceneComponentTypeInfo.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntityRange.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "Evaluation/PreAnimatedState/IMovieScenePreAnimatedStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedCaptureSources.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateExtension.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateTypes.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.h"
#include "Evaluation/PreAnimatedState/MovieSceneRestoreStateParams.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/ObjectKey.h"

class FReferenceCollector;
class UObject;

namespace UE
{
namespace MovieScene
{

struct FPreAnimatedObjectGroupManager;
struct FRestoreStateParams;
struct FRootInstanceHandle;

/** Enum that defines how to cache pre-animated values based on their capture source */
enum class EPreAnimatedCaptureSourceTracking
{
	/** Cache the pre-animated value only if we are already tracking a capture source for it (or if global capture is enabled) */
	CacheIfTracked,
	/** Always cache the pre-animated value, potentially adding tracking meta-data for the scoped capture source */
	AlwaysCache,
};

/**
 * Default task that is provided to BeginTrackingEntitiesTask or CachePreAnimatedValuesTask to constrain
 * the task using an additional filter, or to provide custom short-circuit behavior that allows the client
 * to skip saving pre-animated state for some entities.
 */
template<typename... InputTypes>
struct TPreAnimatedStateTaskParams
{
	FEntityComponentFilter AdditionalFilter;
	EPreAnimatedCaptureSourceTracking TrackingMode;

	TPreAnimatedStateTaskParams()
	{
		TrackingMode = EPreAnimatedCaptureSourceTracking::CacheIfTracked;
		AdditionalFilter.All({ FBuiltInComponentTypes::Get()->Tags.NeedsLink });
	}
};

/**
 * Base class for all pre-animated state traits.
 *
 * Inherit from this class to get the default flag values. The sub-class must/might implement
 * the following members:
 *
 * (mandatory)
 *   typedef or alias KeyType, must be constructible from (InputTypes...)
 *   typedef or alias StorageType
 *   void RestorePreAnimatedValue(const KeyType&, const StorageValue&, const FRestoreStateParams&);
 *
 * (optional, if using the ECS-wide tasks for tracking and caching state)
 *   StorageType CachePreAnimatedValue(InputTypes...);
 *
 * (optional, if NeedsInitialize is true)
 *   void Initialize(FPreAnimatedStorageID, FPreAnimatedStateExtension*);
 *
 * (optional, if SupportsGrouping is true)
 *   FPreAnimatedStorageGroupHandle MakeGroup(InputTypes...);
 *
 * (optional, if SupportsReplaceObject is true)
 *   void ReplaceObject(KeyType&, const FObjectKey&);
 */
struct FPreAnimatedStateTraits
{
	enum
	{
		NeedsInitialize = false,
		SupportsGrouping = false,
		SupportsReplaceObject = false,
	};
};

/**
 * Storage container for a specific type of pre-animated state as defined by the specified traits.
 *
 * Reference collection for KeyType and StorageType is automatically provided by way of an optional AddReferencedObjectForComponent
 * override
 *
 * Traits must include a type definition or using alias for its KeyType and StorageType, defining the type of key to use
 * for associating the pre-animated value and the storage value type respectively.
 * 
 * Additionally, traits must define a RestorePreAnimatedValue function that will be used by the storage
 * container to restore data back to its previous value.

 * An example trait that maps an object and a name identifier to a string would look like this:
 * struct FExampleTraits
 * {
 *   using KeyType = TTuple<FObjectKey, FName>;
 *   using StorageType = FString;
 *
 *   static void RestorePreAnimatedValue(const KeyType& InKey, const FString& PreviousString, const FRestoreStateParams& Params)
 *   {
 *     if (UMyObjectType* Object = Cast<UMyObjectType>(InKey.Get<0>().ResolveObjectPtr()))
 *     {
 *       Object->SetStringValue(InKey.Get<1>(), PreviousString);
 *     }
 *   }
 * }
 *
 * Furthermore, if the CachePreAnimatedValuesTask is used, traits must implement a CachePreAnimatedValue function that receives
 * the contributor component types, and returns the cached value:
 *
 *   static FString CachePreAnimatedValue(UObject* InObject, const FName& StringName)
 *   {
 *     UMyObjectType* Object = CastChecked<UMyObjectType>(InObject);
 *     return Object->GetStringValue(StringName);
 *   }
 *
 * Traits may be stateful if desired. Stateful traits must be provided to the constructor in order to be valid.
 */
template<typename StorageTraits>
struct TPreAnimatedStateStorage : IPreAnimatedStorage
{
	/** The key type to use as a key in the map that associates pre-animated state values to owners (ie, an object and property name)*/
	using KeyType     = typename StorageTraits::KeyType;
	/** The value type this storage should store (ie, the actual property type) */
	using StorageType = typename StorageTraits::StorageType;

	/** Interface used for restoring pre-animated state that allows an external system to control whether state should be restored or not */
	struct IRestoreMask
	{
		virtual ~IRestoreMask()= default;

		virtual bool CanRestore(const KeyType& InKey) const = 0;
	};

	TPreAnimatedStateStorage()
	{}

	TPreAnimatedStateStorage(StorageTraits&& InTraits)
		: Traits(MoveTemp(InTraits))
	{}

	/** Pre-Animated storage is not copyable */
	TPreAnimatedStateStorage(const TPreAnimatedStateStorage&) = delete;
	TPreAnimatedStateStorage& operator=(const TPreAnimatedStateStorage&) = delete;

public:

	/** Called when this storage is created inside the pre-animated state extension on a UMovieSceneEntitySystemLinker */
	void Initialize(FPreAnimatedStorageID InStorageID, FPreAnimatedStateExtension* InParentExtension) override
	{
		ParentExtension = InParentExtension;
		StorageID = InStorageID;

		if constexpr (StorageTraits::NeedsInitialize)
		{
			this->Traits.Initialize(InStorageID, InParentExtension);
		}
	}

	/** Retrieve the ID that uniquely identifies this storage container */
	FPreAnimatedStorageID GetStorageType() const override
	{
		return StorageID;
	}

	/**
	 * Restore a specified index within this storage container.
	 * Usually called when all the things contributing to this storage index have finished evaluating.
	 *
	 * @param StorageIndex      The unique index for the stored state - either an index within PreAnimatedStorage or TransientPreAnimatedStorage.
	 * @param SourceRequirement The source requirement that is requesting to restore state:
	 *                              Persistent - indicates that the storage should be completely restored as a result of a sequence finishing or explicitly restoring state
	 *                              Transient - indicates that all 'Restore State' tracks have finished evaluating for this index, but persistent state may still be cached
	 * @param TargetRequirement The target requirement for storage - None implies that no state needs to remain cached for this index, Persistent implies that keep state entities have finished, but the state still needs to be cached.
	 * @param Params            Additional restore parameters defining the instigator context
	 *
	 * @return The resulting storage requirement for the supplied index
	 */
	EPreAnimatedStorageRequirement RestorePreAnimatedStateStorage(FPreAnimatedStorageIndex StorageIndex, EPreAnimatedStorageRequirement SourceRequirement, EPreAnimatedStorageRequirement TargetRequirement, const FRestoreStateParams& Params) override
	{
		if (RestoreMask)
		{
			if (!RestoreMask->CanRestore(PreAnimatedStorage[StorageIndex].Key))
			{
				return EPreAnimatedStorageRequirement::NoChange;
			}
		}

		if (SourceRequirement == EPreAnimatedStorageRequirement::Persistent)
		{
			// Restoring global state
			if (TargetRequirement == EPreAnimatedStorageRequirement::None)
			{
				FCachedData CachedData = MoveTemp(PreAnimatedStorage[StorageIndex]);

				KeyToStorageIndex.Remove(CachedData.Key);
				PreAnimatedStorage.RemoveAt(StorageIndex, 1);
				TransientPreAnimatedStorage.Remove(StorageIndex);

				if (CachedData.bInitialized)
				{
					Traits.RestorePreAnimatedValue(CachedData.Key, CachedData.Value, Params);
				}
				return EPreAnimatedStorageRequirement::None;
			}
			else
			{
				ensure(TargetRequirement == EPreAnimatedStorageRequirement::NoChange);

				FCachedData& CachedData = PreAnimatedStorage[StorageIndex];
				if (CachedData.bInitialized)
				{
					Traits.RestorePreAnimatedValue(CachedData.Key, CachedData.Value, Params);
				}

				return EPreAnimatedStorageRequirement::NoChange;
			}
		}

		ensure(SourceRequirement == EPreAnimatedStorageRequirement::Transient);

		// Always restore from the transient storage if available
		if (StorageType* CachedData = TransientPreAnimatedStorage.Find(StorageIndex))
		{
			Traits.RestorePreAnimatedValue(PreAnimatedStorage[StorageIndex].Key, *CachedData, Params);

			TransientPreAnimatedStorage.Remove(StorageIndex);

			return EPreAnimatedStorageRequirement::Persistent;
		}

		if (TargetRequirement == EPreAnimatedStorageRequirement::None)
		{
			FCachedData& ActualValue = PreAnimatedStorage[StorageIndex];
			if (ActualValue.bPersistent)
			{
				if (ActualValue.bInitialized)
				{
					Traits.RestorePreAnimatedValue(ActualValue.Key, ActualValue.Value, Params);
				}
				return EPreAnimatedStorageRequirement::Persistent;
			}

			FCachedData Tmp = MoveTemp(PreAnimatedStorage[StorageIndex]);

			KeyToStorageIndex.Remove(Tmp.Key);
			PreAnimatedStorage.RemoveAt(StorageIndex, 1);
			TransientPreAnimatedStorage.Remove(StorageIndex);

			if (Tmp.bInitialized)
			{
				Traits.RestorePreAnimatedValue(Tmp.Key, Tmp.Value, Params);
			}

			return EPreAnimatedStorageRequirement::None;
		}

		if (TargetRequirement == EPreAnimatedStorageRequirement::Persistent)
		{
			ensureMsgf(
					this->ParentExtension->IsCapturingGlobalState(),
					TEXT("We shouldn't move cached values to the persistent storage if we are not capturing global state!"));

			// Restore the value but keep the value cached
			FCachedData& PersistentData = PreAnimatedStorage[StorageIndex];

			if (PersistentData.bInitialized)
			{
				PersistentData.bPersistent = true;
				Traits.RestorePreAnimatedValue(PersistentData.Key, PersistentData.Value, Params);
			}
		}

		return EPreAnimatedStorageRequirement::Persistent;
	}

	/**
	 * Discard a specified index within this storage container.
	 *
	 * @param StorageIndex      The unique index for the stored state - either an index within PreAnimatedStorage or TransientPreAnimatedStorage.
	 * @param SourceRequirement The storage requirement to discard - Persistent will discard all cached state for the object, Transient may leave persistent storage around, if possible.
	 * 
	 * @return The resulting storage requirement for the supplied index
	 */
	EPreAnimatedStorageRequirement DiscardPreAnimatedStateStorage(FPreAnimatedStorageIndex StorageIndex, EPreAnimatedStorageRequirement SourceRequirement) override
	{
		if (RestoreMask)
		{
			if (!RestoreMask->CanRestore(PreAnimatedStorage[StorageIndex].Key))
			{
				return EPreAnimatedStorageRequirement::NoChange;
			}
		}

		if (SourceRequirement == EPreAnimatedStorageRequirement::Persistent)
		{
			KeyType Key = PreAnimatedStorage[StorageIndex].Key;

			KeyToStorageIndex.Remove(Key);
			PreAnimatedStorage.RemoveAt(StorageIndex, 1);
			TransientPreAnimatedStorage.Remove(StorageIndex);

			return EPreAnimatedStorageRequirement::None;
		}
		else
		{
			ensure(SourceRequirement == EPreAnimatedStorageRequirement::Transient);
			const int32 NumTransients = TransientPreAnimatedStorage.Remove(StorageIndex);
			if (NumTransients == 0)
			{
				PreAnimatedStorage[StorageIndex].bPersistent = true;
			}
			return EPreAnimatedStorageRequirement::Persistent;
		}
	}

	/**
	 * Called prior to restoring pre-animated state to control whether this storage should restore state or not
	 */
	void SetRestoreMask(const IRestoreMask* InRestoreMask)
	{
		RestoreMask = InRestoreMask;
	}

	/**
	 * Called by the owning extension to add reference collection tracking for pre-animated state
	 */
	void AddReferencedObjects(FReferenceCollector& ReferenceCollector) override
	{
		if constexpr (THasAddReferencedObjectForComponent<KeyType>::Value)
		{
			for (auto It = KeyToStorageIndex.CreateIterator(); It; ++It)
			{
				AddReferencedObjectForComponent(&ReferenceCollector, &It.Key());
			}
		}
		
		if constexpr (THasAddReferencedObjectForComponent<KeyType>::Value || THasAddReferencedObjectForComponent<StorageType>::Value)
		{
			for (FCachedData& CachedData : PreAnimatedStorage)
			{
				AddReferencedObjectForComponent(&ReferenceCollector, &CachedData.Key);
				AddReferencedObjectForComponent(&ReferenceCollector, &CachedData.Value);
			}
		}
		if constexpr (THasAddReferencedObjectForComponent<StorageType>::Value)
		{
			for (auto It = TransientPreAnimatedStorage.CreateIterator(); It; ++It)
			{
				AddReferencedObjectForComponent(&ReferenceCollector, &It.Value());
			}
		}
	}

	/**
	 * Attempt to find a storage index for the specified key, creating a new one if it doesn't exist
	 */
	FPreAnimatedStorageIndex GetOrCreateStorageIndex(const KeyType& InKey)
	{
		FPreAnimatedStorageIndex Index = KeyToStorageIndex.FindRef(InKey);
		if (!Index)
		{
			Index = PreAnimatedStorage.Add(FCachedData{InKey});
			KeyToStorageIndex.Add(InKey, Index);
		}

		return Index;
	}

	/**
	 * Attempt to find a storage index for the specified key
	 */
	FPreAnimatedStorageIndex FindStorageIndex(const KeyType& InKey) const
	{
		return KeyToStorageIndex.FindRef(InKey);
	}

	/**
	 * Assign the value for a specific storage index. It is an error to re-assign an already cached value.
	 *
	 * @param StorageIndex       The storage index to assign a value for
	 * @param StorageRequirement Whether to assign the value to this index's persistent, or transient value. Transient should be used when this value is directly associated with a track evaluating.
	 * @param InNewValue         The value to assign
	 */
	void AssignPreAnimatedValue(FPreAnimatedStorageIndex StorageIndex, EPreAnimatedStorageRequirement StorageRequirement, StorageType&& InNewValue)
	{
		check(StorageIndex);

		FCachedData& CachedData = PreAnimatedStorage[StorageIndex.Value];

		if (StorageRequirement == EPreAnimatedStorageRequirement::Persistent)
		{
			ensure(!CachedData.bInitialized);
			CachedData.Value = MoveTemp(InNewValue);
			CachedData.bPersistent = true;
			CachedData.bInitialized = true;
		}
		else if (StorageRequirement == EPreAnimatedStorageRequirement::Transient)
		{
			ensure(!CachedData.bInitialized || !TransientPreAnimatedStorage.Contains(StorageIndex));

			// Assign the transient value
			if (!CachedData.bInitialized)
			{
				CachedData.Value = MoveTemp(InNewValue);
				CachedData.bInitialized = true;
			}
			else
			{
				TransientPreAnimatedStorage.Add(StorageIndex, MoveTemp(InNewValue));
			}
		}
	}

	/**
	 * Check whether the storage for a given index and requirement has already been initialized
	 */
	bool IsStorageRequirementSatisfied(FPreAnimatedStorageIndex StorageIndex, EPreAnimatedStorageRequirement StorageRequirement) const
	{
		check(StorageIndex);

		const FCachedData& CachedData = PreAnimatedStorage[StorageIndex.Value];

		if (StorageRequirement == EPreAnimatedStorageRequirement::Persistent)
		{
			return CachedData.bInitialized;
		}
		else if (StorageRequirement == EPreAnimatedStorageRequirement::Transient)
		{
			return (CachedData.bInitialized && CachedData.bPersistent == false)
				|| TransientPreAnimatedStorage.Contains(StorageIndex);
		}

		return true;
	}

	/**
	* Given a list of arguments suitable for building a storage key, builds that key, and then 
	* tries to find an existing pre-animated state entry for that key if it already exists.
	* If it doesn't exist, it returns an empty FPreAnimatedStateEntry.
	*
	* Note that the arguments are also used for find the group handle, if the storage traits required grouping.
	*/
	template<typename... KeyArgs>
	FPreAnimatedStateEntry FindEntry(KeyArgs&&... InKeyArgs)
	{
		KeyType Key{ Forward<KeyArgs>(InKeyArgs)... };

		FPreAnimatedStorageGroupHandle GroupHandle;
		if constexpr (StorageTraits::SupportsGrouping)
		{
			GroupHandle = this->Traits.FindGroup(this->Traits.ResolveComponent(InKeyArgs)...);
		}

		FPreAnimatedStorageIndex StorageIndex = this->FindStorageIndex(Key);

		return FPreAnimatedStateEntry{ GroupHandle, FPreAnimatedStateCachedValueHandle{ this->StorageID, StorageIndex } };
	}

	/**
	 * Creates a new pre-animated state entry from a list of arguments suitable for building the storage key.
	 *
	 * Note that the arguments are also used for getting the group handle, if the storage traits required grouping.
	 */
	template<typename... KeyArgs>
	FPreAnimatedStateEntry MakeEntry(KeyArgs&&... InKeyArgs)
	{
		KeyType Key{ Forward<KeyArgs>(InKeyArgs)... };

		FPreAnimatedStorageGroupHandle GroupHandle;
		if constexpr(StorageTraits::SupportsGrouping)
		{
			GroupHandle = this->Traits.MakeGroup(this->Traits.ResolveComponent(InKeyArgs)...);
		}

		FPreAnimatedStorageIndex StorageIndex = this->GetOrCreateStorageIndex(Key);

		return FPreAnimatedStateEntry{ GroupHandle, FPreAnimatedStateCachedValueHandle{ this->StorageID, StorageIndex } };
	}

	bool IsCapturingGlobalState() const
	{
		return this->ParentExtension->IsCapturingGlobalState();
	}

	/**
	 * Cause a previously cached value to always outlive any actively animating sources.
	 * This is called when a Restore State track overlaps a Keep State track. The Restore State track will initially
	 * save state using the Transient requirement, but the Keep State track may need to keep this cached state alive
	 * if it is capturing _global_ state. As such, we take the previously cached state and make it persistent.
	 */
	void ForciblyPersistStorage(FPreAnimatedStorageIndex StorageIndex)
	{
		check(StorageIndex);
		PreAnimatedStorage[StorageIndex.Value].bPersistent = true;
	}

	/**
	 * Check whether a piece of cached storage has been initialized yet. This will be false where multiple entities contribute to an object's state
	 * but a different entity actually assigns the value (eg multiple blended entiies). Saving state for these requires 2 passes: firstly we gather
	 * all the entities that contribute to the state (which may not be able to even know _how_ to cache the state), then we capture the actual value.
	 */
	bool IsStorageInitialized(FPreAnimatedStorageIndex StorageIndex) const
	{
		return StorageIndex && (PreAnimatedStorage[StorageIndex.Value].bInitialized || TransientPreAnimatedStorage.Contains(StorageIndex));
	}

	/**
	 * Check whether we have ever animated the specified storage index.
	 */
	bool HasEverAnimated(FPreAnimatedStorageIndex StorageIndex) const
	{
		return StorageIndex && PreAnimatedStorage[StorageIndex.Value].bInitialized;
	}

	/**
	 * Get the key associated with a particular storage index
	 */
	const KeyType& GetKey(FPreAnimatedStorageIndex StorageIndex) const
	{
		return PreAnimatedStorage[StorageIndex].Key;
	}

	/**
	 * Replace the key associated with a particular storage index
	 */
	void ReplaceKey(FPreAnimatedStorageIndex StorageIndex, const KeyType& NewKey)
	{
		KeyType OldKey = PreAnimatedStorage[StorageIndex].Key;
		PreAnimatedStorage[StorageIndex].Key = NewKey;

		KeyToStorageIndex.Remove(OldKey);
		KeyToStorageIndex.Add(NewKey, StorageIndex);
	}

	/**
	 * Get the cached value associated with a particular storage index
	 */
	const StorageType& GetCachedValue(FPreAnimatedStorageIndex StorageIndex) const
	{
		static const StorageType DefaultValue = StorageType();

		if (ensure(PreAnimatedStorage.IsValidIndex(StorageIndex.Value)))
		{
			const FCachedData& CachedData = PreAnimatedStorage[StorageIndex.Value];
			if (CachedData.bInitialized)
			{
				return CachedData.Value;
			}
		}
		return DefaultValue;
	}

	/**
	 * Look at any entity with the specified component types, and set up new associations with storage indices for those entities
	 * The provided component values are put together to make up the storage key
	 * WARNING: Does not cache actual pre-animated values
	 */
	template<typename... ContributorTypes>
	void BeginTrackingEntities(UMovieSceneEntitySystemLinker* Linker, TComponentTypeID<ContributorTypes>... InComponentTypes)
	{
		BeginTrackingEntitiesTask(Linker, TPreAnimatedStateTaskParams<ContributorTypes...>(), InComponentTypes...);
	}

	/**
	 * Look at any entity with the specified component types, and set up new associations with storage indices for those entities
	 * The provided component values are put together to make up the storage key
	 * WARNING: Does not cache actual pre-animated values
	 */
	template<typename TaskType, typename... ContributorTypes>
	void BeginTrackingEntitiesTask(UMovieSceneEntitySystemLinker* Linker, const TaskType& InParams, TComponentTypeID<ContributorTypes>... InComponentTypes)
	{
		auto VisitAllocation = [this, InParams](FEntityAllocationIteratorItem Item, TRead<FMovieSceneEntityID> EntityIDs, TRead<FRootInstanceHandle> RootInstanceHandles, TRead<ContributorTypes>... Inputs)
		{
			FPreAnimatedTrackerParams Params(Item);

			const int32 Num = Params.Num;
			const bool  bWantsRestore = Params.bWantsRestoreState;

			if (!this->ParentExtension->IsCapturingGlobalState() && !bWantsRestore)
			{
				return;
			}

			FPreAnimatedEntityCaptureSource* EntityMetaData = this->ParentExtension->GetOrCreateEntityMetaData();
			for (int32 Index = 0; Index < Num; ++Index)
			{
				FPreAnimatedStateEntry Entry = MakeEntry(Inputs[Index]...);
				EntityMetaData->BeginTrackingEntity(Entry, EntityIDs[Index], RootInstanceHandles[Index], bWantsRestore);
			}
		};

		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

		FComponentMask ExcludeMask({ BuiltInComponents->Tags.NeedsUnlink, BuiltInComponents->Tags.Finished, BuiltInComponents->Tags.Ignored });

		FEntityTaskBuilder()
		.ReadEntityIDs()
		.Read(BuiltInComponents->RootInstanceHandle)
		.ReadAllOf(InComponentTypes...)
		.FilterNone(ExcludeMask)
		.CombineFilter(InParams.AdditionalFilter)
		.Iterate_PerAllocation(&Linker->EntityManager, VisitAllocation);
	}

	/**
	 * Set up a new associations with a storage index for the given entity
	 * The provided component values are put together to make up the storage key
	 * WARNING: Does not cache actual pre-animated values
	 */
	template<typename... ContributorTypes>
	void BeginTrackingEntity(FMovieSceneEntityID EntityID, const bool bWantsRestoreState, FRootInstanceHandle RootInstanceHandle, ContributorTypes... InComponents)
	{
		if (!this->ParentExtension->IsCapturingGlobalState() && !bWantsRestoreState)
		{
			return;
		}

		TPreAnimatedStateTaskParams<ContributorTypes...> Params;

		FPreAnimatedStateEntry Entry = MakeEntry(InComponents...);
		FPreAnimatedEntityCaptureSource* EntityMetaData = this->ParentExtension->GetOrCreateEntityMetaData();
		EntityMetaData->BeginTrackingEntity(Entry, EntityID, RootInstanceHandle, bWantsRestoreState);
	}

	/**
	 * Cache pre-animated values for entities with the specified component types
	 */
	template<typename... ContributorTypes>
	void CachePreAnimatedValues(UMovieSceneEntitySystemLinker* Linker, TComponentTypeID<ContributorTypes>... InComponentTypes)
	{
		CachePreAnimatedValuesTask(Linker, TPreAnimatedStateTaskParams<ContributorTypes...>(), InComponentTypes...);
	}

	/**
	 * Cache pre-animated values for entities with the specified component types
	 */
	template<typename TaskType, typename... ContributorTypes>
	void CachePreAnimatedValuesTask(UMovieSceneEntitySystemLinker* Linker, const TaskType& InParams, TComponentTypeID<ContributorTypes>... InComponentTypes)
	{
		auto VisitAllocation = [this, &InParams](FEntityAllocationIteratorItem Item, TRead<ContributorTypes>... Values)
		{
			const int32 Num = Item.GetAllocation()->Num();
			for (int32 Index = 0; Index < Num; ++Index)
			{
				if (!ShouldTrackCaptureSource(InParams.TrackingMode, Values[Index]...))
				{
					continue;
				}

				FPreAnimatedStateEntry Entry = MakeEntry(Values[Index]...);
				TrackCaptureSource(Entry, InParams.TrackingMode);
				
				EPreAnimatedStorageRequirement StorageRequirement = this->ParentExtension->GetStorageRequirement(Entry);
				if (!this->IsStorageRequirementSatisfied(Entry.ValueHandle.StorageIndex, StorageRequirement))
				{
					StorageType NewValue = this->Traits.CachePreAnimatedValue(this->Traits.ResolveComponent(Values[Index])...);
					this->AssignPreAnimatedValue(Entry.ValueHandle.StorageIndex, StorageRequirement, MoveTemp(NewValue));
				}
			}
		};

		FEntityTaskBuilder()
		.ReadAllOf(InComponentTypes...)
		.CombineFilter(InParams.AdditionalFilter)
		.Iterate_PerAllocation(&Linker->EntityManager, VisitAllocation);
	}

	/**
	 * Save pre-animated state for the specified values, using CacheIfTracked tracking
	 * Requires that the traits class implements CachePreAnimatedValue(ContributorTypes...)
	 */
	template<typename... ContributorTypes>
	void CachePreAnimatedValue(ContributorTypes... Values)
	{
		CacheTrackedPreAnimatedValue(EPreAnimatedCaptureSourceTracking::CacheIfTracked, Values...);
	}

	/**
	 * Save pre-animated state for the specified values
	 * Requires that the traits class implements CachePreAnimatedValue(ContributorTypes...)
	 */
	template<typename... ContributorTypes>
	void CacheTrackedPreAnimatedValue(EPreAnimatedCaptureSourceTracking TrackingMode, ContributorTypes... Values)
	{
		if (ShouldTrackCaptureSource(TrackingMode, Values...))
		{
			FPreAnimatedStateEntry Entry = MakeEntry(Values...);

			TrackCaptureSource(Entry, TrackingMode);

			EPreAnimatedStorageRequirement StorageRequirement = this->ParentExtension->GetStorageRequirement(Entry);
			if (!IsStorageRequirementSatisfied(Entry.ValueHandle.StorageIndex, StorageRequirement))
			{
				StorageType NewValue = this->Traits.CachePreAnimatedValue(this->Traits.ResolveComponent(Values)...);
				this->AssignPreAnimatedValue(Entry.ValueHandle.StorageIndex, StorageRequirement, MoveTemp(NewValue));
			}
		}
	}

	/**
	 * Save pre-animated state for the specified group and key using a callback.
	 * Callback will only be invoked if state has not already been saved.
	 */
	template<typename OnCacheValue /* StorageType(const KeyType&) */>
	void CachePreAnimatedValue(const KeyType& InKey, OnCacheValue&& CacheCallback, EPreAnimatedCaptureSourceTracking TrackingMode = EPreAnimatedCaptureSourceTracking::CacheIfTracked)
	{
		static_assert(StorageTraits::SupportsGrouping == false, "Grouped pre-animated state requires passing a group handle");
		CachePreAnimatedValue(FPreAnimatedStorageGroupHandle(), InKey, Forward<OnCacheValue>(CacheCallback), TrackingMode);
	}

	/**
	 * Save pre-animated state for the specified group and key using a callback.
	 * Callback will only be invoked if state has not already been saved.
	 */
	template<typename OnCacheValue /* StorageType(const KeyType&) */>
	void CachePreAnimatedValue(FPreAnimatedStorageGroupHandle GroupHandle, const KeyType& InKey, OnCacheValue&& CacheCallback, EPreAnimatedCaptureSourceTracking TrackingMode = EPreAnimatedCaptureSourceTracking::CacheIfTracked)
	{
		ensureMsgf(GroupHandle.IsValid() || !StorageTraits::SupportsGrouping, TEXT("The group handle must be valid for pre-animated state that supports grouping, and invalid if not"));

		if (!ShouldTrackCaptureSource(TrackingMode, InKey))
		{
			return;
		}
		// Find the storage index for the specific key we're animating
		FPreAnimatedStorageIndex StorageIndex = GetOrCreateStorageIndex(InKey);
		FPreAnimatedStateEntry Entry{ GroupHandle, FPreAnimatedStateCachedValueHandle{ this->StorageID, StorageIndex } };
		TrackCaptureSource(Entry, TrackingMode);

		EPreAnimatedStorageRequirement StorageRequirement = this->ParentExtension->GetStorageRequirement(Entry);
		if (!IsStorageRequirementSatisfied(Entry.ValueHandle.StorageIndex, StorageRequirement))
		{
			StorageType NewValue = CacheCallback(InKey);
			AssignPreAnimatedValue(StorageIndex, StorageRequirement, MoveTemp(NewValue));
		}
	}

	/**
	 * Look at any entity with the specified component types, and set up new associations with storage indices for those entities, whilst also caching pre-animated values at the same time.
	 */
	template<typename... ContributorTypes>
	void BeginTrackingAndCachePreAnimatedValues(UMovieSceneEntitySystemLinker* Linker, TComponentTypeID<ContributorTypes>... InComponentTypes)
	{
		BeginTrackingAndCachePreAnimatedValuesTask(Linker, TPreAnimatedStateTaskParams<ContributorTypes...>(), InComponentTypes...);
	}

	/**
	 * Look at any entity with the specified component types, and set up new associations with storage indices for those entities, whilst also caching pre-animated values at the same time.
	 */
	template<typename TaskType, typename... ContributorTypes>
	void BeginTrackingAndCachePreAnimatedValuesTask(UMovieSceneEntitySystemLinker* Linker, const TaskType& InParams, TComponentTypeID<ContributorTypes>... InComponentTypes)
	{
		auto VisitAllocation = [this, InParams](FEntityAllocationIteratorItem Item, TRead<FMovieSceneEntityID> EntityIDs, TRead<FRootInstanceHandle> RootInstanceHandles, TRead<ContributorTypes>... Inputs)
		{
			FPreAnimatedTrackerParams Params(Item);

			const int32 Num = Params.Num;
			const bool  bWantsRestore = Params.bWantsRestoreState;

			if (!this->ParentExtension->IsCapturingGlobalState() && !bWantsRestore)
			{
				return;
			}

			FPreAnimatedEntityCaptureSource* EntityMetaData = this->ParentExtension->GetOrCreateEntityMetaData();
			for (int32 Index = 0; Index < Num; ++Index)
			{
				FPreAnimatedStateEntry Entry = MakeEntry(Inputs[Index]...);

				EntityMetaData->BeginTrackingEntity(Entry, EntityIDs[Index], RootInstanceHandles[Index], bWantsRestore);

				EPreAnimatedStorageRequirement StorageRequirement = this->ParentExtension->GetStorageRequirement(Entry);
				if (!this->IsStorageRequirementSatisfied(Entry.ValueHandle.StorageIndex, StorageRequirement))
				{
					StorageType NewValue = this->Traits.CachePreAnimatedValue(this->Traits.ResolveComponent(Inputs[Index])...);
					this->AssignPreAnimatedValue(Entry.ValueHandle.StorageIndex, StorageRequirement, MoveTemp(NewValue));
				}
			}
		};

		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

		FEntityTaskBuilder()
		.ReadEntityIDs()
		.Read(BuiltInComponents->RootInstanceHandle)
		.ReadAllOf(InComponentTypes...)
		.CombineFilter(InParams.AdditionalFilter)
		.Iterate_PerAllocation(&Linker->EntityManager, VisitAllocation);
	}

	void OnObjectReplaced(FPreAnimatedStorageIndex StorageIndex, const FObjectKey& OldObject, const FObjectKey& NewObject) override
	{
		KeyType ExistingKey = this->GetKey(StorageIndex);
		if constexpr(StorageTraits::SupportsReplaceObject)
		{
			this->Traits.ReplaceObject(ExistingKey, NewObject);
		}
		this->ReplaceKey(StorageIndex, ExistingKey);
	}

protected:

	template<typename... KeyArgs>
	bool ShouldTrackCaptureSource(EPreAnimatedCaptureSourceTracking TrackingMode, KeyArgs&&... InKeyArgs)
	{
		switch (TrackingMode)
		{
		case EPreAnimatedCaptureSourceTracking::CacheIfTracked:
		{
			// If we're capturing global state we always track changes
			if (this->ParentExtension->IsCapturingGlobalState())
			{
				return true;
			}

			// Only cache the value if an entry already exists, and if metadata already exists.
			// (ie, something is actively animating this and wants restore state)
			if (FPreAnimatedStateEntry Entry = FindEntry(InKeyArgs...))
			{
				return ensureMsgf(this->ParentExtension->MetaDataExists(Entry), TEXT("PreAnimatedStateEntry has allocated storage but no metadata exists to track it."));
			}
			return false;
		}
		break;

		case EPreAnimatedCaptureSourceTracking::AlwaysCache:
		{
			// Always cache the source meta data. If there is no capture source this calls EnsureMetaData as above
			return true;
		}
		break;

		default:
			ensureMsgf(false, TEXT("Unsupported tracking mode, no pre-animated state caching will occur!"));
			return false;
		}
	}

	/**
	 * Begins tracking the current entry with the currently set tracking source, if necessary/desirable.
	 * If the entry is valid, we assume the metadata must already exist.
	 */
	void TrackCaptureSource(const FPreAnimatedStateEntry& Entry, EPreAnimatedCaptureSourceTracking TrackingMode)
	{
		switch (TrackingMode)
		{
		case EPreAnimatedCaptureSourceTracking::CacheIfTracked:
			{
				// If we're capturing global state we always track changes
				if (this->ParentExtension->IsCapturingGlobalState())
				{
					this->ParentExtension->EnsureMetaData(Entry);
					return;
				}

				// Only cache the value if tracking meta-data exists for this entry
				// (ie, something is actively animating this and wants restore state)
				ensureMsgf(Entry.IsValid() && this->ParentExtension->MetaDataExists(Entry), TEXT("PreAnimatedStateEntry has allocated storage but no metadata exists to track it."));
			}
			break;

		case EPreAnimatedCaptureSourceTracking::AlwaysCache:
			{
				// Always cache the source meta data. If there is no capture source this calls EnsureMetaData as above
				this->ParentExtension->AddSourceMetaData(Entry);
				return;
			}
			break;

		default:
			ensureMsgf(false, TEXT("Unsupported tracking mode, no pre-animated state caching will occur!"));
		}
	}

	struct FCachedData
	{
		FCachedData()
			: bInitialized(false)
			, bPersistent(false)
		{}

		FCachedData(const KeyType& InKey)
			: Key(InKey)
			, bInitialized(false)
			, bPersistent(false)
		{}

		FCachedData(const KeyType& InKey, StorageType&& InValue)
			: Key(InKey)
			, Value(MoveTemp(InValue))
			, bInitialized(true)
			, bPersistent(false)
		{}

		KeyType Key;
		StorageType Value;
		bool bInitialized : 1;
		bool bPersistent : 1;
	};

	/** Map that associates a storage index with a key. This indirection allows common code to deal with indices without knowing concrete templated types. */
	TMap<KeyType, FPreAnimatedStorageIndex> KeyToStorageIndex;

	/** Sparse array of cached data that represents persistent, or both persistent _and_ transient storage (but never exclusively transient storage).
	 * FPreAnimatedStorageIndex defines an index into this array (or into the rarely used TransientPreAnimatedStorage map) */
	TSparseArray<FCachedData> PreAnimatedStorage;

	/**
	 * Storage that holds values that need to be kept transiently (for evaluation).
	 * This map is only used if a Keep State section previously captured a value (because it is evaluating in a 'Capture Global State' context)
	 * and we end up animating the _same_ value using a Restore State section. The Restore State section has to re-capture its starting value
	 * whilst also keeping the previously captured value alive to ensure that the section can restore to its starting value in addition to the
	 * sequence itself restoring to the global starting value
	 */
	TSortedMap<FPreAnimatedStorageIndex, StorageType> TransientPreAnimatedStorage;

	/** Pointer to our parent extension */
	FPreAnimatedStateExtension* ParentExtension = nullptr;

	/** Temporary restoration mask */
	const IRestoreMask* RestoreMask = nullptr;

	/** Our storage ID */
	FPreAnimatedStorageID StorageID;

public:

	/** Traits instance - normally not required but some traits require state */
	StorageTraits Traits;
};


/**
 * Simple traits class that works with the simple storage below.
 * The only thing that is needed is a `void RestoreState(KeyType, Params)` method on the state
 * class.
 */
template<typename InKeyType, typename InStorageType>
struct TSimplePreAnimatedStateTraits : FPreAnimatedStateTraits
{
	using KeyType = InKeyType;
	using StorageType = InStorageType;

	void RestorePreAnimatedValue(const KeyType& Key, StorageType& SavedValue, const FRestoreStateParams& Params)
	{
		SavedValue.RestoreState(Key, Params);
	}
};


/**
 * Simple pre-animated state storage that doesn't support any grouping, and lets people simply implement
 * a `void RestoreState(KeyType, Params)` method on their state class.
 */
template<typename KeyType, typename StorageType>
struct TSimplePreAnimatedStateStorage : TPreAnimatedStateStorage<TSimplePreAnimatedStateTraits<KeyType, StorageType>>
{
};

} // namespace MovieScene
} // namespace UE
