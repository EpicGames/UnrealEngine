// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Templates/TypeHash.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.h"

namespace UE
{
namespace MovieScene
{

struct FEntityAllocationProxy;

struct FPreAnimatedTrackerParams
{
	MOVIESCENE_API FPreAnimatedTrackerParams(FEntityAllocationProxy Item);

	int32 Num;
	bool bWantsRestoreState;
};

struct FCachePreAnimatedValueParams
{
	bool bForcePersist = false;
};

enum class EPreAnimatedStorageRequirement : uint8
{
	None,
	Persistent,
	Transient,

	NoChange,
};


struct FPreAnimatedStorageIndex
{
	FPreAnimatedStorageIndex()
		: Value(INDEX_NONE)
	{}

	FPreAnimatedStorageIndex(int32 InNewValue)
		: Value(InNewValue)
	{}

	FPreAnimatedStorageIndex& operator=(int32 InNewValue)
	{
		Value = InNewValue;
		return *this;
	}

	bool IsValid() const
	{
		return Value != INDEX_NONE;
	}

	explicit operator bool() const
	{
		return Value != INDEX_NONE;
	}

	operator int32() const
	{
		check(Value != INDEX_NONE);
		return Value;
	}

	friend uint32 GetTypeHash(const FPreAnimatedStorageIndex& InIndex)
	{
		return ::GetTypeHash(InIndex.Value);
	}
	friend bool operator==(const FPreAnimatedStorageIndex& A, const FPreAnimatedStorageIndex& B)
	{
		return A.Value == B.Value;
	}

	int32 Value;
};

struct FPreAnimatedStorageGroupHandle
{
	FPreAnimatedStorageGroupHandle()
		: Value(INDEX_NONE)
	{}

	FPreAnimatedStorageGroupHandle(int32 InNewValue)
		: Value(InNewValue)
	{}

	FPreAnimatedStorageGroupHandle& operator=(int32 InNewValue)
	{
		Value = InNewValue;
		return *this;
	}

	bool IsValid() const
	{
		return Value != INDEX_NONE;
	}

	explicit operator bool() const
	{
		return Value != INDEX_NONE;
	}

	operator int32() const
	{
		check(Value != INDEX_NONE);
		return Value;
	}

	friend uint32 GetTypeHash(const FPreAnimatedStorageGroupHandle& InHandle)
	{
		return ::GetTypeHash(InHandle.Value);
	}
	friend bool operator==(const FPreAnimatedStorageGroupHandle& A, const FPreAnimatedStorageGroupHandle& B)
	{
		return A.Value == B.Value;
	}

	int32 Value;
};


/**
 * A handle to a particular pre-animated value within a FPreAnimatedStateExtension instance
 */
struct FPreAnimatedStateCachedValueHandle
{
	/** The type identifier for the unique storage implementation (see FPreAnimatedStateExtension::StorageImplementations) */
	FPreAnimatedStorageID TypeID;

	/** The index to the specific value within the storage implementation */
	FPreAnimatedStorageIndex StorageIndex;

	friend uint32 GetTypeHash(const FPreAnimatedStateCachedValueHandle& InEntry)
	{
		return GetTypeHash(InEntry.TypeID) ^ GetTypeHash(InEntry.StorageIndex);
	}
	friend bool operator==(const FPreAnimatedStateCachedValueHandle& A, const FPreAnimatedStateCachedValueHandle& B)
	{
		return A.TypeID == B.TypeID && A.StorageIndex == B.StorageIndex;
	}
};


/**
 * Specifies an entry to a specific piece of pre-animated state within a FPreAnimatedStateExtension instance.
 * An entry is represented by a group handle (usually relating to a UObject* retrieved from FPreAnimatedObjectGroupManager)
 * and the handle to the specific storage entry
 */
struct FPreAnimatedStateEntry
{
	/** An index into FPreAnimatedStateExtension::GroupMetaData */
	FPreAnimatedStorageGroupHandle GroupHandle;
	/** A handle for the actual storage location of the value */
	FPreAnimatedStateCachedValueHandle ValueHandle;

	bool IsValid() const
	{
		return ValueHandle.StorageIndex.IsValid();
	}

	explicit operator bool() const
	{
		return ValueHandle.StorageIndex.IsValid();
	}

	friend uint32 GetTypeHash(const FPreAnimatedStateEntry& InEntry)
	{
		return GetTypeHash(InEntry.GroupHandle) ^ GetTypeHash(InEntry.ValueHandle);
	}
	friend bool operator==(const FPreAnimatedStateEntry& A, const FPreAnimatedStateEntry& B)
	{
		return A.GroupHandle == B.GroupHandle && A.ValueHandle == B.ValueHandle;
	}
};


/**
 * Meta-data pertaining to a specific animating source (could be a FMovieSceneEntityID, FMovieSceneEvaluationKey, UMovieSceneTrackInstance etc)
 * Which specifies an entry to a piece of pre-animated state, the RootInstance that it's playing from, and whether it wants to restore state or not
 */
struct FPreAnimatedStateMetaData
{
	/** The pre-animated state entry */
	FPreAnimatedStateEntry Entry;
	/** The instance handle for the root sequence */
	FRootInstanceHandle RootInstanceHandle;
	/** True if this entry should be restored when it is removed */
	bool bWantsRestoreState;
};
using FPreAnimatedStateMetaDataArray = TArray<FPreAnimatedStateMetaData, TInlineAllocator<1>>;




} // namespace MovieScene
} // namespace UE
