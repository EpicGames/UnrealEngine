// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/TypeHash.h"

namespace UE
{
namespace MovieScene
{

struct FPreAnimatedStorageID
{
public:

	FPreAnimatedStorageID() = default;

	explicit operator bool() const
	{
		return TypeID != ~0u;
	}

	friend uint32 GetTypeHash(FPreAnimatedStorageID In)
	{
		return ::GetTypeHash(In.TypeID);
	}

	friend bool operator<(FPreAnimatedStorageID A, FPreAnimatedStorageID B)
	{
		return A.TypeID < B.TypeID;
	}

	friend bool operator==(FPreAnimatedStorageID A, FPreAnimatedStorageID B)
	{
		return A.TypeID == B.TypeID;
	}

private:
	friend struct FPreAnimatedStateExtension;

	explicit FPreAnimatedStorageID(uint32 InTypeID)
		: TypeID(InTypeID)
	{}

	uint32 TypeID = ~0u;
};

template<typename StorageType>
struct TPreAnimatedStorageID : FPreAnimatedStorageID
{
};

template<typename StorageType>
struct TAutoRegisterPreAnimatedStorageID : TPreAnimatedStorageID<StorageType>
{
	// Definition is defined inside MovieScenePreAnimatedStorageID.inl
	TAutoRegisterPreAnimatedStorageID();
};



} // namespace MovieScene
} // namespace UE
