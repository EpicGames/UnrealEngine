// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateExtension.h"

namespace UE
{
namespace MovieScene
{


template<typename StorageType>
TAutoRegisterPreAnimatedStorageID<StorageType>::TAutoRegisterPreAnimatedStorageID()
	: TPreAnimatedStorageID<StorageType>{ FPreAnimatedStateExtension::template RegisterStorage<StorageType>() }
{}


} // namespace MovieScene
} // namespace UE
