// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"

class IMovieScenePlayer;
class UMovieSceneEntitySystemLinker;

namespace UE
{
namespace MovieScene
{

struct FSharedPlaybackState;

/**
 * Parameters that are passed to IMovieScenePreAnimatedToken::RestoreState and IMovieScenePreAnimatedGlobalToken::RestoreState
 */
struct FRestoreStateParams
{
	/** The linker that originally cached the pre-animated state */
	UMovieSceneEntitySystemLinker* Linker;

	/**
	 * The instance handle that relates to the last sequence that finished animating this state.
	 * Can be invalid in test harnesses.
	 */
	FInstanceHandle TerminalInstanceHandle;

	/**
	 * Retrieve a pointer to the player that is causing this state to be restored.
	 * @note: may not be the same player that originally cached the state in the case where
	 * 2 completely different sequences animated the same object at the same time.
	 * May be null in test harnesses
	 */
	MOVIESCENE_API IMovieScenePlayer* GetTerminalPlayer() const;

	/**
	 * Retrieve a pointer to the shared playback state that is causing this state to be restored.
	 * @note: may not relate to the same sequence instance that originally cached the state in the case where
	 * 2 completely different sequences animated the same object at the same time.
	 * May be null in test harnesses
	 */
	MOVIESCENE_API TSharedPtr<FSharedPlaybackState> GetTerminalPlaybackState() const;
};


} // namespace MovieScene
} // namespace UE
