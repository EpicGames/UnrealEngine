// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "Evaluation/IMovieScenePlaybackCapability.h"
#include "Evaluation/MovieScenePlaybackCapabilities.h"
#include "MovieSceneSequenceID.h"
#include "UObject/GCObject.h"

namespace UE::MovieScene
{

struct FSharedPlaybackState;

/**
 * Playback capability for sequences that have a director blueprint.
 */
struct FSequenceDirectorPlaybackCapability : public IPlaybackCapability
{
	UE_DECLARE_MOVIESCENE_PLAYBACK_CAPABILITY_API(MOVIESCENE_API, FSequenceDirectorPlaybackCapability)

public:

	FSequenceDirectorPlaybackCapability() {}
	FSequenceDirectorPlaybackCapability(bool bInUseStrongCache) : bUseStrongCache(bInUseStrongCache) {}

public:

	/** Remove all director blueprint instances */
	MOVIESCENE_API void ResetDirectorInstances();

	/** Gets a new or existing director blueprint instance for the given root or sub sequence */
	MOVIESCENE_API UObject* GetOrCreateDirectorInstance(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, FMovieSceneSequenceIDRef SequenceID);

public:

	// IPlaybackCapability interface.
	MOVIESCENE_API virtual void InvalidateCachedData(UMovieSceneEntitySystemLinker* Linker) override;

private:

	// The actual cache of BP instances is stored in a heap-allocated object
	// because it neeeds to be an FGCObject to keep those instances alive, and
	// an FGCObject isn't relocatable so we can't put it in a playback 
	// capabailities container.
	struct FDirectorInstanceCache : FGCObject
	{
		virtual FString GetReferencerName() const override;
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		TSortedMap<FMovieSceneSequenceID, TObjectPtr<UObject>> DirectorInstances;
	};
	TUniquePtr<FDirectorInstanceCache> Cache;

	/** An alternative cache using weak-pointers. */
	TSortedMap<FMovieSceneSequenceID, TWeakObjectPtr<UObject>> WeakCache;

	bool bUseStrongCache = true;
};

}  // namespace UE::MovieScene

