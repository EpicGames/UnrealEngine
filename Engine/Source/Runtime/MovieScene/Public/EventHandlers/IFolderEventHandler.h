// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/List.h"
#include "CoreTypes.h"
#include "EventHandlers/MovieSceneDataEventContainer.h"

class UMovieSceneFolder;
class UMovieSceneTrack;
struct FGuid;
struct FMovieSceneBinding;

namespace UE
{
namespace MovieScene
{

class IFolderEventHandler
{
public:

	virtual void OnTrackAdded(UMovieSceneTrack* Track) {}
	virtual void OnTrackRemoved(UMovieSceneTrack* Track) {}

	virtual void OnObjectBindingAdded(const FGuid& ObjectBinding) {}
	virtual void OnObjectBindingRemoved(const FGuid& ObjectBinding) {}

	virtual void OnChildFolderAdded(UMovieSceneFolder* Folder) {}
	virtual void OnChildFolderRemoved(UMovieSceneFolder* Folder) {}

	virtual void OnPostUndo() {}
};

} // namespace MovieScene
} // namespace UE

