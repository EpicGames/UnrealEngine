// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/List.h"
#include "CoreTypes.h"
#include "EventHandlers/MovieSceneDataEventContainer.h"

class UMovieSceneSection;

namespace UE
{
namespace MovieScene
{

class ISectionEventHandler
{
public:

	virtual void OnRowChanged(UMovieSceneSection*) {}

	virtual void OnDecorationAdded(UObject* AddedDecoration) {}

	virtual void OnDecorationRemoved(UObject* RemovedDecoration) {}
};

} // namespace MovieScene
} // namespace UE

