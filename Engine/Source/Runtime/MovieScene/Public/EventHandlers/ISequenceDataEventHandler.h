// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/List.h"
#include "CoreTypes.h"
#include "EventHandlers/MovieSceneDataEventContainer.h"

class UMovieScene;
class UMovieSceneFolder;
class UMovieSceneTrack;
class UMovieSceneClock;
struct FGuid;
struct FMovieSceneBinding;

enum class EUpdateClockSource : uint8;

namespace UE
{
namespace MovieScene
{

class ISequenceDataEventHandler
{
public:

	virtual void OnTrackAdded(UMovieSceneTrack*)           {}

	virtual void OnTrackRemoved(UMovieSceneTrack*)         {}

	virtual void OnBindingAdded(const FMovieSceneBinding&) {}

	virtual void OnBindingRemoved(const FGuid&)            {}

	virtual void OnRootFolderAdded(UMovieSceneFolder*)     {}

	virtual void OnRootFolderRemoved(UMovieSceneFolder*)   {}

	virtual void OnTrackAddedToBinding(UMovieSceneTrack* Track, const FGuid& ObjectBindingID) {}

	virtual void OnTrackRemovedFromBinding(UMovieSceneTrack* Track, const FGuid& ObjectBindingID) {}

	virtual void OnBindingParentChanged(const FGuid& ObjectBindingID, const FGuid& NewParent) {}

	virtual void OnClockChanged(EUpdateClockSource NewClock, UMovieSceneClock* NewCustomClock) {}

	virtual void OnDecorationAdded(UObject* AddedDecoration) {}

	virtual void OnDecorationRemoved(UObject* RemovedDecoration) {}

};

} // namespace MovieScene
} // namespace UE

