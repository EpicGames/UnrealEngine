// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/List.h"
#include "CoreTypes.h"
#include "EventHandlers/MovieSceneDataEventContainer.h"

class UMovieSceneSignedObject;

namespace UE
{
namespace MovieScene
{

class ISignedObjectEventHandler
{
public:

	virtual void OnModifiedIndirectly(UMovieSceneSignedObject*) {}
	virtual void OnModifiedDirectly(UMovieSceneSignedObject*) {}

#if WITH_EDITOR
	virtual void OnPostUndo() {}
#endif
};



} // namespace MovieScene
} // namespace UE

