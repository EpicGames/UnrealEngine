// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneFwd.h"
#include "Delegates/Delegate.h"
#include "Containers/List.h"

namespace UE
{
namespace MovieScene
{

struct FDataEventScope
{
	FSimpleMulticastDelegate OnBracketClosed;

	MOVIESCENE_API FDataEventScope();
	MOVIESCENE_API ~FDataEventScope();
};

template<typename T>
class TIntrusiveEventHandler : public TIntrusiveLinkedList<TIntrusiveEventHandler<T>>, public T
{
protected:
	virtual ~TIntrusiveEventHandler()
	{
		this->Unlink();
	}
};

template<typename T>
struct TNonIntrusiveEventHandler : public TLinkedList<T*>
{
	TNonIntrusiveEventHandler()
	{}

	TNonIntrusiveEventHandler(T* InInstance)
		: TLinkedList<T*>(InInstance)
	{}

	~TNonIntrusiveEventHandler()
	{
		this->Unlink();
	}
};

template<typename EventInterface>
struct TDataEventContainer
{
#if UE_MOVIESCENE_EVENTS
	~TDataEventContainer()
	{
		if (IntrusiveHandlers)
		{
			IntrusiveHandlers->Unlink();
		}
		if (NonIntrusiveHandlers)
		{
			NonIntrusiveHandlers->Unlink();
		}
	}

	template<typename FuncType, typename... ArgTypes>
	void Trigger(FuncType&& Func, ArgTypes&&... Args) const
	{
		for (typename TIntrusiveLinkedList<TIntrusiveEventHandler<EventInterface>>::TIterator It(IntrusiveHandlers); It; )
		{
			TIntrusiveEventHandler<EventInterface>& Current = *It;
			// Increment before invoking to ensure that removals inside the invocation do not cause the iteration to return prematurely
			++It;

			Invoke(Func, Current, Forward<ArgTypes>(Args)...);
		}
		for (typename TNonIntrusiveEventHandler<EventInterface>::TIterator It(NonIntrusiveHandlers); It; )
		{
			EventInterface* Current = *It;
			// Increment before invoking to ensure that removals inside the invocation do not cause the iteration to return prematurely
			++It;

			Invoke(Func, Current, Forward<ArgTypes>(Args)...);
		}
	}

	void Link(TIntrusiveEventHandler<EventInterface>* InLink) const
	{
		check(!InLink->IsLinked());
		InLink->LinkHead(IntrusiveHandlers);
	}

	void Link(TNonIntrusiveEventHandler<EventInterface>& InLink, EventInterface* Instance) const
	{
		check(!InLink.IsLinked());
		InLink = TNonIntrusiveEventHandler<EventInterface>(Instance);
		InLink.LinkHead(NonIntrusiveHandlers);
	}

private:

	mutable TIntrusiveEventHandler<EventInterface>* IntrusiveHandlers = nullptr;
	mutable TLinkedList<EventInterface*>* NonIntrusiveHandlers = nullptr;

#else

	template<typename ...ArgTypes>
	void Trigger(ArgTypes&&...) const
	{}

#endif
};

} // namespace MovieScene
} // namespace UE

