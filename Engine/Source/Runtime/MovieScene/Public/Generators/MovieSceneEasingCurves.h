// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Curves/RichCurve.h"
#include "Generators/MovieSceneEasingFunction.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "UObject/Class.h"
#include "UObject/Interface.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneEasingCurves.generated.h"

class UCurveFloat;

UENUM()
enum class EMovieSceneBuiltInEasing : uint8
{
	// Linear easing
	Linear UMETA(Grouping=Linear,DisplayName="Linear"),
	// Sinusoidal easing
	SinIn UMETA(Grouping=Sinusoidal,DisplayName="Sinusoidal In"), SinOut UMETA(Grouping=Sinusoidal,DisplayName="Sinusoidal Out"), SinInOut UMETA(Grouping=Sinusoidal,DisplayName="Sinusoidal InOut"),
	// Quadratic easing
	QuadIn UMETA(Grouping=Quadratic,DisplayName="Quadratic In"), QuadOut UMETA(Grouping=Quadratic,DisplayName="Quadratic Out"), QuadInOut UMETA(Grouping=Quadratic,DisplayName="Quadratic InOut"),
	// Cubic easing
	Cubic UMETA(Grouping = Cubic, DisplayName = "Cubic"), CubicIn UMETA(Grouping=Cubic,DisplayName="Cubic In"), CubicOut UMETA(Grouping=Cubic,DisplayName="Cubic Out"), CubicInOut UMETA(Grouping=Cubic,DisplayName="Cubic InOut"), HermiteCubicInOut UMETA(Grouping = Cubic, DisplayName = "Hermite-Cubic InOut"),
	// Quartic easing
	QuartIn UMETA(Grouping=Quartic,DisplayName="Quartic In"), QuartOut UMETA(Grouping=Quartic,DisplayName="Quartic Out"), QuartInOut UMETA(Grouping=Quartic,DisplayName="Quartic InOut"),
	// Quintic easing
	QuintIn UMETA(Grouping=Quintic,DisplayName="Quintic In"), QuintOut UMETA(Grouping=Quintic,DisplayName="Quintic Out"), QuintInOut UMETA(Grouping=Quintic,DisplayName="Quintic InOut"),
	// Exponential easing
	ExpoIn UMETA(Grouping=Exponential,DisplayName="Exponential In"), ExpoOut UMETA(Grouping=Exponential,DisplayName="Exponential Out"), ExpoInOut UMETA(Grouping=Exponential,DisplayName="Exponential InOut"),
	// Circular easing
	CircIn UMETA(Grouping=Circular,DisplayName="Circular In"), CircOut UMETA(Grouping=Circular,DisplayName="Circular Out"), CircInOut UMETA(Grouping=Circular,DisplayName="Circular InOut"),
	// Custom
	Custom UMETA(Grouping = Custom, DisplayName = "Custom"),
};


UCLASS(BlueprintType, meta=(DisplayName = "Built-In Function"), MinimalAPI)
class UMovieSceneBuiltInEasingFunction : public UObject, public IMovieSceneEasingFunction
{
public:
	GENERATED_BODY()

	MOVIESCENE_API UMovieSceneBuiltInEasingFunction(const FObjectInitializer& Initiailizer);

	MOVIESCENE_API virtual float Evaluate(float InTime) const override;

#if WITH_EDITOR
	virtual FText GetDisplayName() const override { return StaticEnum<EMovieSceneBuiltInEasing>()->GetDisplayNameTextByValue((int64)Type); }
#endif

	UPROPERTY(EditAnywhere, Category=Easing)
	EMovieSceneBuiltInEasing Type;
};


UCLASS(meta=(DisplayName="Curve Asset"), MinimalAPI)
class UMovieSceneEasingExternalCurve : public UObject, public IMovieSceneEasingFunction
{
public:
	GENERATED_BODY()

	MOVIESCENE_API virtual float Evaluate(float InTime) const override;

#if WITH_EDITOR
	virtual FText GetDisplayName() const override { return StaticClass()->GetDisplayNameText(); }
#endif

	/** Curve data */
	UPROPERTY(EditAnywhere, Category=Easing)
	TObjectPtr<UCurveFloat> Curve;
};
