// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "MovieSceneEasingFunction.generated.h"


UINTERFACE(Category="Sequencer", Blueprintable, meta=(DisplayName = "Easing Function"), MinimalAPI)
class UMovieSceneEasingFunction : public UInterface
{
public:
	GENERATED_BODY()
};

class IMovieSceneEasingFunction
{
public:
	GENERATED_BODY()

	/**
	 * Evaluate using the specified script interface. Handles both native and k2 implemented interfaces.
	 */
	static MOVIESCENE_API float EvaluateWith(const TScriptInterface<IMovieSceneEasingFunction>& ScriptInterface, float Time);

#if WITH_EDITOR
	virtual FText GetDisplayName() const { return FText::GetEmpty(); }
#endif

protected:

	/** Evaluate the easing with an interpolation value between 0 and 1 */
	UFUNCTION(BlueprintCallable, BlueprintImplementableEvent, Category = "Sequencer|Section", meta=(CallInEditor="true"))
	MOVIESCENE_API float OnEvaluate(float Interp) const;

	/** Evaluate the easing with an interpolation value between 0 and 1 */
	virtual float Evaluate(float Interp) const { return 0.f; }
};
