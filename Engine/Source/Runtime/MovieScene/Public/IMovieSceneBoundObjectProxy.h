// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "IMovieSceneBoundObjectProxy.generated.h"


UINTERFACE(MinimalAPI)
class UMovieSceneBoundObjectProxy
	: public UInterface
{
public:
	GENERATED_BODY()
};


class IMovieSceneBoundObjectProxy
{
public:
	GENERATED_BODY()

	/**
	 * Retrieve the bound object that this interface wants to animate. Could be 'this' or a transient child object.
	 *
	 * @return Pointer to the object that should be animated, or nullptr if it's not valid.
	 */
	UFUNCTION(BlueprintCallable, BlueprintImplementableEvent, Category = "Sequencer", DisplayName = "GetBoundObjectForSequencer", meta=(CallInEditor="true"))
	MOVIESCENE_API UObject* BP_GetBoundObjectForSequencer(UObject* ResolvedObject);
	virtual UObject* NativeGetBoundObjectForSequencer(UObject* ResolvedObject) { return ResolvedObject; }
};
