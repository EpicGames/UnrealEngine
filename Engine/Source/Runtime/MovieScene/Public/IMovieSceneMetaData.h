// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/Interface.h"
#include "IMovieSceneMetaData.generated.h"

UINTERFACE(MinimalAPI)
class UMovieSceneMetaDataInterface : public UInterface
{
	GENERATED_BODY()
};

/**
 * Interface that may be implemented by meta-data added movie scene objects that can extend the default behavior
 * such as adding asset registry tags and other meta-data.
 */
class IMovieSceneMetaDataInterface
{
public:

	GENERATED_BODY()

	/**
	 * Called from ULevelSequence::GetAssetRegistryTags in order to
	 * extend its default set of tags to include any from this meta-data object.
	 */
	virtual void ExtendAssetRegistryTags(FAssetRegistryTagsContext Context) const {}
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	virtual void ExtendAssetRegistryTags(TArray<UObject::FAssetRegistryTag>& OutTags) const {}

#if WITH_EDITOR

	/**
	 * Called from ULevelSequence::GetAssetRegistryTagMetadata in order to
	 * extend its default set of tag meta-data to include any from this meta-data object.
	 */
	virtual void ExtendAssetRegistryTagMetaData(TMap<FName, UObject::FAssetRegistryTagMetadata>& OutMetadata) const {}

#endif
};
