// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Decorations/MovieSceneDecorationContainer.h"

struct FMovieSceneEvaluationGroupParameters
{
	FMovieSceneEvaluationGroupParameters()
		: EvaluationPriority(0xFF)
	{
	}

	FMovieSceneEvaluationGroupParameters(uint16 InPriority)
		: EvaluationPriority(InPriority)
	{}

	/** Prioirty assigned to this group. Higher priorities are evaluated first */
	uint16 EvaluationPriority;
};

/**
 * The public interface of the MovieScene module
 */
class IMovieSceneModule
	: public IModuleInterface
{
public:

	/**
	 * Singleton-like access to IMovieScene
	 *
	 * @return Returns MovieScene singleton instance, loading the module on demand if needed
	 */
	static inline IMovieSceneModule& Get()
	{
		return FModuleManager::LoadModuleChecked< IMovieSceneModule >( "MovieScene" );
	}

	static inline IMovieSceneModule& Get_Concurrent()
	{
		return FModuleManager::GetModuleChecked< IMovieSceneModule >("MovieScene");
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded( "MovieScene" );
	}

	/**
	 * Register template parameters for compilation
	 */
	virtual void RegisterEvaluationGroupParameters(FName GroupName, const FMovieSceneEvaluationGroupParameters& GroupParameters) = 0;

	/**
	 * Find group parameters for a specific evaluation group
	 */
	virtual FMovieSceneEvaluationGroupParameters GetEvaluationGroupParameters(FName GroupName) const = 0;

	/**
	 * Get this module ptr as a weak ptr.
	 * @note: resulting weak ptr should not be used to hold persistent strong references
	 */
	virtual TWeakPtr<IMovieSceneModule> GetWeakPtr() = 0;
	
	/**
	 * Register a compatible decoration class for a specific container type. 
	 * This allows modules to externally register compatible decorations for a container type without the container type itself explicitly defining that compatibility.
	 */
	virtual void RegisterCompatibleDecoration(TSubclassOf<UMovieSceneDecorationContainerObject> ContainerClass, TSubclassOf<UObject> DecorationClass) = 0;

	/**
	 * Unregister a compatible decoration class for a specific container type.
	 */
	virtual void UnregisterCompatibleDecoration(TSubclassOf<UMovieSceneDecorationContainerObject> ContainerClass, TSubclassOf<UObject> DecorationClass) = 0;

	/**
	 * Get all compatible decoration classes for a specific container instance.
	 * This includes both decorations internally marked compatible by a container type and externally registered decorations.
	 */
	virtual void GetCompatibleDecorationsForContainer(const UMovieSceneDecorationContainerObject* Container, TSet<UClass*>& OutClasses) const = 0;

	/**
	 * Get every decoration class registered as compatible with at least one container type.
	 */
	virtual void GetAllRegisteredDecorationClasses(TSet<UClass*>& OutClasses) const = 0;
};
