// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MovieSceneSequenceID.h"
#include "Templates/ValueOrError.h"

struct FMovieSceneSpawnable;
class IMovieScenePlayer;
class UMovieScene;
class IMovieSceneObjectSpawner;
class ISequencer;
class USequencerSettings;
class UActorFactory;
struct FTransformData;

namespace UE::MovieScene
{
	struct FSharedPlaybackState;
}

/** A delegate which will create an object spawner */
DECLARE_DELEGATE_RetVal(TSharedRef<IMovieSceneObjectSpawner>, FOnCreateMovieSceneObjectSpawner);

/** Struct used for defining a new spawnable type */
struct FNewSpawnable
{
	FNewSpawnable() : ObjectTemplate(nullptr) {}
	FNewSpawnable(UObject* InObjectTemplate, FString InName) : ObjectTemplate(InObjectTemplate), Name(MoveTemp(InName)) {}

	/** The archetype object template that defines the spawnable */
	UObject* ObjectTemplate;

	/** The desired name of the new spawnable */
	FString Name;
};


/** Interface used to extend spawn registers to support extra types */
class IMovieSceneObjectSpawner
{
public:
	/**
	 * Returns the type of object we can spawn
	 *
	 * @return the class of the object we support for spawning
	 */
	virtual UClass* GetSupportedTemplateType() const = 0;

	/**
	 * Spawn an object for the specified GUID, from the specified sequence instance.
	 *
	 * @param Object 		ID of the object to spawn
	 * @param TemplateID 	Identifier for the current template we're evaluating
	 * @param Player 		Movie scene player that is ultimately responsible for spawning the object
	 * @return the spawned object, or nullptr on failure
	 */
	virtual UObject* SpawnObject(FMovieSceneSpawnable& Spawnable, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) = 0;

	/**
	 * Destroy a specific previously spawned object
	 *
	 * @param Object 		The object to destroy
	 */
	virtual void DestroySpawnedObject(UObject& Object) = 0;

	/** 
	  * @return true if this spawner is used in the editor, or false if it is purely runtime. We use this to prioritize the
	  * use of editor spawners when in editor
	  */
	virtual bool IsEditor() const { return false; }

	/** 
	  * @return Spawner priority in order to sort the list of the spawners. If several spawners 
	  *	match the template type, the one with the highest priority will be picked. 
	  */
	virtual int32 GetSpawnerPriority() const { return 0; }

#if WITH_EDITOR
	
	/**
	 * Create a new spawnable type from the given source object
	 *
	 * @param SourceObject		The source object to create the spawnable from
	 * @param OwnerMovieScene	The owner movie scene that this spawnable type should reside in
	 * @param ActorFactory      Optional actor factory to use to create spawnable type
	 * @return the new spawnable type, or error
	 */
	virtual TValueOrError<FNewSpawnable, FText> CreateNewSpawnableType(UObject& SourceObject, UMovieScene& OwnerMovieScene, UActorFactory* ActorFactory = nullptr) { return MakeError(NSLOCTEXT("IMovieSceneObjectSpawner", "NotSupported", "Not supported")); }

	/**
	 * Setup a new spawnable object with some default tracks and keys
	 *
	 * @param	SpawnedObject	The newly spawned object. This may be NULL in the case of a spawnable that has not yet neen spawned.
	 * @param	Guid			The ID of the spawnable to setup defaults for
	 * @param	TransformData	The transform of the object to be spawned. This will usually be valid in the case of converting a possessable to a spawnable.
	 * @param	Sequencer		The sequencer this spawnable was just created by
	 * @param	Settings		The settings for this sequencer
	 */
	virtual void SetupDefaultsForSpawnable(UObject* SpawnedObject, const FGuid& Guid, const TOptional<FTransformData>& TransformData, TSharedRef<ISequencer> Sequencer, USequencerSettings* Settings) {}

	/*
 	 * Whether this spawner can set up defaults
	 *
	 * @param	SpawnedObject	The newly spawned object. This may be NULL in the case of a spawnable that has not yet been spawned.
	 * @return whether this spawned can set up defaults
	 */
	virtual bool CanSetupDefaultsForSpawnable(UObject* SpawnedObject) const { return SpawnedObject ? SpawnedObject->IsA(GetSupportedTemplateType()) : false; }

	/**
	 * Check whether the specified Spawnable can become a Possessable.
	 * @param	Spawnable	The spawnable to check
	 * @return whether the conversion from Spawnable to Possessable can occur.
	 */
	virtual bool CanConvertSpawnableToPossessable(FMovieSceneSpawnable& Spawnable) const { return true; }

#endif

public:

	/** Virtual destructor. */
	virtual ~IMovieSceneObjectSpawner() = default;
};
