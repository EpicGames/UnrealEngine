// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "Evaluation/IMovieScenePlaybackCapability.h"
#include "MovieSceneSequenceID.h"
#include "IMovieScenePlaybackClient.generated.h"

struct FMovieSceneEvaluationRange;
class UMovieSceneSequence;

/** Movie scene binding overrides interface */
UINTERFACE(MinimalAPI)
class UMovieScenePlaybackClient
	: public UInterface
{
public:
	GENERATED_BODY()
};


class IMovieScenePlaybackClient
{
public:
	GENERATED_BODY()

	UE_DECLARE_MOVIESCENE_PLAYBACK_CAPABILITY_API(MOVIESCENE_API, IMovieScenePlaybackClient)

	/**
	 * Locate bound objects that relate to the specified binding ID
	 *
	 * @param InBindingId 		The GUID of the object binding in the movie scene
	 * @param OutObjects 		Array to populate with bound objects
	 * @return true to prevent default lookup of the binding in the sequence itself, else false
	 */
	virtual bool RetrieveBindingOverrides(const FGuid& InBindingId, FMovieSceneSequenceID InSequenceID, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const = 0;

	/**
	 * Retrieve the optional instance data that should be used for this evaluation
	 */
	virtual UObject* GetInstanceData() const = 0;

	/**
	 * Whether this playback client wants a specific aspect ratio axis constraint during playback.
	 */
	virtual TOptional<EAspectRatioAxisConstraint> GetAspectRatioAxisConstraint() const
	{
		return TOptional<EAspectRatioAxisConstraint>();
	}

	/*
	 * Whether this playback client wants replicated playback.
	 */
	virtual bool GetIsReplicatedPlayback() const
	{
		return false;
	}

	/*
	 * Warp the time range right before we evaluate
	 * This gives clients an opportunity to apply time-warping effects without manipulating the actual user-facing playback position
	 */
	virtual void WarpEvaluationRange(FMovieSceneEvaluationRange& InOutRange) const {}
	
#if WITH_EDITOR
	/*
	 * Whether this playback client can play the given sequence in editor preview.
	 * This gives client actors an opportunity to add themselves to the list of preview contexts in the level editor.
	 */
	virtual bool CanPreview(const UMovieSceneSequence& InSequence) const
	{ 
		return false; 
	}
#endif
};
