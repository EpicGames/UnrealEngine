// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Containers/ArrayView.h"
#include "Misc/InlineValue.h"

#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "Evaluation/IMovieScenePlaybackCapability.h"
#include "Evaluation/MovieSceneAnimTypeID.h"
#include "Evaluation/MovieSceneEvaluationOperand.h"
#include "Evaluation/MovieSceneEvaluationState.h"
#include "Evaluation/MovieScenePreAnimatedState.h"
#include "Evaluation/SequenceDirectorPlaybackCapability.h"
#include "MovieSceneSpawnRegister.h"

enum class EMovieSceneBuiltInEasing : uint8;

struct FMovieSceneContext;
class UMovieSceneSequence;
class FViewportClient;
class IMovieScenePlaybackClient;
class UMovieSceneEntitySystemLinker;
struct FMovieSceneRootEvaluationTemplateInstance;
class FMovieSceneSequenceInstance;
class IMovieScenePlayer;
class IMovieSceneSequencePlayerObserver;
struct EMovieSceneViewportParams;

namespace UE::MovieScene
{
	enum class ESequenceInstanceUpdateFlags : uint8;
	struct FSharedPlaybackState;

	/**
	 * Playback capability for storing an IMovieScenePlayer unique index.
	 */
	struct FPlayerIndexPlaybackCapability
	{
		UE_DECLARE_MOVIESCENE_PLAYBACK_CAPABILITY_API(MOVIESCENE_API, FPlayerIndexPlaybackCapability)

		static MOVIESCENE_API IMovieScenePlayer* GetPlayer(TSharedRef<const FSharedPlaybackState> Owner);
		static MOVIESCENE_API uint16 GetPlayerIndex(TSharedRef<const FSharedPlaybackState> Owner);

		FPlayerIndexPlaybackCapability(uint16 InPlayerIndex)
			: PlayerIndex(InPlayerIndex)
		{}

		uint16 PlayerIndex = (uint16)-1;
	};
}

/** Camera cut parameters */
struct FMovieSceneCameraCutParams
{
	/** If this is not null, release actor lock only if currently locked to this object */
	UObject* UnlockIfCameraObject = nullptr;
	/** Whether this is a jump cut, i.e. the cut jumps from one shot to another shot */
	bool bJumpCut = false;

	/** Blending time to get to the new shot instead of cutting */
	float BlendTime = -1.f;
	/** Blending type to use to get to the new shot (only used when BlendTime is greater than 0) */
	TOptional<EMovieSceneBuiltInEasing> BlendType;

	/** The previous camera object, if any */
	UObject* PreviousCameraObject = nullptr;

	/** The computed blend factor, if blending is enabled */
	float PreviewBlendFactor = -1.f;

	/** When blending, whether to lock the previous camera */
	bool bLockPreviousCamera = false;
	
	/** Whether the camera cut track had blending enabled */
	bool bCanBlend = false;
};

/** Backwards compatibility to old struct name with typo */
using EMovieSceneCameraCutParams = FMovieSceneCameraCutParams; 

/**
 * Interface for movie scene players
 * Provides information for playback of a movie scene
 */
class IMovieScenePlayer 
	: public UE::MovieScene::IObjectBindingNotifyPlaybackCapability
{
public:
	MOVIESCENE_API IMovieScenePlayer();

	MOVIESCENE_API virtual ~IMovieScenePlayer();

	/**
	 * Access the evaluation template that we are playing back
	 */
	virtual FMovieSceneRootEvaluationTemplateInstance& GetEvaluationTemplate() = 0;

	/**
	 * Called to retrieve or construct an entity linker for the specified playback context
	 */
	virtual UMovieSceneEntitySystemLinker* ConstructEntitySystemLinker() { return nullptr; }

	/**
	 * Cast this player instance as a UObject if possible
	 */
	virtual UObject* AsUObject() { return nullptr; }

	/*
	 * Set the perspective viewport settings
	 *
	 * @param ViewportParamMap A map from the viewport client to its settings
	 */
	UE_DEPRECATED(5.5, "Viewport settings management has moved to FViewportSettingsPlaybackCapability")
	virtual void SetViewportSettings(const TMap<FViewportClient*, EMovieSceneViewportParams>& ViewportParamsMap) {}

	/*
	 * Get the current perspective viewport settings
	 *
	 * @param ViewportParamMap A map from the viewport client to its settings
	 */
	UE_DEPRECATED(5.5, "Viewport settings management has moved to FViewportSettingsPlaybackCapability")
	virtual void GetViewportSettings(TMap<FViewportClient*, EMovieSceneViewportParams>& ViewportParamsMap) const {}

	/** @return whether the player is currently playing, scrubbing, etc. */
	virtual EMovieScenePlayerStatus::Type GetPlaybackStatus() const = 0;

	/** 
	* @param PlaybackStatus The playback status to set
	*/
	virtual void SetPlaybackStatus(EMovieScenePlayerStatus::Type InPlaybackStatus) = 0;

	/**
	 * Resolve objects bound to the specified binding ID
	 *
	 * @param InBindingId	The ID relating to the object(s) to resolve
	 * @param OutObjects	Container to populate with the bound objects
	 */
	UE_DEPRECATED(5.4, "Please either call IMovieScenePlayer::FindBoundObjects, FMovieSceneObjectBindingID::ResolveBoundObjects, or FMovieSceneEvaluationState::FindBoundObjects")
	MOVIESCENE_API virtual void ResolveBoundObjects(const FGuid& InBindingId, FMovieSceneSequenceID SequenceID, UMovieSceneSequence& Sequence, UObject* ResolutionContext, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const;

	/**
	 * Access the client in charge of playback
	 *
	 * @return A pointer to the playback client, or nullptr if one is not available
	 */
	virtual IMovieScenePlaybackClient* GetPlaybackClient() { return nullptr; }

	const IMovieScenePlaybackClient* GetPlaybackClient() const { return const_cast<IMovieScenePlayer*>(this)->GetPlaybackClient(); }

	/**
	 * Obtain an object responsible for managing movie scene spawnables
	 */
	virtual FMovieSceneSpawnRegister& GetSpawnRegister() { return NullRegister; }

	/*
	 * Called when an object is spawned by sequencer
	 * 
	 */
	virtual void OnObjectSpawned(UObject* InObject, const FMovieSceneEvaluationOperand& Operand) {}

	/**
	 * Called whenever an object binding has been resolved to give the player a chance to interact with the objects before they are animated
	 * 
	 * @param InGuid		The guid of the object binding that has been resolved
	 * @param InSequenceID	The ID of the sequence in which the object binding resides
	 * @param Objects		The array of objects that were resolved
	 */
	virtual void NotifyBindingUpdate(const FGuid& InGuid, FMovieSceneSequenceIDRef InSequenceID, TArrayView<TWeakObjectPtr<>> Objects) override { NotifyBindingsChanged(); }

	/**
	 * Called whenever any object bindings have changed
	 */
	virtual void NotifyBindingsChanged() override {}

	/** 
	 * Retrieves any override for the given operand
	 */
	FMovieSceneEvaluationOperand* GetBindingOverride(const FMovieSceneEvaluationOperand& InOperand);

	/** 
	 * Adds an override for the given operand 
	 */
	void AddBindingOverride(const FMovieSceneEvaluationOperand& InOperand, const FMovieSceneEvaluationOperand& InOverrideOperand);

	/** 
	 * Removes any override set for the given operand
	 */
	void RemoveBindingOverride(const FMovieSceneEvaluationOperand& InOperand);

	/**
	 * Remove all director blueprint instances
	 */
	UE_DEPRECATED(5.4, "Director instances are now automanaged via FSequenceDirectorPlaybackCapability")
	void ResetDirectorInstances();

	/**
	 * Gets a new or existing director blueprint instance for the given root or sub sequence
	 */
	UE_DEPRECATED(5.4, "Director instances are now automanaged via FSequenceDirectorPlaybackCapability")
	UObject* GetOrCreateDirectorInstance(TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, FMovieSceneSequenceIDRef SequenceID);

	/**
	 * Called to initialize the flag structure that denotes what functions need to be called on this updater
	 */
	MOVIESCENE_API virtual void PopulateUpdateFlags(UE::MovieScene::ESequenceInstanceUpdateFlags& OutFlags);

	/**
	 * Access the playback context for this movie scene player
	 */
	virtual UObject* GetPlaybackContext() const { return nullptr; }

	/**
	 * Access the event contexts for this movie scene player
	 */
	MOVIESCENE_API virtual TArray<UObject*> GetEventContexts() const;

	/**
	 * Returns whether event triggers are disabled and if so, until what time.
	 */
	MOVIESCENE_API virtual bool IsDisablingEventTriggers(FFrameTime& DisabledUntilTime) const;

	/**
	 * Test whether this is a preview player or not. As such, playback range becomes insignificant for things like spawnables
	 */
	virtual bool IsPreview() const { return false; }

	/**
	 * Whether this player utilizes dynamic weighting
	 */
	virtual bool HasDynamicWeighting() const { return false; }

	/**
	 * Called by the evaluation system when evaluation has just started.
	 **/
	virtual void PreEvaluation(const FMovieSceneContext& Context) {}

	/**
	 * Called by the evaluation system after evaluation has occured
	 */
	virtual void PostEvaluation(const FMovieSceneContext& Context) {}

	/*
	* Used to access the Observer in MovieSceneSequencePlayer
	*/
	virtual TScriptInterface<IMovieSceneSequencePlayerObserver> GetObserver() { return nullptr; }

	/*
	* Attempts to create a binding for the given object in the given sequence.
	*/
	MOVIESCENE_API virtual FGuid CreateBinding(UMovieSceneSequence* InSequence, UObject* InObject);

public:

	/**
	 * Locate objects bound to the specified object guid, in the specified sequence
	 * @note: Objects lists are cached internally until they are invalidate.
	 *
	 * @param ObjectBindingID 		The object to resolve
	 * @param SequenceID 			ID of the sequence to resolve for
	 *
	 * @return Iterable list of weak object pointers pertaining to the specified GUID
	 */
	MOVIESCENE_API TArrayView<TWeakObjectPtr<>> FindBoundObjects(const FGuid& ObjectBindingID, FMovieSceneSequenceIDRef SequenceID);

	/**
	 * Locate objects bound to the specified sequence operand
	 * @note: Objects lists are cached internally until they are invalidate.
	 *
	 * @param Operand 			The movie scene operand to resolve
	 *
	 * @return Iterable list of weak object pointers pertaining to the specified GUID
	 */
	TArrayView<TWeakObjectPtr<>> FindBoundObjects(const FMovieSceneEvaluationOperand& Operand)
	{
		return FindBoundObjects(Operand.ObjectBindingID, Operand.SequenceID);
	}

	/**
	 * Attempt to find the object binding ID for the specified object, in the specified sequence
	 * @note: Will forcably resolve all out-of-date object mappings in the sequence
	 *
	 * @param InObject 			The object to find a GUID for
	 * @param SequenceID 		The sequence ID to search within
	 *
	 * @return The guid of the object's binding, or zero guid if it was not found
	 */
	FGuid FindObjectId(UObject& InObject, FMovieSceneSequenceIDRef SequenceID)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return State.FindObjectId(InObject, SequenceID, GetSharedPlaybackState());
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	* Attempt to find the object binding ID for the specified object, in the specified sequence
	* @note: Does not clear the existing cache
	*
	* @param InObject 			The object to find a GUID for
	* @param SequenceID 		The sequence ID to search within
	*
	* @return The guid of the object's binding, or zero guid if it was not found
	*/
	FGuid FindCachedObjectId(UObject& InObject, FMovieSceneSequenceIDRef SequenceID)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return State.FindCachedObjectId(InObject, SequenceID, GetSharedPlaybackState());
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Attempt to save specific state for the specified token state before it animates an object.
	 * @note: Will only call IMovieSceneExecutionToken::CacheExistingState if no state has been previously cached for the specified token type
	 *
	 * @param InObject			The object to cache state for
	 * @param InTokenType		Unique marker that identifies the originating token type
	 * @param InProducer		Producer implementation that defines how to create the preanimated token, if it doesn't already exist
	 */
	inline void SavePreAnimatedState(UObject& InObject, FMovieSceneAnimTypeID InTokenType, const IMovieScenePreAnimatedTokenProducer& InProducer)
	{
		PreAnimatedState.SavePreAnimatedState(InObject, InTokenType, InProducer);
	}

	/**
	 * Attempt to save specific state for the specified token state before it mutates state.
	 * @note: Will only call IMovieSceneExecutionToken::CacheExistingState if no state has been previously cached for the specified token type
	 *
	 * @param InTokenType		Unique marker that identifies the originating token type
	 * @param InProducer		Producer implementation that defines how to create the preanimated token, if it doesn't already exist
	 */
	inline void SavePreAnimatedState(FMovieSceneAnimTypeID InTokenType, const IMovieScenePreAnimatedGlobalTokenProducer& InProducer)
	{
		PreAnimatedState.SavePreAnimatedState(InTokenType, InProducer);
	}

	/**
	 * Restore all pre-animated state
	 */
	void RestorePreAnimatedState()
	{
		PreAnimatedState.RestorePreAnimatedState();
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		State.ClearObjectCaches(GetSharedPlaybackState());
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Discard all pre-animated state without restoring it
	 */
	void DiscardPreAnimatedState()
	{
		PreAnimatedState.DiscardPreAnimatedState();
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		State.ClearObjectCaches(GetSharedPlaybackState());
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Invalidate any cached state contained within this player causing all entities to be forcibly re-linked and evaluated
	 */
	MOVIESCENE_API void InvalidateCachedData();

	MOVIESCENE_API static IMovieScenePlayer* Get(uint16 InUniqueIndex);

	MOVIESCENE_API static void Get(TArray<IMovieScenePlayer*>& OutPlayers, bool bOnlyUnstoppedPlayers);

	MOVIESCENE_API static void SetIsEvaluatingFlag(uint16 InUniqueIndex, bool bIsUpdating);

	MOVIESCENE_API bool IsEvaluating() const;

	/**
	 * Returns the evaluated sequence instance's shared playback state, if any.
	 */
	MOVIESCENE_API TSharedPtr<UE::MovieScene::FSharedPlaybackState> FindSharedPlaybackState();
	MOVIESCENE_API TSharedPtr<const UE::MovieScene::FSharedPlaybackState> FindSharedPlaybackState() const;
	/**
	 * Returns the evaluated sequence instance's shared playback state, asserts if there is none.
	 */
	MOVIESCENE_API TSharedRef<UE::MovieScene::FSharedPlaybackState> GetSharedPlaybackState();
	MOVIESCENE_API TSharedRef<const UE::MovieScene::FSharedPlaybackState> GetSharedPlaybackState() const;

	uint16 GetUniqueIndex() const
	{
		return UniqueIndex;
	}

public:

	/**
	 * Initializes a new root sequence instance and its shared playback state.
	 * This adds all the player's playback capabilities to the given state.
	 */
	MOVIESCENE_API virtual void InitializeRootInstance(TSharedRef<UE::MovieScene::FSharedPlaybackState> NewSharedPlaybackState);

public:

	UE_DEPRECATED(5.4, "Camera cut management has moved to UMovieSceneCameraCutTrackInstance")
	virtual bool CanUpdateCameraCut() const { return true; }

	UE_DEPRECATED(5.4, "Camera cut management has moved to UMovieSceneCameraCutTrackInstance")
	virtual void UpdateCameraCut(UObject* CameraObject, UObject* UnlockIfCameraObject = nullptr, bool bJumpCut = false)
	{
		EMovieSceneCameraCutParams CameraCutParams;
		CameraCutParams.UnlockIfCameraObject = UnlockIfCameraObject;
		CameraCutParams.bJumpCut = bJumpCut;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		UpdateCameraCut(CameraObject, CameraCutParams);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UE_DEPRECATED(5.4, "Camera cut management has moved to UMovieSceneCameraCutTrackInstance")
	virtual void UpdateCameraCut(UObject* CameraObject, const EMovieSceneCameraCutParams& CameraCutParams) {}

protected:

	friend struct FMovieSceneObjectCache;
	/**
	 * Resolve objects bound to the specified binding ID
	 *
	 * @param InBindingId	The ID relating to the object(s) to resolve
	 * @param OutObjects	Container to populate with the bound objects
	 */
	MOVIESCENE_API virtual void ResolveBoundObjects(UE::UniversalObjectLocator::FResolveParams& ResolveParams, const FGuid& InBindingId, FMovieSceneSequenceID SequenceID, UMovieSceneSequence& Sequence, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const;

public:

	/** Gets the evaluation state that stores global state to do with the playback operation */
	MOVIESCENE_API FMovieSceneEvaluationState* GetEvaluationState();

	/** Gets the list of binding overrides to use for the sequence */
	MOVIESCENE_API UE::MovieScene::IStaticBindingOverridesPlaybackCapability* GetStaticBindingOverrides();

public:

	/** Evaluation state that stores global state to do with the playback operation */
	UE_DEPRECATED(5.6, "Please use GetEvaluationState().")
	FMovieSceneEvaluationState State;

	/** Container that stores any per-animated state tokens  */
	FMovieScenePreAnimatedState PreAnimatedState;

	/** List of binding overrides to use for the sequence */
	UE_DEPRECATED(5.6, "Please use GetStaticBindingOverrides().")
	TMap<FMovieSceneEvaluationOperand, FMovieSceneEvaluationOperand>& BindingOverrides;

private:

	/** Null register that asserts on use */
	FNullMovieSceneSpawnRegister NullRegister;

	/** Static binding overrides */
	UE::MovieScene::FStaticBindingOverrides StaticBindingOverrides;

	/** This player's unique Index */
	uint16 UniqueIndex;
};
