// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/FrameRate.h"

#define UE_API MOVIESCENE_API

struct FFrameNumber;
struct FFrameTime;
class UMovieScene;

namespace UE::MovieScene
{

struct IRetimingInterface
{
	virtual ~IRetimingInterface() = default;

	MOVIESCENE_API FFrameNumber RemapTime(FFrameNumber InTime) const;

	virtual double GetScale() const = 0;
	virtual FFrameTime RemapTime(FFrameTime InTime) const = 0;

	virtual TUniquePtr<IRetimingInterface> RecurseInto(UMovieScene* InMovieScene) const = 0;

	virtual void Begin(UMovieScene* InMovieScene) const = 0;
	virtual void End(UMovieScene* InMovieScene) const = 0;
};


struct FFrameRateRetiming : IRetimingInterface
{
	FFrameRate SourceRate;
	FFrameRate DestinationRate;

	UE_API FFrameRateRetiming(FFrameRate InSourceRate, FFrameRate InDestinationRate);

	UE_API virtual double GetScale() const override;
	UE_API virtual FFrameTime RemapTime(FFrameTime InTime) const override;

	UE_API virtual TUniquePtr<IRetimingInterface> RecurseInto(UMovieScene* InMovieScene) const override;

	UE_API virtual void Begin(UMovieScene* InMovieScene) const override;
	UE_API virtual void End(UMovieScene* InMovieScene) const override;
};


} // namespace UE::MovieScene

#undef UE_API
