// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "IMovieSceneSequencePlayerObserver.generated.h"


UINTERFACE(MinimalAPI)
class UMovieSceneSequencePlayerObserver : public UInterface
{
public:
	GENERATED_BODY()
};

class IMovieSceneSequencePlayerObserver
{
public:
	GENERATED_BODY()

	virtual bool CanObserveSequence() const = 0;

	virtual TObjectPtr<UObject> GetInstigator() { return nullptr; }
};
