// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"

class UMovieSceneTrack;
class UMovieSceneSection;

class IMovieSceneTrackVirtualAPI
{
protected:

	/**
	 * Add a section to this track.
	 *
	 * @param Section The section to add.
	 */
	virtual void AddSection(UMovieSceneSection& Section) PURE_VIRTUAL(IMovieSceneTrackVirtualAPI::AddSection,);

	/**
	 * Removes a section from this track.
	 *
	 * @param Section The section to remove.
	 */
	virtual void RemoveSection(UMovieSceneSection& Section) PURE_VIRTUAL(IMovieSceneTrackVirtualAPI::RemoveSection, );

	/**
	 * Removes a section from this track at a particular index
	 *
	 * @param SectionIndex The section index to remove.
	 */
	virtual void RemoveSectionAt(int32 SectionIndex) PURE_VIRTUAL(IMovieSceneTrackVirtualAPI::RemoveSectionAt, );

private:
	friend UMovieSceneTrack;

	void CallAddSection(UMovieSceneSection& Section)
	{
		AddSection(Section);
	}
	void CallRemoveSection(UMovieSceneSection& Section)
	{
		RemoveSection(Section);
	}
	void CallRemoveSectionAt(int32 SectionIndex)
	{
		RemoveSectionAt(SectionIndex);
	}
};