// Copyright Epic Games, Inc. All Rights Reserved.

/**
* Sequencer Animation Track Support interface - this is required for animation track to work
*/

#pragma once
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Channels/MovieSceneChannelEditorData.h"
#include "HAL/Platform.h"
#include "UObject/Interface.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "INodeAndChannelMappings.generated.h"

class FName;
class UMovieSceneSection;
class UObject;

/** Enumeration specifying the control type. Must match ERigControlType in order */
enum class FFBXControlRigTypeProxyEnum : uint8
{
	Bool,
	Float,
	Integer,
	Vector2D,
	Position,
	Scale,
	Rotator,
	Transform,
	TransformNoScale,
	EulerTransform
};

class UMovieSceneTrack;
struct FMovieSceneBoolChannel;
struct FMovieSceneByteChannel;
struct FMovieSceneDoubleChannel;
struct FMovieSceneFloatChannel;
struct FMovieSceneIntegerChannel;

// For import data onto channels directly
struct FRigControlFBXNodeAndChannels
{
	UMovieSceneTrack* MovieSceneTrack;
	FFBXControlRigTypeProxyEnum ControlType;
	FString NodeName;
	FName ControlName;
	
	TArray< FMovieSceneDoubleChannel*> DoubleChannels;
	TArray< FMovieSceneFloatChannel*> FloatChannels;
	//will really only have one ever.
	TArray< FMovieSceneBoolChannel*> BoolChannels;
	TArray< FMovieSceneIntegerChannel*> IntegerChannels;
	TArray< FMovieSceneByteChannel*> EnumChannels;

};

struct FControlRigFbxNodeMapping
{
	// The UE channel type name
	FName ChannelType;

	// The UE control type proxy
	FFBXControlRigTypeProxyEnum ControlType;

	// The UE channel transform attribute index (0-9, invalid if non-transform) 
	int8 ChannelAttrIndex = INDEX_NONE;

	// The FBX transform attribute index (0-9)
	uint8 FbxAttrIndex = 0;

	bool bNegate = false;
};

struct FControlRigFbxCurveData
{
	// The name of the control associated to this curve
	FName ControlName;

	// The type of the control associated to this curve
	FFBXControlRigTypeProxyEnum ControlType;

	// The name of the node holding this curve
	FString NodeName;

	// the name of the FBX node's attribute (i.e. 'X', 'Weight'...) holding this curve
	FString AttributeName;

	// the name of the FBX node's property (i.e. 'Rotation', 'Scale'...) holding the channel attribute. Usually empty for non transform curves
	FString AttributePropertyName;

	// @returns whether the control is a node in itself instead of being grouped as an attribute of another existing control's node 
	bool IsControlNode() const { return NodeName == ControlName.ToString(); }
};


UINTERFACE(meta = (CannotImplementInterfaceInBlueprint), MinimalAPI)
class UNodeAndChannelMappings : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class INodeAndChannelMappings
{
	GENERATED_IINTERFACE_BODY()

	/** Get The Node And Mappings for this Track. Note Callee is responsible for deleting 
	*  InSection The section from which to get the nodes and the channels from. If empty, up to implementor to decide which section to use, usually the Section To Key.
	*/
	virtual TArray<FRigControlFBXNodeAndChannels>* GetNodeAndChannelMappings(UMovieSceneSection* InSection)  = 0;
	
	/** Get Selected Nodes */
	virtual void GetSelectedNodes(TArray<FName>& OutSelectedNodes) = 0;

#if WITH_EDITOR
	/**
	 * @param MetaData The channel metadata
	 * @param OutCurveData The data to be associated to the FBX curve corresponding to the given channel 
	 */
	virtual bool GetFbxCurveDataFromChannelMetadata(const FMovieSceneChannelMetaData& MetaData, FControlRigFbxCurveData& OutCurveData) = 0;

#endif // WITH_EDITOR

};
