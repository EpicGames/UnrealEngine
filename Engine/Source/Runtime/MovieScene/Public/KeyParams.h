// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#include "KeyParams.generated.h"

UENUM()
enum class EMovieSceneKeyInterpolation : uint8
{
	/** Auto. */
	Auto UMETA(DisplayName="Auto"),

	/** User. */
	User UMETA(DisplayName="User"),

	/** Break. */
	Break UMETA(DisplayName="Break"),

	/** Linear. */
	Linear UMETA(DisplayName="Linear"),

	/** Constant. */
	Constant UMETA(DisplayName="Constant"),

	/** Smart Auto. */
	SmartAuto UMETA(DisplayName = "Smart Auto"),
};

/** Defines different modes for adding keyframes in sequencer. */
enum class ESequencerKeyMode
{
	/** Keyframes are being generated automatically by the user interacting with object properties or transforms. */
	AutoKey,
	/** The user has requested to manually add a keyframe.  Keys will be added for unchanged and empty components of multi-component tracks depending on current settings. */
	ManualKey,
	/** The user has requested to manually add a keyframe.  Keys will be added for unchanged and empty components of multi-component tracks regardless of the current settings. */
	ManualKeyForced
};
