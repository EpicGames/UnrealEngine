// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMovieSceneMetaData.h"
#include "UObject/Object.h"
#include "MovieSceneShotMetaData.generated.h"

#define UE_API MOVIESCENE_API

struct FAssetData;

/**
 * Holds meta data that is common for rating shots in cinematic workflows.
 * The purpose is to share this data among related tools.
 *
 * Stored in ULevelSequence as meta  data. 
 * @see ULevelSequenceShotMetaDataLibrary for managing data.
 */
UCLASS(MinimalAPI)
class UMovieSceneShotMetaData
	: public UObject
	, public IMovieSceneMetaDataInterface
{
	GENERATED_BODY()
public:

	/** The asset registry tag that contains whether this shot is good or not */
	static UE_API const FName AssetRegistryTag_bIsNoGood;
	/** The asset registry tag that contains whether this was flagged by a user */
	static UE_API const FName AssetRegistryTag_bIsFlagged;
	/** The asset registry tag that contains whether this was recorded. */
	static UE_API const FName AssetRegistryTag_bIsRecorded;
	/** The asset registry tag that contains whether this is a subsequence. */
	static UE_API const FName AssetRegistryTag_bIsSubSequence;
	/** The asset registry tag that contains its favorite status */
	static UE_API const FName AssetRegistryTag_FavoriteRating;

	static TArray<FName> GetAllTags()
	{
		return
		{
			AssetRegistryTag_bIsNoGood,
			AssetRegistryTag_bIsFlagged,
			AssetRegistryTag_bIsRecorded,
			AssetRegistryTag_bIsSubSequence,
			AssetRegistryTag_FavoriteRating
		};
	}

	const TOptional<bool>& GetIsNoGood() const { return bIsNoGood; }
	const TOptional<bool>& GetIsFlagged() const { return bIsFlagged; }
	const TOptional<bool>& GetIsRecorded() const { return bIsRecorded; }
	const TOptional<bool>& GetIsSubSequence() const { return bIsSubSequence; }
	const TOptional<int32>& GetFavoriteRating() const { return FavoriteRating; }

	void SetIsNoGood(bool bInIsNoGood) { bIsNoGood = bInIsNoGood; }
	void SetIsFlagged(bool bInIsFlagged) { bIsFlagged = bInIsFlagged; }
	void SetIsRecorded(bool bInIsRecorded) { bIsRecorded = bInIsRecorded; }
	void SetIsSubSequence(bool bInIsSubSequence) { bIsSubSequence = bInIsSubSequence; }
	void SetFavoriteRating(int32 InFavoriteRating) { FavoriteRating = InFavoriteRating; }
	
	void ClearIsNoGood() { bIsNoGood.Reset(); }
	void ClearIsFlagged() { bIsFlagged.Reset(); }
	void ClearIsRecorded() { bIsRecorded.Reset(); }
	void ClearIsSubSequence() { bIsSubSequence.Reset(); }
	void ClearFavoriteRating() { FavoriteRating.Reset(); }

	static UE_API bool GetIsNoGoodByAssetData(const FAssetData& InAssetData, bool& bOutNoGood);
	static UE_API bool GetIsFlaggedByAssetData(const FAssetData& InAssetData, bool& bOutIsFlagged);
	static UE_API bool GetIsRecordedByAssetData(const FAssetData& InAssetData, bool& bOutIsRecorded);
	static UE_API bool GetIsSubSequenceByAssetData(const FAssetData& InAssetData, bool& bOutIsSubSequence);
	static UE_API bool GetFavoriteRatingByAssetData(const FAssetData& InAssetData, int32& OutFavoriteRating); 

	// Begin IMovieSceneMetaDataInterface Interface
	UE_API virtual void ExtendAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
#if WITH_EDITOR
	UE_API virtual void ExtendAssetRegistryTagMetaData(TMap<FName, UObject::FAssetRegistryTagMetadata>& OutMetadata) const override;
#endif
	// End IMovieSceneMetaDataInterface Interface
	
private:

	/** Whether this shot is marked as not good */
	UPROPERTY()
	TOptional<bool> bIsNoGood;
	
	/** The asset registry tag that contains whether this was flagged by a user */
	UPROPERTY()
	TOptional<bool> bIsFlagged;

	/** If this sequence was recorded. */
	UPROPERTY()
	TOptional<bool> bIsRecorded;

	/** If this is a subsequence. */
	UPROPERTY()
	TOptional<bool> bIsSubSequence;
	
	/** The favorite rating is like a star rating, usually 1-3 if it was rated. */
	UPROPERTY()
	TOptional<int32> FavoriteRating;
};

#undef UE_API
