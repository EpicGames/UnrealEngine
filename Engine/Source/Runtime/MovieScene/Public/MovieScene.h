// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "EditorSequenceNavigationDefs.h"
#include "EventHandlers/MovieSceneDataEventContainer.h"
#include "Decorations/MovieSceneDecorationContainer.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Math/Range.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "Misc/Guid.h"
#include "MovieSceneBinding.h"
#include "MovieSceneFrameMigration.h"
#include "MovieSceneFwd.h"
#include "MovieSceneMarkedFrame.h"
#include "MovieSceneObjectBindingID.h"
#include "MovieScenePossessable.h"
#include "MovieSceneSequenceID.h"
#include "MovieSceneSignedObject.h"
#include "MovieSceneSpawnable.h"
#include "MovieSceneTimeController.h"
#include "MovieSceneTrack.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "MovieScene.generated.h"

class FArchive;
class FObjectPreSaveContext;
class UClass;
class UMovieSceneClock;
class UK2Node;
class UMovieSceneFolder;
class UMovieSceneSection;
class UMovieSceneTrack;
class UMovieSceneGroupCondition;
namespace UE { namespace MovieScene { class ISequenceDataEventHandler; } }
struct FMovieSceneChannelMetaData;
struct FMovieSceneTimeController;
struct FMovieSceneTimecodeSource;
template <typename FuncType> class TFunctionRef;

//delegates for use when some data in the MovieScene changes, WIP right now, hopefully will replace delegates on ISequencer
//and be used for moving towards a true MVC system

UE_DEPRECATED(5.8, "FMovieSceneOnChannelChanged is now deprecated. Use FMovieSceneOnChannelChangedWithTime")
DECLARE_MULTICAST_DELEGATE_TwoParams(FMovieSceneOnChannelChanged, const FMovieSceneChannelMetaData* MetaData, UMovieSceneSection*)

DECLARE_MULTICAST_DELEGATE_ThreeParams(FMovieSceneOnChannelChangedWithTime, const TArray<FFrameNumber>&, const FMovieSceneChannelMetaData* MetaData, UMovieSceneSection*)


/** @todo: remove this type when support for intrinsics on TMap values is added? */
USTRUCT()
struct FMovieSceneExpansionState
{
	GENERATED_BODY()

	FMovieSceneExpansionState(bool bInExpanded = true) : bExpanded(bInExpanded) {}

	UPROPERTY()
	bool bExpanded;
};

/**
 * Editor only data that needs to be saved between sessions for editing but has no runtime purpose
 */
USTRUCT()
struct FMovieSceneEditorData
{
	GENERATED_USTRUCT_BODY()

	FMovieSceneEditorData()
		: ViewStart(0.0), ViewEnd(0.0)
		, WorkStart(0.0), WorkEnd(0.0)
	{}

	TRange<double> GetViewRange() const
	{
		return TRange<double>(ViewStart, ViewEnd);
	}

	TRange<double> GetWorkingRange() const
	{
		return TRange<double>(WorkStart, WorkEnd);
	}

	/** Map of node path -> expansion state. */
	UPROPERTY()
	TMap<FString, FMovieSceneExpansionState> ExpansionStates;

	/** List of Pinned nodes. */
	UPROPERTY()
	TArray<FString> PinnedNodes;

	/** The last view-range start that the user was observing */
	UPROPERTY(Transient)
	double ViewStart;
	/** The last view-range end that the user was observing */
	UPROPERTY(Transient)
	double ViewEnd;

	/** User-defined working range start in which the entire sequence should reside. */
	UPROPERTY(Transient)
	double WorkStart;
	/** User-defined working range end in which the entire sequence should reside. */
	UPROPERTY(Transient)
	double WorkEnd;

	/** Deprecated */
	UPROPERTY()
	TSet<FFrameNumber> MarkedFrames_DEPRECATED;
	UPROPERTY()
	FFloatRange WorkingRange_DEPRECATED;
	UPROPERTY()
	FFloatRange ViewRange_DEPRECATED;

	/** Saved Sequence Navigator state to restore */
	UPROPERTY()
	FNavigationToolSaveState NavigationToolState;

	/** Nodes currently marked Solo, stored as node tree paths */
	UPROPERTY(Transient)
	TArray<FString> SoloNodes;

	/** Nodes currently marked Mute, stored as node tree paths */
	UPROPERTY(Transient)
	TArray<FString> MuteNodes;
};


/**
 * Structure for labels that can be assigned to movie scene tracks.
 */
USTRUCT()
struct FMovieSceneTrackLabels
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FString> Strings;

	void FromString(const FString& LabelString)
	{
		LabelString.ParseIntoArray(Strings, TEXT(" "));
	}

	FString ToString() const
	{
		return FString::Join(Strings, TEXT(" "));
	}
};


/**
 * Structure that comprises a list of object binding IDs
 */
USTRUCT()
struct FMovieSceneObjectBindingIDs
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FMovieSceneObjectBindingID> IDs;
};

/**
 * Structure that represents a group of sections
 */
USTRUCT()
struct FMovieSceneSectionGroup
{
	GENERATED_BODY()
public:
	/* @return Whether the section is part of this group */
	MOVIESCENE_API bool Contains(const UMovieSceneSection& Section) const;
	
	/* Add the section to this group */
	MOVIESCENE_API void Add(UMovieSceneSection& Section);

	/* Remove the section from this group */
	MOVIESCENE_API void Remove(const UMovieSceneSection& Section);

	/* Add all members of a group to this group */
	MOVIESCENE_API void Append(const FMovieSceneSectionGroup& SectionGroup);

	/* Removes any sections which the pointers are stale or otherwise not valid */
	MOVIESCENE_API void Clean();

	int32 Num() const { return Sections.Num(); }

protected:
	UPROPERTY()
	TArray<TWeakObjectPtr<UMovieSceneSection> > Sections;

public:
	/**
	 * Comparison operators
	 * We only need these for being stored in a container, to check if it's the same object.
	 * Not intended for direct use.
	 */
	inline bool operator==(const FMovieSceneSectionGroup& Other) const
	{
		return &Sections == &Other.Sections;
	}
	inline bool operator!=(const FMovieSceneSectionGroup& Other) const
	{
		return &Sections != &Other.Sections;
	}

	/**
	 * DO NOT USE DIRECTLY
	 * STL-like iterators to enable range-based for loop support.
	 */
	inline TArray<TWeakObjectPtr<UMovieSceneSection> >::RangedForIteratorType      begin() { return Sections.begin(); }
	inline TArray<TWeakObjectPtr<UMovieSceneSection> >::RangedForConstIteratorType begin() const { return Sections.begin(); }
	inline TArray<TWeakObjectPtr<UMovieSceneSection> >::RangedForIteratorType      end() { return Sections.end(); }
	inline TArray<TWeakObjectPtr<UMovieSceneSection> >::RangedForConstIteratorType end() const { return Sections.end(); }
};

/**
 * Structure that represents a group of nodes
 */
UCLASS(MinimalAPI)
class UMovieSceneNodeGroup : public UObject
{
	GENERATED_BODY()

	virtual bool IsEditorOnly() const override { return true; }

#if WITH_EDITORONLY_DATA
public:
	const FName GetName() const { return Name; }
	MOVIESCENE_API void SetName(const FName& Name);

	MOVIESCENE_API void AddNode(const FString& Path);
	MOVIESCENE_API void RemoveNode(const FString& Path);
	TArrayView<FString> GetNodes() { return Nodes; }
	MOVIESCENE_API bool ContainsNode(const FString& Path) const;

	MOVIESCENE_API void UpdateNodePath(const FString& OldPath, const FString& NewPath);

	bool GetEnableFilter() const { return bEnableFilter; }
	MOVIESCENE_API void SetEnableFilter(bool bInEnableFilter);

	/** Event that is triggered whenever this node group has changed */
	DECLARE_EVENT(UMovieSceneNodeGroup, FOnNodeGroupChanged)
	FOnNodeGroupChanged& OnNodeGroupChanged() { return OnNodeGroupChangedEvent; }

private:
	UPROPERTY()
	FName Name;

	/** Nodes that are part of this node group, stored as node tree paths */
	UPROPERTY()
	TArray<FString> Nodes;

	/** Whether sequencer should filter to only show this nodes in this group */
	bool bEnableFilter;

	/** Event that is triggered whenever this node group has changed */
	FOnNodeGroupChanged OnNodeGroupChangedEvent;

	/**
	 * Comparison operators
	 * We only need these for being stored in a container, to check if it's the same object.
	 * Not intended for direct use.
	 */
	inline bool operator==(const UMovieSceneNodeGroup& Other) const
	{
		return &Nodes == &Other.Nodes;
	}
	inline bool operator!=(const UMovieSceneNodeGroup& Other) const
	{
		return &Nodes != &Other.Nodes;
	}
#endif
};

/**
 * Structure that represents a collection of NodeGroups
 */
UCLASS(MinimalAPI)
class UMovieSceneNodeGroupCollection : public UObject
{
	GENERATED_BODY()

	virtual bool IsEditorOnly() const override { return true; }

#if WITH_EDITORONLY_DATA
public:
	/** Called after this object has been deserialized */
	MOVIESCENE_API virtual void PostLoad() override;
	MOVIESCENE_API virtual void PostEditUndo() override;

	MOVIESCENE_API void AddNodeGroup(UMovieSceneNodeGroup* NodeGroup);
	MOVIESCENE_API void RemoveNodeGroup(UMovieSceneNodeGroup* NodeGroup);

	bool Contains(UMovieSceneNodeGroup* NodeGroup) const { return NodeGroups.Contains(NodeGroup); }
	const int32 Num() const { return NodeGroups.Num(); }
	bool HasAnyActiveFilter() const { return bAnyActiveFilter; }

	MOVIESCENE_API void UpdateNodePath(const FString& OldPath, const FString& NewPath);

	/** Event that is triggered whenever this collection of node groups, or an included node group, has changed */
	DECLARE_EVENT(UMovieSceneNodeGroupCollection, FOnNodeGroupCollectionChanged)
	FOnNodeGroupCollectionChanged& OnNodeGroupCollectionChanged() { return OnNodeGroupCollectionChangedEvent; }

private:

	MOVIESCENE_API void Refresh();

	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneNodeGroup>> NodeGroups;

	bool bAnyActiveFilter;

	MOVIESCENE_API void OnNodeGroupChanged();

	/** Event that is triggered whenever this collection of node groups, or an included node group, has changed */
	FOnNodeGroupCollectionChanged OnNodeGroupCollectionChangedEvent;

public:

	/**
	 * DO NOT USE DIRECTLY
	 * STL-like iterators to enable range-based for loop support.
	 */
	inline auto begin()	{ return NodeGroups.begin(); }
	inline auto begin()	const { return NodeGroups.begin(); }
	inline auto end()	{ return NodeGroups.end(); }
	inline auto end()	const { return NodeGroups.end(); }
#endif
};

/**
 * Implements a movie scene asset.
 */
UCLASS(DefaultToInstanced, MinimalAPI)
class UMovieScene
	: public UMovieSceneDecorationContainerObject
{
	GENERATED_UCLASS_BODY()

public:

	/**~ UObject implementation */
	MOVIESCENE_API virtual void Serialize( FArchive& Ar ) override;
	MOVIESCENE_API virtual bool IsPostLoadThreadSafe() const override;
	MOVIESCENE_API virtual void PostInitProperties() override;
	MOVIESCENE_API virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	static MOVIESCENE_API void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif


#if WITH_EDITOR
	MOVIESCENE_API virtual void PostEditUndo() override;
#endif

public:

	UE::MovieScene::TDataEventContainer<UE::MovieScene::ISequenceDataEventHandler> EventHandlers;

	/**
	 * Add a spawnable to this movie scene's list of owned blueprints.
	 *
	 * These objects are stored as "inners" of the MovieScene.
	 *
	 * @param Name Name of the spawnable.
	 * @param ObjectTemplate The object template to use for the spawnable
	 * @return Guid of the newly-added spawnable.
	 */
	MOVIESCENE_API FGuid AddSpawnable(const FString& Name, UObject& ObjectTemplate);

	/*
	 * Adds an existing spawnable to this movie scene.
	 *
	 * @param InNewSpawnable The posssesable to add.
	 * @param InNewBinding The object binding to add.
	 */
	MOVIESCENE_API void AddSpawnable(const FMovieSceneSpawnable& InNewSpawnable, const FMovieSceneBinding& InNewBinding);

	/**
	 * Removes a spawnable from this movie scene.
	 *
	 * @param Guid The guid of a spawnable to find and remove.
	 * @return true if anything was removed.
	 */
	MOVIESCENE_API bool RemoveSpawnable(const FGuid& Guid);

	/**
	 * Attempt to find a spawnable using some custom predicate
	 *
	 * @param InPredicate A predicate to test each spawnable against
	 * @return Spawnable object that was found (or nullptr if not found).
	 */
	MOVIESCENE_API FMovieSceneSpawnable* FindSpawnable( const TFunctionRef<bool(FMovieSceneSpawnable&)>& InPredicate );

	/**
	 * Tries to locate a spawnable in this MovieScene for the specified spawnable GUID.
	 *
	 * @param Guid The spawnable guid to search for.
	 * @return Spawnable object that was found (or nullptr if not found).
	 */
	MOVIESCENE_API FMovieSceneSpawnable* FindSpawnable(const FGuid& Guid);

	/**
	 * Grabs a reference to a specific spawnable by index.
	 *
	 * @param Index of spawnable to return. Must be between 0 and GetSpawnableCount()
	 * @return Returns the specified spawnable by index.
	 */
	MOVIESCENE_API FMovieSceneSpawnable& GetSpawnable(int32 Index);

	/**
	 * Get the number of spawnable objects in this scene.
	 *
	 * @return Spawnable object count.
	 */
	MOVIESCENE_API int32 GetSpawnableCount() const;
	
public:

	/**
	 * Adds a possessable to this movie scene.
	 *
	 * @param Name Name of the possessable.
	 * @param Class The class of object that will be possessed.
	 * @return Guid of the newly-added possessable.
	 */
	MOVIESCENE_API FGuid AddPossessable(const FString& Name, UClass* Class);

	/*
	 * Adds an existing possessable to this movie scene.
	 *
	 * @param InNewPossessable The posssesable to add.
	 * @param InNewBinding The object binding to add.
	 */
	MOVIESCENE_API void AddPossessable(const FMovieScenePossessable& InNewPossessable, const FMovieSceneBinding& InNewBinding);

	/**
	 * Removes a possessable from this movie scene.
	 *
	 * @param PossessableGuid Guid of possessable to remove.
	 */
	MOVIESCENE_API bool RemovePossessable(const FGuid& PossessableGuid);
	
	/*
	* Replace an existing possessable with another 
	*/
	MOVIESCENE_API bool ReplacePossessable(const FGuid& OldGuid, const FMovieScenePossessable& InNewPosessable);

	/**
	 * Tries to locate a possessable in this MovieScene for the specified possessable GUID.
	 *
	 * @param Guid The possessable guid to search for.
	 * @return Possessable object that was found (or nullptr if not found).
	 */
	MOVIESCENE_API struct FMovieScenePossessable* FindPossessable(const FGuid& Guid);

	/**
	 * Attempt to find a possessable using some custom prdeicate
	 *
	 * @param InPredicate A predicate to test each possessable against
	 * @return Possessable object that was found (or nullptr if not found).
	 */
	MOVIESCENE_API FMovieScenePossessable* FindPossessable( const TFunctionRef<bool(FMovieScenePossessable&)>& InPredicate );

	/**
	 * Grabs a reference to a specific possessable by index.
	 *
	 * @param Index of possessable to return.
	 * @return Returns the specified possessable by index.
	 */
	MOVIESCENE_API FMovieScenePossessable& GetPossessable(const int32 Index);

	/**
	 * Get the number of possessable objects in this scene.
	 *
	 * @return Possessable object count.
	 */
	MOVIESCENE_API int32 GetPossessableCount() const;

public:

	/**
	 * Adds a track.
	 *
	 * Note: The type should not already exist.
	 *
	 * @param TrackClass The class of the track to create.
	 * @param ObjectGuid The runtime object guid that the type should bind to.
	 * @param Type The newly created type.
	 * @see  FindTrack, RemoveTrack
	 */
	MOVIESCENE_API UMovieSceneTrack* AddTrack(TSubclassOf<UMovieSceneTrack> TrackClass, const FGuid& ObjectGuid);

	/**
	* Adds a given track.
	*
	* @param InTrack The track to add.
	* @param ObjectGuid The runtime object guid that the type should bind to
	* @see  FindTrack, RemoveTrack
	* @return true if the track is successfully added, false otherwise.
	*/
	MOVIESCENE_API bool AddGivenTrack(UMovieSceneTrack* InTrack, const FGuid& ObjectGuid);

	/**
	 * Adds a track.
	 *
	 * Note: The type should not already exist.
	 *
	 * @param TrackClass The class of the track to create.
	 * @param ObjectGuid The runtime object guid that the type should bind to.
	 * @param Type The newly created type.
	 * @see FindTrack, RemoveTrack
	 */
	template<typename TrackClass>
	TrackClass* AddTrack(const FGuid& ObjectGuid)
	{
		return Cast<TrackClass>(AddTrack(TrackClass::StaticClass(), ObjectGuid));
	}

	/**
	 * Finds a track.
	 *
	 * @param TrackClass The class of the track to find.
	 * @param ObjectGuid The runtime object guid that the track is bound to.
	 * @param TrackName The name of the track to differentiate the one we are searching for from other tracks of the same class (optional).
	 * @return The found track or nullptr if one does not exist.
	 * @see AddTrack, RemoveTrack, FindTracks
	 */
	MOVIESCENE_API UMovieSceneTrack* FindTrack(TSubclassOf<UMovieSceneTrack> TrackClass, const FGuid& ObjectGuid, const FName& TrackName = NAME_None) const;
	
	/**
	 * Finds a track.
	 *
	 * @param TrackClass The class of the track to find.
	 * @param ObjectGuid The runtime object guid that the track is bound to.
	 * @param TrackName The name of the track to differentiate the one we are searching for from other tracks of the same class (optional).
	 * @return The found track or nullptr if one does not exist.
	 * @see AddTrack, RemoveTrack, FindTracks
	 */
	template<typename TrackClass>
	TrackClass* FindTrack(const FGuid& ObjectGuid, const FName& TrackName = NAME_None) const
	{
		return Cast<TrackClass>(FindTrack(TrackClass::StaticClass(), ObjectGuid, TrackName));
	}

	/**
	 * Find all tracks of a given class.
	 *
	 * @param TrackClass The class of the track to find.
	 * @param ObjectGuid The runtime object guid that the track is bound to.
	 * @param TrackName The name of the track to differentiate the one we are searching for from other tracks of the same class (optional).
	 * @return The found tracks or an empty array if none exist
	 * @see AddTrack, RemoveTrack, FindTrack
	 */
	MOVIESCENE_API TArray<UMovieSceneTrack*> FindTracks(TSubclassOf<UMovieSceneTrack> TrackClass, const FGuid& ObjectGuid, const FName& TrackName = NAME_None) const;

	/**
	 * Removes a track.
	 *
	 * @param Track The track to remove.
	 * @return true if anything was removed.
	 * @see AddTrack, FindTrack
	 */
	MOVIESCENE_API bool RemoveTrack(UMovieSceneTrack& Track);

	/**
	 * Find a track binding Guid from a UMovieSceneTrack
	 * 
	 * @param	InTrack		The track to find the binding for.
	 * @param	OutGuid		The binding's Guid if one was found.
	 * @return true if a binding was found for this track.
	 */
	MOVIESCENE_API bool FindTrackBinding(const UMovieSceneTrack& InTrack, FGuid& OutGuid) const;

#if WITH_EDITOR

	DECLARE_DELEGATE_RetVal_OneParam(bool, FIsTrackClassAllowedEvent, UClass*);

	static MOVIESCENE_API FIsTrackClassAllowedEvent IsTrackClassAllowedEvent;

	static MOVIESCENE_API bool IsTrackClassAllowed(UClass* InClass);

	DECLARE_DELEGATE_RetVal_OneParam(bool, FIsCustomBindingClassAllowedEvent, UClass*);

	static MOVIESCENE_API FIsCustomBindingClassAllowedEvent IsCustomBindingClassAllowedEvent;

	static MOVIESCENE_API bool IsCustomBindingClassAllowed(UClass* InClass);

	DECLARE_DELEGATE_RetVal_OneParam(bool, FIsConditionClassAllowedEvent, const UClass*);

	static MOVIESCENE_API FIsConditionClassAllowedEvent IsConditionClassAllowedEvent;

	static MOVIESCENE_API bool IsConditionClassAllowed(const UClass* InClass);


	void OnDynamicBindingUserDefinedPinRenamed(UK2Node* InNode, FName OldPinName, FName NewPinName)
	{
		FixupDynamicBindingPayloadParameterNameEvent.Broadcast(this, InNode, OldPinName, NewPinName);
	}

	DECLARE_MULTICAST_DELEGATE_FourParams(FFixupDynamicBindingPayloadParameterNameEvent, UMovieScene*, UK2Node*, FName, FName);

	static MOVIESCENE_API FFixupDynamicBindingPayloadParameterNameEvent FixupDynamicBindingPayloadParameterNameEvent;

	void OnDirectorBlueprintConditionUserDefinedPinRenamed(UK2Node* InNode, FName OldPinName, FName NewPinName)
	{
		FixupDirectorBlueprintConditionPayloadParameterNameEvent.Broadcast(this, InNode, OldPinName, NewPinName);
	}

	DECLARE_MULTICAST_DELEGATE_FourParams(FFixupDirectorBlueprintConditionPayloadParameterNameEvent, UMovieScene*, UK2Node*, FName, FName);

	static MOVIESCENE_API FFixupDirectorBlueprintConditionPayloadParameterNameEvent FixupDirectorBlueprintConditionPayloadParameterNameEvent;

#endif

public:

	/**
	 * Adds a track.
	 *
	 * Note: The type should not already exist.
	 *
	 * @param TrackClass The class of the track to create
	 * @param Type	The newly created type
	 * @see FindTrack, GetTracks, IsTrack, RemoveTrack
	 */
	MOVIESCENE_API UMovieSceneTrack* AddTrack(TSubclassOf<UMovieSceneTrack> TrackClass);
	
	/**
	 * Adds a track.
	 *
	 * Note: The type should not already exist.
	 *
	 * @param TrackClass The class of the track to create
	 * @param Type	The newly created type
	 * @see FindTrack, GetTracks, IsTrack, RemoveTrack
	 */
	template<typename TrackClass>
	TrackClass* AddTrack()
	{
		return Cast<TrackClass>(AddTrack(TrackClass::StaticClass()));
	}

	/**
	* Adds a given track as a track
	*
	* @param InTrack The track to add.
	* @see  FindTrack, RemoveTrack
	* @return true if the track is successfully added, false otherwise.
	*/
	MOVIESCENE_API bool AddGivenTrack(UMovieSceneTrack* InTrack);

	/**
	 * Finds a track (one not bound to a runtime objects).
	 *
	 * @param TrackClass The class of the track to find.
	 * @return The found track or nullptr if one does not exist.
	 * @see AddTrack, GetTracks, IsTrack, RemoveTrack
	 */
	MOVIESCENE_API UMovieSceneTrack* FindTrack(TSubclassOf<UMovieSceneTrack> TrackClass) const;

	/**
	 * Finds a track (one not bound to a runtime objects).
	 *
	 * @param TrackClass The class of the track to find.
	 * @return The found track or nullptr if one does not exist.
	 * @see AddTrack, GetTracks, IsTrack, RemoveTrack
	 */
	template<typename TrackClass>
	TrackClass* FindTrack() const
	{
		return Cast<TrackClass>(FindTrack(TrackClass::StaticClass()));
	}

	/**
	 * Get all tracks.
	 *
	 * @return Track collection.
	 * @see AddTrack, FindTrack, IsTrack, RemoveTrack
	 */
	const TArray<UMovieSceneTrack*>& GetTracks() const
	{
		return Tracks;
	}

	/**
	 * Check whether the specified track is a track in this movie scene.
	 *
	 * @return true if the track is a track, false otherwise.
	 * @see AddTrack, FindTrack, GetTracks, RemoveTrack
	 */
	MOVIESCENE_API bool ContainsTrack(const UMovieSceneTrack& Track) const;

	/**
	 * Move all the contents (tracks, child bindings) of the specified binding ID onto another
	 *
	 * @param SourceBindingId The identifier of the binding ID to move all tracks and children from
	 * @param DestinationBindingId The identifier of the binding ID to move the contents to
	 */
	MOVIESCENE_API void MoveBindingContents(const FGuid& SourceBindingId, const FGuid& DestinationBindingId);

	/**
	 * Tries to find an FMovieSceneBinding for the specified Guid.
	 * 
	 * @param ForGuid	The binding's Guid to look for.
	 * @return			Pointer to the binding, otherwise nullptr.
	 */
	MOVIESCENE_API FMovieSceneBinding* FindBinding(const FGuid& ForGuid);

	/**
	* Tries to find an FMovieSceneBinding for the specified Guid.
	*
	* @param ForGuid	The binding's Guid to look for.
	* @return			Pointer to the binding, otherwise nullptr.
	*/
	MOVIESCENE_API const FMovieSceneBinding* FindBinding(const FGuid& ForGuid) const;

public:

	// @todo sequencer: the following methods really shouldn't be here

	/**
	 * Adds a new camera cut track if it doesn't exist 
	 * A camera cut track allows for cutting between camera views
	 * There is only one per movie scene. 
	 *
	 * @param TrackClass  The camera cut track class type
	 * @return The created camera cut track
	 */
	MOVIESCENE_API UMovieSceneTrack* AddCameraCutTrack( TSubclassOf<UMovieSceneTrack> TrackClass );
	
	/** @return The camera cut track if it exists. */
	MOVIESCENE_API UMovieSceneTrack* GetCameraCutTrack() const;

	/** Removes the camera cut track if it exists. */
	MOVIESCENE_API void RemoveCameraCutTrack();

	MOVIESCENE_API void SetCameraCutTrack(UMovieSceneTrack* Track);

public:

	/**
	 * Returns all sections and their associated binding data.
	 *
	 * @return A list of sections with object bindings and names.
	 */
	MOVIESCENE_API TArray<UMovieSceneSection*> GetAllSections() const;

	/**
	 * @return All object bindings.
	 */
	const TArray<FMovieSceneBinding>& GetBindings() const
	{
		return ObjectBindings;
	}

	UE_DEPRECATED(5.7, "Getting non-const access to GetBindings is no longer allowed. Please use const GetBindings()")
	TArray<FMovieSceneBinding>& GetBindings()
	{
		return ObjectBindings;
	}

	/** Get the current selection range. */
	TRange<FFrameNumber> GetSelectionRange() const
	{
		return SelectionRange.Value;
	}

	/**
	 * Get the display name of the object with the specified identifier.
	 *
	 * @param ObjectId The object identifier.
	 * @return The object's display name.
	 */
	MOVIESCENE_API FText GetObjectDisplayName(const FGuid& ObjectId) const;

	/** Get the playback time range of this movie scene, relative to its 0-time offset. */
	TRange<FFrameNumber> GetPlaybackRange() const
	{
		return PlaybackRange.Value;
	}

	/**
	 * Retrieve the tick resolution at which all frame numbers within this movie scene are defined
	 */
	FFrameRate GetTickResolution() const
	{
		return TickResolution;
	}

	/**
	 * Directly set the tick resolution for this movie scene without applying any conversion whatsoever, or modifying the data
	 */
	MOVIESCENE_API void SetTickResolutionDirectly(FFrameRate InTickResolution);

	/**
	 * Retrieve the display frame rate for this data, in which frame numbers should be displayed on UI, and interacted with in movie scene players
	 */
	FFrameRate GetDisplayRate() const
	{
		return DisplayRate;
	}

	/**
	 * Set the play rate for this movie scene
	 */
	void SetDisplayRate(FFrameRate InDisplayRate)
	{
		DisplayRate = InDisplayRate;
	}

	/**
	 * Retrieve a value signifying how to evaluate this movie scene data
	 */
	EMovieSceneEvaluationType GetEvaluationType() const
	{
		return EvaluationType;
	}

	/**
	 * Assign a value signifying how to evaluate this movie scene data
	 */
	void SetEvaluationType(EMovieSceneEvaluationType InNewEvaluationType)
	{
		EvaluationType = InNewEvaluationType;

		if (EvaluationType == EMovieSceneEvaluationType::FrameLocked && ClockSource == EUpdateClockSource::Tick)
		{
			ClockSource = EUpdateClockSource::Platform;
		}
	}

	/**
	 * retrieve the clock source to be used for this moviescene
	 */
	EUpdateClockSource GetClockSource() const
	{
		return ClockSource;
	}

	UMovieSceneClock* GetCustomClock() const
	{
		return ClockSource == EUpdateClockSource::Custom ? CustomClock : nullptr;
	}

	template<typename T>
	T* GetCustomClockAs() const
	{
		return Cast<T>(GetCustomClock());
	}

	/**
	 * Retrieve a time controller from this sequence instance, if the clock source is set to custom
	 */
	MOVIESCENE_API TSharedPtr<FMovieSceneTimeController> MakeCustomTimeController(UObject* PlaybackContext);

	/**
	 * Assign the clock source to be used for this moviescene
	 */
	MOVIESCENE_API void SetClockSource(EUpdateClockSource InNewClockSource);

	/**
	 * Assign the clock source to be used for this moviescene
	 */
	UE_DEPRECATED(5.6, "Please call SetCustomClock with an instance of UMovieSceneExternalClock")
	MOVIESCENE_API void SetClockSource(UObject* InNewClockSource);

	MOVIESCENE_API void SetCustomClock(UMovieSceneClock* InNewClockSource);

	/**
	 * Get the earliest timecode source out of all of the movie scene sections contained within this movie scene.
	 */
	MOVIESCENE_API FMovieSceneTimecodeSource GetEarliestTimecodeSource() const;

	/*
	* Replace an existing binding with another 
	*/
	MOVIESCENE_API void ReplaceBinding(const FGuid& OldGuid, const FGuid& NewGuid, const FString& Name);

	/*
	* Replace an existing binding with another. Assumes ownership of any
	* tracks listed in the binding. Does nothing if no binding can be found.
	* @param BindingToReplaceGuid	Binding Guid that should be replaced
	* @param NewBinding				Binding Data that should replace the original one specified by BindingToReplaceGuid.
	*/
	MOVIESCENE_API void ReplaceBinding(const FGuid& BindingToReplaceGuid, const FMovieSceneBinding& NewBinding);

#if WITH_EDITORONLY_DATA
	/**
	 */
	TMap<FString, FMovieSceneTrackLabels>& GetObjectsToLabels()
	{
		return ObjectsToLabels;
	}

	/** Set the selection range. */
	void SetSelectionRange(TRange<FFrameNumber> Range)
	{
		SelectionRange.Value = Range;
	}

	/**
	 * Get the display name of the object with the specified identifier.
	 *
	 * @param ObjectId The object identifier.
	 * @return The object's display name.
	 */
	MOVIESCENE_API void SetObjectDisplayName(const FGuid& ObjectId, const FText& DisplayName);

	/**
	 * Gets the root folders for this movie scene.
	 */
	MOVIESCENE_API TArrayView<UMovieSceneFolder* const> GetRootFolders();

	/**
	 * Gets a copy of the root folders for this movie scene.
	 */
	MOVIESCENE_API void GetRootFolders(TArray<UMovieSceneFolder*>& InRootFolders);

	/**
	 * Gets the number of root folders in this movie scene.
	 */
	MOVIESCENE_API int32 GetNumRootFolders() const;

	/**
	 * Gets the i-th root folder for this movie scene.
	 */
	MOVIESCENE_API UMovieSceneFolder* GetRootFolder(int32 FolderIndex) const;

	/**
	 * Adds a root folder for this movie scene.
	 */
	MOVIESCENE_API void AddRootFolder(UMovieSceneFolder* Folder);

	/**
	 * Removes a root folder for this movie scene (does not delete tracks or objects contained within)
	 */
	MOVIESCENE_API int32 RemoveRootFolder(UMovieSceneFolder* Folder);

	/**
	 * Removes a root folder for this movie scene (does not delete tracks or objects contained within)
	 */
	MOVIESCENE_API bool RemoveRootFolder(int32 FolderIndex);

	/**
	 * Removes all root folders from this movie scene (does not delete tracks or objects contained within)
	 */
	MOVIESCENE_API void EmptyRootFolders();

	/**
	 * Gets the nodes marked as transient solo in the editor, as node tree paths
	 */
	UE_DEPRECATED(5.7, "GetSoloNodes is now deprecated. Use FindDecoration")
	TArray<FString>& GetSoloNodes() { return EditorData.SoloNodes; }

	/**
	 * Gets the nodes marked as transient muted in the editor, as node tree paths
	 */
	UE_DEPRECATED(5.7, "GetMuteNodes is now deprecated. Use FindDecoration")
	TArray<FString>& GetMuteNodes() { return EditorData.MuteNodes; }
	
	//WIP Set of Delegates
	/** Gets a multicast delegate which is executed whenever a channel is changed, currently only set by Python/BP actions.
	*
	*/

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "FMovieSceneOnChannelChanged OnChannelChanged() is now deprecated. Use FMovieSceneOnChannelChangedWithTime OnChannelChangedWithTime()")
	FMovieSceneOnChannelChanged& OnChannelChanged() { return OnChannelChangedDelegate_DEPRACRATED; }
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FMovieSceneOnChannelChangedWithTime& OnChannelChangedWithTime() { return OnChannelChangedDelegate; }
#endif

	/**
	 * Set the start and end playback positions (playback range) for this movie scene
	 *
	 * @param Start The offset from 0-time to start playback of this movie scene
	 * @param Duration The number of frames the movie scene should play for
	 * @param bAlwaysMarkDirty Whether to always mark the playback range dirty when changing it. 
	 *        In the case where the playback range is dynamic and based on section bounds, the playback range doesn't need to be dirtied when set
	 */
	MOVIESCENE_API void SetPlaybackRange(FFrameNumber Start, int32 Duration, bool bAlwaysMarkDirty = true);

	/**
	 * Set the playback range for this movie scene
	 *
	 * @param Range The new playback range. Must not have any open bounds (ie must be a finite range)
	 * @param bAlwaysMarkDirty Whether to always mark the playback range dirty when changing it. 
	 *        In the case where the playback range is dynamic and based on section bounds, the playback range doesn't need to be dirtied when set
	 */
	MOVIESCENE_API void SetPlaybackRange(const TRange<FFrameNumber>& NewRange, bool bAlwaysMarkDirty = true);

	/**
	 * Set the start and end working range (outer) for this movie scene
	 *
	 * @param Start The offset from 0-time to view this movie scene.
	 * @param End The offset from 0-time to view this movie scene
	 */
	MOVIESCENE_API void SetWorkingRange(double Start, double End);

	/**
	 * Set the start and end view range (inner) for this movie scene
	 *
	 * @param Start The offset from 0-time to view this movie scene
	 * @param End The offset from 0-time to view this movie scene
	 */
	MOVIESCENE_API void SetViewRange(double Start, double End);

#if WITH_EDITORONLY_DATA

public:

	/*
	* @return Returns whether this movie scene is read only
	*/
	bool IsReadOnly() const { return bReadOnly; }

	/**
	* Set whether this movie scene is read only.
	*/
	void SetReadOnly(bool bInReadOnly) { bReadOnly = bInReadOnly; }

	/**
	* Return whether the playback range is locked.
	*/
	MOVIESCENE_API bool IsPlaybackRangeLocked() const;

	/**
	* Set whether the playback range is locked.
	*/
	MOVIESCENE_API void SetPlaybackRangeLocked(bool bLocked);

	/**
	 * Return whether marked frames are locked.
	 */
	MOVIESCENE_API bool AreMarkedFramesLocked() const;

	/**
	 * Set whether marked frames are locked.
	 */
	MOVIESCENE_API void SetMarkedFramesLocked(bool bLocked);

	/**
	 * @return The editor only data for use with this movie scene
	 */
	FMovieSceneEditorData& GetEditorData()
	{
		return EditorData;
	}
	const FMovieSceneEditorData& GetEditorData() const
	{
		return EditorData;
	}

	void SetEditorData(FMovieSceneEditorData& InEditorData)
	{
		EditorData = InEditorData;
	}

	/*
	 * @return Whether the section is in a group
	 */
	MOVIESCENE_API bool IsSectionInGroup(const UMovieSceneSection& InSection) const;

	/*
	 * Create a group containing InSections, merging any existing groups the sections are in
	 */
	MOVIESCENE_API void GroupSections(const TArray<UMovieSceneSection*> InSections);

	/*
	 * Remove InSection from any group it currently is in
	 */
	MOVIESCENE_API void UngroupSection(const UMovieSceneSection& InSection);

	/*
	 * @return The group containing the InSection, or a nullptr if InSection is not grouped.
	 */
	MOVIESCENE_API const FMovieSceneSectionGroup* GetSectionGroup(const UMovieSceneSection& InSection) const;

	/*
	 * Cleans stale UMovieSceneSection pointers, and removes any section groups which are no longer valid, e.g. contain less that two valid sections
	 */
	MOVIESCENE_API void CleanSectionGroups();

	UMovieSceneNodeGroupCollection& GetNodeGroups() { return *NodeGroupCollection; }

#endif	// WITH_EDITORONLY_DATA

public:

	/*
	 * @return Return the user marked frames
	 */
	const TArray<FMovieSceneMarkedFrame>& GetMarkedFrames() const { return MarkedFrames; }

	/*
	 * Sets the frame number for the given marked frame index. Does not maintain sort. Call SortMarkedFrames
	 *
	 * @InMarkIndex The given user marked frame index to edit
	 * @InFrameNumber The frame number to set
	 */
	MOVIESCENE_API void SetMarkedFrame(int32 InMarkIndex, FFrameNumber InFrameNumber);

	/*
	 * Add a given user marked frame.
	 * A unique label will be generated if the marked frame label is empty
	 *
	 * @InMarkedFrame The given user marked frame to add
	 * @return The index to the newly added marked frame
	 */
	MOVIESCENE_API int32 AddMarkedFrame(const FMovieSceneMarkedFrame& InMarkedFrame);

	/*
	 * Delete the user marked frame by index.
	 *
	 * @DeleteIndex The index to the user marked frame to delete
	 */
	MOVIESCENE_API void DeleteMarkedFrame(int32 DeleteIndex);

	/*
	 * Delete all user marked frames
	 */
	MOVIESCENE_API void DeleteMarkedFrames();

	/*
	 * Sort the marked frames in chronological order
	 */
	MOVIESCENE_API void SortMarkedFrames();

	/*
	 * Find the user marked frame by label
	 *
	 * @InLabel The label to the user marked frame to find
	 */
	MOVIESCENE_API int32 FindMarkedFrameByLabel(const FString& InLabel) const;

	/*
	 * Find the user marked frame by frame number
	 *
	 * @InFrameNumber The frame number of the user marked frame to find
	 */
	MOVIESCENE_API int32 FindMarkedFrameByFrameNumber(FFrameNumber InFrameNumber) const;

	/*
	 * Find the next/previous user marked frame from the given frame number
	 *
	 * @InFrameNumber The frame number to find the next/previous user marked frame from
	 * @bForward Find forward from the given frame number.
	 */
	MOVIESCENE_API int32 FindNextMarkedFrame(FFrameNumber InFrameNumber, bool bForward);

#if WITH_EDITORONLY_DATA
	/*
	 * Set whether this scene's marked frames should be shown globally
	 */
	void SetGloballyShowMarkedFrames(bool bShowMarkedFrames) { bGloballyShowMarkedFrames = bShowMarkedFrames; }

	/*
	 * Toggle whether this scene's marked frames should be shown globally
	 */
	void ToggleGloballyShowMarkedFrames() { bGloballyShowMarkedFrames = !bGloballyShowMarkedFrames; }

	/*
	 * Returns whether this scene's marked frames should be shown globally
	 */
	bool GetGloballyShowMarkedFrames() const { return bGloballyShowMarkedFrames; }
#endif

	/*
	 * Retrieve all the tagged binding groups for this movie scene
	 */
	const TMap<FName, FMovieSceneObjectBindingIDs>& AllTaggedBindings() const
	{
		return BindingGroups;
	}

	/*
	 * Add a new binding group for the specified name
	 */
	MOVIESCENE_API void AddNewBindingTag(const FName& NewTag);

	/*
	 * Tag the specified binding ID with the specified name
	 */
	MOVIESCENE_API void TagBinding(const FName& NewTag, const UE::MovieScene::FFixedObjectBindingID& BindingToTag);

	/*
	 * Remove a tag from the specified object binding
	 */
	MOVIESCENE_API void UntagBinding(const FName& Tag, const UE::MovieScene::FFixedObjectBindingID& Binding);

	/*
	 * Remove the specified tag from any binding and forget about it completely
	 */
	MOVIESCENE_API void RemoveTag(const FName& TagToRemove);

	/* Called during compilation to add a new generated condition to store in the movie scene. */
	void AddGeneratedCondition(UMovieSceneGroupCondition* InGeneratedCondition) { GeneratedConditions.Add(InGeneratedCondition); }

	/* Called by the compiler to empty the list of generated conditions*/
	void ResetGeneratedConditions() { GeneratedConditions.Reset(); }

	/* Remove null tracks on this movie scene. Returns true if any discovered and removed. */
	MOVIESCENE_API bool RemoveNullTracks();

protected:

	/**
	 * Removes animation data bound to a GUID.
	 *
	 * @param Guid The guid bound to animation data to remove
	 */
	MOVIESCENE_API void RemoveBinding(const FGuid& Guid);


	/**
	 * Tries to find an FMovieSceneBinding for the specified Guid.
	 *
	 * @param ForGuid	The binding's Guid to look for.
	 * @return			Index of the binding in ObjectBindings array, otherwise INDEX_NONE.
	 */
	int32 IndexOfBinding(const FGuid& ForGuid) const;

	/**
	* Tries to find an FMovieSceneSpawnable for the specified Guid.
	*
	* @param ForGuid	The spawnable's Guid to look for.
	* @return			Index of the binding in Spawnables array, otherwise INDEX_NONE.
	*/
	int32 IndexOfSpawnable(const FGuid& ForGuid) const;


	/**
	* Tries to find an FMovieScenePossessable for the specified Guid.
	*
	* @param ForGuid	The possessable's Guid to look for.
	* @return			Index of the binding in Possessables array, otherwise INDEX_NONE.
	*/
	int32 IndexOfPossessable(const FGuid& ForGuid) const;

protected:

	/** Called before this object is being deserialized. */
	MOVIESCENE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	MOVIESCENE_API virtual void OnDecorationAdded(UObject* NewDecoration) override;
	MOVIESCENE_API virtual void OnDecorationRemoved(UObject* NewDecoration) override;

	/** Perform legacy upgrade of time ranges */
	MOVIESCENE_API void UpgradeTimeRanges();

private:

#if WITH_EDITOR
	MOVIESCENE_API void OptimizeForCook();
#endif

private:

	/**
	 * Data-only blueprints for all of the objects that we we're able to spawn.
	 * These describe objects and actors that we may instantiate at runtime,
	 * or create proxy objects for previewing in the editor.
	 */
	UPROPERTY()
	TArray<FMovieSceneSpawnable> Spawnables;

	/** Typed slots for already-spawned objects that we are able to control with this MovieScene */
	UPROPERTY()
	TArray<FMovieScenePossessable> Possessables;

	/** Tracks bound to possessed or spawned objects */
	UPROPERTY()
	TArray<FMovieSceneBinding> ObjectBindings;

	/** Map of persistent tagged bindings for this sequence */
	UPROPERTY()
	TMap<FName, FMovieSceneObjectBindingIDs> BindingGroups;

	/** Tracks which are not bound to spawned or possessed objects */
	UPROPERTY(Instanced)
	TArray<TObjectPtr<UMovieSceneTrack>> Tracks;

	/** The camera cut track is a specialized track for switching between cameras on a cinematic */
	UPROPERTY(Instanced)
	TObjectPtr<UMovieSceneTrack> CameraCutTrack;

	/** A custom clock implementation */
	UPROPERTY(Instanced)
	TObjectPtr<UMovieSceneClock> CustomClock;

	/** User-defined selection range. */
	UPROPERTY()
	FMovieSceneFrameRange SelectionRange;

	/** User-defined playback range for this movie scene. Must be a finite range. Relative to this movie-scene's 0-time origin. */
	UPROPERTY()
	FMovieSceneFrameRange PlaybackRange;

	/** The resolution at which all frame numbers within this movie-scene data are stored */
	UPROPERTY()
	FFrameRate TickResolution;

	/** The rate at which we should interact with this moviescene data on UI, and to movie scene players. Also defines the frame locked frame rate. */
	UPROPERTY()
	FFrameRate DisplayRate;

	/** The type of evaluation to use when playing back this sequence */
	UPROPERTY()
	EMovieSceneEvaluationType EvaluationType;

	UPROPERTY()
	EUpdateClockSource ClockSource;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FSoftObjectPath CustomClockSourcePath_DEPRECATED;
#endif

	/** The set of user-marked frames */
	UPROPERTY()
	TArray<FMovieSceneMarkedFrame> MarkedFrames;

	/* List of compiler generated group conditions, stored here to prevent garbage collection. */
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneGroupCondition>> GeneratedConditions;

#if WITH_EDITORONLY_DATA

	/** Indicates whether this movie scene is read only */
	UPROPERTY()
	bool bReadOnly;

	/** User-defined playback range is locked. */
	UPROPERTY()
	bool bPlaybackRangeLocked;

	/** User-defined marked frames are locked. */
	UPROPERTY()
	bool bMarkedFramesLocked;

	/** Maps object GUIDs to user defined display names. */
	UPROPERTY()
	TMap<FString, FText> ObjectsToDisplayNames;

	/** Maps object GUIDs to user defined labels. */
	UPROPERTY()
	TMap<FString, FMovieSceneTrackLabels> ObjectsToLabels;

	/** Editor only data that needs to be saved between sessions for editing but has no runtime purpose */
	UPROPERTY()
	FMovieSceneEditorData EditorData;

	/** The root folders for this movie scene. */
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneFolder>> RootFolders;

	/** Nodes currently marked Solo, stored as node tree paths */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Solo nodes are now saved in FMovieSceneEditorData as transient"))
	TArray<FString> SoloNodes_DEPRECATED;

	/** Nodes currently marked Mute, stored as node tree paths */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Mute nodes are now saved in FMovieSceneEditorData as transient"))
	TArray<FString> MuteNodes_DEPRECATED;

	/** Groups of sections which should maintain the same relative offset */
	UPROPERTY()
	TArray<FMovieSceneSectionGroup> SectionGroups;

	/** Collection of user-defined groups */
	UPROPERTY()
	TObjectPtr<UMovieSceneNodeGroupCollection> NodeGroupCollection;

	/** Whether this scene's marked frames should be shown globally */
	UPROPERTY()
	bool bGloballyShowMarkedFrames;

private:

	UPROPERTY()
	float InTime_DEPRECATED;

	UPROPERTY()
	float OutTime_DEPRECATED;

	UPROPERTY()
	float StartTime_DEPRECATED;

	UPROPERTY()
	float EndTime_DEPRECATED;

	UPROPERTY()
	bool bForceFixedFrameIntervalPlayback_DEPRECATED;

	UPROPERTY()
	float FixedFrameInterval_DEPRECATED;

	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneTrack>> MasterTracks_DEPRECATED;

	//delegates
	private:

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FMovieSceneOnChannelChanged OnChannelChangedDelegate_DEPRACRATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FMovieSceneOnChannelChangedWithTime OnChannelChangedDelegate;
#endif
};
