// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Decorations/MovieSceneDecorationContainer.h"
#include "Misc/Guid.h"
#include "MovieSceneTrack.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"

#include "MovieSceneBinding.generated.h"

class UMovieScene;
class UMovieSceneTrack;

/**
 * A set of tracks bound to runtime objects
 */
USTRUCT()
struct FMovieSceneBinding 
	: public FMovieSceneDecorationContainer
{
	GENERATED_USTRUCT_BODY()

	/** Default constructor. */
	FMovieSceneBinding()
#if WITH_EDITORONLY_DATA
		: SortingOrder(-1)
#endif
	{ }

private:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InObjectGuid
	 * @param InTracks
	 */
	FMovieSceneBinding(const FGuid& InObjectGuid, const TArray<UMovieSceneTrack*>& InTracks)
		: ObjectGuid(InObjectGuid)
		, Tracks(InTracks)
#if WITH_EDITORONLY_DATA
		, SortingOrder(-1)
#endif
	{ }

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InObjectGuid
	 */
	FMovieSceneBinding(const FGuid& InObjectGuid)
		: ObjectGuid(InObjectGuid)
#if WITH_EDITORONLY_DATA
		, SortingOrder(-1)
#endif
	{ }

	/**
	 * Set the object guid
	 *
	 * @param InObjectGuid
	 */
	void SetObjectGuid_Private(const FGuid& InObjectGuid)
	{
		ObjectGuid = InObjectGuid;
	}

public:

	UE_DEPRECATED(5.7, "SetObjectGuid is deprecated because guids can no longer be set directly.")
	void SetObjectGuid(const FGuid& InObjectGuid)
	{
		ObjectGuid = InObjectGuid;
	}

	/**
	 * @return The guid of runtime objects in this binding
	 */
	const FGuid& GetObjectGuid() const
	{
		return ObjectGuid;
	}

	UE_DEPRECATED(5.7, "SetName is deprecated. Use SetName on FMovieScenePossessable or FMovieSceneSpawnable")
	void SetName(const FString& InBindingName) { } 

	UE_DEPRECATED(5.7, "GetName is deprecated. Use GetName on FMovieScenePossessable or FMovieSceneSpawnable")
	const FString& GetName() const { return BindingName_DEPRECATED; }

	/**
	 * Adds a new track to this binding
	 *
	 * @param NewTrack	The track to add
	 */
	MOVIESCENE_API void AddTrack(UMovieSceneTrack& NewTrack, UMovieScene* Owner);

	/**
	 * Removes a track from this binding
	 * 
	 * @param Track	The track to remove
	 * @return true if the track was successfully removed, false if the track could not be found
	 */
	MOVIESCENE_API bool RemoveTrack(UMovieSceneTrack& Track, UMovieScene* Owner);

	/**
	 * Removes all null tracks from this binding
	 */
	MOVIESCENE_API bool RemoveNullTracks();

	/**
	 * @return All tracks in this binding
	 */
	const TArray<UMovieSceneTrack*>& GetTracks() const
	{
		return Tracks;
	}

	/**
	 * Reset all tracks in this binding, returning the previous array of tracks
	 */
	MOVIESCENE_API TArray<UMovieSceneTrack*> StealTracks(UMovieScene* Owner);

	/**
	 * Assign all tracks in this binding
	 */
	MOVIESCENE_API void SetTracks(TArray<UMovieSceneTrack*>&& InTracks, UMovieScene* Owner);

	/* For sorts so we can search quickly by Guid */
	inline bool operator<(const FMovieSceneBinding& RHS) const { return ObjectGuid < RHS.ObjectGuid; }
	inline bool operator<(const FGuid& InGuid) const { return ObjectGuid < InGuid; }
	inline friend bool operator<(const FGuid& InGuid, const FMovieSceneBinding& RHS) { return InGuid < RHS.GetObjectGuid(); }

#if WITH_EDITORONLY_DATA
	/**
	* Get this folder's desired sorting order
	*/
	int32 GetSortingOrder() const
	{
		return SortingOrder;
	}

	/**
	* Set this folder's desired sorting order.
	*
	* @param InSortingOrder The higher the value the further down the list the folder will be.
	*/
	void SetSortingOrder(const int32 InSortingOrder)
	{
		SortingOrder = InSortingOrder;
	}
#endif

	friend class UMovieScene;

private:

	/** Object binding guid for runtime objects */
	UPROPERTY()
	FGuid ObjectGuid;
	
	/** Display name */
	UPROPERTY()
	FString BindingName_DEPRECATED;

	/** All tracks in this binding */
	UPROPERTY(Instanced)
	TArray<TObjectPtr<UMovieSceneTrack>> Tracks;

#if WITH_EDITORONLY_DATA
	/** The desired sorting order for this binding in Sequencer */
	UPROPERTY()
	int32 SortingOrder;
#endif
};
