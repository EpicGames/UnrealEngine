// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "MovieSceneObjectBindingID.h"
#include "MovieSceneSequencePlayer.h"
#include "MovieSceneBindingEventReceiverInterface.generated.h"


class UMovieSceneSequence;
class IPropertyHandle;
class FStructOnScope;


/** Interface for objects to implement used by the binding lifetime track to provide events when sequnecer activates or deactivates a binding.  */
UINTERFACE(MinimalAPI)
class UMovieSceneBindingEventReceiverInterface
	: public UInterface
{
public:
	GENERATED_BODY()
};

class IMovieSceneBindingEventReceiverInterface
{
public:
	GENERATED_BODY()

	UFUNCTION(BlueprintNativeEvent, CallInEditor)
	void OnObjectBoundBySequencer(UMovieSceneSequencePlayer* Player, FMovieSceneObjectBindingID BindingID);
	virtual void OnObjectBoundBySequencer_Implementation(UMovieSceneSequencePlayer* Player, FMovieSceneObjectBindingID BindingID) { };

	UFUNCTION(BlueprintNativeEvent, CallInEditor)
	void OnObjectUnboundBySequencer(UMovieSceneSequencePlayer* Player, FMovieSceneObjectBindingID BindingID);
	virtual void OnObjectUnboundBySequencer_Implementation(UMovieSceneSequencePlayer* Player, FMovieSceneObjectBindingID BindingID) { };
};
