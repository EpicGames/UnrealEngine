// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "HAL/Platform.h"
#include "Misc/Guid.h"
#include "MovieScene.h"
#include "MovieSceneObjectBindingID.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneBindingOverrides.generated.h"

struct FMovieSceneSequenceID;
struct FPropertyChangedEvent;


/** Movie scene binding override data */
USTRUCT()
struct FMovieSceneBindingOverrideData
{
	GENERATED_BODY()

	FMovieSceneBindingOverrideData()
		: bOverridesDefault(true)
	{
	}

	/** Specifies the object binding to override. */
	UPROPERTY(EditAnywhere, Category="Binding")
	FMovieSceneObjectBindingID ObjectBindingId;

	/** Specifies the object to override the binding with. */
	UPROPERTY(EditAnywhere, Category="Binding")
	TSoftObjectPtr<UObject> Object;

	/** Specifies whether the default assignment should remain bound (false) or if this should completely override the default binding (true). */
	UPROPERTY(EditAnywhere, Category="Binding")
	bool bOverridesDefault;
};


/**
 * A one-to-many definition of movie scene object binding IDs to overridden objects that should be bound to that binding.
 */
UCLASS(DefaultToInstanced, EditInlineNew, MinimalAPI)
class UMovieSceneBindingOverrides
	: public UObject
{
public:

	GENERATED_BODY()

	/** Default constructor */
	MOVIESCENE_API UMovieSceneBindingOverrides(const FObjectInitializer& Init);

	MOVIESCENE_API bool LocateBoundObjects(const FGuid& InBindingId, FMovieSceneSequenceID InSequenceID, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const;

public:

	/**
	 * Overrides the specified binding with the specified objects,
	 * optionally still allowing the bindings defined in the sequence.
	 */
	MOVIESCENE_API void SetBinding(FMovieSceneObjectBindingID Binding, const TArray<UObject*>& Objects, bool bAllowBindingsFromAsset = false);

	/**
	 * Adds the specified actor to the overridden bindings for the specified binding ID,
	 * optionally still allowing the bindings defined in the sequence.
	 */
	MOVIESCENE_API void AddBinding(FMovieSceneObjectBindingID Binding, UObject* Object, bool bAllowBindingsFromAsset = false);

	/**
	 * Removes the specified actor from the specified binding's actor array.
	 */
	MOVIESCENE_API void RemoveBinding(FMovieSceneObjectBindingID Binding, UObject* Object);

	/**
	 * Resets the specified binding back to the defaults defined by the sequence.
	 */
	MOVIESCENE_API void ResetBinding(FMovieSceneObjectBindingID Binding);

	/** Resets all overridden bindings back to the defaults defined by the sequence. */
	MOVIESCENE_API void ResetBindings();

protected:

	/** Rebuild the lookup map for efficient lookup. */
	MOVIESCENE_API void RebuildLookupMap() const;

#if WITH_EDITOR

	MOVIESCENE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

#endif

private:

	/** The actual binding data */
	UPROPERTY(EditAnywhere, Category="General", DisplayName="Binding Overrides")
	TArray<FMovieSceneBindingOverrideData> BindingData;

	/** Runtime lookup map. note this is only from GUID -> index. We do not hash the sequence ID into the map. this must be checked manually. */
	mutable bool bLookupDirty;
	mutable TMultiMap<FGuid, int32> LookupMap;
};
