// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "MovieSceneBindingOwnerInterface.generated.h"


class UMovieSceneSequence;
class IPropertyHandle;
class FStructOnScope;
struct FMovieScenePossessable;


/** Interface used in the editor to provide contextual information about movie scene bindings */
UINTERFACE(MinimalAPI)
class UMovieSceneBindingOwnerInterface
	: public UInterface
{
public:
	GENERATED_BODY()
};

class IMovieSceneBindingOwnerInterface
{
public:
	GENERATED_BODY()

#if WITH_EDITOR
	/** Retrieve the sequence that we own */
	virtual UMovieSceneSequence* RetrieveOwnedSequence() const = 0;

	/** Return a proxy struct used for editing the bound object */
	virtual TSharedPtr<FStructOnScope> GetObjectPickerProxy(TSharedPtr<IPropertyHandle> ObjectPropertyHandle) = 0;

	/** Update the specified object property handle based on the proxy structure's contents */
	virtual void UpdateObjectFromProxy(FStructOnScope& Proxy, IPropertyHandle& ObjectPropertyHandle) = 0;

	/** Find an IMovieSceneBindingOwnerInterface ptr from the specified object or its outers */
	static IMovieSceneBindingOwnerInterface* FindFromObject(UObject* InObject)
	{
		for ( ; InObject; InObject = InObject->GetOuter() )
		{
			if (IMovieSceneBindingOwnerInterface* Result = Cast<IMovieSceneBindingOwnerInterface>(InObject))
			{
				return Result;
			}
		}

		return nullptr;
	}
#endif
};
