// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "Templates/TypeHash.h"

#include "MovieSceneBindingProxy.generated.h"

class UMovieScene;
class UMovieSceneSequence;

//
// Movie Scene Binding Proxy represents the binding ID (an FGuid) and the corresponding sequence that it relates to. 
// This is primarily used for scripting where there is no support for FMovieSceneSequenceID and use cases where 
// managing multiple bindings with their corresponding sequences is necessary.
//
USTRUCT(BlueprintType)
struct FMovieSceneBindingProxy
{
	GENERATED_BODY()

	FMovieSceneBindingProxy()
		: Sequence(nullptr)
	{}

	FMovieSceneBindingProxy(const FGuid& InBindingID, UMovieSceneSequence* InSequence)
		: BindingID(InBindingID)
		, Sequence(InSequence)
	{}

	inline friend bool operator==(const FMovieSceneBindingProxy &LHS, const FMovieSceneBindingProxy &RHS)
	{
		return LHS.BindingID == RHS.BindingID && LHS.Sequence == RHS.Sequence;
	}

	inline friend bool operator!=(const FMovieSceneBindingProxy &LHS, const FMovieSceneBindingProxy &RHS)
	{
		return LHS.BindingID != RHS.BindingID || LHS.Sequence != RHS.Sequence;
	}

	inline friend uint32 GetTypeHash(const FMovieSceneBindingProxy& In)
	{
		return HashCombine(GetTypeHash(In.BindingID), GetTypeHash(In.Sequence));
	}

	MOVIESCENE_API UMovieScene* GetMovieScene() const;

	UPROPERTY(BlueprintReadOnly, Category=Binding)
	FGuid BindingID;

	UPROPERTY(BlueprintReadOnly, Category=Binding)
	TObjectPtr<UMovieSceneSequence> Sequence;
};
