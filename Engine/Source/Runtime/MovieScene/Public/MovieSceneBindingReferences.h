// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UniversalObjectLocator.h"
#include "UniversalObjectLocatorResolveParams.h"
#include "MovieSceneSequenceID.h"
#include "Bindings/MovieSceneCustomBinding.h"
#include "MovieSceneBindingReferences.generated.h"

class UWorld;
struct FWorldPartitionResolveData;
class UMovieSceneSequence;

namespace UE::MovieScene
{
	struct FSharedPlaybackState;
}


/**
 * An array of binding references
 */
USTRUCT()
struct FMovieSceneBindingReference
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid ID;

	UPROPERTY()
	FUniversalObjectLocator Locator;

	UPROPERTY()
	ELocatorResolveFlags ResolveFlags = ELocatorResolveFlags::None;

	UPROPERTY(Instanced)
	TObjectPtr<UMovieSceneCustomBinding> CustomBinding;

	void InitializeLocatorResolveFlags();
};

USTRUCT()
struct FMovieSceneBindingResolveParams
{
	GENERATED_BODY()

	/** The sequence that contains the object binding being resolved */
	UPROPERTY()
	TObjectPtr<UMovieSceneSequence> Sequence;

	/** The ID of the object binding being resolved */
	UPROPERTY()
	FGuid ObjectBindingID;

	/** The sequence ID of the object binding being resolved */
	UPROPERTY()
	FMovieSceneSequenceID SequenceID;

	/* The outer context with which to resolve this binding. May be the world, or may be an outer UObject.*/
	UPROPERTY()
	TObjectPtr<UObject> Context = nullptr;
};

/**
 * Structure that stores a one to many mapping from object binding ID, to object references that pertain to that ID.
 */
USTRUCT()
struct FMovieSceneBindingReferences
{
	GENERATED_BODY()

	MOVIESCENE_API TArrayView<const FMovieSceneBindingReference> GetAllReferences() const;

	MOVIESCENE_API TArrayView<FMovieSceneBindingReference> GetAllReferences();

	MOVIESCENE_API TArrayView<const FMovieSceneBindingReference> GetReferences(const FGuid& ObjectId) const;

	MOVIESCENE_API const FMovieSceneBindingReference* GetReference(const FGuid& ObjectId, int32 BindingIndex) const;

	/**
	 * Check whether this map has a binding for the specified object id
	 * @return true if this map contains a binding for the id, false otherwise
	 */
	MOVIESCENE_API bool HasBinding(const FGuid& ObjectId) const;

	/*
	* @return If a custom binding exists at the given id and index, returns it.
	*/
	MOVIESCENE_API UMovieSceneCustomBinding* GetCustomBinding(const FGuid& ObjectId, int32 BindingIndex);

	/*
	* @return If a custom binding exists at the given id and index, returns it.
	*/
	MOVIESCENE_API const UMovieSceneCustomBinding* GetCustomBinding(const FGuid& ObjectId, int32 BindingIndex) const;

	/**
	 * Remove a binding for the specified ID
	 *
	 * @param ObjectId	The ID to remove
	 */
	MOVIESCENE_API void RemoveBinding(const FGuid& ObjectId);

	/**
	 * Remove specific object references
	 *
	 * @param ObjectId	The ID to remove
	 * @param InObjects The objects to remove
	 * @param InContext A context in which InObject resides (either a UWorld, or an AActor)
	 */
	MOVIESCENE_API void RemoveObjects(const FGuid& ObjectId, const TArray<UObject*>& InObjects, UObject *InContext);

	/**
	 * Remove specific object references that do not resolve
	 *
	 * @param ObjectId	The ID to remove
	 * @param InContext A context in which InObject resides (either a UWorld, or an AActor)
	 */
	MOVIESCENE_API void RemoveInvalidObjects(const FGuid& ObjectId, UObject *InContext);

	/**
	 * Add a binding for the specified ID
	 *
	 * @param ObjectId	The ID to associate the object with
	 * @param InContext	A context in which InObject resides (either a UWorld, or an AActor)
	 */
	MOVIESCENE_API const FMovieSceneBindingReference* AddBinding(const FGuid& ObjectId, FUniversalObjectLocator&& NewLocator);

	/**
	 * Add a custom binding for the specified ID, no locator necessary.
	 *
	 * @param ObjectId	The ID to associate the object with
	 * @param CustomBinding A created custom binding for the object
	 */
	const FMovieSceneBindingReference* AddBinding(const FGuid& ObjectId, UMovieSceneCustomBinding* CustomBinding) { return AddBinding(ObjectId, FUniversalObjectLocator(), ELocatorResolveFlags::None, CustomBinding); }

	/**
	 * Add a binding for the specified ID
	 *
	 * @param ObjectId	The ID to associate the object with
	 * @param InContext	A context in which InObject resides (either a UWorld, or an AActor)
	 */
	MOVIESCENE_API const FMovieSceneBindingReference* AddBinding(const FGuid& ObjectId, FUniversalObjectLocator&& NewLocator, ELocatorResolveFlags InResolveFlags, UMovieSceneCustomBinding* CustomBinding=nullptr);

	/**
	 * Replace the binding associated with the ObjectId at the given BindingIndex with a new custom binding.
	 * If no such binding exists, then one will be created at the nearest new BindingIndex.
	 *
	 * @param ObjectId	The ID to associate the object with
	 * @param CustomBinding A created custom binding for the object
	 * @param BindingIndex The binding index of the binding to replace.
	 */
	MOVIESCENE_API const FMovieSceneBindingReference* AddOrReplaceBinding(const FGuid& ObjectId, UMovieSceneCustomBinding* NewCustomBinding, int32 BindingIndex);

	/**
 * Replace the binding associated with the ObjectId at the given BindingIndex with a new possessable locator binding.
	 * If no such binding exists, then one will be created at the nearest new BindingIndex.
	 *
	 * @param ObjectId	The ID to associate the object with
	 * @param NewLocator A created locator for the object
	 * @param BindingIndex The binding index of the binding to replace.
	 */
	MOVIESCENE_API const FMovieSceneBindingReference* AddOrReplaceBinding(const FGuid& ObjectId, FUniversalObjectLocator&& NewLocator, int32 BindingIndex);

	/**
	 * Resolve a binding for the specified ID using a given context.
	 * Calling this version will not resolve custom bindings/spawnables- to resolve those, please call the override with BindingResolveParams and SharedPlaybackState
	 *
	 * @param ObjectId					The ID to associate the object with
	 * @param Params					Resolve parameters specifying the context and fragment-specific parameters
	 * @param OutObjects				Array to populate with resolved object bindings
	 */
	MOVIESCENE_API void ResolveBinding(const FGuid& ObjectId, const UE::UniversalObjectLocator::FResolveParams& LocatorResolveParams, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const;

	/**
	 * Resolve a binding for the specified ID using a given context
	 *
	 * @param ObjectId					The ID to associate the object with
	 * @param BindingResolveParams		Resolve parameters
	 * @param LocatorResolveParams      Locator-specific resolve parameters
	 * @param SharedPlaybackState       Shared Playback State from the Sequencer
	 * @param OutObjects				Array to populate with resolved object bindings
	 */
	MOVIESCENE_API void ResolveBinding(const FMovieSceneBindingResolveParams& BindingResolveParams, const UE::UniversalObjectLocator::FResolveParams& LocatorResolveParams, TSharedPtr<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const;

	/**
	 * Resolve a binding for the specified ID and BindingIndex using a given context
	 *
	 * @param ObjectId					The ID to associate the object with
	 * @param BindingResolveParams		Resolve parameters
	 * @param LocatorResolveParams      Locator-specific resolve parameters
	 * @param SharedPlaybackState       Shared Playback State from the Sequencer
	 * @param OutObjects				Array to populate with resolved objectss
	 * */
	MOVIESCENE_API void ResolveSingleBinding(const FMovieSceneBindingResolveParams& BindingResolveParams, int32 BindingIndex, const UE::UniversalObjectLocator::FResolveParams& LocatorResolveParams, TSharedPtr<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const;

	UE_DEPRECATED(5.7, "This method has been deprecated. Please use ResolveSingleBinding which returns an array of bound objects.")
	MOVIESCENE_API UObject* ResolveSingleBinding(const FMovieSceneBindingResolveParams& BindingResolveParams, int32 BindingIndex, const UE::UniversalObjectLocator::FResolveParams& LocatorResolveParams, TSharedPtr<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const;

	/**
	 * Resolve a binding for the specified ID using a given context
	 *
	 * @param ObjectId					The ID to associate the object with
	 * @param InContext					A context in which InObject resides
	 * @oaram StreamedLevelAssetPath    The path to the streamed level asset that contains the level sequence actor playing back the sequence. 'None' for any non-instance-level setups.
	 * @param OutObjects				Array to populate with resolved object bindings
	 */
	UE_DEPRECATED(5.5, "This method has been deprecated as it produces errors for subobjects. Please call UMovieSceneSequence::FindBindingFromObject passing in a SharedPlaybackState")
	MOVIESCENE_API FGuid FindBindingFromObject(UObject* InObject, UObject* InContext) const;

	/**
	 * Filter out any bindings that do not match the specified set of GUIDs
	 *
	 * @param ValidBindingIDs A set of GUIDs that are considered valid. Anything references not matching these will be removed.
	 */
	MOVIESCENE_API void RemoveInvalidBindings(const TSet<FGuid>& ValidBindingIDs);

	/**
	 * Unloads an object that has been loaded via a locator.
	 *  @param ObjectId	The ID of the binding to unload
	 *  @param BindingIndex	The index of the binding to unload
	 */

	UE_DEPRECATED(5.5, "UnloadBoundObject no longer supported")
	void UnloadBoundObject(const UE::UniversalObjectLocator::FResolveParams& ResolveParams, const FGuid& ObjectId, int32 BindingIndex) {}

private:

	void ResolveBindingInternal(const FMovieSceneBindingResolveParams& BindingResolveParams, const UE::UniversalObjectLocator::FResolveParams& LocatorResolveParams, int32 BindingIndex, int32 InternalIndex, TSharedPtr<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const;
	UObject* ResolveBindingFromLocator(int32 Index, const UE::UniversalObjectLocator::FResolveParams& LocatorResolveParams) const;

	/** The map from object binding ID to an array of references that pertain to that ID */
	UPROPERTY()
	TArray<FMovieSceneBindingReference> SortedReferences;
};
