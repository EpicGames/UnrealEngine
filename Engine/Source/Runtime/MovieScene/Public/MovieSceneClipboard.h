// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "Evaluation/MovieSceneTimeTransform.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Misc/DateTime.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "Misc/InlineValue.h"
#include "Misc/Optional.h"
#include "MovieSceneTrack.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UnrealNames.h"

#if WITH_EDITOR


namespace MovieSceneClipboard
{
	template<typename T> static FName GetKeyTypeName()
	{
		// Use the template type here to stop the compiler generating the code unless it's actually used
		static_assert(!std::is_same_v<T, T>, "This function must be specialized to use with the specified type");
		return NAME_None;
	}
}

class UMovieSceneTrack;

/**
 * A key in the clipboard representing a time and a value of a specific type
 * Client code must define MovieSceneClipboard::GetKeyTypeName<> in order to use a type with this class.
 * In general, keys are stored relaative to the minumum key-time in the clipboard, or some arbitrary time.
 * This cardinal time is stored with the clipboard environment.
 */
class FMovieSceneClipboardKey
{
public:
	/** Templated constructor accepting a specific value type */
	template<typename T>
	FMovieSceneClipboardKey(FFrameNumber InTime, T InValue)
		: Time(InTime)
		, Data(TKey<T>(MoveTemp(InValue)))
	{
	}

	/** Copy construction/assignment */
	MOVIESCENE_API FMovieSceneClipboardKey(const FMovieSceneClipboardKey& In);
	MOVIESCENE_API FMovieSceneClipboardKey& operator=(const FMovieSceneClipboardKey& In);

	/** Get the time at which this key is defined */
	MOVIESCENE_API FFrameNumber GetTime() const;

	/** Set the time at which this key is defined */
	MOVIESCENE_API void SetTime(FFrameNumber InTime);

	/**
	 * Get the value of this key as the specified type.
	 *
	 * @return the key as the templated type, or a default-constructed T where no conversion is possible
	 */
	template<typename T>
	T GetValue() const
	{
		T Default = T();
		TryGetValue<T>(Default);
		return Default;
	}

	/**
	 * Attempt to get the value of this key as the specified type.
	 *
	 * @param InValue	Reference to receive the value
	 * @return true if the value was set, or false if no conversion between the source and T was possible
	 */
	template<typename T>
	bool TryGetValue(T& Value) const
	{
		if (!Data.IsValid())
		{
			return false;
		}

		// Check for same type
		if (Data->GetTypeName() == MovieSceneClipboard::GetKeyTypeName<T>())
		{
			Value = static_cast<const TKey<T>&>(Data.GetValue()).Value;
			return true;
		}

		// Check for conversion possibility
		FConversionFunction* ConversionFunction = Data->FindConversionTo(MovieSceneClipboard::GetKeyTypeName<T>());
		if (ConversionFunction)
		{
			FMovieSceneClipboardKey Key = ConversionFunction->operator()(*this);
			Value = static_cast<const TKey<T>&>(Key.Data.GetValue()).Value;
			return true;
		}

		return false;
	}

private:

	typedef TFunction<FMovieSceneClipboardKey(const FMovieSceneClipboardKey&)> FConversionFunction;

	/** Abstract base class for all key types */
	struct IKey
	{
		virtual ~IKey() = default;
		virtual void CopyTo(TInlineValue<IKey, 64>& OutDest) const = 0;
		virtual FConversionFunction* FindConversionTo(FName DestinationTypeName) const = 0;
		virtual FName GetTypeName() const = 0;
	};

	/** Implementation of any key-value type */
	template<typename T>
	struct TKey : IKey
	{
		TKey(T InValue) : Value(MoveTemp(InValue)) {}

		/** Find a conversion from this type to the specified destination type */
		virtual FConversionFunction* FindConversionTo(FName DestinationTypeName) const
		{
			TMap<FName, FConversionFunction>* ConversionBucket = ConversionMap.Find(MovieSceneClipboard::GetKeyTypeName<T>());
			if (ConversionBucket)
			{
				return ConversionBucket->Find(DestinationTypeName);
			}
			return nullptr;
		}

		/** Get the name of this value type */
		virtual FName GetTypeName() const
		{
			return MovieSceneClipboard::GetKeyTypeName<T>();
		}

		/** Copy this value to another destination ptr */
		virtual void CopyTo(TInlineValue<IKey, 64>& OutDest) const
		{
			OutDest = TKey(Value);
		}

		/** The actual value */
		T Value;
	};

	/** The time that this key is defined at */
	FFrameNumber Time;

	/** Type-erased bytes containing the key's value */
	TInlineValue<IKey, 64> Data;

public:

	/** Define a conversion from one type to another type */
	template<typename FromType, typename ToType>
	static void DefineConversion(TFunction<ToType(const FromType&)> InFunction)
	{
		auto Facade = [=](const FMovieSceneClipboardKey& InKey) -> FMovieSceneClipboardKey {
			const TKey<FromType>& TypedKey = static_cast<const TKey<FromType>&>(InKey.Data.GetValue());

			return FMovieSceneClipboardKey(InKey.GetTime(), InFunction(TypedKey.Value));
		};

		ConversionMap
			.Add(MovieSceneClipboard::GetKeyTypeName<FromType>())
			.Add(MovieSceneClipboard::GetKeyTypeName<ToType>(), Facade);
	}

private:
	/** Static map of conversions between types */
	static MOVIESCENE_API TMap<FName, TMap<FName, FConversionFunction>> ConversionMap;
};

/** Container for a collection of keys arranged in a track. */
class FMovieSceneClipboardKeyTrack
{
public:

	/** Create a key track that wraps the specified key type */
	template<typename KeyType>
	static FMovieSceneClipboardKeyTrack Create(FName InName, FFrameNumber KeyOffset = 0)
	{
		FMovieSceneClipboardKeyTrack Track;
		Track.TypeName = MovieSceneClipboard::GetKeyTypeName<KeyType>();
		Track.Name = MoveTemp(InName);
		Track.KeyOffset = KeyOffset;
		return Track;
	}

public:

	/** Move construction/assignment */
	MOVIESCENE_API FMovieSceneClipboardKeyTrack(FMovieSceneClipboardKeyTrack&& In);
	MOVIESCENE_API FMovieSceneClipboardKeyTrack& operator=(FMovieSceneClipboardKeyTrack&& In);

	/** Copy construction/assignment */
	MOVIESCENE_API FMovieSceneClipboardKeyTrack(const FMovieSceneClipboardKeyTrack& In);
	MOVIESCENE_API FMovieSceneClipboardKeyTrack& operator=(const FMovieSceneClipboardKeyTrack& In);

	/** Check the type of the keys contained within this track */
	template<typename KeyType>
	bool IsKeyOfType() const
	{
		return TypeName == MovieSceneClipboard::GetKeyTypeName<KeyType>();
	}

	/** Add a key of the specified type to this track. KeyType must match the type this track was constructed with */
	template<typename KeyType>
	void AddKey(FFrameNumber Time, KeyType Value)
	{
		Time -= KeyOffset;

		checkf(IsKeyOfType<KeyType>(), TEXT("Unable to add a key of a different value type to the track"));
		Keys.Add(FMovieSceneClipboardKey(Time, MoveTemp(Value)));
	}

	/**
	 * Iterate the keys contained within this track
	 * @param Iter		Predicate function to call for every key in this track.
	 *					return true to continue iteration, false to abort
	 */
	bool IterateKeys(TFunctionRef<bool(const FMovieSceneClipboardKey& Key)> Iter) const
	{
		for (const FMovieSceneClipboardKey& Key : Keys)
		{
			if (!Iter(Key))
			{
				return false;
			}
		}
		return true;
	}
	bool IterateKeys(TFunctionRef<bool(FMovieSceneClipboardKey& Key)> Iter)
	{
		for (FMovieSceneClipboardKey& Key : Keys)
		{
			if (!Iter(Key))
			{
				return false;
			}
		}
		return true;
	}

	/** Get the name of this track */
	MOVIESCENE_API const FName& GetName() const;

private:

	FMovieSceneClipboardKeyTrack()
	{}

	/** Collection of keys contained within this track */
	TArray<FMovieSceneClipboardKey> Keys;

	/** Type name for the value of the keys this track supports */
	FName TypeName;

	/** Generic name of this track (generally the name of a key area within a track e.g. Location.X) */
	FName Name;

	/** Optional key offset that is applied to all keys added to the clipboard */
	FFrameNumber KeyOffset;
};

/** Structure representing an environment a clipboard applies to */
struct FMovieSceneClipboardEnvironment
{
	FMovieSceneClipboardEnvironment()
		: CardinalTime(0)
		, DateTime(FDateTime::UtcNow())
	{}

	/** The cardinal time for a copy-paste operation. */
	FFrameTime CardinalTime;

	/** The time transform for the copy-paste operation. */
	FMovieSceneTimeTransform TimeTransform;

	/** The date/time at which the copy operation was performed */
	FDateTime DateTime;

	/** The tick resolution within which all clipboard data is stored */
	FFrameRate TickResolution;
};

/** A clipboard representing serializable copied data for a movie scene */
class FMovieSceneClipboard
{
public:
	/** Default Constructor */
	MOVIESCENE_API FMovieSceneClipboard();

	/** Move construction/assignment */
	MOVIESCENE_API FMovieSceneClipboard(FMovieSceneClipboard&& In);
	MOVIESCENE_API FMovieSceneClipboard& operator=(FMovieSceneClipboard&& In);

	/** Copy construction/assignment */
	MOVIESCENE_API FMovieSceneClipboard(const FMovieSceneClipboard& In);
	MOVIESCENE_API FMovieSceneClipboard& operator=(const FMovieSceneClipboard& In);

	/** Get the key track groups that were copied */
	MOVIESCENE_API const TArray<TArray<FMovieSceneClipboardKeyTrack>>& GetKeyTrackGroups() const;

	/** Get a text description of this clipboard for display on UI */
	MOVIESCENE_API FText GetDisplayText() const;

	/** Get the environment to which this clipboard relates */
	MOVIESCENE_API const FMovieSceneClipboardEnvironment& GetEnvironment() const;

	/** Get the environment to which this clipboard relates */
	MOVIESCENE_API FMovieSceneClipboardEnvironment& GetEnvironment();

private:
	friend class FMovieSceneClipboardBuilder;

	/** The environment to which this clipboard relates */
	FMovieSceneClipboardEnvironment Environment;

	/** Collection of groups of key tracks that have been copied */
	TArray<TArray<FMovieSceneClipboardKeyTrack>> KeyTrackGroups;
};

/** Class responsible for building a clipboard for a movie scene */
class FMovieSceneClipboardBuilder
{
public:

	/** Optional key offset that is applied to all keys added to the clipboard */
	FFrameNumber KeyOffset;

	/** Generate a clipboard for the current state of this builder, resetting the builder back to its default state */
	MOVIESCENE_API FMovieSceneClipboard Commit(TOptional<FFrameNumber> CopyRelativeTo);

	/**
	 * Find or add a key track. Key tracks are group primarily by MovieSceneTrack instance, then by name
	 * 
	 * @param InName		The name of this track (e.g. 'R', 'Location.X', etc)
	 * @param ParentTrack	The parent track within which this track resides
	 * @return A key track which can be populated with keys
	 */
	template<typename KeyType>
	FMovieSceneClipboardKeyTrack& FindOrAddKeyTrack(FName InName, UMovieSceneTrack& ParentTrack)
	{
		check(IsValid(&ParentTrack));

		TArray<FMovieSceneClipboardKeyTrack>& Tracks = TrackIndex.FindOrAdd(&ParentTrack);

		FMovieSceneClipboardKeyTrack* Existing = Tracks.FindByPredicate([&](const FMovieSceneClipboardKeyTrack& In){
			return In.GetName() == InName;
		});

		if (Existing)
		{
			return *Existing;
		}

		int32 Index = Tracks.Num();
		Tracks.Add(FMovieSceneClipboardKeyTrack::Create<KeyType>(MoveTemp(InName)));
		return Tracks.GetData()[Index];
	}

private:
	/** Map of key tracks for a given moviescene track ptr */
	TMap<UMovieSceneTrack*, TArray<FMovieSceneClipboardKeyTrack>> TrackIndex;
};

/** Helper functions for defining conversions between types */
namespace MovieSceneClipboard
{
	template<typename From, typename To>
	struct TImplicitConversionFacade
	{
		static To Cast(const From& Value) { return Value; }
	};

	template<typename From>
	struct TImplicitConversionFacade<From, bool>
	{
		static bool Cast(const From& Value) { return !!Value; }
	};

	template<typename From, typename To>
	void DefineExplicitConversion(TFunction<To(const From&)> InConversion)
	{
		FMovieSceneClipboardKey::DefineConversion<From, To>(InConversion);
	}

	template<typename From, typename To>
	void DefineImplicitConversion()
	{
		DefineExplicitConversion<From, To>(&TImplicitConversionFacade<From, To>::Cast);
	}
}

#endif
