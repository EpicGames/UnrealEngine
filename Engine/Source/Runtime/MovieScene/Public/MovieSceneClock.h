// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/FrameRate.h"
#include "MovieSceneSignedObject.h"
#include "UObject/Object.h"
#include "MovieSceneClock.generated.h"

struct FMovieSceneTimeController;
struct FMovieSceneSequenceTransform;
struct FMovieSceneSectionTimingParametersFrames;

class UMovieSceneSubSection;

/**
 * 
 */
UCLASS(MinimalAPI)
class UMovieSceneClock : public UMovieSceneSignedObject
{
public:

	GENERATED_BODY()

	MOVIESCENE_API virtual TSharedPtr<FMovieSceneTimeController> MakeTimeController(UObject* PlaybackContext) const;

	virtual bool MakeSubSequenceTransform(const FMovieSceneSectionTimingParametersFrames& Timing, const UMovieSceneSubSection* SubSection, FMovieSceneSequenceTransform& OutTransform) const
	{
		return false;
	}

	virtual void HandleTickResolutionChange(FFrameRate PreviousTickResolution, FFrameRate NewTickResolution)
	{}
};


UCLASS(MinimalAPI)
class UMovieSceneExternalClock : public UMovieSceneClock
{
public:

	GENERATED_BODY()

	MOVIESCENE_API virtual TSharedPtr<FMovieSceneTimeController> MakeTimeController(UObject* PlaybackContext) const override;

	UPROPERTY()
	FSoftObjectPath CustomClockSourcePath;
};


