// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "UObject/UnrealType.h"
#include "UObject/ObjectKey.h"
#include "Curves/KeyHandle.h"
#include "Misc/FrameNumber.h"
#include "UObject/WeakFieldPtr.h"

#include "TrackInstancePropertyBindings.h"  // For backwards compatibility

class AActor;
class UCameraComponent;
class UMovieScene;
class UMovieSceneSection;
class UMovieSceneSubSection;
class UMovieSceneSequence;
class USceneComponent;
class USoundBase;
template <class TClass> class TSubclassOf;
class UMovieSceneCustomBinding;
struct FRichCurve;
enum class EMovieSceneKeyInterpolation : uint8;
struct FMovieSceneChannel;
struct FMovieSceneChannelMetaData;
struct FMovieSceneSequenceID;
class UMovieSceneCondition;
class UMovieSceneTrack;

namespace UE::MovieScene
{
	struct FSharedPlaybackState;
}

class MovieSceneHelpers
{
public:

	/*
	* Helper struct to cache the package dirty state and then to restore it
	* after this leaves scope. This is for a few minor areas where calling
	* functions on actors dirties them, but Sequencer doesn't actually want
	* the package to be dirty as it causes Sequencer to unnecessairly dirty
	* actors.
	*/
	struct FMovieSceneScopedPackageDirtyGuard
	{
		MOVIESCENE_API FMovieSceneScopedPackageDirtyGuard(class USceneComponent* InComponent);
		MOVIESCENE_API virtual ~FMovieSceneScopedPackageDirtyGuard();

	private:
		class USceneComponent* Component;
		bool bPackageWasDirty;
	};

	/** Get displayable names */
	static MOVIESCENE_API FText GetDisplayPathName(const UMovieSceneTrack* Track);
	static MOVIESCENE_API FText GetDisplayPathName(const UMovieSceneSection* Section);
	static MOVIESCENE_API FText GetDisplayPathName(const UMovieSceneSection* Section, const FMovieSceneChannel* Channel, const FMovieSceneChannelMetaData& ChannelMetaData);

	/** 
	 * @return Whether the section is considered keyable. The section must be active and on a track that is not muted. 
	 * If there are existing channels, the channels must be active and marked as keyable. A section with no channels is 
	 * considered keyable.
	 */
	static MOVIESCENE_API bool IsSectionKeyable(const UMovieSceneSection*);

	/**
	 * Finds a section that exists at a given time
	 *
	 * @param Time	The time to find a section at
	 * @param RowIndex  Limit the search to a given row index
	 * @return The found section or null
	 */
	static MOVIESCENE_API UMovieSceneSection* FindSectionAtTime( TArrayView<UMovieSceneSection* const> Sections, FFrameNumber Time, int32 RowIndex = INDEX_NONE );

	/**
	 * Finds the nearest section to the given time
	 *
	 * @param Time	The time to find a section at
	 * @param RowIndex  Limit the search to a given row index
	 * @return The found section or null
	 */
	static MOVIESCENE_API UMovieSceneSection* FindNearestSectionAtTime( TArrayView<UMovieSceneSection* const> Sections, FFrameNumber Time, int32 RowIndex = INDEX_NONE );

	/** Find the next section that doesn't overlap - the section that has the next closest start time to the requested start time */
	static MOVIESCENE_API UMovieSceneSection* FindNextSection(TArrayView<UMovieSceneSection* const> Sections, FFrameNumber Time);

	/** Find the previous section that doesn't overlap - the section that has the previous closest start time to the requested start time */
	static MOVIESCENE_API UMovieSceneSection* FindPreviousSection(TArrayView<UMovieSceneSection* const> Sections, FFrameNumber Time);

	/*
	 * Fix up consecutive sections so that there are no gaps
	 * 
	 * @param Sections All the sections
	 * @param Section The section that was modified 
	 * @param bDelete Was this a deletion?
	 * @param bCleanUp Should we cleanup any invalid sections?
	 * @return Whether the list of sections was modified as part of the clean-up
	 */
	static MOVIESCENE_API bool FixupConsecutiveSections(TArray<UMovieSceneSection*>& Sections, UMovieSceneSection& Section, bool bDelete, bool bCleanUp = false);

	/**
	 * Fix up consecutive sections so that there are no gaps, but there can be overlaps, in which case the sections
	 * blend together.
	 *
	 * @param Sections All the sections
	 * @param Section The section that was modified 
	 * @param bDelete Was this a deletion?
	 * @param bCleanUp Should we cleanup any invalid sections?
	 * @return Whether the list of sections was modified as part of the clean-up
	 */
	static MOVIESCENE_API bool FixupConsecutiveBlendingSections(TArray<UMovieSceneSection*>& Sections, UMovieSceneSection& Section, bool bDelete, bool bCleanUp = false);

	/*
 	 * Sort consecutive sections so that they are in order based on start time
 	 */
	static MOVIESCENE_API void SortConsecutiveSections(TArray<UMovieSceneSection*>& Sections);

	/*
	 * Gather up descendant movie scenes from the incoming sequence
	 */
	static MOVIESCENE_API void GetDescendantMovieScenes(UMovieSceneSequence* InSequence, TArray<UMovieScene*> & InMovieScenes);

	/*
	 * Gather up descendant movie scene sub-sections from the incoming movie scene
	 */
	static MOVIESCENE_API void GetDescendantSubSections(const UMovieScene* InMovieScene, TArray<UMovieSceneSubSection*>& InSubSections);
	
	/*
	 * 	Disable the entire track or each of the rows
 	 */
	static MOVIESCENE_API void DisableTrack(UMovieSceneTrack* InTrack);

	/**
	 * Get the scene component from the runtime object
	 *
	 * @param Object The object to get the scene component for
	 * @return The found scene component
	 */	
	static MOVIESCENE_API USceneComponent* SceneComponentFromRuntimeObject(UObject* Object);
	static MOVIESCENE_API UObject* ResolveSceneComponentBoundObject(UObject* Object);

	/**
	 * Get the active camera component from the actor 
	 *
	 * @param InActor The actor to look for the camera component on
	 * @return The active camera component
	 */
	static MOVIESCENE_API UCameraComponent* CameraComponentFromActor(const AActor* InActor);

	/**
	 * Find and return camera component from the runtime object
	 *
	 * @param Object The object to get the camera component for
	 * @return The found camera component
	 */	
	static MOVIESCENE_API UCameraComponent* CameraComponentFromRuntimeObject(UObject* RuntimeObject);

	/*
	 * Get the duration for the given sound

	 * @param Sound The sound to get the duration for
	 * @return The duration in seconds
	 */
	static MOVIESCENE_API float GetSoundDuration(USoundBase* Sound);

	/**
	 * Sort predicate that sorts lower bounds of a range
	 */
	static bool SortLowerBounds(TRangeBound<FFrameNumber> A, TRangeBound<FFrameNumber> B)
	{
		return TRangeBound<FFrameNumber>::MinLower(A, B) == A && A != B;
	}

	/**
	 * Sort predicate that sorts upper bounds of a range
	 */
	static bool SortUpperBounds(TRangeBound<FFrameNumber> A, TRangeBound<FFrameNumber> B)
	{
		return TRangeBound<FFrameNumber>::MinUpper(A, B) == A && A != B;
	}

	/**
	 * Sort predicate that sorts overlapping sections by row primarily, then by overlap priority
	 */
	static MOVIESCENE_API bool SortOverlappingSections(const UMovieSceneSection* A, const UMovieSceneSection* B);

	/*
	* Get weight needed to modify the global difference in order to correctly key this section due to it possibly being blended by other sections.
	* @param Section The Section who's weight we are calculating.
	* @param  Time we are at.
	* @return Returns the weight that needs to be applied to the global difference to correctly key this section.
	*/
	static MOVIESCENE_API float CalculateWeightForBlending(UMovieSceneSection* SectionToKey, FFrameNumber Time);

	/*
	 * Return a name unique to the binding names in the given movie scene
	 * @param InMovieScene The movie scene to look for existing possessables.
	 * @param InName The requested name to make unique.
	 * @return The unique name
	 */
	static MOVIESCENE_API FString MakeUniqueBindingName(UMovieScene* InMovieScene, const FString& InName);

	/*
	 * Return a name unique to the spawnable names in the given movie scene
	 * @param InMovieScene The movie scene to look for existing spawnables.
	 * @param InName The requested name to make unique.
	 * @return The unique name
	 */
	static MOVIESCENE_API FString MakeUniqueSpawnableName(UMovieScene* InMovieScene, const FString& InName);

	/**
	 * Return a copy of the source object, suitable for use as a spawnable template.
	 * @param InSourceObject The source object to convert into a spawnable template
	 * @param InMovieScene The movie scene the spawnable template will be associated with
	 * @param InName The name to use for the spawnable template
	 * @return The spawnable template
	 */
	static MOVIESCENE_API UObject* MakeSpawnableTemplateFromInstance(UObject& InSourceObject, UMovieScene* InMovieScene, FName InName);

	/*
	* Returns whether the given ObjectId is valid and is currently bound to at least 1 spawnable give the current context.
	* More specifically, if a FMovieSceneSpawnable exists with this ObjectId, true will be returned.
	* If a Level Sequence binding reference exists with a Custom Binding implementing MovieSceneSpawnableBindingBase, true will be returned.
	* If a Level Sequence binding reference exists with a Custom Binding implementing MovieSceneReplaceableBindingBase and the Context is an editor world, then true will be returned.
	* Otherwise, false will be returned.
	*/
	static MOVIESCENE_API bool IsBoundToAnySpawnable(UMovieSceneSequence* Sequence, const FGuid& ObjectId, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState);

	/*
	* Returns whether the given ObjectId is valid and is the given bindingindex is currently bound to a spawnable give the current context.
	* More specifically, if a FMovieSceneSpawnable exists with this ObjectId, true will be returned.
	* If a Level Sequence binding reference for this guid with the given BindingIndex exists with a Custom Binding implementing MovieSceneSpawnableBindingBase, true will be returned.
	* If a Level Sequence binding reference for this guid with the given BindingIndex exists with a Custom Binding implementing MovieSceneReplaceableBindingBase and the Context is an editor world, then true will be returned.
	* Otherwise, false will be returned.
	*/
	static MOVIESCENE_API bool IsBoundToSpawnable(UMovieSceneSequence* Sequence, const FGuid& ObjectId, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, int32 BindingIndex = 0);

	/*
	* Attempts to create a new custom spawnable binding for the passed in UObject*. 
	* Where possible, it is preferred to call FSequencerUtilities::CreateOrReplaceBinding as it handles more cases. This should only be called in cases where there is no editor or sequencer context.
	* FactoryCreatedActor may be passed in as an alternative option for creating the binding in the case an actor factory was able to create an actor from this object.
	*/

	static MOVIESCENE_API FGuid TryCreateCustomSpawnableBinding(UMovieSceneSequence* Sequence, UObject* CustomBindingObject);
	

	/*
	* Returns the objects currently bound to the given objectid and binding index (optional).
	*/
	static MOVIESCENE_API TArray<UObject*> GetBoundObjects(const FMovieSceneSequenceID& SequenceID, const FGuid& ObjectId, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, int32 BindingIndex = 0);

	UE_DEPRECATED(5.7, "This method has been deprecated. Please use GetBoundObjects which returns an array of bound objects.")
	static MOVIESCENE_API UObject* GetSingleBoundObject(UMovieSceneSequence* Sequence, const FGuid& ObjectId, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, int32 BindingIndex = 0);

	/*
	* If the binding for the given ObjectId supports object templates, returns the template, otherwise returns nullptr
	*/
	static MOVIESCENE_API UObject* GetObjectTemplate(UMovieSceneSequence* Sequence, const FGuid& ObjectId, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, int32 BindingIndex = 0);

	/*
	* If the binding for the given ObjectId supports object templates, sets the template and returns true, otherwise returns false
	*/
	static MOVIESCENE_API bool SetObjectTemplate(UMovieSceneSequence* Sequence, const FGuid& ObjectId, UObject* InSourceObject, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, int32 BindingIndex = 0);

	/*
	* Returns whether the binding for the given ObjectId supports object templates
	*/
	static MOVIESCENE_API bool SupportsObjectTemplate(UMovieSceneSequence* Sequence, const FGuid& ObjectId, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, int32 BindingIndex = 0);

	/*
	* If the binding for the given ObjectId supports object templates, copies the object template into the binding and returns true, otherwise returns false
	*/
	static MOVIESCENE_API bool CopyObjectTemplate(UMovieSceneSequence* Sequence, const FGuid& ObjectId, UObject* InSourceObject, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, int32 BindingIndex = 0);
#if WITH_EDITORONLY_DATA
	/*
	* Returns the bound object class for the binding for the given ObjectId.
	*/
	static MOVIESCENE_API const UClass* GetBoundObjectClass(UMovieSceneSequence* Sequence, const FGuid& ObjectId, int32 BindingIndex = 0);
#endif

	/*  Will set the track with this order of sections. Will return false if unsuccessful, with the operation failing. This can happen if sections
	are not owned by the track or if there are any duplicates or if there are any missing sections. This function will not allow adding or removing
	sections, just re-ordering them*/
	static bool MOVIESCENE_API ResetSectionOrder(UMovieSceneTrack* Track, const TArray<UMovieSceneSection*>& NewSections);

	/* Returns a sorted list of all custom binding type classes currently known. Slow, may desire to cache result*/
	static void MOVIESCENE_API GetPrioritySortedCustomBindingTypes(TArray<const TSubclassOf<UMovieSceneCustomBinding>>& OutCustomBindingTypes);

	/* For cases where the user does not have a IMovieScenePlayer with a shared playback state, creates a transient one. Use sparingly. */
	static TSharedRef<UE::MovieScene::FSharedPlaybackState> MOVIESCENE_API CreateTransientSharedPlaybackState(UObject* WorldContext, UMovieSceneSequence* Sequence);

	/* Finds the resolution context to use to resolve the given guid. */
	static MOVIESCENE_API UObject* GetResolutionContext(UMovieSceneSequence* Sequence, const FGuid& ObjectId, const FMovieSceneSequenceID& SequenceID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState);

	/* Given a movie scene track and an optional section inside it, returns an optional single condition that needs to be evaluated.
	* If multiple conditions exist in the given scope (for example a track condition, a track row condition for the row the section is on, and a section),
	* a UMovieSceneGroupCondition will be generated, and the caller is responsible for holding a reference to this new UObject.
	* If bFromCompilation is true, then any generated conditions will be stored on the movie scene.
	*/
	static MOVIESCENE_API const UMovieSceneCondition* GetSequenceCondition(const UMovieSceneTrack* Track, const UMovieSceneSection* Section, bool bFromCompilation=false);
	
	/* Helper function for evaluating a condition in a movie scene, taking advantage of any cacheing that may apply. */
	static MOVIESCENE_API bool EvaluateSequenceCondition(const FGuid& BindingID, const FMovieSceneSequenceID& SequenceID, const UMovieSceneCondition* Condition, UObject* ConditionOwnerObject, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState);
};

