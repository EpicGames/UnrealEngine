// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "MovieSceneSequenceID.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"

#include "MovieSceneDynamicBinding.generated.h"

class UMovieScene;
class UMovieSceneSequence;
class UMovieSceneEntitySystemLinker;

/** Value definition for any type-agnostic variable (exported as text) */
USTRUCT(BlueprintType)
struct FMovieSceneDynamicBindingPayloadVariable
{
	GENERATED_BODY()
		
	UPROPERTY()
	FSoftObjectPath ObjectValue;

	UPROPERTY(EditAnywhere, Category = "Sequencer|Dynamic Binding")
	FString Value;
};

/**
 * Data for a dynamic binding endpoint call.
 */
USTRUCT()
struct FMovieSceneDynamicBinding
{
	GENERATED_BODY()

	/** The function to call (normally a generated blueprint function on the sequence director) */
	UPROPERTY()
	TObjectPtr<UFunction> Function;

	/** Property pointer for the function parameter that should receive the resolve params */
	UPROPERTY()
	TFieldPath<FProperty> ResolveParamsProperty;

#if WITH_EDITORONLY_DATA

	/** Array of payload variables to be added to the generated function */
	UPROPERTY(EditAnywhere, Category = "Sequencer|Dynamic Binding")
	TMap<FName, FMovieSceneDynamicBindingPayloadVariable> PayloadVariables;

	/** Name of the generated blueprint function */
	UPROPERTY(transient)
	FName CompiledFunctionName;

	/** Pin name for passing the resolve params */
	UPROPERTY(EditAnywhere, Category="Sequencer|Dynamic Binding")
	FName ResolveParamsPinName;

	/** Endpoint node in the sequence director */
	UPROPERTY(EditAnywhere, Category="Sequencer|Dynamic Binding")
	TWeakObjectPtr<UObject> WeakEndpoint;

#endif
};

/**
 * Optional parameter struct for dynamic binding resolver functions.
 */
USTRUCT(BlueprintType)
struct FMovieSceneDynamicBindingResolveParams
{
	GENERATED_BODY()

	/** The sequence that contains the object binding being resolved */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="General")
	TObjectPtr<UMovieSceneSequence> Sequence;

	/** The ID of the object binding being resolved */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="General")
	FGuid ObjectBindingID;

	/** The root sequence */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="General")
	TObjectPtr<UMovieSceneSequence> RootSequence;
};

USTRUCT(BlueprintType)
struct FMovieSceneDynamicBindingResolveResult
{
	GENERATED_BODY()

	/** The resolved objects */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "General")
	TArray<TObjectPtr<UObject>> Objects;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "General", meta = (DeprecatedProperty, DeprecationMessage = "This property is deprecated, please use the array of Objects instead."))
	TObjectPtr<UObject> Object = nullptr;

	/**
	 * Whether the resolved object is external to the sequence
	 *
	 * This property is ignored for possessables, who are always treated as external.
	 * When resolving a spawnable, setting this to true will not destroy the object
	 * when the spawnable track ends, or the sequence finishes.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="General")
	bool bIsPossessedObject = true;
};

/**
 * Dummy structure for showing an FMovieSceneDynamicBinding inside a details view,
 * and having a function signature that we can get a valid UFunction from to prepare
 * blueprint function graphs.
 */
USTRUCT()
struct FMovieSceneDynamicBindingContainer
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Sequencer")
	FMovieSceneDynamicBinding DynamicBinding;
};

/**
 * Default dynamic binding resolver library, with several basic resolver functions.
 */
UCLASS(meta=(SequencerBindingResolverLibrary), MinimalAPI)
class UBuiltInDynamicBindingResolverLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** Resolve the bound object to the player's pawn */
	UFUNCTION(BlueprintPure, Category="Sequencer|Dynamic Binding", meta=(WorldContext="WorldContextObject"))
	static MOVIESCENE_API FMovieSceneDynamicBindingResolveResult ResolveToPlayerPawn(UObject* WorldContextObject, int32 PlayerControllerIndex = 0);
};

