// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "MovieSceneDynamicBinding.h"

#define UE_API MOVIESCENE_API

class UObject;
class IMovieScenePlayer;
class UMovieSceneSequence;
struct FGuid;
struct FMovieScenePossessable;
struct FMovieSceneSequenceID;
struct FMovieSceneSpawnable;

namespace UE::MovieScene
{
	struct FSharedPlaybackState;
}

/**
 * Utility class for invoking dynamic binding endpoints.
 */
struct FMovieSceneDynamicBindingInvoker
{
	using FSharedPlaybackState = UE::MovieScene::FSharedPlaybackState;

	static UE_API FMovieSceneDynamicBindingResolveResult ResolveDynamicBinding(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, UMovieSceneSequence* Sequence, const FMovieSceneSequenceID& SequenceID, const FGuid& InGuid, const FMovieSceneDynamicBinding& DynamicBinding);


private:
	static FMovieSceneDynamicBindingResolveResult InvokeDynamicBinding(UObject* DirectorInstance, const FMovieSceneDynamicBinding& DynamicBinding, const FMovieSceneDynamicBindingResolveParams& Params);
};

#undef UE_API
