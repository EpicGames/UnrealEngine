// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MovieSceneFwd.h"
#include "Misc/InlineValue.h"
#include "Evaluation/MovieSceneAnimTypeID.h"
#include "Evaluation/PreAnimatedState/MovieSceneRestoreStateParams.h"

class IMovieScenePlayer;
struct FMovieSceneContext;
struct FMovieSceneEvaluationOperand;
struct FPersistentEvaluationData;

/**
 * Base class for all pre-animated state tokens that apply to UObjects. Store any cached state in derived classes
 */
struct IMovieScenePreAnimatedToken
{
	virtual ~IMovieScenePreAnimatedToken() = default;

	/**
	 * Restore state for the specified object, only called when this token was created with a bound object
	 *
	 * @param Object The object to restore state for
	 * @param Params Parameters for restoring state
	 */
	virtual void RestoreState(UObject& Object, const UE::MovieScene::FRestoreStateParams& Params) = 0;
};
typedef TInlineValue<IMovieScenePreAnimatedToken, 32> IMovieScenePreAnimatedTokenPtr;

/**
 * Type required for production of pre-animated state tokens.
 * Implemented as a type rather than a callback to ensure efficient construction (these types are often constructed, but rarely utilized)
 */
struct IMovieScenePreAnimatedTokenProducer
{
	virtual ~IMovieScenePreAnimatedTokenProducer()= default;

	/**
	 * Perform any initial set up required to animate the specified object
	 * @note Only ever called when Object is in an unanimated state, as according to the AnimTypeID that this producer is operating on.
	 * 
	 * @param Object 		The object to initialize
	 */
	virtual void InitializeObjectForAnimation(UObject& Object) const
	{}

	/**
	 * Produce a token that can be used to return the specified object back to its current state
	 * @note Under some circumstances, the object may already be animated (for instance, after something has animated the object, but didn't restore state)
	 * 
	 * @param Object 		The object to initialize
	 *
	 * @return A valid pre animated token that will restore this object back to its current state
	 */
	virtual IMovieScenePreAnimatedTokenPtr CacheExistingState(UObject& Object) const = 0;
};

/**
 * Base class for all pre-animated state tokens that do not apply to UObjects. Store any cached state in derived classes.
 */
struct IMovieScenePreAnimatedGlobalToken
{
	virtual ~IMovieScenePreAnimatedGlobalToken() = default;

	/**
	 * Restore global state for a root track.
	 *
	 * @param Params Parameters for restoring state
	 */
	virtual void RestoreState(const UE::MovieScene::FRestoreStateParams& Params) = 0;
};
typedef TInlineValue<IMovieScenePreAnimatedGlobalToken, 32> IMovieScenePreAnimatedGlobalTokenPtr;

/**
 * Type required for production of pre-animated state tokens.
 * Implemented as a type rather than a callback to ensure efficient construction (these types are often constructed, but rarely utilized)
 */
struct IMovieScenePreAnimatedGlobalTokenProducer
{
	virtual ~IMovieScenePreAnimatedGlobalTokenProducer() = default;

	/**
	 * Perform any initial set up required to animate the playback environment
	 * @note Only ever called when environment is in an unanimated state, as according to the AnimTypeID that this producer is operating on.
	 */
	virtual void InitializeForAnimation() const {}

	/**
	 * Produce a token that can be used to return the playback environment back to its current state
	 * @note Under some circumstances, the environment may already be animated (for instance, after something has animated, but didn't restore state)
	 *
	 * @return A valid pre animated token that will restore the environment back to its current state
	 */
	virtual IMovieScenePreAnimatedGlobalTokenPtr CacheExistingState() const = 0;
};

/**
 * Base class for all execution tokens that are produced by evaluation templates
 */
struct IMovieSceneExecutionToken
{
	virtual ~IMovieSceneExecutionToken() = default;
	virtual void Execute(const FMovieSceneContext& Context, const FMovieSceneEvaluationOperand& Operand, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) = 0;
};

/** Identifiyable and sortable execution token. Gets evaluated before (Order <=0) or after (Order > 0) IMovieSceneExecutionTokens */
struct IMovieSceneSharedExecutionToken
{
	IMovieSceneSharedExecutionToken()
		: Order(0)
	{}
	
	virtual ~IMovieSceneSharedExecutionToken() = default;

	/** Execute this token */
	virtual void Execute(FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) = 0;

	/** The order at which this token should be executed. <= 0 executed before IMovieSceneExecutionTokens, > 0 after */
	int32 Order;
};

/** Stateless pre-animated state token producer that simply calls a static function as the token */
struct FStatelessPreAnimatedTokenProducer : IMovieScenePreAnimatedTokenProducer
{
	typedef void (*StaticFunction)(UObject& Object, const UE::MovieScene::FRestoreStateParams& Params);

	FStatelessPreAnimatedTokenProducer(StaticFunction InFunction) : Function(InFunction) {}

	virtual IMovieScenePreAnimatedTokenPtr CacheExistingState(UObject& Object) const override
	{
		return FToken(Function);
	}
	
	struct FToken : IMovieScenePreAnimatedToken
	{
		FToken(StaticFunction InFunctionPtr) : FunctionPtr(InFunctionPtr) {}

		virtual void RestoreState(UObject& Object, const UE::MovieScene::FRestoreStateParams& Params) override
		{
			(*FunctionPtr)(Object, Params);
		}

		StaticFunction FunctionPtr;
	};
	StaticFunction Function;
};

/** Templated stateless pre-animated state token producer that simply creates the templated type */
template<typename TokenType>
struct TStatelessPreAnimatedTokenProducer : IMovieScenePreAnimatedTokenProducer
{
	virtual IMovieScenePreAnimatedTokenPtr CacheExistingState(UObject& Object) const override
	{
		return TokenType();
	}
};

/** Templated pre-animated state token producer that forwards the object onto the templated type */
template<typename TokenType>
struct TForwardingPreAnimatedTokenProducer : IMovieScenePreAnimatedTokenProducer
{
	virtual IMovieScenePreAnimatedTokenPtr CacheExistingState(UObject& Object) const override
	{
		return TokenType(Object);
	}
};

