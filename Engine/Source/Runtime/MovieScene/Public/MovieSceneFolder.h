// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Decorations/MovieSceneDecorationContainer.h"
#include "EventHandlers/IFolderEventHandler.h"
#include "EventHandlers/MovieSceneDataEventContainer.h"
#include "Math/Color.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneFolder.generated.h"

class FArchive;
class ITransactionObjectAnnotation;
class UMovieSceneTrack;
namespace UE { namespace MovieScene { class IFolderEventHandler; } }

/** Represents a folder used for organizing objects in tracks in a movie scene. */
UCLASS(DefaultToInstanced, MinimalAPI, BlueprintType)
class UMovieSceneFolder
	: public UMovieSceneDecorationContainerObject
{
	GENERATED_UCLASS_BODY()

	/** Gets the name of this folder. */
	MOVIESCENE_API FName GetFolderName() const;

	/** Sets the name of this folder. Automatically calls Modify() on the folder object. */
	MOVIESCENE_API void SetFolderName( FName InFolderName );

	/** Gets the folders contained by this folder. */
	MOVIESCENE_API TArrayView<UMovieSceneFolder* const> GetChildFolders() const;

	/** Adds a child folder to this folder. Removes the folder from any other folders (including root folders). Automatically calls Modify() on the folder object. */
	MOVIESCENE_API void AddChildFolder( UMovieSceneFolder* InChildFolder );

	/** Removes a child folder from this folder. Automatically calls Modify() on the folder object. */
	MOVIESCENE_API void RemoveChildFolder( UMovieSceneFolder* InChildFolder );

	/** Gets the tracks contained by this folder. */
	MOVIESCENE_API const TArray<UMovieSceneTrack*>& GetChildTracks() const;

	/** Adds a track to this folder. Automatically calls Modify() on the folder object. */
	MOVIESCENE_API void AddChildTrack( UMovieSceneTrack* InTrack );

	/** Removes a track from this folder. Automatically calls Modify() on the folder object. */
	MOVIESCENE_API void RemoveChildTrack( UMovieSceneTrack* InTrack );

	/** Clear all child tracks from this folder. */
	MOVIESCENE_API void ClearChildTracks();

	/** Gets the guids for the object bindings contained by this folder. */
	MOVIESCENE_API const TArray<FGuid>& GetChildObjectBindings() const;

	/** Adds a guid for an object binding to this folder. Automatically calls Modify() on the folder object. */
	MOVIESCENE_API void AddChildObjectBinding(const FGuid& InObjectBinding );

	/** Removes a guid for an object binding from this folder. Automatically calls Modify() on the folder object. */
	MOVIESCENE_API void RemoveChildObjectBinding( const FGuid& InObjectBinding );

	/** Clear all child object bindings from this folder. */
	MOVIESCENE_API void ClearChildObjectBindings();

	/** Called after this object has been deserialized */
	MOVIESCENE_API virtual void PostLoad() override;

	/** Searches for a guid in this folder and its child folders, if found returns the folder containing the guid. */
	MOVIESCENE_API UMovieSceneFolder* FindFolderContaining(const FGuid& InObjectBinding);

	/** Searches for a track in this folder and its child folders, if found returns the folder containing the track. */
	MOVIESCENE_API UMovieSceneFolder* FindFolderContaining(const UMovieSceneTrack* InTrack);

	/** Get the folder path for this folder, stopping at the given root folders */
	static MOVIESCENE_API void CalculateFolderPath(UMovieSceneFolder* InFolder, TArrayView<UMovieSceneFolder* const> RootFolders, TArray<FName>& FolderPath);
	
	/** For the given set of folders, return the folder that has the matching folder path */
	static MOVIESCENE_API UMovieSceneFolder* GetFolderWithPath(const TArray<FName>& InFolderPath, const TArray<UMovieSceneFolder*>& InFolders, TArrayView<UMovieSceneFolder* const> RootFolders);

	MOVIESCENE_API virtual void Serialize( FArchive& Archive );

	MOVIESCENE_API FName MakeUniqueChildFolderName(FName InName) const;

	static MOVIESCENE_API FName MakeUniqueChildFolderName(FName InName, TArrayView<UMovieSceneFolder* const> InFolders);

#if WITH_EDITORONLY_DATA
	/**
	 * Get this folder's color.
	 *
	 * @return The folder color.
	 */
	const FColor& GetFolderColor() const
	{
		return FolderColor;
	}

	/**
	 * Set this folder's color. Does not call Modify() on the folder object for legacy reasons.
	 *
	 * @param InFolderColor The folder color to set.
	 */
	void SetFolderColor(const FColor& InFolderColor)
	{
		FolderColor = InFolderColor;
	}

	/**
	 * Get this folder's desired sorting order 
	 */
	int32 GetSortingOrder() const
	{
		return SortingOrder;
	}

	/**
	 * Set this folder's desired sorting order. Does not call Modify() internally for legacy reasons.
	 *
	 * @param InSortingOrder The higher the value the further down the list the folder will be.
	 */
	void SetSortingOrder(const int32 InSortingOrder)
	{
		SortingOrder = InSortingOrder;
	}
#endif

#if WITH_EDITOR
	MOVIESCENE_API virtual void PostEditUndo() override;
	MOVIESCENE_API virtual void PostEditUndo(TSharedPtr<ITransactionObjectAnnotation> TransactionAnnotation) override;
#endif

	/** Event handlers for handling changes to this object */
	UE::MovieScene::TDataEventContainer<UE::MovieScene::IFolderEventHandler> EventHandlers;

private:
	/** The name of this folder. */
	UPROPERTY()
	FName FolderName;

	/** The folders contained by this folder. */
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneFolder>> ChildFolders;

	/** The tracks contained by this folder. */
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneTrack>> ChildTracks;

	/** The guid strings used to serialize the guids for the object bindings contained by this folder. */
	UPROPERTY()
	TArray<FString> ChildObjectBindingStrings;

#if WITH_EDITORONLY_DATA
	/** This folder's color */
	UPROPERTY(EditAnywhere, Category=General, DisplayName=Color)
	FColor FolderColor;

	/** This folder's desired sorting order */
	UPROPERTY()
	int32 SortingOrder;
#endif

	/** The guids for the object bindings contained by this folder. */
	TArray<FGuid> ChildObjectBindings;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneTrack>> ChildMasterTracks_DEPRECATED;
#endif
};

MOVIESCENE_API void GetMovieSceneFoldersRecursive(TArrayView<UMovieSceneFolder* const> InFoldersToRecurse, TArray<UMovieSceneFolder*>& OutFolders);


