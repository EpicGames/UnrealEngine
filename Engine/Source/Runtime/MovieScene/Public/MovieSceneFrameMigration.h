// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "Math/Range.h"
#include "Math/RangeBound.h"
#include "Misc/FrameNumber.h"
#include "MovieSceneFwd.h"
#include "Serialization/StructuredArchive.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"

#include "MovieSceneFrameMigration.generated.h"

class FArchive;
class FOutputDevice;
class FString;
class UObject;
struct FPropertyTag;

/**
 * Type used to convert from a FFloatRange to a TRange<FFrameNumber>
 */
USTRUCT()
struct FMovieSceneFrameRange
{
	GENERATED_BODY()

	/**
	 * The actual frame number range, custom serialized
	 */
	TRange<FFrameNumber> Value;

#if WITH_EDITORONLY_DATA

	/**
	 * Default value to use during serialization to ensure that values previously serialized with deltas get upgraded correctly
	 */
	FFloatRange MigrationDefault;

#endif

	/**
	 * Default construction to an empty frame range
	 */
	FMovieSceneFrameRange()
		: Value(TRange<FFrameNumber>::Empty())
	{}

	/**
	 * Construction from a frame range
	 */
	FMovieSceneFrameRange(const TRange<FFrameNumber>& InValue)
		: Value(InValue)
	{}

	/**
	 * Convert this frame number from a float range
	 */
	MOVIESCENE_API static TRange<FFrameNumber> FromFloatRange(const TRange<float>& InFloatRange);

	/**
	 * Custom serializer for the frame number range
	 */
	MOVIESCENE_API bool Serialize(FArchive& Ar);

	/**
	 * Serialize this frame range from a mismatched type (only FFloatRange supported)
	 */
	MOVIESCENE_API bool SerializeFromMismatchedTag(FPropertyTag const& Tag, FStructuredArchive::FSlot Slot);

	/**
	 * Get this range's lower bound
	 */
	TRangeBound<FFrameNumber> GetLowerBound() const
	{
		return Value.GetLowerBound();
	}

	/**
	 * Get this range's upper bound
	 */
	TRangeBound<FFrameNumber> GetUpperBound() const
	{
		return Value.GetUpperBound();
	}

	//Needed for copy and pasting of tracks since that mechanism uses string export and import
	bool ExportTextItem(FString& ValueStr, FMovieSceneFrameRange const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const;
	bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText);


private:

	/**
	 * Equality operator
	 */
	friend bool operator==(const FMovieSceneFrameRange& A, const FMovieSceneFrameRange& B)
	{
		return A.Value == B.Value;
	}
};


template<>
struct TStructOpsTypeTraits<FMovieSceneFrameRange> : public TStructOpsTypeTraitsBase2<FMovieSceneFrameRange>
{
	enum { WithStructuredSerializeFromMismatchedTag = true, WithSerializer = true, WithIdenticalViaEquality = true,
		   WithExportTextItem = true, WithImportTextItem = true};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

/**
*  Set the value at the specified time and index into the arrays, sorted and converted to frame numbers
*  We need this since some legacy channels and sections may not be sorted and we now expect time,value
*  arrays to be sorted.
*/
template <typename T> inline void ConvertInsertAndSort(int32 IndexAt, FFrameNumber NewFrame, T& Value, TArray<FFrameNumber> &Times, TArray<T> &Values)
{
	if (IndexAt > 0)
	{
		int32 PrevWhich = IndexAt - 1;
		//this seems to happen 99% of the time, it is properly sorted.
		if (NewFrame >= Times[PrevWhich])
		{
			Times.Emplace(NewFrame);
			Values.Emplace(Value);
		}
		else
		{
			while (--PrevWhich >= 0)
			{
				if (NewFrame >= Times[PrevWhich])
				{
					Times.Insert(NewFrame, PrevWhich + 1);
					Values.Insert(Value, PrevWhich + 1);
					break;
				}
			}
			if (PrevWhich < 0)
			{
				Times.Insert(NewFrame, 0);
				Values.Insert(Value, 0);
			}
		}
	}
	else
	{
		Times.Emplace(NewFrame);
		Values.Emplace(Value);
	}
}