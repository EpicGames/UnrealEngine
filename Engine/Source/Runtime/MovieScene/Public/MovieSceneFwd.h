// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Stats/Stats.h"
#include "Misc/EnumClassFlags.h"
#include "HAL/PreprocessorHelpers.h"

#include "MovieSceneFwd.generated.h"

#ifndef UE_MOVIESCENE_EVENTS
	#define UE_MOVIESCENE_EVENTS WITH_EDITOR
#endif

#ifndef UE_MOVIESCENE_EXPENSIVE_CONSISTENCY_CHECKS
	#define UE_MOVIESCENE_EXPENSIVE_CONSISTENCY_CHECKS 0
#endif

#ifndef UE_MOVIESCENE_ENTITY_DEBUG
	#define UE_MOVIESCENE_ENTITY_DEBUG !UE_BUILD_SHIPPING
#endif

#ifndef MOVIESCENE_DETAILED_STATS
	#define MOVIESCENE_DETAILED_STATS 0
#endif

struct FFrameRate;
struct FFrameNumber;
class UObject;

namespace UE::MovieScene
{

	class ISequenceDataEventHandler;

} // namespace UE::MovieScene

UENUM()
namespace EMovieScenePlayerStatus
{
	enum Type : int
	{
		Stopped,
		Playing,
		Scrubbing,
		Jumping,
		Stepping,
		Paused,
		MAX
	};
}

UENUM(BlueprintType)
enum class EMovieSceneEvaluationType : uint8
{
	/** Play the sequence frame-locked to its playback rate (snapped to the tick resolution - no sub-frames) */
	FrameLocked,

	/** Play the sequence in real-time, with sub-frame interpolation if necessary */
	WithSubFrames,
};

/**
 * Enum used to define how to update to a particular time
 */
UENUM(BlueprintType)
enum class EUpdateClockSource : uint8
{
	/** Use the default world tick delta for timing. Honors world and actor pause state, but is susceptible to accumulation errors */
	Tick,

	/** Use the platform clock for timing. Does not honor world or actor pause state. */
	Platform,

	/** Use the audio clock for timing. Does not honor world or actor pause state. */
	Audio,

	/** Time relative to the timecode provider for timing. Does not honor world or actor pause state. */
	RelativeTimecode,

	/** Use current timecode provider for timing. Does not honor world or actor pause state. */
	Timecode,

	/** Debugging Tool: Hold on each whole frame for a Sequencer.SecondsPerFrame many wall-clock seconds before advancing to the next one. Does not honor world or actor pause state or time dilation and audio will be out of sync. */
	PlayEveryFrame,

	/** Custom clock source created and defined externally. */
	Custom,
};


/**
 * Bitfield flags that define special behavior for any UMovieSceneSequence.
 */
UENUM(Flags)
enum class EMovieSceneSequenceFlags : uint8
{
	/** Symbolic entry for no flags */
	None = 0 UMETA(Hidden),

	/**
	 * Flag signifying that this sequence can change dynamically at runtime or during the game so the template must be checked for validity and recompiled as necessary before each evaluation.
	 * The absence of this flag will result in the same compiled data being used for the duration of the program, as well as being pre-built during cook. As such, any dynamic changes to the 
	 * sequence will not be reflected in the evaluation itself. This flag *must* be set if *any* procedural changes will be made to the source sequence data in-game.
	 */
	Volatile = 1 << 0,

	/**
	 * Indicates that a sequence must fully evaluate and apply its state every time it is updated, blocking until complete. Should be used sparingly as it will severely affect performance.
	 */
	BlockingEvaluation = 1 << 1,

	/**
	 * Indicates that a sequence will utilize dynamic weighting when it is played back. Setting this flag ensures that initial values are cached correctly so the sequence can be blended into and out of
	 */
	DynamicWeighting = 1 << 2,

	/**
	 * When enabled, the sequence considers looping as a jump/cut. When disabled, the sequence considers looping to be seamless.
	 */
	LoopCuts = 1 << 3,

	/** Symbolic entry for all flags that should be inherited by parent sequences when present on a sub sequence */
	InheritedFlags = Volatile UMETA(Hidden),
};
ENUM_CLASS_FLAGS(EMovieSceneSequenceFlags);


/**
 * 
 */
UENUM(Flags, meta=(Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EMovieSceneServerClientMask : uint8
{
	None           = 0 UMETA(Hidden),
	Server         = 1 << 0,
	Client         = 1 << 1,

	All            = Server | Client UMETA(Hidden)
};
ENUM_CLASS_FLAGS(EMovieSceneServerClientMask)

MOVIESCENE_API DECLARE_LOG_CATEGORY_EXTERN(LogMovieScene, Log, All);
MOVIESCENE_API DECLARE_LOG_CATEGORY_EXTERN(LogMovieSceneECS, Log, All);

DECLARE_STATS_GROUP(TEXT("Movie Scene Evaluation"), STATGROUP_MovieSceneEval, STATCAT_Advanced);

MOVIESCENE_API FFrameRate GetLegacyConversionFrameRate();
MOVIESCENE_API void EmitLegacyOutOfBoundsError(UObject* ErrorContext, FFrameRate InFrameRate, double InTime);
MOVIESCENE_API FFrameNumber UpgradeLegacyMovieSceneTime(UObject* ErrorContext, FFrameRate InFrameRate, double InTime);


#if MOVIESCENE_DETAILED_STATS
	#define MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER SCOPE_CYCLE_COUNTER
#else
	#define MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(...)
#endif


#if defined(_MSC_VER)
	#define UE_MOVIESCENE_TODO_IMPL(x) __pragma (x)
#else
	#define UE_MOVIESCENE_TODO_IMPL(x) _Pragma (#x)
#endif

#ifdef UE_MOVIESCENE_TODOS
	#define UE_MOVIESCENE_TODO(MSG) UE_MOVIESCENE_TODO_IMPL(message("" __FILE__ "(" UE_STRINGIZE(__LINE__) "): warning TODO: " # MSG))
#else
	#define UE_MOVIESCENE_TODO(MSG)
#endif
