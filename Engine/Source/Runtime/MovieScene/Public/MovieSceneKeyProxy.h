// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "MovieSceneSection.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "Channels/MovieSceneChannel.h"
#include "Curves/KeyHandle.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "MovieScene.h"
#include "MovieSceneKeyProxy.generated.h"

UINTERFACE(MinimalAPI)
class UMovieSceneKeyProxy : public UInterface
{
	GENERATED_BODY()
};


/**
 * Interface that can be implemented by any object that is used as a key editing proxy for a moviescene channel.
 * When used, UpdateValuesFromRawData should be called every frame to optionally retrieve the current values
 * of the key for this proxy.
 */
class IMovieSceneKeyProxy
{
public:

	GENERATED_BODY()

	/**
	 * To be called by the edit interface to update this instance's properties with the underlying raw data
	 */
	virtual void UpdateValuesFromRawData() = 0;

protected:

	/**
	 * Implementation function that sets the underlying key time/value to the specified values if possible.
	 * If the section is locked, InOutValue and InOutTime will be reset back to the current key's value
	 *
	 * @param InChannelHandle      The channel on which the underlying key value resides
	 * @param InSection            The section that owns the channel
	 * @param InKeyHandle          A handle to the key that we are reflecting
	 * @param InOutValue           Value to assign to the underlying key. If the section is locked, this will receive the existing key's value without changing the underlying key value.
	 * @param InOutTime            Time to move the underlying key to. If the section is locked, this will receive the existing key's time without changing the underlying time.
	 */
	template<typename ChannelType, typename ValueType>
	void OnProxyValueChanged(TMovieSceneChannelHandle<ChannelType> InChannelHandle, UMovieSceneSignedObject* InSignedObject, FKeyHandle InKeyHandle, ValueType& InOutValue, FFrameNumber& InOutTime);

	/**
	 * Implementation function that retrieves the underlying key time/value and applies then to the specified value and time parameters. Normally called once per tick.
	 *
	 * @param InChannelHandle      The channel on which the underlying key value resides
	 * @param InKeyHandle          A handle to the key that we are reflecting
	 * @param OutValue             (Out) Value to receive the underlying key's value
	 * @param OutTime              (Out) Time to receive the underlying key time
	 */
	template<typename ChannelType, typename ValueType>
	void RefreshCurrentValue(TMovieSceneChannelHandle<ChannelType> InChannelHandle, FKeyHandle InKeyHandle, ValueType& OutValue, FFrameNumber& OutTime);
};



template<typename ChannelType, typename ValueType>
void IMovieSceneKeyProxy::OnProxyValueChanged(TMovieSceneChannelHandle<ChannelType> InChannelHandle, UMovieSceneSignedObject* InSignedObject, FKeyHandle InKeyHandle, ValueType& InOutValue, FFrameNumber& InOutTime)
{
	auto* Channel = InChannelHandle.Get();
	if (!Channel || !InSignedObject)
	{
		return;
	}

	UMovieSceneSection* Section = Cast<UMovieSceneSection>(InSignedObject);
	if (!Section)
	{
		Section = InSignedObject->GetTypedOuter<UMovieSceneSection>();
	}

	auto ChannelData = Channel->GetData();

	int32 KeyIndex = ChannelData.GetIndex(InKeyHandle);
	if (KeyIndex != INDEX_NONE)
	{
		// If we have no signed object, or it's locked, don't let the user change the value
		if (!InSignedObject || (Section && !Section->TryModify()))
		{
			InOutTime  = ChannelData.GetTimes()[KeyIndex];
			InOutValue = ChannelData.GetValues()[KeyIndex];
		}
		else
		{
			InSignedObject->Modify();

			ChannelData.GetValues()[KeyIndex] = InOutValue;

			ChannelData.MoveKey(KeyIndex, InOutTime);

			if (Section)
			{
				Section->ExpandToFrame(InOutTime);
			}
		}
#if WITH_EDITOR
		const FMovieSceneChannelMetaData* MetaData = InChannelHandle.GetMetaData();
		if (MetaData && Section)
		{
			if (UMovieScene* MovieScene = Section->GetTypedOuter<UMovieScene>())
			{
				TArray<FFrameNumber> Frames;
				Frames.Add(InOutTime);
				MovieScene->OnChannelChangedWithTime().Broadcast(Frames, MetaData, Section);
			}
		}
#endif
		Channel->PostEditChange();
	}
}



template<typename ChannelType, typename ValueType>
void IMovieSceneKeyProxy::RefreshCurrentValue(TMovieSceneChannelHandle<ChannelType> InChannelHandle, FKeyHandle InKeyHandle, ValueType& OutValue, FFrameNumber& OutTime)
{
	auto* Channel = InChannelHandle.Get();
	if (Channel)
	{
		auto ChannelData = Channel->GetData();
		int32 KeyIndex = ChannelData.GetIndex(InKeyHandle);
		if (KeyIndex != INDEX_NONE && KeyIndex < FMath::Min(ChannelData.GetValues().Num(), ChannelData.GetTimes().Num()))
		{
			OutValue = ChannelData.GetValues()[KeyIndex];
			OutTime  = ChannelData.GetTimes()[KeyIndex];
		}
	}
}
