// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MovieSceneKeyStructHelper.h"
#include "UObject/StructOnScope.h"
#include "MovieSceneKeyStruct.generated.h"

struct FPropertyChangedEvent;

/**
 * Base class for movie scene section key structs that need to manually
 * have their changes propagated to key values.
 */
USTRUCT()
struct FMovieSceneKeyStruct
{
	GENERATED_BODY()

	/**
	 * Propagate changes from this key structure to the corresponding key values.
	 *
	 * @param ChangeEvent The property change event.
	 */
	virtual void PropagateChanges(const FPropertyChangedEvent& ChangeEvent) { }

	virtual ~FMovieSceneKeyStruct() = default;
};

USTRUCT()
struct FGeneratedMovieSceneKeyStruct
{
	GENERATED_BODY()

	virtual ~FGeneratedMovieSceneKeyStruct() = default;

	/**
	 * Function that is called when a property is changed on this struct
	 */
	TFunction<void(const FPropertyChangedEvent&)> OnPropertyChangedEvent;
};

USTRUCT()
struct FMovieSceneKeyTimeStruct : public FMovieSceneKeyStruct
{
	GENERATED_BODY()

	FMovieSceneKeyTimeStruct(){}

	UPROPERTY(EditAnywhere, Category="Key", meta=(Units=s))
	FFrameNumber Time;

	FMovieSceneKeyStructHelper KeyStructInterop;

	/**
	 * Propagate changes from this key structure to the corresponding key values.
	 *
	 * @param ChangeEvent The property change event.
	 */
	virtual void PropagateChanges(const FPropertyChangedEvent& ChangeEvent)
	{
		KeyStructInterop.Apply(Time);
	}
};
template<> struct TStructOpsTypeTraits<FMovieSceneKeyTimeStruct> : public TStructOpsTypeTraitsBase2<FMovieSceneKeyTimeStruct> { enum { WithCopy = false }; };
