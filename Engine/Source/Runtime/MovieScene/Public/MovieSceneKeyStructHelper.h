// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannelHandle.h"
#include "Channels/MovieSceneChannelTraits.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreTypes.h"
#include "Curves/KeyHandle.h"
#include "Evaluation/MovieSceneRootOverridePath.h"
#include "Misc/FrameNumber.h"
#include "Misc/InlineValue.h"
#include "Misc/Optional.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"

struct FMovieSceneChannelValueHelper;

/**
 * Helper class that marshals user-facing data from an edit UI into particluar keys on various underlying channels
 */
struct FMovieSceneKeyStructHelper
{
	/**
	 * Default construction
	 */
	FMovieSceneKeyStructHelper() = default;

	/**
	 * Non-copyable
	 */
	FMovieSceneKeyStructHelper(const FMovieSceneKeyStructHelper&) = delete;
	FMovieSceneKeyStructHelper& operator=(const FMovieSceneKeyStructHelper&) = delete;

	/**
	 * Add a particular key value to this helper that should be applied when the edit UI is committed
	 *
	 * @param InHelper    The utility class to add, bound to the key handle it wants to edit
	 */
	MOVIESCENE_API void Add(FMovieSceneChannelValueHelper&& InHelper);

	/**
	 * Set the user facing values on the UI based on the unified starting time
	 */
	MOVIESCENE_API void SetStartingValues();

	/**
	 * Get unified starting time that should be shown on the UI
	 */
	MOVIESCENE_API TOptional<FFrameNumber> GetUnifiedKeyTime() const;

	/**
	 * Propagate the user-facing UI values to the keys that are being represented in this class
	 *
	 * @param InUnifiedTime  A time to set all keys to
	 */
	MOVIESCENE_API void Apply(FFrameNumber InUnifiedTime);

private:
	/** Unified key time that represents all the keys */
	TOptional<FFrameNumber> UnifiedKeyTime;

	/** Array of value accessors that are being shown on the edit UI */
	TArray<FMovieSceneChannelValueHelper> Helpers;
};



/**
 * Utility class that gets and sets a specific key value for a key struct
 */
struct FMovieSceneChannelValueHelper
{
	/**
	 * Construction from a channel handle, a pointer to an external user-facing value on the key struct, and an array of handles
	 *
	 * @param InChannel          A handle to the channel that contains the key we're editing
	 * @param InUserValue        Pointer to a user-facing value on an editable UStruct instance
	 * @param AllKeyHandles      Array of all key handles that should be edited using this helper
	 */
	template<typename ChannelType, typename ValueType>
	FMovieSceneChannelValueHelper(const TMovieSceneChannelHandle<ChannelType>& InChannel, ValueType* InUserValue, TArrayView<const FKeyHandle> AllKeyHandles);

	/**
	 * Construction from a channel handle, a pointer to an external user-facing value on the key struct, and a specific key handle/time
	 *
	 * @param InChannel          A handle to the channel that contains the key we're editing
	 * @param InUserValue        Pointer to a user-facing value on an editable UStruct instance
	 * @param InKeyHandleAndTime (Optional) Specific key value and time that should be edited
	 */
	template<typename ChannelType, typename ValueType>
	FMovieSceneChannelValueHelper(const TMovieSceneChannelHandle<ChannelType>& InChannel, ValueType* InUserValue, TOptional<TTuple<FKeyHandle, FFrameNumber>> InKeyHandleAndTime);

	/**
	 * Attempt to find a single key handle that exists on the specified channel
	 *
	 * @param InChannel          The channel to look on
	 * @param AllKeyHandles      The key handles to search for
	 * @return (Optional) A key handle and time pair for the first valid key handle found
	 */
	template<typename ChannelType>
	static TOptional<TTuple<FKeyHandle, FFrameNumber>> FindFirstKey(ChannelType* InChannel, TArrayView<const FKeyHandle> AllKeyHandles);


	/** Underlying implementation interface */
	struct IChannelValueHelper
	{
		virtual ~IChannelValueHelper()= default;
		/** Set the user value from the specified time */
		virtual void SetUserValueFromTime(FFrameNumber InUnifiedTime) = 0;
		/** Set the curve's key value and time from the user value, with the specified time */
		virtual void SetKeyFromUserValue(FFrameNumber InUnifiedTime) = 0;

		/** The key handle and time for the key we're editing */
		TOptional<TTuple<FKeyHandle, FFrameNumber>> KeyHandleAndTime;
	};

	/**
	 * Pointer operator overload that allows access to the underlying interface
	 */
	IChannelValueHelper* operator->()
	{
		return &Impl.GetValue();
	}

private:

	/** Pointer to the underlying value helper utility */
	TInlineValue<IChannelValueHelper> Impl;
};


namespace MovieSceneImpl
{

/** Templated channel value accessor utility that get's sets a channel value and time */
template<typename ChannelType, typename ValueType>
struct TChannelValueHelper : FMovieSceneChannelValueHelper::IChannelValueHelper
{
	TChannelValueHelper(const TMovieSceneChannelHandle<ChannelType>& InChannel, ValueType* InValue)
		: ChannelHandle(InChannel), UserValue(InValue)
	{}

	/** Set the user facing value to the curve's current value at the specified time */
	virtual void SetUserValueFromTime(FFrameNumber InUnifiedTime)
	{
		if (ChannelType* Channel = ChannelHandle.Get())
		{
			using namespace UE::MovieScene;
			EvaluateChannel(Channel, InUnifiedTime, *UserValue);
		}
	}

	/** Set the key's time and value to the user facing value, and the specified time */
	virtual void SetKeyFromUserValue(FFrameNumber InUnifiedTime)
	{
		ChannelType* Channel = ChannelHandle.Get();

		using namespace UE::MovieScene;
		if (Channel && KeyHandleAndTime.IsSet())
		{
			FKeyHandle Handle = KeyHandleAndTime->Get<0>();

			AssignValue(Channel, Handle, *UserValue);
			Channel->SetKeyTime(Handle, InUnifiedTime);
		}
	}

	/** Handle to the channel itself */
	TMovieSceneChannelHandle<ChannelType> ChannelHandle;

	/** Pointer to the user facing value on the edit interface */
	ValueType* UserValue;
};

}	// namespace MovieSceneImpl

template<typename ChannelType, typename ValueType>
FMovieSceneChannelValueHelper::FMovieSceneChannelValueHelper(const TMovieSceneChannelHandle<ChannelType>& InChannel, ValueType* InUserValue, TArrayView<const FKeyHandle> AllKeyHandles)
	: Impl(MovieSceneImpl::TChannelValueHelper<ChannelType, ValueType>(InChannel, InUserValue))
{
	Impl->KeyHandleAndTime = FindFirstKey(InChannel.Get(), AllKeyHandles);
}



template<typename ChannelType, typename ValueType>
FMovieSceneChannelValueHelper::FMovieSceneChannelValueHelper(const TMovieSceneChannelHandle<ChannelType>& InChannel, ValueType* InUserValue, TOptional<TTuple<FKeyHandle, FFrameNumber>> InKeyHandleAndTime)
	: Impl(MovieSceneImpl::TChannelValueHelper<ChannelType, ValueType>(InChannel, InUserValue))
{
	Impl->KeyHandleAndTime = InKeyHandleAndTime;
}



template<typename ChannelType>
TOptional<TTuple<FKeyHandle, FFrameNumber>> FMovieSceneChannelValueHelper::FindFirstKey(ChannelType* InChannel, TArrayView<const FKeyHandle> AllKeyHandles)
{
	if (InChannel)
	{
		auto ChannelData = InChannel->GetData();
		for (FKeyHandle Handle : AllKeyHandles)
		{
			const int32 KeyIndex = ChannelData.GetIndex(Handle);
			if (KeyIndex != INDEX_NONE)
			{
				return MakeTuple(Handle, ChannelData.GetTimes()[KeyIndex]);
			}
		}
	}

	return TOptional<TTuple<FKeyHandle, FFrameNumber>>();
}
