// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Templates/Function.h"
#include "Containers/ContainersFwd.h"

class UObject;

// using "not checked" user policy (means race detection is disabled) because this delegate is stored in a TArray and causes its reallocation
// from inside delegate's execution. This is incompatible with race detection that needs to access the delegate instance after its execution
using FMovieSceneSequenceLatentActionDelegate = TDelegate<void(), FNotThreadSafeNotCheckedDelegateUserPolicy>;

/**
 * Utility class for running latent actions created from sequence players.
 */
class FMovieSceneLatentActionManager
{
public:
	MOVIESCENE_API void AddLatentAction(FMovieSceneSequenceLatentActionDelegate Delegate);
	MOVIESCENE_API void ClearLatentActions(UObject* Object);
	MOVIESCENE_API void ClearLatentActions();

	MOVIESCENE_API void RunLatentActions(TFunctionRef<void()> FlushCallback);

	bool IsEmpty() const { return LatentActions.Num() == 0; }

private:
	TArray<FMovieSceneSequenceLatentActionDelegate> LatentActions;

	bool bIsRunningLatentActions = false;
};
