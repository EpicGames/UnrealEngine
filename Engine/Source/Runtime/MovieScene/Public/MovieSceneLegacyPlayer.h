// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMovieScenePlayer.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Templates/SharedPointerFwd.h"

namespace UE::MovieScene
{

	struct FSharedPlaybackState;

	struct ILegacyPlayerProviderPlaybackCapability
	{
		UE_DECLARE_MOVIESCENE_PLAYBACK_CAPABILITY_API(MOVIESCENE_API, ILegacyPlayerProviderPlaybackCapability)

		virtual ~ILegacyPlayerProviderPlaybackCapability() = default;

		virtual IMovieScenePlayer* CreateLegacyPlayer(TSharedRef<FSharedPlaybackState> InSharedPlaybackState) = 0;
	};

}  // namespace UE::MovieScene

/**
 * A legacy player implementation only meant for running legacy tracks using the older
 * evaluation template system.
 */
class FMovieSceneLegacyPlayer : public IMovieScenePlayer
{
public:

	FMovieSceneLegacyPlayer();
	FMovieSceneLegacyPlayer(TSharedRef<UE::MovieScene::FSharedPlaybackState> InSharedPlaybackState);

public:

	// IMovieScenePlayer interface.
	virtual FMovieSceneRootEvaluationTemplateInstance& GetEvaluationTemplate() override;
	virtual UMovieSceneEntitySystemLinker* ConstructEntitySystemLinker() override;
	virtual UObject* AsUObject() override;
	virtual EMovieScenePlayerStatus::Type GetPlaybackStatus() const override;
	virtual void SetPlaybackStatus(EMovieScenePlayerStatus::Type InPlaybackStatus) override;
	virtual IMovieScenePlaybackClient* GetPlaybackClient() override;
	virtual FMovieSceneSpawnRegister& GetSpawnRegister() override;
	virtual UObject* GetPlaybackContext() const override;

	virtual void InitializeRootInstance(TSharedRef<UE::MovieScene::FSharedPlaybackState> NewSharedPlaybackState) override;

private:

	TSharedPtr<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState;

	FMovieSceneRootEvaluationTemplateInstance EvaluationTemplateInstance;
};

