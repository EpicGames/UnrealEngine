// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Misc/FrameNumber.h"
#include "Math/Color.h"

#include "MovieSceneMarkedFrame.generated.h"

USTRUCT(BlueprintType)
struct FMovieSceneMarkedFrame
{
	GENERATED_BODY()

	FMovieSceneMarkedFrame()
		: Label(FString())
#if WITH_EDITORONLY_DATA
		, Comment(FString())
		, CustomColor(0.f, 1.f, 1.f, 0.4f)	
#endif
		, bIsDeterminismFence(false)
		, bIsInclusiveTime(false)
	{}

	FMovieSceneMarkedFrame(FFrameNumber InFrameNumber)
		: FrameNumber(InFrameNumber)
		, Label(FString())
#if WITH_EDITORONLY_DATA
		, Comment(FString())
		, CustomColor(0.f, 1.f, 1.f, 0.4f)
#endif
		, bIsDeterminismFence(false)
		, bIsInclusiveTime(false)
	{}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Marked Frame")
	FFrameNumber FrameNumber;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Marked Frame")
	FString Label;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Marked Frame")
	FString Comment;

	UPROPERTY(EditAnywhere, Category = "Marked Frame", DisplayName = "Color", meta=(EditCondition = bUseCustomColor))
	FLinearColor CustomColor;

	UPROPERTY(EditAnywhere, Category = "Marked Frame")
	bool bUseCustomColor = false;

	UPROPERTY()
	FLinearColor Color_DEPRECATED = FLinearColor(0.f, 1.f, 1.f, 0.4f);
#endif

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Marked Frame", DisplayName="Is Determinism Fence?", meta=(Tooltip="When checked, treat this mark as a fence for evaluation purposes. Fences cannot be crossed in a single evaluation, and force the evaluation to be split into 2 separate parts."))
	bool bIsDeterminismFence;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Marked Frame", DisplayName="Inclusive?", meta=(EditCondition="bIsDeterminismFence", Tooltip="Defines how this determinism fence determines the previous and next range: when true, the range will be dissected as (X, Y] -> (Y, Z], when false it will be (X, Y) -> [Y, Z]."))
	bool bIsInclusiveTime;
};
