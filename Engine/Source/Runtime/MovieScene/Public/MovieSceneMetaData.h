// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "IMovieSceneMetaData.h"
#include "Misc/DateTime.h"
#include "Misc/QualifiedFrameTime.h"
#include "MovieSceneMetaData.generated.h"

/**
 * Movie scene meta-data that is stored on UMovieScene assets
 * Meta-data is retrieved through ULevelSequence::FindMetaData<ULevelSequenceMetaData>()
 */
UCLASS(config = EditorSettings, PerObjectConfig, BlueprintType, MinimalAPI)
class UMovieSceneMetaData : public UObject, public IMovieSceneMetaDataInterface
{
public:
	GENERATED_BODY()

	MOVIESCENE_API UMovieSceneMetaData (const FObjectInitializer& ObjInit);

public:

	/** The asset registry tag that contains the author for this meta-data */
	static MOVIESCENE_API const FName AssetRegistryTag_Author;

	/** The asset registry tag that contains the notes for this meta-data */
	static MOVIESCENE_API const FName AssetRegistryTag_Notes;

	/** The asset registry tag that contains the created date for this meta-data */
	static MOVIESCENE_API const FName AssetRegistryTag_Created;

	/**
	 * Access the global config instance that houses default settings for movie scene meta data for a given project
	 */
	static MOVIESCENE_API UMovieSceneMetaData* GetConfigInstance();

	/**
	 * Create a new meta-data object from the project defaults
	 *
	 * @param Outer    The object to allocate the new meta-data within
	 * @param Name     The name for the new object. Must not already exist
	 */
	static MOVIESCENE_API UMovieSceneMetaData* CreateFromDefaults(UObject* Outer, FName Name);

public:

	/**
	 * Extend the default asset registry tags
	 */
	MOVIESCENE_API virtual void ExtendAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	virtual void ExtendAssetRegistryTags(TArray<UObject::FAssetRegistryTag>& OutTags) const override {}

#if WITH_EDITOR

	/**
	 * Extend the default asset registry tag meta-data
	 */
	MOVIESCENE_API virtual void ExtendAssetRegistryTagMetaData(TMap<FName, UObject::FAssetRegistryTagMetadata>& OutMetadata) const override;

#endif

	/** Return whether this metadata has any valid data */
	MOVIESCENE_API bool IsEmpty() const;

public:

	/**
	 * @return The author for this metadata
	 */
	UFUNCTION(BlueprintCallable, Category = "Meta Data")
	MOVIESCENE_API FString GetAuthor() const;

	/**
	 * @return The created date for this metadata
	 */
	UFUNCTION(BlueprintCallable, Category = "Meta Data")
	MOVIESCENE_API FDateTime GetCreated() const;

	/**
	 * @return The notes for this metadata
	 */
	UFUNCTION(BlueprintCallable, Category = "Meta Data")
	MOVIESCENE_API FString GetNotes() const;

public:

	/**
	 * Set this metadata's author
	 */
	UFUNCTION(BlueprintCallable, Category = "Meta Data")
	MOVIESCENE_API void SetAuthor(FString InAuthor);

	/**
	 * Set this metadata's created date
	 */
	UFUNCTION(BlueprintCallable, Category = "Meta Data")
	MOVIESCENE_API void SetCreated(FDateTime InCreated);

	/**
	 * Set this metadata's notes
	 */
	UFUNCTION(BlueprintCallable, Category = "Meta Data")
	MOVIESCENE_API void SetNotes(FString InNotes);

private:

	/** The author that created this metadata */
	UPROPERTY(EditAnywhere, Category = "Meta Data")
	FString Author;

	/** The created date at which the metadata was initiated */
	UPROPERTY(EditAnywhere, Category = "Meta Data")
	FDateTime Created;

	/** Notes for the metadata */
	UPROPERTY(EditAnywhere, Category = "Meta Data")
	FString Notes;
};
