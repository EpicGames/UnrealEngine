// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Internationalization/Text.h"
#include "MovieSceneTrack.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneNameableTrack.generated.h"

class UObject;


/**
 * Base class for movie scene tracks that can be renamed by the user.
 */
UCLASS(abstract, MinimalAPI)
class UMovieSceneNameableTrack
	: public UMovieSceneTrack
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
public:

	/**
	 * Set the track's display name.
	 *
	 * @param NewName The name to set.
	 */
	MOVIESCENE_API virtual void SetDisplayName(const FText& NewDisplayName);

	/**
	 * Set the track row's display name.
	 *
	 * @param NewName The name to set.
	 * @param RowIndex The track row index to set.
	 */
	MOVIESCENE_API virtual void SetTrackRowDisplayName(const FText& NewDisplayName, int32 TrackRowIndex);

	/**
	 * Can rename this track.
	 *
	 * @return Whether this track can be renamed.
	 */
	virtual bool CanRename() const { return true; }

	/** 
	 * Validate the new display name. 
	 *
	 * @return True if the given display name is valid, false if it is not. 
	 * Error message should be set if the name is not valid.
	 */
	MOVIESCENE_API virtual bool ValidateDisplayName(const FText& NewDisplayName, FText& OutErrorMessage) const;

public:

	// UMovieSceneTrack interface

	MOVIESCENE_API virtual FText GetDisplayName() const override;
	MOVIESCENE_API virtual FText GetTrackRowDisplayName(int32 RowIndex) const override;
	MOVIESCENE_API virtual FText GetDefaultDisplayName() const;

	MOVIESCENE_API virtual void OnRowIndicesChanged(const TMap<int32, int32>& NewToOldRowIndices) override;

protected:
	const FText& GetDisplayName_Internal() const { return DisplayName; }

private:

	/** The track's human readable display name. */
	UPROPERTY(EditAnywhere, Category="Track")
	FText DisplayName;

	/** The track display name per row. */
	UPROPERTY(EditAnywhere, Category="Track")
	TArray<FText> TrackRowDisplayNames;

#endif
};
