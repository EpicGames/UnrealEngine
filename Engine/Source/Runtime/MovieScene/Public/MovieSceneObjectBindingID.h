// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Guid.h"
#include "MovieSceneFwd.h"
#include "MovieSceneSequenceID.h"
#include "Templates/TypeHash.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "MovieSceneObjectBindingID.generated.h"

class FArchive;
class IMovieScenePlayer;
class UMovieSceneSequence;
struct FMovieSceneSequenceHierarchy;


namespace UE
{
namespace MovieScene
{


struct FSequenceInstance;
struct FSharedPlaybackState;


/**
 * A binding ID whose target is resolved relative to the owner of this ID (either internal or external to that sequence through a parent or grandparent)
 * Should be used for asignment to any serialized FMovieSceneObjectBindingID
 * Represented by a local object binding ID and a parent index to which it is relative
 */
struct FRelativeObjectBindingID
{
	FRelativeObjectBindingID(const FGuid& InGuid)
		: Guid(InGuid)
		, SequenceID(MovieSceneSequenceID::Root)
		, ResolveParentIndex(0)
	{}

	FRelativeObjectBindingID(const FGuid& InGuid, FMovieSceneSequenceID InSequenceID)
		: Guid(InGuid)
		, SequenceID(InSequenceID)
		, ResolveParentIndex(0)
	{}

	FRelativeObjectBindingID(const FGuid& InGuid, FMovieSceneSequenceID InSequenceID, int32 InParentIndex)
		: Guid(InGuid)
		, SequenceID(InSequenceID)
		, ResolveParentIndex(InParentIndex)
	{}

	/**
	 * Construction from a root sequence asset
	 *
	 * @param SourceSequenceID      Absolute sequence ID within RootSequence that represents the start of the path (ie, the sequence that this binding ID is serialized within)
	 * @param TargetSequenceID      Absolute sequence ID within RootSequence that holds the target object
	 * @param TargetGuid            The GUID of the object binding within TargetSequenceID that represents the object at runtime
	 * @param RootSequence          The root sequence asset
	 */
	MOVIESCENE_API FRelativeObjectBindingID(FMovieSceneSequenceID SourceSequenceID, FMovieSceneSequenceID TargetSequenceID, const FGuid& TargetGuid, UMovieSceneSequence* RootSequence);

	/**
	 * Construction from a root sequence Hierarchy
	 *
	 * @param SourceSequenceID      Absolute sequence ID within Hierarchy that represents the start of the path (ie, the sequence that this binding ID is serialized within)
	 * @param TargetSequenceID      Absolute sequence ID within Hierarchy that holds the target object
	 * @param TargetGuid            The GUID of the object binding within TargetSequenceID that represents the object at runtime
	 * @param Hierarchy             The hierachy to use for computation of the relative path. Many be nullptr where Source and Targets are both MovieSceneSequenceID::Root
	 */
	MOVIESCENE_API FRelativeObjectBindingID(FMovieSceneSequenceID SourceSequenceID, FMovieSceneSequenceID TargetSequenceID, const FGuid& TargetGuid, const FMovieSceneSequenceHierarchy* Hierarchy);

	/**
	 * Construction from a root sequence Player
	 *
	 * @param SourceSequenceID      Absolute sequence ID within Player's root sequence hierarchy that represents the start of the path (ie, the sequence that this binding ID is serialized within)
	 * @param TargetSequenceID      Absolute sequence ID within Player's root sequence hierarchy that holds the target object
	 * @param TargetGuid            The GUID of the object binding within TargetSequenceID that represents the object at runtime
	 * @param Player                The movie scene player that is currently playing the root sequence
	 */
	MOVIESCENE_API FRelativeObjectBindingID(FMovieSceneSequenceID SourceSequenceID, FMovieSceneSequenceID TargetSequenceID, const FGuid& TargetGuid, TSharedRef<const FSharedPlaybackState> SharedPlaybackState);

	MOVIESCENE_API FRelativeObjectBindingID(FMovieSceneSequenceID SourceSequenceID, FMovieSceneSequenceID TargetSequenceID, const FGuid& TargetGuid, IMovieScenePlayer& Player);

	/** GUID of the Object Binding (ie, FMovieSceneBinding::GetObjectGuid) */
	FGuid Guid;

	/** The sequence ID that the object binding is found within - relative to nth parent specified by ResolveParentIndex */
	FMovieSceneSequenceID SequenceID;

	/** The parent index denoting where to resolve this binding from */
	int32 ResolveParentIndex;

private:

	void ConstructInternal(FMovieSceneSequenceID SourceSequenceID, FMovieSceneSequenceID TargetSequenceID, const FGuid& TargetGuid, const FMovieSceneSequenceHierarchy* Hierarchy);
};


/**
 * A binding ID that is only resolveable through a fixed root sequence.
 * Use of this type should be reserved for editor code dealing with a specific sequence hierarchy
 * where hashing or comparison is required.
 * The SequenceID member is always an absolute sequence ID
 */
struct FFixedObjectBindingID
{
	FFixedObjectBindingID()
	{}

	FFixedObjectBindingID(const FGuid& InGuid, FMovieSceneSequenceID InSequenceID)
		: Guid(InGuid)
		, SequenceID(InSequenceID)
	{}

public:

	friend uint32 GetTypeHash(const FFixedObjectBindingID& A)
	{
		return GetTypeHash(A.Guid) ^ GetTypeHash(A.SequenceID);
	}

	friend bool operator==(const FFixedObjectBindingID& A, const FFixedObjectBindingID& B)
	{
		return A.Guid == B.Guid && A.SequenceID == B.SequenceID;
	}

	friend bool operator!=(const FFixedObjectBindingID& A, const FFixedObjectBindingID& B)
	{
		return A.Guid != B.Guid || A.SequenceID != B.SequenceID;
	}

public:

	/**
	 * Convert this fixed binding ID to one that is resolved relative to the specified Sequence.
	 *
	 * @param SourceSequenceID    The sequence ID to make this fixed ID relative to
	 * @param InPlayer            The active movie scene player to retrieve a root sequence hierarchy from
	 * @return A new binding ID that will resolve relative to the specified sequence
	 */
	MOVIESCENE_API FRelativeObjectBindingID ConvertToRelative(FMovieSceneSequenceID SourceSequenceID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState) const;

	MOVIESCENE_API FRelativeObjectBindingID ConvertToRelative(FMovieSceneSequenceID SourceSequenceID, IMovieScenePlayer& InPlayer) const;

	/**
	 * Convert this fixed binding ID to one that is resolved relative to the specified Sequence.
	 *
	 * @param SourceSequenceID    The sequence ID to make this fixed ID relative to
	 * @param Hierarchy           The hierarchy that represents the root sequence that SourceSequenceID is contained within
	 * @return A new binding ID that will resolve relative to the specified sequence
	 */
	MOVIESCENE_API FRelativeObjectBindingID ConvertToRelative(FMovieSceneSequenceID SourceSequenceID, const FMovieSceneSequenceHierarchy* Hierarchy) const;

public:

	/** GUID of the Object Binding (ie, FMovieSceneBinding::GetObjectGuid) */
	FGuid Guid;

	/** The sequence ID that the object binding is found within */
	FMovieSceneSequenceID SequenceID;
};


} // namespace MovieScene
} // namespace UE



/** Enumeration specifying how a movie scene object binding ID relates to the sequence */
UENUM()
enum class EMovieSceneObjectBindingSpace : uint8
{
	/** The object binding sequence ID resolves from a local sequence (ie, it may need to accumulate a parent sequence ID before it resolves correctly) */
	Local,
	/** The object binding sequence ID resolves from the root of the sequence */
	Root,

	/** Default value for any newly-constructed bindings */
	Unused UMETA(Hidden),
};


/**
 * Persistent identifier to a specific object binding within a sequence hierarchy.
 * 
 * Binding IDs come in 3 flavors with Local and External being preferred as they are reslilient towards sequences being authored in isolation or included in other root sequences:
 *     Local: (ResolveParentIndex == 0) SequenceID relates to _this sequence's_ local hierarchy; represents an object binding within the same sequence as the ID is resolved, or inside one of its sub-sequences. Sequence ID must be remapped at runtime.
 *     External: (ResolveParentIndex > 0) SequenceID is local to the parent sequence of this one denoted by the parent index (ie, 1 = parent, 2 = grandparent etc). Sequence ID must be remapped at runtime.
 *     Fixed: Represents a binding anywhere in the sequence; always resolved from the root sequence.
 *
 * Fixed bindings will break if the sequence is evaluated inside a different root sequence.
 */
USTRUCT(BlueprintType, meta=(HasNativeMake))
struct FMovieSceneObjectBindingID
{
	using FSharedPlaybackState = UE::MovieScene::FSharedPlaybackState;

	GENERATED_BODY()

	/**
	 * Default construction to a root empty. Only to be used for UObject serialization and default UObject construction
	 */
	FMovieSceneObjectBindingID()
		// Defaults purposefully match the equivalent of a legacy root binding
		// such that any pre-existing ::Root space binding upgrades to ResolveParentIndex = -1
		// and ::Local space bindings will get upgraded in PostSerialize
		: SequenceID(int32(MovieSceneSequenceID::Root.GetInternalValue()))
		, ResolveParentIndex(FixedRootSequenceParentIndex)
	{
		LegacyInit();
	}

	/**
	 * Construction from an external object binding ID
	 */
	FMovieSceneObjectBindingID(const UE::MovieScene::FRelativeObjectBindingID& InID)
		: Guid(InID.Guid)
		, SequenceID(InID.SequenceID.GetInternalValue())
		, ResolveParentIndex(InID.ResolveParentIndex)
	{
		LegacyInit();
	}

	/**
	 * Construction from a fixed object binding ID
	 */
	FMovieSceneObjectBindingID(const UE::MovieScene::FFixedObjectBindingID& InID)
		: Guid(InID.Guid)
		, SequenceID(InID.SequenceID.GetInternalValue())
		, ResolveParentIndex(FixedRootSequenceParentIndex)
	{
		LegacyInit();
	}


public:


	/**
	 * Assignment from an external object binding ID
	 */
	FMovieSceneObjectBindingID& operator=(const UE::MovieScene::FRelativeObjectBindingID& InID)
	{
		Guid = InID.Guid;
		SequenceID = InID.SequenceID.GetInternalValue();
		ResolveParentIndex = InID.ResolveParentIndex;
		return *this;
	}

	/**
	 * Assignment from a fixed object binding ID
	 */
	FMovieSceneObjectBindingID& operator=(const UE::MovieScene::FFixedObjectBindingID& InID)
	{
		Guid = InID.Guid;
		SequenceID = InID.SequenceID.GetInternalValue();
		ResolveParentIndex = FixedRootSequenceParentIndex;
		return *this;
	}

public:

	friend uint32 GetTypeHash(const FMovieSceneObjectBindingID& A)
	{
		return GetTypeHash(A.Guid) ^ GetTypeHash(A.SequenceID);
	}

	friend bool operator==(const FMovieSceneObjectBindingID& A, const FMovieSceneObjectBindingID& B)
	{
		return A.Guid == B.Guid && A.SequenceID == B.SequenceID && A.ResolveParentIndex == B.ResolveParentIndex;
	}

	friend bool operator!=(const FMovieSceneObjectBindingID& A, const FMovieSceneObjectBindingID& B)
	{
		return A.Guid != B.Guid || A.SequenceID != B.SequenceID ||  A.ResolveParentIndex != B.ResolveParentIndex;
	}


public:

	/**
	 * Check whether this object binding ID has been set to something valied
	 * @note: does not imply that the ID resolves to a valid object
	 */
	bool IsValid() const
	{
		return Guid.IsValid();
	}

	/**
	 * Check whether this binding is fixed to the root sequence - these bindings should be avoided or fixed up where possible to ensure portability
	 */
	bool IsFixedBinding() const
	{
		return Guid.IsValid() && ResolveParentIndex == FixedRootSequenceParentIndex;
	}

	/**
	 * Access the guid that identifies the object binding within the sequence
	 */
	const FGuid& GetGuid() const
	{
		return Guid;
	}

	/**
	 * Set the guid that identifies the object binding within the sequence
	 */
	void SetGuid(const FGuid& InGuid)
	{
#if WITH_EDITORONLY_DATA
		ensureMsgf(Space_DEPRECATED == EMovieSceneObjectBindingSpace::Unused, TEXT("Assigning a GUID for a binding which has no type specifier - consider using Local / External / Fixed instead"));
#endif
		Guid = InGuid;
	}

	/**
	 * Get the relative sequence ID for this binding.
	 * @note: The meaning of this sequence ID is dependent upon ResolveParentIndex so should only be used where the context is known
	 */
	FMovieSceneSequenceID GetRelativeSequenceID() const
	{
		return FMovieSceneSequenceID(SequenceID);
	}

	/**
	 * Resolve this binding ID to a fixed object binding ID resolvable from the root sequence
	 *
	 * @param SourceSequenceID      The source sequence ID that owns this binding ID
	 * @param Player                The movie scene player that is currently playing the root sequence
	 * @return A fixed binding ID whose SequenceID relates to the root sequence hierarchy
	 */
	MOVIESCENE_API UE::MovieScene::FFixedObjectBindingID ResolveToFixed(FMovieSceneSequenceID SourceSequenceID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState) const;

	MOVIESCENE_API UE::MovieScene::FFixedObjectBindingID ResolveToFixed(FMovieSceneSequenceID SourceSequenceID, IMovieScenePlayer& Player) const;

	/**
	 * Resolve the sequence ID for this binding for the root sequence
	 *
	 * @param SourceSequenceID      The source sequence ID that owns this binding ID
	 * @param Player                The movie scene player that is currently playing the sequence
	 * @return The sequence ID that holds the target object
	 */
	MOVIESCENE_API FMovieSceneSequenceID ResolveSequenceID(FMovieSceneSequenceID SourceSequenceID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState) const;

	MOVIESCENE_API FMovieSceneSequenceID ResolveSequenceID(FMovieSceneSequenceID SourceSequenceID, IMovieScenePlayer& Player) const;

	/**
	 * Resolve the sequence ID for this binding for the root sequence
	 *
	 * @param SourceSequenceID      The source sequence ID that owns this binding ID
	 * @param Hierarchy             The hierachy to use for computation of the relative path. Many be nullptr where Source and Targets are both MovieSceneSequenceID::Root
	 * @return The sequence ID that holds the target object
	 */
	MOVIESCENE_API FMovieSceneSequenceID ResolveSequenceID(FMovieSceneSequenceID SourceSequenceID, const FMovieSceneSequenceHierarchy* Hierarchy) const;

	/**
	 * Resolve all the bound objects for this binding ID
	 *
	 * @param SourceSequenceID      The source sequence ID that owns this binding ID
	 * @param Hierarchy             The hierachy to use for computation of the relative path. Many be nullptr where Source and Targets are both MovieSceneSequenceID::Root
	 * @return An array view of all bound objects
	 */
	MOVIESCENE_API TArrayView<TWeakObjectPtr<>> ResolveBoundObjects(FMovieSceneSequenceID SourceSequenceID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState) const;

	MOVIESCENE_API TArrayView<TWeakObjectPtr<>> ResolveBoundObjects(FMovieSceneSequenceID SourceSequenceID, IMovieScenePlayer& Player) const;

	/**
	 * Resolve all the bound objects for this binding ID
	 *
	 * @param SequenceInstance      The sequence instance for the sub sequence that is resolving the binding
	 * @return An array view of all bound objects
	 */
	MOVIESCENE_API TArrayView<TWeakObjectPtr<>> ResolveBoundObjects(const UE::MovieScene::FSequenceInstance& SequenceInstance) const;


	/**
	 * Reinterpret this binding ID as a fixed binding. Should only be used when ResolveParentIndex == -1, although this is not enforced.
	 */
	UE::MovieScene::FFixedObjectBindingID ReinterpretAsFixed() const
	{
		return UE::MovieScene::FFixedObjectBindingID(Guid, FMovieSceneSequenceID(SequenceID));
	}

public:

#if WITH_EDITORONLY_DATA
	void PostSerialize(const FArchive& Ar);
#endif

private:

	static constexpr int32 FixedRootSequenceParentIndex = -1;

	/** Identifier for the object binding within the sequence */
	UPROPERTY(EditAnywhere, Category="Binding")
	FGuid Guid;

	/** Sequence ID stored as an int32 so that it can be used in the blueprint VM */
	UPROPERTY()
	int32 SequenceID;

	/** The parent index to resolve the sequence ID from. 0 signifies the sequence this binding ID is serialized within, -1 signifies the root sequence (previously EMovieSceneObjectBindingSpace::Root) */
	UPROPERTY()
	int32 ResolveParentIndex;

private:
	void LegacyInit()
	{
#if WITH_EDITORONLY_DATA
		Space_DEPRECATED = EMovieSceneObjectBindingSpace::Unused;
#endif
	}


#if WITH_EDITORONLY_DATA

	/** DEPRECATED: The binding's resolution space */
	UPROPERTY()
	EMovieSceneObjectBindingSpace Space_DEPRECATED;

#endif // WITH_EDITORONLY_DATA
};


template<> struct TStructOpsTypeTraits<FMovieSceneObjectBindingID> : public TStructOpsTypeTraitsBase2<FMovieSceneObjectBindingID>
{
#if WITH_EDITORONLY_DATA
	enum 
	{
		WithPostSerialize = true,
	};
#endif
};
