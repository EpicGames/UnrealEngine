// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Misc/Guid.h"
#include "MovieSceneDynamicBinding.h"
#include "MovieSceneObjectBindingID.h"
#include "MovieSceneSequenceID.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPtr.h"

#include "MovieScenePossessable.generated.h"

class IMovieScenePlayer;
class UClass;
class UMovieScene;
struct FMovieSceneSequenceID;

namespace UE::MovieScene
{
	struct FSharedPlaybackState;
}

/**
 * MovieScenePossessable is a "typed slot" used to allow the MovieScene to control an already-existing object
 */
USTRUCT()
struct FMovieScenePossessable
{
	GENERATED_USTRUCT_BODY(FMovieScenePossessable)

public:

	/** Default constructor. */
	FMovieScenePossessable() { }

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InitName
	 * @param InitPossessedObjectClass
	 */
	FMovieScenePossessable(const FString& InitName, UClass* InitPossessedObjectClass)
		: Guid(FGuid::NewGuid())
		, Name(InitName)
#if WITH_EDITORONLY_DATA
		, PossessedObjectClass(InitPossessedObjectClass)
#endif
	{ }

public:

	/**
	 * Get the unique identifier of the possessed object.
	 *
	 * @return Object GUID.
	 * @see GetName, GetPossessedObjectClass
	 */
	const FGuid& GetGuid() const
	{
		return Guid;
	}

	/** 
	* Set the unique identifier
	*
	* @param InGuid
	*/

	void SetGuid(const FGuid& InGuid)
	{
		Guid = InGuid;
	}

	/**
	 * Get the name of the possessed object.
	 *
	 * @return Object name.
	 * @see GetGuid, GetPossessedObjectClass
	 */
	const FString& GetName() const
	{
		return Name;
	}

	/**
	* Set the name of the possessed object
	*
	* @param InName
	*/
	void SetName(const FString& InName)
	{
		Name = InName;
	}

#if WITH_EDITORONLY_DATA

	/**
	 * Get the class of the possessed object. Will load the class if it hasn't been loaded yet.
	 *
	 * @return Object class.
	 * @see GetGuid, GetName
	 */
	const UClass* GetPossessedObjectClass() const
	{
		if (PossessedObjectClass.ToSoftObjectPath().IsNull())
		{
			return nullptr;
		}

		return LoadObject<UClass>(nullptr, *PossessedObjectClass.ToSoftObjectPath().ToString(), nullptr, LOAD_EditorOnly);
	}

	/**
	 * Get the class of the possessed object. Can return null if the class hasn't been loaded by any other means yet.
	 *
	 * @return Object class.
	 */
	const UClass* GetLoadedPossessedObjectClass() const
	{
		return (const UClass*)(PossessedObjectClass.Get());
	}

	/**
	 * Set the class of the possessed object.
	 *
	 * @return Object class.
	 * @see GetGuid, GetName
	 */
	void SetPossessedObjectClass(UClass* InClass)
	{
		PossessedObjectClass = InClass;
	}

	/**
	 * Fixup the possessed object class by resolving the possessable and determining the most common class
	 */
	MOVIESCENE_API void FixupPossessedObjectClass(UMovieSceneSequence* InSequence, UObject* Context);

#endif

	/**
	 * Get the guid of this possessable's parent, if applicable
	 *
	 * @return The guid.
	 */
	const FGuid& GetParent() const
	{
		return ParentGuid;
	}

	UE_DEPRECATED(5.1, "Please use the overload that takes a UMovieScene* to ensure that events are triggered correctly")
	MOVIESCENE_API void SetParent(const FGuid& InParentGuid);

	/**
	 * Set the guid of this possessable's parent
	 *
	 * @param InParentGuid The guid of this possessable's parent.
	 */
	MOVIESCENE_API void SetParent(const FGuid& InParentGuid, UMovieScene* Owner);

	/** Array of tags that can be used for grouping and categorizing. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category=Actor)
	TArray<FName> Tags;

	// Property has been deprecated- 'Dynamic Bindings' are now handled by UMovieSceneReplaceableDirectorBlueprintBinding and the custom binding system
	UPROPERTY()
	FMovieSceneDynamicBinding DynamicBinding_DEPRECATED;

	/* Get the optional binding id for binding to a spawnable */
	const FMovieSceneObjectBindingID& GetSpawnableObjectBindingID() const 
	{
		return SpawnableObjectBindingID;
	}

	/* Set the optional binding id for binding to a spawnable */
	void SetSpawnableObjectBindingID(const FMovieSceneObjectBindingID& InSpawnableObjectBindingID)
	{
		SpawnableObjectBindingID = InSpawnableObjectBindingID;
	}

	/* Bind the potential spawnable object to this possessable by setting the ObjectBindingID */
	MOVIESCENE_API bool BindSpawnableObject(FMovieSceneSequenceID SequenceID, UObject* Object, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState);

	UE_DEPRECATED(5.4, "Please use the FSharedPlaybackState version of this method")
	MOVIESCENE_API bool BindSpawnableObject(FMovieSceneSequenceID SequenceID, UObject* Object, IMovieScenePlayer* Player);

	/* For sorts so we can search quickly by Guid */
	inline bool operator<(const FMovieScenePossessable& RHS) const { return Guid < RHS.Guid; }
	inline friend bool operator<(const FGuid& InGuid, const FMovieScenePossessable& RHS) { return InGuid < RHS.GetGuid(); }
	inline bool operator<(const FGuid& InGuid) const { return Guid < InGuid; }

private:

	/** Unique identifier of the possessable object. */
	// @todo sequencer: Guids need to be handled carefully when the asset is duplicated (or loaded after being copied on disk).
	//					Sometimes we'll need to generate fresh Guids.
	UPROPERTY()
	FGuid Guid;

	/** Name label for this slot */
	// @todo sequencer: Should be editor-only probably
	UPROPERTY()
	FString Name;

#if WITH_EDITORONLY_DATA

	/** Type of the object we'll be possessing */
	UPROPERTY()
	TSoftClassPtr<UObject> PossessedObjectClass;

#endif

	/** GUID relating to this possessable's parent, if applicable. */
	UPROPERTY()
	FGuid ParentGuid;

	/** Optional object binding ID if this possessable possesses a spawnable */
	UPROPERTY()
	FMovieSceneObjectBindingID SpawnableObjectBindingID;
};
