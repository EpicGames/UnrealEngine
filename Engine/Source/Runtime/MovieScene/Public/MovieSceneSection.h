// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Conditions/MovieSceneCondition.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"
#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Evaluation/Blending/MovieSceneBlendType.h"
#include "Evaluation/MovieSceneCompletionMode.h"
#include "Evaluation/MovieSceneEvaluationCustomVersion.h"
#include "EventHandlers/ISectionEventHandler.h"
#include "EventHandlers/MovieSceneDataEventContainer.h"
#include "Decorations/MovieSceneDecorationContainer.h"
#include "HAL/PlatformCrt.h"
#include "Math/Range.h"
#include "Math/RangeBound.h"
#include "Misc/AssertionMacros.h"
#include "Misc/FrameNumber.h"
#include "Misc/QualifiedFrameTime.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "Misc/Optional.h"
#include "Misc/Timecode.h"
#include "MovieSceneFrameMigration.h"
#include "MovieSceneSequenceID.h"
#include "MovieSceneSignedObject.h"
#include "MovieSceneTrack.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptInterface.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneSection.generated.h"

class FArchive;
class FStructOnScope;
class IMovieSceneEasingFunction;
class IMovieScenePlayer;
class UMovieSceneEntitySystemLinker;
class UObject;
class UMovieSceneTrack;
enum class ECookOptimizationFlags;
struct FEasingComponentData;
struct FFrame;
struct FFrameRate;
struct FGuid;
struct FKeyHandle;
struct FMovieSceneBlendTypeField;
struct FMovieSceneChannelProxy;
struct FMovieSceneEvalTemplatePtr;
struct FMovieSceneSequenceHierarchy;
struct FMovieSceneSequenceID;
struct FPropertyChangedEvent;
struct FQualifiedFrameTime;
struct FMovieSceneTimeWarpVariant;

namespace UE::MovieScene
{
	class ISectionEventHandler;
	struct FEntityImportParams;
	struct FFixedObjectBindingID;
	struct FImportedEntity;
	struct FSharedPlaybackState;
	struct IRetimingInterface;
}

/** Enumeration defining how a section's channel proxy behaves. */
enum class EMovieSceneChannelProxyType : uint8
{
	/** Once constructed, the channel proxy will not change even on serialization. The channel proxy has a static layout and the memory for each channel is always valid. */
	Static,

	/** The channel proxy layout can be affected by serialization or duplication and must be updated on such changes. */
	Dynamic
};

/** Delegate for when a section is added to a track. */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnMovieSceneSectionAddedToTrack, UMovieSceneTrack*, UMovieSceneSection*);
/** Delegate for when a section is removed from a track. */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnMovieSceneSectionChanged, UMovieSceneSection*);

USTRUCT()
struct FMovieSceneSectionEvalOptions
{
	GENERATED_BODY()
	
	FMovieSceneSectionEvalOptions()
		: bCanEditCompletionMode(false)
		, CompletionMode(EMovieSceneCompletionMode::KeepState)
	{}

	void EnableAndSetCompletionMode(EMovieSceneCompletionMode NewCompletionMode)
	{
		bCanEditCompletionMode = true;
		CompletionMode = NewCompletionMode;
	}

	UPROPERTY()
	bool bCanEditCompletionMode;

	/** When set to "RestoreState", this section will restore any animation back to its previous state  */
	UPROPERTY(EditAnywhere, DisplayName="When Finished", Category="Section")
	EMovieSceneCompletionMode CompletionMode;
};

USTRUCT()
struct FMovieSceneEasingSettings
{
	GENERATED_BODY()

	FMovieSceneEasingSettings()
		: AutoEaseInDuration(0), AutoEaseOutDuration(0)
		, EaseIn(nullptr), bManualEaseIn(false), ManualEaseInDuration(0)
		, EaseOut(nullptr), bManualEaseOut(false), ManualEaseOutDuration(0)
#if WITH_EDITORONLY_DATA
		, AutoEaseInTime_DEPRECATED(0.f), AutoEaseOutTime_DEPRECATED(0.f), ManualEaseInTime_DEPRECATED(0.f), ManualEaseOutTime_DEPRECATED(0.f)
#endif
	{}

public:

	int32 GetEaseInDuration() const
	{
		return bManualEaseIn ? ManualEaseInDuration : AutoEaseInDuration;
	}

	int32 GetEaseOutDuration() const
	{
		return bManualEaseOut ? ManualEaseOutDuration : AutoEaseOutDuration;
	}

public:

	/** Automatically applied ease in duration in frames */
	UPROPERTY()
	int32 AutoEaseInDuration;

	/** Automatically applied ease out time */
	UPROPERTY()
	int32 AutoEaseOutDuration;

	UPROPERTY()
	TScriptInterface<IMovieSceneEasingFunction> EaseIn;

	/** Whether to manually override this section's ease in time */
	UPROPERTY()
	bool bManualEaseIn;

	/** Manually override this section's ease in duration in frames */
	UPROPERTY()
	int32 ManualEaseInDuration;

	UPROPERTY()
	TScriptInterface<IMovieSceneEasingFunction> EaseOut;

	/** Whether to manually override this section's ease out time */
	UPROPERTY()
	bool bManualEaseOut;

	/** Manually override this section's ease-out duration in frames */
	UPROPERTY()
	int32 ManualEaseOutDuration;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	float AutoEaseInTime_DEPRECATED;
	UPROPERTY()
	float AutoEaseOutTime_DEPRECATED;
	UPROPERTY()
	float ManualEaseInTime_DEPRECATED;
	UPROPERTY()
	float ManualEaseOutTime_DEPRECATED;
#endif
};

USTRUCT(BlueprintType)
struct FMovieSceneTimecodeSource
{
	GENERATED_BODY()

	FMovieSceneTimecodeSource(FTimecode InTimecode)
		: Timecode(InTimecode)
	{}

	FMovieSceneTimecodeSource()
		: Timecode(FTimecode())
	{}

	inline bool operator==(const FMovieSceneTimecodeSource& Other) const
	{
		return Timecode == Other.Timecode;
	}
	inline bool operator!=(const FMovieSceneTimecodeSource& Other) const
	{
		return Timecode != Other.Timecode;
	}

public:

	/** The global timecode at which this target is based (ie. the timecode at the beginning of the movie scene section when it was recorded) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Timecode")
	FTimecode Timecode;
};
/**
* Defines for common transform 'type' sections. Moved here to avoid extra module dependencies
*/
enum class EMovieSceneTransformChannel : uint32
{
	None = 0x000,

	TranslationX = 0x001,
	TranslationY = 0x002,
	TranslationZ = 0x004,
	Translation = TranslationX | TranslationY | TranslationZ,

	RotationX = 0x008,
	RotationY = 0x010,
	RotationZ = 0x020,
	Rotation = RotationX | RotationY | RotationZ,

	ScaleX = 0x040,
	ScaleY = 0x080,
	ScaleZ = 0x100,
	Scale = ScaleX | ScaleY | ScaleZ,

	AllTransform = Translation | Rotation | Scale,

	Weight = 0x200,

	All = Translation | Rotation | Scale | Weight,
};

/**
 * Base class for movie scene sections
 */
UCLASS(abstract, DefaultToInstanced, MinimalAPI, BlueprintType)
class UMovieSceneSection
	: public UMovieSceneDecorationContainerObject
{
	GENERATED_UCLASS_BODY()

public:

	UPROPERTY(EditAnywhere, Category="Section", meta=(ShowOnlyInnerProperties))
	FMovieSceneSectionEvalOptions EvalOptions;

	UE::MovieScene::TDataEventContainer<UE::MovieScene::ISectionEventHandler> EventHandlers;

public:

	/**
	 * Calls Modify if this section can be modified, i.e. can't be modified if it's locked
	 *
	 * @return Returns whether this section is locked or not
	 */
	MOVIESCENE_API bool TryModify(bool bAlwaysMarkDirty=true);

	/*
	 * A section is read only if it or its outer movie are read only
	 * 
	 * @return Returns whether this section is read only
	 */
	MOVIESCENE_API bool IsReadOnly() const;

	/**
	 * @return The range of times of the section
	 */
	TRange<FFrameNumber> GetRange() const
	{
		return SectionRange.Value;
	}

	/**
	 * A true representation of this section's range with an inclusive start frame and an exclusive end frame.
	 * The resulting range defines that the section lies between { lower <= time < upper }
	 */
	TRange<FFrameNumber> GetTrueRange() const
	{
		TRangeBound<FFrameNumber> SectionLower = SectionRange.Value.GetLowerBound();
		TRangeBound<FFrameNumber> SectionUpper = SectionRange.Value.GetUpperBound();

		// Make exclusive lower bounds inclusive on the next frame
		if (SectionLower.IsExclusive())
		{
			SectionLower = TRangeBound<FFrameNumber>::Inclusive(SectionLower.GetValue() + 1);
		}
		// Make inclusive upper bounds exclusive on the next frame
		if (SectionUpper.IsInclusive())
		{
			SectionUpper = TRangeBound<FFrameNumber>::Exclusive(SectionUpper.GetValue() + 1);
		}
		return TRange<FFrameNumber>(SectionLower, SectionUpper);
	}

	/**
	 * Expands this section's range to include the specified time if bAutoExpandToKeyframes is enabled for the track
	 */
	void ExpandToFrame(FFrameNumber InFrame)
	{
		UMovieSceneTrack* Track = GetTypedOuter<UMovieSceneTrack>();
		check(Track != nullptr);

		if (!Track->bAutoExpandToKeyframes)
		{
			return;
		}

		FFrameNumber StartOffset = HasStartFrame() ? FMath::Min(GetInclusiveStartFrame(), InFrame) - GetInclusiveStartFrame() : 0;

		SetRange(TRange<FFrameNumber>::Hull(GetRange(), TRange<FFrameNumber>::Inclusive(InFrame, InFrame)));
		if (StartOffset != 0)
		{
			FixupRelativeKeyframes(StartOffset);
		}
	}

	/**
	 * Sets a new range of times for this section
	 * 
	 * @param NewRange	The new range of times
	 */
	virtual void SetRange(const TRange<FFrameNumber>& NewRange)
	{
		// Skip TryModify for objects that still need initialization (i.e. we're in the object's constructor), because modifying objects in their constructor can lead to non-deterministic cook issues.
		bool bCanSetRange = true;
		if (!HasAnyFlags(RF_NeedInitialization))
		{
			const bool bHasChanged = NewRange != SectionRange.Value; 
			bCanSetRange = !IsReadOnly() && bHasChanged && TryModify();
		}

		const bool bInvariants = ensure(
			NewRange.GetLowerBound().IsOpen() || NewRange.GetUpperBound().IsOpen()
			|| NewRange.GetLowerBoundValue() <= NewRange.GetUpperBoundValue()
			);
		if (bCanSetRange && bInvariants)
		{
			SectionRange.Value = NewRange;
		}
	}

	/**
	 * Check whether this section has a start frame (else infinite)
	 * @return true if this section has an inclusive or exclusive start frame, false if it's open (infinite)
	 */
	bool HasStartFrame() const
	{
		return !SectionRange.Value.GetLowerBound().IsOpen();
	}

	/**
	 * Check whether this section has an end frame (else infinite)
	 * @return true if this section has an inclusive or exclusive end frame, false if it's open (infinite)
	 */
	bool HasEndFrame() const
	{
		return !SectionRange.Value.GetUpperBound().IsOpen();
	}

	/**
	 * Gets the frame number at which this section starts
	 *
	 * @note Assumes a non-infinite start time. Check HasStartFrame first.
	 * @return The frame number at which this section starts.
	 */
	FFrameNumber GetInclusiveStartFrame() const
	{
		TRangeBound<FFrameNumber> LowerBound = SectionRange.GetLowerBound();
		return LowerBound.IsInclusive() ? LowerBound.GetValue() : LowerBound.GetValue() + 1;
	}

	/**
	 * Gets the first frame number after the end of this section
	 *
	 * @note Assumes a non-infinite end time. Check HasEndFrame first.
	 * @return The first frame after this section ends
	 */
	FFrameNumber GetExclusiveEndFrame() const
	{
		TRangeBound<FFrameNumber> UpperBound = SectionRange.GetUpperBound();
		return UpperBound.IsInclusive() ? UpperBound.GetValue() + 1 : UpperBound.GetValue();
	}

	/**
	 * Set this section's start frame in sequence resolution space.
	 * @note: Will be clamped to the current end frame if necessary
	 */
	MOVIESCENE_API virtual void SetStartFrame(TRangeBound<FFrameNumber> NewStartFrame);

	/**
	 * Set this section's end frame in sequence resolution space
	 * @note: Will be clamped to the current start frame if necessary
	 */
	MOVIESCENE_API virtual void SetEndFrame(TRangeBound<FFrameNumber> NewEndFrame);

	/**
	 * Returns whether or not a provided position in time is within the timespan of the section 
	 *
	 * @param Position	The position to check
	 * @return true if the position is within the timespan, false otherwise
	 */
	bool IsTimeWithinSection(FFrameNumber Position) const 
	{
		return SectionRange.Value.Contains(Position);
	}

	/*
	 * Returns the range to auto size this section to, if there is one. This defaults to the 
	 * range of all the keys.
	 *
	 * @return the range of this section to auto size to
	 */
	MOVIESCENE_API virtual TOptional<TRange<FFrameNumber> > GetAutoSizeRange() const;

	/**
	 * Gets this section's completion mode
	 */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	EMovieSceneCompletionMode GetCompletionMode() const
	{
		return EvalOptions.CompletionMode;
	}

	/*
	 * Sets this section's completion mode
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	void SetCompletionMode(EMovieSceneCompletionMode InCompletionMode)
	{
		EvalOptions.CompletionMode = InCompletionMode;
	}

	/**
	 * Gets this section's blend type
	 */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	FOptionalMovieSceneBlendType GetBlendType() const
	{
		return BlendType;
	}

	/**
	 * Sets this section's blend type
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	virtual void SetBlendType(EMovieSceneBlendType InBlendType)
	{
		if (GetSupportedBlendTypes().Contains(InBlendType))
		{
			BlendType = InBlendType;
		}
	}

	/**
	 * Gets what kind of blending is supported by this section
	 */
	MOVIESCENE_API FMovieSceneBlendTypeField GetSupportedBlendTypes() const;

	/**
	 * Get the optional order for the blending of this section that may or may not be used by the blending system
	 * Lower values blend first, value of INDEX_NONE means no order is set
	 */
	MOVIESCENE_API int32 GetBlendingOrder() const;

	/**
	 * Moves the section by a specific amount of time
	 *
	 * @param DeltaTime	The distance in time to move the curve
	 */
	MOVIESCENE_API virtual void MoveSection(FFrameNumber DeltaTime);

	/**
	 * Return the range within which this section is effective. Used for automatic calculation of sequence bounds.
	 *
	 * @return the range within which this section is effective
	 */
	MOVIESCENE_API TRange<FFrameNumber> ComputeEffectiveRange() const;

	/**
	 * Split a section in two at the split time
	 *
	 * @param SplitTime The time at which to split
	 * @param bDeleteKeys Delete keys outside the split ranges
	 * @return The newly created split section
	 */
	MOVIESCENE_API virtual UMovieSceneSection* SplitSection(FQualifiedFrameTime SplitTime, bool bDeleteKeys);

	/**
	 * Trim a section at the trim time
	 *
	 * @param TrimTime The time at which to trim
	 * @param bTrimLeft Whether to trim left or right
	 * @param bDeleteKeys Delete keys outside the split ranges
	 */
	MOVIESCENE_API virtual void TrimSection(FQualifiedFrameTime TrimTime, bool bTrimLeft, bool bDeleteKeys);

	/**
	 * Get the data structure representing the specified keys.
	 *
	 * @param KeyHandles The handles of the keys.
	 * @return The keys' data structure representation, or nullptr if key not found or no structure available.
	 */
	MOVIESCENE_API virtual TSharedPtr<FStructOnScope> GetKeyStruct(TArrayView<const FKeyHandle> KeyHandles);

	/**
	 * Retrieve the time warp variant for this section, if it has one
	 */
	virtual FMovieSceneTimeWarpVariant* GetTimeWarp()
	{
		return nullptr;
	}

	/**
	 * Gets all snap times for this section
	 *
	 * @param OutSnapTimes The array of times we will to output
	 * @param bGetSectionBorders Gets the section borders in addition to any custom snap times
	 */
	virtual void GetSnapTimes(TArray<FFrameNumber>& OutSnapTimes, bool bGetSectionBorders) const
	{
		if (bGetSectionBorders)
		{
			if (SectionRange.Value.GetLowerBound().IsClosed())
			{
				OutSnapTimes.Add(SectionRange.Value.GetLowerBoundValue());
			}

			if (SectionRange.Value.GetUpperBound().IsClosed())
			{
				OutSnapTimes.Add(SectionRange.Value.GetUpperBoundValue());
			}
		}
	}

	/** Sets this section's new row index */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	MOVIESCENE_API void SetRowIndex(int32 NewRowIndex);

	/** Gets the row index for this section */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	int32 GetRowIndex() const { return RowIndex; }
	
	/** Sets this section's priority over overlapping sections (higher wins) */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	void SetOverlapPriority(int32 NewPriority)
	{
		OverlapPriority = NewPriority;
	}

	/** Gets this section's priority over overlapping sections (higher wins) */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	int32 GetOverlapPriority() const
	{
		return OverlapPriority;
	}

	/**
	 * Checks to see if this section overlaps with an array of other sections
	 * given an optional time and track delta.
	 *
	 * @param Sections Section array to check against.
	 * @param TrackDelta Optional offset to this section's track index.
	 * @param TimeDelta Optional offset to this section's time delta.
	 * @return The first section that overlaps, or null if there is no overlap.
	 */
	virtual MOVIESCENE_API const UMovieSceneSection* OverlapsWithSections(const TArray<UMovieSceneSection*>& Sections, int32 TrackDelta = 0, int32 TimeDelta = 0) const;
	
	/**
	 * Places this section at the first valid row at the specified time. Good for placement upon creation.
	 *
	 * @param Sections Sections that we can not overlap with.
	 * @param InStartTime The new start time.
	 * @param InDuration The duration.
	 * @param bAllowMultipleRows If false, it will move the section in the time direction to make it fit, rather than the row direction.
	 */
	virtual MOVIESCENE_API void InitialPlacement(const TArray<UMovieSceneSection*>& Sections, FFrameNumber InStartTime, int32 InDuration, bool bAllowMultipleRows);

	/**
	 * Places this section at the specified row at the specified time. Overlapping sections will be moved down a row. Good for placement upon creation.
	 *
	 * @param Sections Sections that we can not overlap with.
	 * @param InStartTime The new start time.
	 * @param InDuration The duration.
	 * @param InRowIndex The row index to place this section on.
	 */
	virtual MOVIESCENE_API void InitialPlacementOnRow(const TArray<UMovieSceneSection*>& Sections, FFrameNumber InStartTime, int32 InDuration, int32 InRowIndex);

	/** Whether or not this section is active. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	void SetIsActive(bool bInIsActive) { if (TryModify()) { bIsActive = bInIsActive; } }
	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	bool IsActive() const { return bIsActive; }

	/** Whether or not this section is locked. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	void SetIsLocked(bool bInIsLocked) { Modify(); bIsLocked = bInIsLocked; }
	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	bool IsLocked() const { return bIsLocked; }

#if WITH_EDITOR
	/** Whether evaluation of this section is disabled due to local muting/soloing (not serialized). Virtual so subclasses (e.g. transitions) can add additional disable conditions. */
	MOVIESCENE_API virtual bool IsLocalEvalDisabled() const;
	void SetLocalEvalDisabled(bool bInEvalDisabled) { bIsLocalEvalDisabled = bInEvalDisabled; }
#endif

	/** Gets the number of frames to prepare this section for evaluation before it actually starts. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	void SetPreRollFrames(int32 InPreRollFrames) { PreRollFrames = InPreRollFrames; }
	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	int32 GetPreRollFrames() const { return PreRollFrames.Value; }

	/** Gets/sets the number of frames to continue 'postrolling' this section for after evaluation has ended. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	void SetPostRollFrames(int32 InPostRollFrames) { PostRollFrames = InPostRollFrames; }
	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	int32 GetPostRollFrames() const { return PostRollFrames.Value; }

	/** Set this section's color tint. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	MOVIESCENE_API void SetColorTint(const FColor& InColorTint);
	/** Get this section's color tint. */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	MOVIESCENE_API FColor GetColorTint() const;

	/** The optional offset time of this section */
	virtual TOptional<FFrameTime> GetOffsetTime() const { return TOptional<FFrameTime>(); }

	/* Migrate the frame times of the movie scene section from the source frame rate to the destination frame rate */
	virtual void MigrateFrameTimes(FFrameRate SourceRate, FFrameRate DestinationRate) {}
	virtual void MigrateFrameTimes(const UE::MovieScene::IRetimingInterface& Retimer) {}

	/** When guid bindings are updated to allow this section to fix-up any internal bindings */
	virtual void OnBindingIDsUpdated(const TMap<UE::MovieScene::FFixedObjectBindingID, UE::MovieScene::FFixedObjectBindingID>& OldFixedToNewFixedMap, FMovieSceneSequenceID LocalSequenceID, TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) {}

	/** Get the referenced bindings for this section */
	virtual void GetReferencedBindings(TArray<FGuid>& OutBindings) {}

	/**
	 * Gets a list of all overlapping sections
	 */
	MOVIESCENE_API void GetOverlappingSections(TArray<UMovieSceneSection*>& OutSections, bool bSameRow, bool bIncludeThis);

	/* Returns whether this section can have an open lower bound. This will generally be false if sections of this type cannot be blended and there is another section on the same row before this one.*/
	MOVIESCENE_API bool CanHaveOpenLowerBound() const;

	/* Returns whether this section can have an open upper bound. This will generally be false if sections of this type cannot be blended and there is another section on the same row after this one.*/
	MOVIESCENE_API bool CanHaveOpenUpperBound() const;

	/**
	 * Evaluate this sections's easing functions based on the specified time
	 */
	MOVIESCENE_API float EvaluateEasing(FFrameTime InTime) const;

	/**
	 * Evaluate this sections's easing functions based on the specified time
	 */
	MOVIESCENE_API void EvaluateEasing(FFrameTime InTime, TOptional<float>& OutEaseInValue, TOptional<float>& OutEaseOutValue, float* OutEaseInInterp, float* OutEaseOutInterp) const;

	MOVIESCENE_API TRange<FFrameNumber> GetEaseInRange() const;

	MOVIESCENE_API TRange<FFrameNumber> GetEaseOutRange() const;

	/**
	 * Access this section's channel proxy, containing pointers to all existing data channels in this section
	 * @note: Proxy can be reallocated at any time; this accessor is only for immediate use.
	 *
	 * @return A reference to this section's channel proxy.
	 */
	MOVIESCENE_API FMovieSceneChannelProxy& GetChannelProxy() const;

	/**
	 * Invalidate this section's channel proxy, causing it to be re-constructed when it is next used
	 */
	MOVIESCENE_API void InvalidateChannelProxy();

	/** Does this movie section support infinite ranges for evaluation */
	bool GetSupportsInfiniteRange() const { return bSupportsInfiniteRange; }

	/**
	*  Whether or not we draw a curve for a particular channel owned by this section.
	*  Defaults to true.
	*/
	virtual bool ShowCurveForChannel(const void *Channel) const  { return true; }

	/** 
	*  Get The Total Weight Value for this Section
	*  For Most Sections it's just the Ease Value, but for some Sections also have an extra Weight Curve
	*/
	virtual float GetTotalWeightValue(FFrameTime InTime) const { return EvaluateEasing(InTime); }


	/**
	*  Get the implicit owner of this section, usually this will be the section's outer possessable or spawnable,
	*  but some sections, like Control Rig, this will be the Control Rig object instead.
	*
	**/
	MOVIESCENE_API virtual UObject* GetImplicitObjectOwner();

	/** Get the object associated with this section */
	virtual UObject* GetSourceObject() const { return nullptr; }

	void OnAddedToTrack(UMovieSceneTrack* Track);
	void OnRemovedFromTrack();

#if WITH_EDITOR
	MOVIESCENE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	MOVIESCENE_API virtual void PostPaste();
#endif

#if WITH_EDITOR

	/**
	 * Called when this section's movie scene is being cooked to determine if/how this section should be cooked.
	 * @return ECookOptimizationFlags detailing how to optimize this section
	 */
	MOVIESCENE_API virtual ECookOptimizationFlags GetCookOptimizationFlags() const;

	/**
	 * Called when this section should be removed for cooking
	 */
	MOVIESCENE_API virtual void RemoveForCook();

#endif

public:

	MOVIESCENE_API void BuildDefaultComponents(UMovieSceneEntitySystemLinker* EntityLinker, const UE::MovieScene::FEntityImportParams& Params, UE::MovieScene::FImportedEntity* OutLedgerEntry);

#if WITH_EDITORONLY_DATA
	MOVIESCENE_API static void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif

#if WITH_EDITORONLY_DATA

public:

	/** A delegate for when a section is added to a track. */
	MOVIESCENE_API static FOnMovieSceneSectionAddedToTrack& GetOnSectionAddedToTrack();
	/** A delegate for when a section is removed from a track. */
	MOVIESCENE_API static FOnMovieSceneSectionChanged& GetOnSectionRemovedFromTrack();

private:

	static FOnMovieSceneSectionAddedToTrack OnAddedToTrackEvent;
	static FOnMovieSceneSectionChanged OnRemovedFromTrackEvent;

#endif

protected:

	//~ UObject interface
	MOVIESCENE_API virtual void PostInitProperties() override;
	MOVIESCENE_API virtual bool IsPostLoadThreadSafe() const override;
	MOVIESCENE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	MOVIESCENE_API virtual void PostRename(UObject* OldOuter, const FName OldName) override;
	MOVIESCENE_API virtual void PostEditImport() override;
	MOVIESCENE_API virtual void Serialize(FArchive& Ar) override;

	MOVIESCENE_API virtual void OnDecorationAdded(UObject* NewDecoration) override;
	MOVIESCENE_API virtual void OnDecorationRemoved(UObject* NewDecoration) override;

	virtual void OnAddedToTrackImpl(UMovieSceneTrack* Track)
	{
	}
	virtual void OnRemovedFromTrackImpl()
	{
	}

	virtual void OnMoved(int32 DeltaTime) {}
	virtual void OnDilated(float DilationFactor, FFrameNumber Origin) {}

	MOVIESCENE_API void FixupRelativeKeyframes(FFrameNumber Offset);

	MOVIESCENE_API bool ShouldUpgradeEntityData(FArchive& Ar, FMovieSceneEvaluationCustomVersion::Type UpgradeVersion) const;

private:

	/**
	 * Cache this section's channel proxy
	 */
	MOVIESCENE_API virtual EMovieSceneChannelProxyType CacheChannelProxy();

	void MoveSectionImpl(FFrameNumber DeltaTime);

public:

	UPROPERTY(EditAnywhere, Category="Easing", meta=(ShowOnlyInnerProperties))
	FMovieSceneEasingSettings Easing;

	/** The range in which this section is active */
	UPROPERTY(EditAnywhere, Category="Section")
	FMovieSceneFrameRange SectionRange;

	/** The timecode at which this movie scene section is based (ie. when it was recorded) */
	UPROPERTY(EditAnywhere, Category="Section")
	FMovieSceneTimecodeSource TimecodeSource;

	/** Optional dynamic condition for whether this section evaluates at runtime. */
	UPROPERTY(EditAnywhere, Category="Section")
	FMovieSceneConditionContainer ConditionContainer;

private:

	/** The amount of time to prepare this section for evaluation before it actually starts. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Section", meta=(UIMin=0, UIFrameDisplayAs="NonMusical|Duration"))
	FFrameNumber PreRollFrames;

	/** The amount of time to continue 'postrolling' this section for after evaluation has ended. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Section", meta=(UIMin=0, UIFrameDisplayAs="NonMusical|Duration"))
	FFrameNumber PostRollFrames;

	/** The row index that this section sits on */
	UPROPERTY()
	int32 RowIndex;

	/** This section's priority over overlapping sections */
	UPROPERTY()
	int32 OverlapPriority;

	/** Toggle whether this section is active/inactive */
	UPROPERTY(EditAnywhere, Category="Section")
	uint32 bIsActive : 1;

	/** Toggle whether this section is locked/unlocked */
	UPROPERTY(EditAnywhere, Category="Section")
	uint32 bIsLocked : 1;

#if WITH_EDITORONLY_DATA
	/** The color tint for this section */
	UPROPERTY(EditAnywhere, Category = "Section")
	FColor ColorTint;

	/** Whether evaluation of this section is disabled due to local muting/soloing (transient, not serialized) */
	bool bIsLocalEvalDisabled = false;
#endif

protected:

	/** The start time of the section */
	UPROPERTY()
	float StartTime_DEPRECATED;

	/** The end time of the section */
	UPROPERTY()
	float EndTime_DEPRECATED;

	/** The amount of time to prepare this section for evaluation before it actually starts. */
	UPROPERTY()
	float PreRollTime_DEPRECATED;

	/** The amount of time to continue 'postrolling' this section for after evaluation has ended. */
	UPROPERTY()
	float PostRollTime_DEPRECATED;

	/** Toggle to set this section to be infinite */
	UPROPERTY()
	uint32 bIsInfinite_DEPRECATED : 1;

protected:
	/** Does this section support infinite ranges in the track editor? */
	UPROPERTY()
	bool bSupportsInfiniteRange;

	UPROPERTY()
	FOptionalMovieSceneBlendType BlendType;

	/**
	 * Channel proxy that contains all the channels in this section - must be populated and invalidated by derived types.
	 * Must be re-allocated any time any channel pointer in derived types is reallocated (such as channel data stored in arrays)
	 * to ensure that any weak handles to channels are invalidated correctly. Allocation is via MakeShared<FMovieSceneChannelProxy>().
	 */
	mutable TSharedPtr<FMovieSceneChannelProxy> ChannelProxy;

	/** Defines whether the channel proxy can change over the lifetime of the section */
	mutable EMovieSceneChannelProxyType ChannelProxyType;
};

template<typename SectionParams>
inline FFrameNumber GetFirstLoopStartOffsetAtTrimTime(FQualifiedFrameTime TrimTime, const SectionParams& Params, FFrameNumber StartFrame, FFrameRate FrameRate)
{
	const float AnimPlayRate = FMath::IsNearlyZero(Params.PlayRate) ? 1.0f : Params.PlayRate;
	const float AnimPosition = static_cast<float>((TrimTime.Time - StartFrame) / TrimTime.Rate * AnimPlayRate);
	const float SeqLength = static_cast<float>(Params.GetSequenceLength() - FrameRate.AsSeconds(Params.StartFrameOffset + Params.EndFrameOffset) / AnimPlayRate);

	FFrameNumber NewOffset = FrameRate.AsFrameNumber(FMath::Fmod(AnimPosition, SeqLength));
	NewOffset += Params.FirstLoopStartFrameOffset;

	const FFrameNumber SeqLengthInFrames = FrameRate.AsFrameNumber(SeqLength);
	NewOffset = NewOffset % SeqLengthInFrames;

	return NewOffset;
}
