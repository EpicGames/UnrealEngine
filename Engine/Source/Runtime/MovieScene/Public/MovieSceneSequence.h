// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "UniversalObjectLocatorFwd.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Evaluation/MovieSceneCompletionMode.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Guid.h"
#include "MovieSceneFwd.h"
#include "MovieSceneObjectBindingID.h" // only for FMovieSceneObjectBindingID in .gen.cpp
#include "MovieSceneSection.h" // only for FMovieSceneTimecodeSource in .gen.cpp
#include "MovieSceneSequenceID.h"
#include "MovieSceneSignedObject.h"
#include "UniversalObjectLocatorResolveParams.h"
#include "MovieSceneTrack.h"
#include "Templates/SubclassOf.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneSequence.generated.h"

class FArchive;
class FObjectPreSaveContext;
class IMovieScenePlayer;
class ITargetPlatform;
class UMovieScene;
class UMovieSceneCompiledData;
class UMovieSceneEntitySystemLinker;
class UMovieSceneTrack;
class UObject;
struct FFrame;
struct FMovieSceneObjectCache;
struct FMovieScenePossessable;
struct FMovieSceneTimecodeSource;
struct FUniversalObjectLocator;
struct FMovieSceneBindingReferences;
struct FMovieSceneDynamicBinding;

#if WITH_EDITOR
struct FSlateBrush;
#endif


namespace UE::MovieScene
{
	struct FSharedPlaybackState;
}

enum class ETrackSupport
{
	/** This track is not supported */
	NotSupported,
	/** This track is supported */
	Supported,
	/** Default behavior determined by the track */
	Default
};


/**
 * Abstract base class for movie scene animations (C++ version).
 */
UCLASS(abstract, MinimalAPI, Config = Engine, BlueprintType)
class UMovieSceneSequence
	: public UMovieSceneSignedObject
{
public:

	using FSharedPlaybackState = UE::MovieScene::FSharedPlaybackState;

	GENERATED_BODY()

	MOVIESCENE_API UMovieSceneSequence(const FObjectInitializer& Init);

public:

	/**
	 * Called when Sequencer has created an object binding for a possessable object
	 * 
	 * @param ObjectId The guid used to map to the possessable object.  Note the guid can be bound to multiple objects at once
	 * @param PossessedObject The runtime object which was possessed.
	 * @param Context Optional context required to bind the specified object (for instance, a parent spawnable object)
	 * @see UnbindPossessableObjects
	 */
	MOVIESCENE_API virtual bool MakeLocatorForObject(UObject* Object, UObject* Context, FUniversalObjectLocator& OutLocator) const;

	/**
	 * Retrieve core UOL-based binding references for this sequence type.
	 */
	MOVIESCENE_API FMovieSceneBindingReferences* GetBindingReferences();

	/**
	 * (Optional) Retrieve core UOL-based binding references for this sequence type.
	 */
	MOVIESCENE_API virtual const FMovieSceneBindingReferences* GetBindingReferences() const;

	/**
	 * Unloads an object that has been loaded via a locator.
	 */
	UE_DEPRECATED(5.5, "UnloadBoundObject no longer supported")
	void UnloadBoundObject(const UE::UniversalObjectLocator::FResolveParams& ResolveParams, const FGuid& ObjectId, int32 BindingIndex) {}


	/**
	 * Called when Sequencer has created an object binding for a possessable object
	 * 
	 * @param ObjectId The guid used to map to the possessable object.  Note the guid can be bound to multiple objects at once
	 * @param PossessedObject The runtime object which was possessed.
	 * @param Context Optional context required to bind the specified object (for instance, a parent spawnable object)
	 * @see UnbindPossessableObjects
	 */
	virtual void BindPossessableObject(const FGuid& ObjectId, UObject& PossessedObject, UObject* Context) PURE_VIRTUAL(UMovieSceneSequence::BindPossessableObject,);

	/**
	 * Check whether the given object can be possessed by this animation.
	 *
	 * @param Object The object to check.
	 * @param InPlaybackContext The current playback context
	 * @return true if the object can be possessed, false otherwise.
	 */
	virtual bool CanPossessObject(UObject& Object, UObject* InPlaybackContext) const PURE_VIRTUAL(UMovieSceneSequence::CanPossessObject, return false;);

	/**
	 * Locate all the objects that correspond to the specified object ID, using the specified context. Called when GetBindingReferences() is null.
	 *
	 * @param ObjectId				The unique identifier of the object.
	 * @param Context				Optional context to use to find the required object (for instance, a parent spawnable object)
	 * @param OutObjects			Destination array to add found objects to
	 */
	UE_DEPRECATED(5.4, "Please call the FResolveParams overload")
	virtual void LocateBoundObjects(const FGuid& ObjectId, UObject* Context, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const {}

	/**
	 * Locate all the objects that correspond to the specified object ID, using the specified parameters.
	 * Calling this directly instead of IMovieScenePlayer::ResolveBoundObjects means that you won't be able to locate Spawnable objects or objects from Custom Bindings.
	 *
	 * @param ObjectId				The unique identifier of the object.
	 * @param Params				Resolve parameters specifying the context and fragment-specific parameters
	 * @param OutObjects			Destination array to add found objects to
	 */
	UE_DEPRECATED(5.5, "Please use the version that also takes a SharedPlaybackState")
	MOVIESCENE_API void LocateBoundObjects(const FGuid& ObjectId, const UE::UniversalObjectLocator::FResolveParams& ResolveParams, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const;

	/**
	 * Locate all the objects that correspond to the specified object ID, using the specified context
	 * Calling this directly instead of IMovieScenePlayer::ResolveBoundObjects means that you won't be able to locate Spawnable objects or objects from Custom Bindings.
	 *
	 * @param ObjectId				The unique identifier of the object.
	 * @param Context				Optional context to use to find the required object (for instance, a parent spawnable object)
	 * @return An array of all bound objects
	 */
	UE_DEPRECATED(5.5, "Please use the version that also takes a SharedPlaybackState")
	TArray<UObject*, TInlineAllocator<1>> LocateBoundObjects(const FGuid& ObjectId, const UE::UniversalObjectLocator::FResolveParams& Context) const
	{
		TArray<UObject*, TInlineAllocator<1>> OutObjects;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		LocateBoundObjects(ObjectId, Context, OutObjects);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		return OutObjects;
	}

	/**
	 * Locate all the objects that correspond to the specified object ID, using the specified parameters.
	 * Calling this directly instead of IMovieScenePlayer::ResolveBoundObjects means that you won't be able to locate Spawnable objects or objects from Custom Bindings.
	 *
	 * @param ObjectId				The unique identifier of the object.
	 * @param ResolveParams			Resolve parameters specifying the context and fragment-specific parameters
	 * @param SharedPlaybackState   Optional ptr to SharedPlaybackState
	 * @param OutObjects			Destination array to add found objects to
	 */
	MOVIESCENE_API virtual void LocateBoundObjects(const FGuid& ObjectId, const UE::UniversalObjectLocator::FResolveParams& ResolveParams, TSharedPtr<const FSharedPlaybackState> SharedPlaybackState, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const;

	/**
	 * Attempt to find the guid relating to the specified object
	 *
	 * @param ObjectId				The unique identifier of the object.
	 * @param Context				Optional context to use to find the required object (for instance, a parent spawnable object or its world)
	 * @return The object's guid, or zero guid if the object is not a valid possessable in the current context
	 */
	MOVIESCENE_API FGuid FindPossessableObjectId(UObject& Object, UObject* Context) const;

	UE_DEPRECATED(5.5, "Use the version that takes a SharedPlaybackState")
	MOVIESCENE_API virtual FGuid FindBindingFromObject(UObject* InObject, UObject* Context) const;

	/**
	 * Optional method for efficient lookup of an object binding from an actual object in the world
	 *
	 * @param ObjectId				The unique identifier of the object.
	 * @param SharedPlaybackState   A shared playback state for the binding- create a transient one if necessary.
	 * @return The object's guid, or zero guid if the object is not a valid possessable in the current context
	 */
	virtual FGuid FindBindingFromObject(UObject* InObject, TSharedRef<const FSharedPlaybackState> SharedPlaybackState) const { return FGuid(); }

	/**
	 * Called to validate the specified object cache by removing anything that should be deemed out of date
	 * 
	 * @param InObjectCache 		The object cache container that contains all the objects currently animated by this sequence
	 * @param OutInvalidIDs 		(Out) Array to populate with any object bindings that should be invalidated
	 */
	virtual void GatherExpiredObjects(const FMovieSceneObjectCache& InObjectCache, TArray<FGuid>& OutInvalidIDs) const {}

	/**
	 * Get the movie scene that controls this animation.
	 *
	 * The returned movie scene represents the root movie scene.
	 * It may contain additional child movie scenes.
	 *
	 * @return The movie scene.
	 */
	virtual UMovieScene* GetMovieScene() const PURE_VIRTUAL(UMovieSceneSequence::GetMovieScene(), return nullptr;);

	/**
	 * Get the logical parent object for the supplied object (not necessarily its outer).
	 *
	 * @param Object The object whose parent to get.
	 * @return The parent object, or nullptr if the object has no logical parent.
	 */
	virtual UObject* GetParentObject(UObject* Object) const PURE_VIRTUAL(UMovieSceneSequence::GetParentObject(), return nullptr;);

	/**
	 * Whether objects can be spawned at run-time.
	 *
	 * @return true if objects can be spawned by sequencer, false if only existing objects can be possessed.
	 */
	virtual bool AllowsSpawnableObjects() const { return false; }

	/**
	 * Whether the sequence type supports custom bindings.
	 *
	 * @return true if custom bindings are supported, false if they are not
	 */
	virtual bool AllowsCustomBindings() const { return false; }

	/**
	 * Unbinds all possessable objects from the provided GUID.
	 *
	 * @param ObjectId The guid bound to possessable objects that should be removed.
	 * @see BindPossessableObject
	 */
	virtual void UnbindPossessableObjects(const FGuid& ObjectId) PURE_VIRTUAL(UMovieSceneSequence::UnbindPossessableObjects, );

	/**
	 * Unbinds specific objects from the provided GUID
	 *
	 * @param ObjectId The guid bound to possessable objects that should be removed.
	 * @param InObjects The objects to unbind.
	 * @param Context Optional context required to bind the specified object (for instance, a parent spawnable object)
	 */
	virtual void UnbindObjects(const FGuid& ObjectId, const TArray<UObject*>& InObjects, UObject* Context) PURE_VIRTUAL(UMovieSceneSequence::UnbindObjects, );

	/**
	 * Unbinds specific objects from the provided GUID that do not resolve
	 *
	 * @param ObjectId The guid bound to possessable objects that should be removed.
	 * @param Context Optional context required to bind the specified object (for instance, a parent spawnable object)
	 */
	virtual void UnbindInvalidObjects(const FGuid& ObjectId, UObject* Context) PURE_VIRTUAL(UMovieSceneSequence::UnbindInvalidObjects, );

	/**
	 * Create a spawnable object template from the specified source object
	 *
	 * @param InSourceObject The source object to copy
	 * @param ObjectName The name of the object
	 * @return A new object template of the specified name
	 */
	virtual UObject* MakeSpawnableTemplateFromInstance(UObject& InSourceObject, FName ObjectName) { return nullptr; }

	/**
	 * Specifies whether this sequence allows rebinding of the specified possessable
	 *
	 * @param InPossessable The possessable to check
	 * @return true if rebinding this possessable is valid at runtime, false otherwise
	 */
	virtual bool CanRebindPossessable(const FMovieScenePossessable& InPossessable) const { return false; }

	/**
	 * Specifies whether this sequence can animate the object in question (either as a spawnable or possessable)
	 *
	 * @param	InObject	The object to check
	 * @return true if this object can be animated.
	 */
	virtual bool CanAnimateObject(UObject& InObject) const { return true; }

	/**
	 * Called to add a new possessable for the specified object
	 */
	virtual FGuid CreatePossessable(UObject* ObjectToPossess) { return FGuid(); }

	/**
	 * Called to add a new spawnable for the specified object
	 */
	virtual FGuid CreateSpawnable(UObject* ObjectToSpawn) { return FGuid(); }

	/**
	 * Called to retrieve or construct a director instance to be used for the specified player
	 */
	virtual UObject* CreateDirectorInstance(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, FMovieSceneSequenceID SequenceID) { return nullptr; }

	UE_DEPRECATED(5.4, "Please use the version that takes a SharedPlaybackState")
	UObject* CreateDirectorInstance(IMovieScenePlayer& Player, FMovieSceneSequenceID SequenceID);

	MOVIESCENE_API virtual EMovieSceneServerClientMask OverrideNetworkMask(EMovieSceneServerClientMask InDefaultMask) const;

	/**
	 * Find the first object binding ID associated with the specified tag name (set up through RMB->Expose on Object bindings from within sequencer)
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence")
	MOVIESCENE_API FMovieSceneObjectBindingID FindBindingByTag(FName InBindingName) const;

	/**
	 * Find all object binding IDs associated with the specified tag name (set up through RMB->Expose on Object bindings from within sequencer)
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence")
	MOVIESCENE_API const TArray<FMovieSceneObjectBindingID>& FindBindingsByTag(FName InBindingName) const;

	/**
	 * Get the earliest timecode source out of all of the movie scene sections contained within this sequence's movie scene.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence")
	MOVIESCENE_API FMovieSceneTimecodeSource GetEarliestTimecodeSource() const;

#if WITH_EDITOR
	virtual void IterateDynamicBindings(TFunction<void(const FGuid&, FMovieSceneDynamicBinding&)> InCallback) {}

	virtual const FSlateBrush* GetCustomBrushForBinding(FGuid BindingID) const { return nullptr; }
#endif

public:

	MOVIESCENE_API virtual void PostLoad() override;
	MOVIESCENE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	MOVIESCENE_API virtual void BeginDestroy() override;
	MOVIESCENE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;

	MOVIESCENE_API virtual void Serialize(FArchive& Ar) override;

public:

	/**
	 * true if the result of GetParentObject is significant in object resolution for LocateBoundObjects.
	 */
	bool AreParentContextsSignificant() const
	{
		return bParentContextsAreSignificant;
	}

	/**
	 * Check whether this sequence is playable directly outside of a root sub sequence or not
	 *
	 * @return True if this sequences cooked data will include all the necessary information to be played back on its own, false if this data is not present in cooked builds
	 */
	bool IsPlayableDirectly() const
	{
		return bPlayableDirectly;
	}

	/**
	 * Assign whether this sequence is playable directly outside of a root sub sequence or not
	 *
	 * @param bInPlayableDirectly   When true, this sequence's cooked data will include all the necessary information to be played back on its own. When false this data will be culled resulting in less memory usage.
	 */
	void SetPlayableDirectly(bool bInPlayableDirectly)
	{
		Modify();
		bPlayableDirectly = bInPlayableDirectly;
	}
	
	/**
	 * Access the flags that define this sequence's characteristics and behavior
	 */
	EMovieSceneSequenceFlags GetFlags() const
	{
		return SequenceFlags;
	}

	void SetSequenceFlags(EMovieSceneSequenceFlags InFlags)
	{
		SequenceFlags = InFlags;
	}

	UMovieSceneCompiledData* GetCompiledData() const;
	UMovieSceneCompiledData* GetOrCreateCompiledData();

private:

#if WITH_EDITOR
	void OptimizeForCook();
#endif

	/** Serialized compiled data - should only be used through UMovieSceneCompiledDataManager */
	UPROPERTY(Instanced)
	TObjectPtr<UMovieSceneCompiledData> CompiledData;

public:

	/* The default completion mode for this movie scene when a section's completion mode is set to project default */
	UPROPERTY(config)
	EMovieSceneCompletionMode DefaultCompletionMode;

protected:

	/**
	 * true if the result of GetParentObject is significant in object resolution for LocateBoundObjects.
	 * When true, if GetParentObject returns nullptr, the PlaybackContext will be used for LocateBoundObjects, other wise the object's parent will be used
	 * When false, the PlaybackContext will always be used for LocateBoundObjects
	 */
	UPROPERTY()
	bool bParentContextsAreSignificant;

	/**
	 * When true, this sequence should be compiled as if it is playable directly (outside of a root sequence). When false, various compiled data will be omitted, preventing direct playback at runtime (although will still play as a sub sequence)
	 */
	UPROPERTY()
	bool bPlayableDirectly;

	/** Flags used to define this sequence's behavior and characteristics */
	UPROPERTY()
	EMovieSceneSequenceFlags SequenceFlags;

public:

#if WITH_EDITOR

	/**
	 * Get the display name for this movie sequence
	 */
	virtual FText GetDisplayName() const { return FText::FromName(GetFName()); }
	
	/*
	 * Sequences can determine whether they support a particular track type
	 */
	MOVIESCENE_API ETrackSupport IsTrackSupported(TSubclassOf<UMovieSceneTrack> InTrackClass) const;

	/** Subclasses can implement this to determine whether they support a particular track type */
	virtual ETrackSupport IsTrackSupportedImpl(TSubclassOf<UMovieSceneTrack> InTrackClass) const { return ETrackSupport::Default; }

	/*
	 * Sequences can determine whether they support a particular filter type
	 */
	MOVIESCENE_API bool IsFilterSupported(const FString& InFilterName) const;

	/** Subclasses can implement this to determine whether they support a particular filter type */
	virtual bool IsFilterSupportedImpl(const FString& InFilterName) const { return true; }

#endif

	/** 
	 * Given a sequence, checks if it is compatible as a subsequence to this sequence. 
	 */
	MOVIESCENE_API bool IsSubSequenceCompatible(const UMovieSceneSequence& SubSequence) const;

	/**
	 * Sequences can determine if the given sequence would be compatible as a subsequence.
	 */
	virtual bool IsCompatibleSubSequence(const UMovieSceneSequence& SubSequence) const { return true; }

	/**
	 * Sequences can determine if they would be compatible as a subsequence to the given parent sequence.
	 */
	virtual bool IsCompatibleAsSubSequence(const UMovieSceneSequence& ParentSequence) const { return true; }
};
