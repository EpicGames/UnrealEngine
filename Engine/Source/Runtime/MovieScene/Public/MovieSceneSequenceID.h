// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "EntitySystem/MovieSceneComponentDebug.h"
#include "MovieSceneSequenceID.generated.h"

USTRUCT()
struct FMovieSceneSequenceID
{
	GENERATED_BODY()

	constexpr FMovieSceneSequenceID()
		: Value(-1)
	{}

	constexpr explicit FMovieSceneSequenceID(uint32 InValue)
		: Value(InValue)
	{}

	inline friend bool operator==(FMovieSceneSequenceID LHS, FMovieSceneSequenceID RHS)
	{
		return LHS.Value == RHS.Value;
	}
	
	inline friend bool operator!=(FMovieSceneSequenceID LHS, FMovieSceneSequenceID RHS)
	{
		return LHS.Value != RHS.Value;
	}

	inline friend bool operator<(FMovieSceneSequenceID LHS, FMovieSceneSequenceID RHS)
	{
		return LHS.Value < RHS.Value;
	}

	inline friend bool operator>(FMovieSceneSequenceID LHS, FMovieSceneSequenceID RHS)
	{
		return LHS.Value > RHS.Value;
	}

	inline friend uint32 GetTypeHash(FMovieSceneSequenceID In)
	{
		return GetTypeHash(In.Value);
	}

	inline FMovieSceneSequenceID AccumulateParentID(FMovieSceneSequenceID InParentID) const
	{
		return Value == 0 ? InParentID : FMovieSceneSequenceID(HashCombine(Value, InParentID.Value));
	}

	inline bool Serialize(FArchive& Ar)
	{
		Ar << Value;
		return true;
	}

	inline friend FArchive& operator<<(FArchive& Ar, FMovieSceneSequenceID& SequenceID)
	{
		SequenceID.Serialize(Ar);
		return Ar;
	}

	inline uint32 GetInternalValue() const
	{
		return Value;
	}

	inline bool IsValid() const
	{
		return Value != -1;
	}

private:

	UPROPERTY()
	uint32 Value;
};

template<>
struct TStructOpsTypeTraits<FMovieSceneSequenceID> : public TStructOpsTypeTraitsBase2<FMovieSceneSequenceID>
{
	enum
	{
		WithSerializer = true,
		WithCopy = true
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

typedef TCallTraits<FMovieSceneSequenceID>::ParamType FMovieSceneSequenceIDRef;

namespace MovieSceneSequenceID
{
	inline constexpr FMovieSceneSequenceID Invalid = FMovieSceneSequenceID(-1);
	inline constexpr FMovieSceneSequenceID Root = FMovieSceneSequenceID(0);
}

inline FString LexToString(const FMovieSceneSequenceID& SequenceID)
{
	return *FString::Printf(TEXT("SeqID(%d)"), SequenceID.GetInternalValue());
}
