// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectMacros.h"
#include "MovieSceneSequenceTickInterval.h"
#include "MovieSceneSequencePlaybackSettings.generated.h"


/** POD struct that represents a number of loops where -1 signifies infinite looping, 0 means no loops, etc
 * Defined as a struct rather than an int so a property type customization can be bound to it
 */
USTRUCT(BlueprintType)
struct FMovieSceneSequenceLoopCount
{
	FMovieSceneSequenceLoopCount()
		: Value(0)
	{}

	GENERATED_BODY()

	/** Serialize this count from an int */
	bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot );

	/** Whether or not to loop playback. If Loop Exactly is chosen, you can specify the number of times to loop */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Playback", meta=(UIMin=1, DisplayName="Loop"))
	int32 Value;
};
template<> struct TStructOpsTypeTraits<FMovieSceneSequenceLoopCount> : public TStructOpsTypeTraitsBase2<FMovieSceneSequenceLoopCount>
{
	enum { WithStructuredSerializeFromMismatchedTag = true };
};

/* Whether to override track sections' default completion mode when a sequence finishes.*/
UENUM(BlueprintType)
enum class EMovieSceneCompletionModeOverride : uint8
{
	None UMETA(DisplayName = "None"),
	ForceKeepState UMETA(DisplayName="Force Keep State"),
	ForceRestoreState UMETA(DisplayName="Force Restore State")
};


/**
 * Settings for the level sequence player actor.
 */
USTRUCT(BlueprintType)
struct FMovieSceneSequencePlaybackSettings
{
	FMovieSceneSequencePlaybackSettings()
		: bAutoPlay(false)
		, PlayRate(1.f)
		, StartTime(0.f)
		, bRandomStartTime(false)
		, bDisableMovementInput(false)
		, bDisableLookAtInput(false)
		, bHidePlayer(false)
		, bHideHud(false)
		, bDisableCameraCuts(false)
		, bPauseAtEnd(false)
		, bInheritTickIntervalFromOwner(true)
		, bDynamicWeighting(false)
	{ }

	GENERATED_BODY()

	/** Auto-play the sequence when created */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Playback")
	uint32 bAutoPlay : 1;

	/** Number of times to loop playback. -1 for infinite, else the number of times to loop before stopping */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Playback", meta=(UIMin=1, DisplayName="Loop"))
	FMovieSceneSequenceLoopCount LoopCount;

	/** Overridable tick interval for this sequence to update at. When not overridden, the owning actor or component's tick interval will be used */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Playback", meta=(Units=s, EditCondition="!bInheritTickIntervalFromOwner"))
	FMovieSceneSequenceTickInterval TickInterval;

	/** The rate at which to playback the animation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Playback", meta=(Units=Multiplier))
	float PlayRate;

	/** Start playback at the specified offset from the start of the sequence's playback range */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Playback", DisplayName="Start Offset", meta=(Units=s, EditCondition="!bRandomStartTime"))
	float StartTime;

	/** Start playback at a random time */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Playback")
	uint32 bRandomStartTime : 1;

	/** Disable Input from player during play */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Cinematic")
	uint32 bDisableMovementInput : 1;

	/** Disable LookAt Input from player during play */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Cinematic")
	uint32 bDisableLookAtInput : 1;

	/** Hide Player Pawn during play */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Cinematic")
	uint32 bHidePlayer : 1;

	/** Hide HUD during play */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Cinematic")
	uint32 bHideHud : 1;

	/** Disable camera cuts */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Cinematic")
	uint32 bDisableCameraCuts : 1;
	
	/* If set to something other than none, when a sequence ends, the completion mode of any track sections still active will be overridden
	* by the chosen value, either keep state or restore state. Otherwise, completion mode will be determined by each track section.
	* Note that any track sections that finish before the end of a sequence will have their completion mode determined by the section settings rather than this override.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Cinematic")
	EMovieSceneCompletionModeOverride FinishCompletionStateOverride = EMovieSceneCompletionModeOverride::None;

	/** Pause the sequence when playback reaches the end rather than stopping it */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Playback")
	uint32 bPauseAtEnd : 1;

	/** When checked, a custom tick interval can be provided to define how often to update this sequence */
	UPROPERTY(EditAnywhere, Category="Playback", meta=(InlineEditConditionToggle))
	uint32 bInheritTickIntervalFromOwner : 1;

	/** Whether to enable dynamic weighting on this player or not */
	UPROPERTY(EditAnywhere, Category="Playback")
	uint32 bDynamicWeighting : 1;


	MOVIESCENE_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);
};

template<> struct TStructOpsTypeTraits<FMovieSceneSequencePlaybackSettings> : public TStructOpsTypeTraitsBase2<FMovieSceneSequencePlaybackSettings>
{
	enum { WithStructuredSerializeFromMismatchedTag = true };
};
