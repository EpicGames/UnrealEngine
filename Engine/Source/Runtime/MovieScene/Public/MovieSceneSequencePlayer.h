// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "IMovieScenePlayer.h"
#include "MovieScene.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "IMovieScenePlaybackClient.h"
#include "Misc/QualifiedFrameTime.h"
#include "MovieSceneTimeController.h"
#include "Evaluation/MovieScenePlayback.h"
#include "Evaluation/MovieScenePlayback.h"
#include "MovieSceneSequencePlaybackSettings.h"
#include "MovieSceneSequenceTickManagerClient.h"
#include "MovieSceneSequencePlaybackSettings.h"
#include "MovieSceneLatentActionManager.h"
#include "IMovieSceneSequencePlayerObserver.h"
#include "EntitySystem/MovieSceneEntityIDs.h"

#include "MovieSceneSequencePlayer.generated.h"

class UMovieSceneSequenceTickManager;

namespace UE::MovieScene
{
	class FSequenceWeights;
}

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnMovieSceneSequencePlayerEvent);
DECLARE_DELEGATE(FOnMovieSceneSequencePlayerNativeEvent);

/**
 * Enum used to define how to update to a particular time
 */
UENUM(BlueprintType)
enum class EUpdatePositionMethod : uint8
{
	/** Update from the current position to a specified position (including triggering events), using the current player status */
	Play,
	/** Jump to a specified position (without triggering events in between), using the current player status */
	Jump,
	/** Jump to a specified position, temporarily using EMovieScenePlayerStatus::Scrubbing */
	Scrub,
};


/**
 * Properties that are broadcast from server->clients for time/state synchronization
 */
USTRUCT()
struct FMovieSceneSequenceReplProperties
{
	GENERATED_BODY()

	FMovieSceneSequenceReplProperties()
		: LastKnownStatus(EMovieScenePlayerStatus::Stopped)
		, LastKnownNumLoops(0)
		, LastKnownSerialNumber(0)
	{}

	/** The last known position of the sequence on the server */
	UPROPERTY()
	FFrameTime LastKnownPosition;

	/** The last known playback status of the sequence on the server */
	UPROPERTY()
	TEnumAsByte<EMovieScenePlayerStatus::Type> LastKnownStatus;

	/** The last known number of loops of the sequence on the server */
	UPROPERTY()
	int32 LastKnownNumLoops;

	/** The last known serial number on the server */
	UPROPERTY()
	int32 LastKnownSerialNumber;
};


UENUM(BlueprintType)
enum class EMovieScenePositionType : uint8
{
	Frame,
	Time,
	MarkedFrame,
	Timecode
};

USTRUCT(BlueprintType)
struct FMovieSceneSequencePlaybackParams
{
	GENERATED_BODY()

	FMovieSceneSequencePlaybackParams()
		: Time(0.f)
		, PositionType(EMovieScenePositionType::Frame)
		, UpdateMethod(EUpdatePositionMethod::Play)
		, bHasJumped(false)
	{}

	FMovieSceneSequencePlaybackParams(FFrameTime InFrame, EUpdatePositionMethod InUpdateMethod)
		: Frame(InFrame)
		, Time(0.f)
		, PositionType(EMovieScenePositionType::Frame)
		, UpdateMethod(InUpdateMethod)
		, bHasJumped(false)
	{}

	FMovieSceneSequencePlaybackParams(float InTime, EUpdatePositionMethod InUpdateMethod)
		: Time(InTime)
		, PositionType(EMovieScenePositionType::Time)
		, UpdateMethod(InUpdateMethod)
		, bHasJumped(false)
	{}

	FMovieSceneSequencePlaybackParams(const FString& InMarkedFrame, EUpdatePositionMethod InUpdateMethod)
		: Time(0.f)
		, MarkedFrame(InMarkedFrame)
		, PositionType(EMovieScenePositionType::MarkedFrame)
		, UpdateMethod(InUpdateMethod)
		, bHasJumped(false)
	{}

	FMovieSceneSequencePlaybackParams(const FTimecode& InTimecode, EUpdatePositionMethod InUpdateMethod)
		: Time(0.f)
		, Timecode(InTimecode)
		, PositionType(EMovieScenePositionType::Timecode)
		, UpdateMethod(InUpdateMethod)
		, bHasJumped(false)
	{}

	// Get the playback position using the player's tick resolution and display rate	
	MOVIESCENE_API FFrameTime GetPlaybackPosition(UMovieSceneSequencePlayer* Player) const;

	// Get the playback position using the sequence's tick resolution and display rate
	MOVIESCENE_API FFrameTime GetPlaybackPosition(UMovieSceneSequence* Sequence) const;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Cinematic", meta=(EditCondition="PositionType == EMovieScenePositionType::Frame"))
	FFrameTime Frame;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Cinematic", meta=(EditCondition="PositionType == EMovieScenePositionType::Time", unit=s))
	float Time;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Cinematic", meta=(EditCondition="PositionType == EMovieScenePositionType::MarkedFrame"))
	FString MarkedFrame;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Cinematic", meta=(EditCondition="PositionType == EMovieScenePositionType::Timecode"))
	FTimecode Timecode;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Cinematic")
	EMovieScenePositionType PositionType;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Cinematic")
	EUpdatePositionMethod UpdateMethod;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Cinematic")
	bool bHasJumped;
};

USTRUCT(BlueprintType)
struct FMovieSceneSequencePlayToParams
{
	GENERATED_BODY()

	/** Should the PlayTo time be considered exclusive? Defaults to true as end frames in Sequencer are exclusive by default. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Cinematic")
	bool bExclusive = true;
};

/**
 * Abstract class that provides consistent player behaviour for various animation players
 */
UCLASS(Abstract, BlueprintType, MinimalAPI)
class UMovieSceneSequencePlayer
	: public UObject
	, public IMovieScenePlayer
	, public IMovieSceneSequenceTickManagerClient
{
public:
	GENERATED_BODY()

	/** Obeserver interface used for controlling whether this sequence can be played. */
	UPROPERTY(replicated)
	TScriptInterface<IMovieSceneSequencePlayerObserver> Observer;

	MOVIESCENE_API UMovieSceneSequencePlayer(const FObjectInitializer&);
	MOVIESCENE_API virtual ~UMovieSceneSequencePlayer();

	/** Start playback forwards from the current time cursor position, using the current play rate. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	MOVIESCENE_API void Play();

	/** Reverse playback. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	MOVIESCENE_API void PlayReverse();

	/** Changes the direction of playback (go in reverse if it was going forward, or vice versa) */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	MOVIESCENE_API void ChangePlaybackDirection();

	/**
	 * Start playback from the current time cursor position, looping the specified number of times.
	 * @param NumLoops - The number of loops to play. -1 indicates infinite looping.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	MOVIESCENE_API void PlayLooping(int32 NumLoops = -1);
	
	/** Pause playback. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	MOVIESCENE_API void Pause();
	
	/** Scrub playback. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	MOVIESCENE_API void Scrub();

	/** Stop playback and move the cursor to the end (or start, for reversed playback) of the sequence. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	MOVIESCENE_API void Stop();

	/** Stop playback without moving the cursor. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	MOVIESCENE_API void StopAtCurrentTime();

	/** Go to end and stop. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player", meta = (ToolTip = "Go to end of the sequence and stop. Adheres to 'When Finished' section rules."))
	MOVIESCENE_API void GoToEndAndStop();

public:

	/**
	 * Get the current playback position
	 * @return The current playback position
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	MOVIESCENE_API FQualifiedFrameTime GetCurrentTime() const;

	/**
	 * Get the total duration of the sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	MOVIESCENE_API FQualifiedFrameTime GetDuration() const;

	/**
	 * Get this sequence's duration in frames
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	MOVIESCENE_API int32 GetFrameDuration() const;

	/**
	 * Get this sequence's display rate.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	FFrameRate GetFrameRate() const { return PlayPosition.GetInputRate(); }

	/**
	 * Set the frame-rate that this player should play with, making all frame numbers in the specified time-space
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	MOVIESCENE_API void SetFrameRate(FFrameRate FrameRate);

	/**
	 * Get the offset within the level sequence to start playing
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	FQualifiedFrameTime GetStartTime() const { return FQualifiedFrameTime(StartTime, PlayPosition.GetInputRate()); }

	/**
	 * Get the offset within the level sequence to finish playing
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	FQualifiedFrameTime GetEndTime() const { return FQualifiedFrameTime(StartTime + DurationFrames, PlayPosition.GetInputRate()); }

	/**
	 * Set a manual weight to be multiplied with all blendable elements within this sequence
	 * @note: It is recommended that a weight between 0 and 1 is supplied, though this is not enforced
	 * @note: It is recommended that either FMovieSceneSequencePlaybackSettings::DynamicWeighting should be true for this player or the asset it's playing back should be set to enable dynamic weight to avoid undesirable behavior
	 *
	 * @param InWeight    The weight to suuply to all elements in this sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	MOVIESCENE_API void SetWeight(double InWeight);

	/**
	 * Removes a previously assigned weight
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	MOVIESCENE_API void RemoveWeight();

	/**
	 * Set a manual weight to be multiplied with all blendable elements within the specified sequence
	 * @note: It is recommended that a weight between 0 and 1 is supplied, though this is not enforced
	 * @note: It is recommended that either FMovieSceneSequencePlaybackSettings::DynamicWeighting should be true for this player or the asset it's playing back should be set to enable dynamic weight to avoid undesirable behavior
	 *
	 * @param InWeight    The weight to suuply to all elements in this sequence
	 */
	MOVIESCENE_API void SetWeight(double InWeight, FMovieSceneSequenceID SequenceID);

	/**
	 * Removes a previously assigned weight
	 */
	MOVIESCENE_API void RemoveWeight(FMovieSceneSequenceID SequenceID);

public:

	/**
	 * Set the valid play range for this sequence, determined by a starting frame number (in this sequence player's plaback frame), and a number of frames duration
	 *
	 * @param StartFrame      The frame number to start playing back the sequence
	 * @param Duration        The number of frames to play
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player", DisplayName="Set Play Range (Frames)")
	MOVIESCENE_API void SetFrameRange( int32 StartFrame, int32 Duration, float SubFrames = 0.f );

	/**
	 * Set the valid play range for this sequence, determined by a starting time  and a duration (in seconds)
	 *
	 * @param StartTime       The time to start playing back the sequence in seconds
	 * @param Duration        The length to play for
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player", DisplayName="Set Play Range (Seconds)")
	MOVIESCENE_API void SetTimeRange( float StartTime, float Duration );

public:

	/**
	 * Play from the current position to the requested position and pause. If requested position is before the current position, 
	 * playback will be reversed. Playback to the requested position will be cancelled if Stop() or Pause() is invoked during this 
	 * playback.
	 *
	 * @param PlaybackParams The position settings (ie. the position to play to)
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	MOVIESCENE_API void PlayTo(FMovieSceneSequencePlaybackParams PlaybackParams, FMovieSceneSequencePlayToParams PlayToParams);

	/**
	 * Set the current time of the player by evaluating from the current time to the specified time, as if the sequence is playing. 
	 * Triggers events that lie within the evaluated range. Does not alter the persistent playback status of the player (IsPlaying).
	 *
	 * @param PlaybackParams The position settings (ie. the position to set playback to)
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	MOVIESCENE_API void SetPlaybackPosition(FMovieSceneSequencePlaybackParams PlaybackParams);

	/**
	 * Restore any changes made by this player to their original state
	 */
	UFUNCTION(BlueprintCallable, Category = "Game|Cinematic")
	MOVIESCENE_API void RestoreState();

	/** Set the state of the completion mode override. Note, setting the state to force restore state will only take effect if the sequence hasn't started playing */
	UFUNCTION(BlueprintCallable, Category = "Game|Cinematic")
	MOVIESCENE_API void SetCompletionModeOverride(EMovieSceneCompletionModeOverride CompletionModeOverride);

	/** Get the state of the completion mode override */
	UFUNCTION(BlueprintCallable, Category = "Game|Cinematic")
	MOVIESCENE_API EMovieSceneCompletionModeOverride GetCompletionModeOverride() const;

public:

	/** Check whether the sequence is actively playing. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	MOVIESCENE_API bool IsPlaying() const;

	/** Check whether the sequence is paused. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	MOVIESCENE_API bool IsPaused() const;

	/** Check whether playback is reversed. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	MOVIESCENE_API bool IsReversed() const;

	/** Get the playback rate of this player. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	MOVIESCENE_API float GetPlayRate() const;

	/**
	 * Set the playback rate of this player. Negative values will play the animation in reverse.
	 * @param PlayRate - The new rate of playback for the animation.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	MOVIESCENE_API void SetPlayRate(float PlayRate);

	/** Get if the hud is hidden during play. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	MOVIESCENE_API bool GetHideHud() const;

	/**
	 * Set if hiding the hud during play.
	 * @param HideHud - The new value of Hide Hud during play.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	MOVIESCENE_API void SetHideHud(bool HideHud);

	/** Set whether to disable camera cuts */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	void SetDisableCameraCuts(bool bInDisableCameraCuts) { PlaybackSettings.bDisableCameraCuts = bInDisableCameraCuts; }

	/** Set whether to disable camera cuts */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	bool GetDisableCameraCuts() { return PlaybackSettings.bDisableCameraCuts; }

	/** An event that is broadcast each time this level sequence player is updated */
	DECLARE_EVENT_ThreeParams( UMovieSceneSequencePlayer, FOnMovieSceneSequencePlayerUpdated, const UMovieSceneSequencePlayer&, FFrameTime /*current time*/, FFrameTime /*previous time*/ );
	FOnMovieSceneSequencePlayerUpdated& OnSequenceUpdated() const { return OnMovieSceneSequencePlayerUpdate; }

	/** Event triggered when the level sequence player is played */
	UPROPERTY(BlueprintAssignable, Category = "Sequencer|Player")
	FOnMovieSceneSequencePlayerEvent OnPlay;

	/** Event triggered when the level sequence player is played in reverse */
	UPROPERTY(BlueprintAssignable, Category = "Sequencer|Player")
	FOnMovieSceneSequencePlayerEvent OnPlayReverse;

	/** Event triggered when the level sequence player is stopped */
	UPROPERTY(BlueprintAssignable, Category = "Sequencer|Player")
	FOnMovieSceneSequencePlayerEvent OnStop;

	/** Event triggered when the level sequence player is paused */
	UPROPERTY(BlueprintAssignable, Category = "Sequencer|Player")
	FOnMovieSceneSequencePlayerEvent OnPause;

	/** Event triggered when the level sequence player finishes naturally (without explicitly calling stop) */
	UPROPERTY(BlueprintAssignable, Category = "Sequencer|Player")
	FOnMovieSceneSequencePlayerEvent OnFinished;

	/** Native event triggered when the level sequence player finishes naturally (without explicitly calling stop) */
	FOnMovieSceneSequencePlayerNativeEvent OnNativeFinished;

public:

	/** Retrieve all objects currently bound to the specified binding identifier */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	MOVIESCENE_API TArray<UObject*> GetBoundObjects(FMovieSceneObjectBindingID ObjectBinding);

	/** Get the object bindings for the requested object */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	MOVIESCENE_API TArray<FMovieSceneObjectBindingID> GetObjectBindings(UObject* InObject);

	/* Invalidates the given binding, forcing it to be refetched. This may be useful for some custom bindings that wish their resolution code to be called again.*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	MOVIESCENE_API void RequestInvalidateBinding(FMovieSceneObjectBindingID ObjectBinding);

public:

	/** Ensure that this player's tick manager is set up correctly for the specified context */
	MOVIESCENE_API void InitializeForTick(UObject* Context);

	/** Assign this player's playback settings */
	MOVIESCENE_API void SetPlaybackSettings(const FMovieSceneSequencePlaybackSettings& InSettings);

	/** Initialize this player using its existing playback settings */
	MOVIESCENE_API void Initialize(UMovieSceneSequence* InSequence);

	/** Initialize this player with a sequence and some settings */
	MOVIESCENE_API void Initialize(UMovieSceneSequence* InSequence, const FMovieSceneSequencePlaybackSettings& InSettings);

	/** Update the sequence for the current time, if playing */
	MOVIESCENE_API void Update(const float DeltaSeconds);

	/** Update the sequence for the current time, if playing, asynchronously */
	MOVIESCENE_API void UpdateAsync(const float DeltaSeconds);

	/** Perform any tear-down work when this player is no longer (and will never) be needed */
	MOVIESCENE_API void TearDown();

	/** Returns whether this player is valid, i.e. it has been initialized and not torn down yet */
	MOVIESCENE_API bool IsValid() const;


public:

	/**
	 * Access the sequence this player is playing
	 * @return the sequence currently assigned to this player
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	UMovieSceneSequence* GetSequence() const { return Sequence; }

	/**
	 * Get the name of the sequence this player is playing
	 * @param bAddClientInfo  If true, add client index if running as a client
	 * @return the name of the sequence, or None if no sequence is set
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	MOVIESCENE_API FString GetSequenceName(bool bAddClientInfo = false) const;

	/**
	 * Access this player's tick manager
	 */
	UMovieSceneSequenceTickManager* GetTickManager() const { return TickManager; }

	/**
	 * Assign a playback client interface for this sequence player, defining instance data and binding overrides
	 */
	MOVIESCENE_API void SetPlaybackClient(TScriptInterface<IMovieScenePlaybackClient> InPlaybackClient);

	/**
	 * Retrieve the currently assigned time controller
	 */
	MOVIESCENE_API TSharedPtr<FMovieSceneTimeController> GetTimeController() const;

	/**
	 * Assign a time controller for this sequence player allowing custom time management implementations.
	 * Will reset the supplied time controller to the current time.
	 */
	MOVIESCENE_API void SetTimeController(TSharedPtr<FMovieSceneTimeController> InTimeController);
	

	/**
	 * Assign a time controller for this sequence player allowing custom time management implementations.
	 * Will not reset the supplied time controller in any way, so the sequence will receive its time directly from the controller.
	 */
	MOVIESCENE_API void SetTimeControllerDirectly(TSharedPtr<FMovieSceneTimeController> InTimeController);

	/**
	 * Sets whether to listen or ignore playback replication events.
	 * @param bState If true, ignores playback replication.
	 */
	MOVIESCENE_API void SetIgnorePlaybackReplication(bool bState);

protected:

	MOVIESCENE_API void PlayInternal();
	MOVIESCENE_API void StopInternal(FFrameTime TimeToResetTo);
	MOVIESCENE_API void FinishPlaybackInternal(FFrameTime TimeToFinishAt);

	struct FMovieSceneUpdateArgs
	{
		bool bHasJumped = false;
		bool bHasLooped = false;
		bool bIsAsync = false;
	};

	MOVIESCENE_API void UpdateMovieSceneInstance(FMovieSceneEvaluationRange InRange, EMovieScenePlayerStatus::Type PlayerStatus, bool bHasJumped = false, bool bHasLooped = false);
	MOVIESCENE_API virtual void UpdateMovieSceneInstance(FMovieSceneEvaluationRange InRange, EMovieScenePlayerStatus::Type PlayerStatus, const FMovieSceneUpdateArgs& Args);

	MOVIESCENE_API void UpdateTimeCursorPosition(FFrameTime NewPosition, EUpdatePositionMethod Method, bool bHasJumpedOverride = false);
	MOVIESCENE_API bool ShouldStopOrLoop(FFrameTime NewPosition) const;
	/** 
	* If the current sequence should pause (due to NewPosition overshooting a previously set ShouldPause) 
	* then a range of time that should be evaluated to reach there will be returned, in tick resolution.
	* If we should not pause then the TOptional will be unset.
	* */
	MOVIESCENE_API TOptional<TRange<FFrameTime>> GetPauseRange(const FFrameTime& NewPosition) const;
	/**
	 * Same as GetPauseRange but returns the next time to update to (the upper or lower bound value of
	 * the range returned by GetPauseRange, depending on the playback direction), in tick resolution.
	 */
	MOVIESCENE_API TOptional<FFrameTime> GetPauseTimeForNewPosition(const FFrameTime& NewPosition) const;

	MOVIESCENE_API UWorld* GetPlaybackWorld() const;

	MOVIESCENE_API FFrameTime GetLastValidTime() const;

	MOVIESCENE_API FFrameRate GetDisplayRate() const;

	MOVIESCENE_API bool NeedsQueueLatentAction() const;
	MOVIESCENE_API void QueueLatentAction(FMovieSceneSequenceLatentActionDelegate Delegate);
	MOVIESCENE_API void RunLatentActions();

public:
	//~ IMovieScenePlayer interface
	virtual FMovieSceneRootEvaluationTemplateInstance& GetEvaluationTemplate() override { return RootTemplateInstance; }

protected:
	//~ IMovieScenePlayer interface
	MOVIESCENE_API virtual UMovieSceneEntitySystemLinker* ConstructEntitySystemLinker() override;
	MOVIESCENE_API virtual EMovieScenePlayerStatus::Type GetPlaybackStatus() const override;
	MOVIESCENE_API virtual FMovieSceneSpawnRegister& GetSpawnRegister() override;
	virtual UObject* AsUObject() override { return this; }

	virtual void SetPlaybackStatus(EMovieScenePlayerStatus::Type InPlaybackStatus) override {}
	virtual void SetViewportSettings(const TMap<FViewportClient*, EMovieSceneViewportParams>& ViewportParamsMap) override {}
	virtual void GetViewportSettings(TMap<FViewportClient*, EMovieSceneViewportParams>& ViewportParamsMap) const override {}

	MOVIESCENE_API virtual void ResolveBoundObjects(UE::UniversalObjectLocator::FResolveParams& ResolveParams, const FGuid& InBindingId, FMovieSceneSequenceID SequenceID, UMovieSceneSequence& Sequence, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const override;
	virtual IMovieScenePlaybackClient* GetPlaybackClient() override { return PlaybackClient ? &*PlaybackClient : nullptr; }
	MOVIESCENE_API virtual bool HasDynamicWeighting() const override;
	MOVIESCENE_API virtual void PreEvaluation(const FMovieSceneContext& Context) override;
	MOVIESCENE_API virtual void PostEvaluation(const FMovieSceneContext& Context) override;

	virtual TScriptInterface<IMovieSceneSequencePlayerObserver> GetObserver() override { return Observer; }

	/*~ Begin UObject interface */
	virtual bool IsSupportedForNetworking() const { return true; }
	MOVIESCENE_API virtual int32 GetFunctionCallspace(UFunction* Function, FFrame* Stack) override;
	MOVIESCENE_API virtual bool CallRemoteFunction(UFunction* Function, void* Parameters, FOutParmRec* OutParms, FFrame* Stack) override;
	MOVIESCENE_API virtual void PostNetReceive() override;
	MOVIESCENE_API virtual void BeginDestroy() override;
	MOVIESCENE_API virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;
	/*~ End UObject interface */

	//~ Begin IMovieSceneSequenceTickManagerClient interface
	MOVIESCENE_API virtual void TickFromSequenceTickManager(float DeltaSeconds, FMovieSceneEntitySystemRunner* Runner) override;
	//~ End IMovieSceneSequenceTickManagerClient interface

protected:

	virtual bool CanPlay() const { return true; }
	virtual void OnStartedPlaying() {}
	virtual void OnLooped() {}
	virtual void OnPaused() {}
	virtual void OnStopped() {}
	
private:
	MOVIESCENE_API void StartTimeControllerAndBroadcastPlayState();

	MOVIESCENE_API void UpdateTimeCursorPosition_Internal(FFrameTime NewPosition, EUpdatePositionMethod Method, bool bHasJumpedOverride);

	MOVIESCENE_API void RunPreEvaluationCallbacks();
	MOVIESCENE_API void RunPostEvaluationCallbacks();

	void IncrementServerSerialNumber();
	void AdvanceClientSerialNumberTo(int32 NewSerialNumber);
	
private:

	/**
	 * Called on the server whenever an explicit change in time has occurred through one of the (Play|Jump|Scrub)To methods
	 */
	UFUNCTION(netmulticast, reliable)
	MOVIESCENE_API void RPC_ExplicitServerUpdateEvent(EUpdatePositionMethod Method, FFrameTime RelevantTime, int32 NewSerialNumber);

	/**
	 * Called on the server when Stop() is called in order to differentiate Stops from Pauses.
	 */
	UFUNCTION(netmulticast, reliable)
	MOVIESCENE_API void RPC_OnStopEvent(FFrameTime StoppedTime, int32 NewSerialNumber);

	/**
	 * Called on the server when playback has reached the end. Could lead to stopping or pausing.
	 */
	UFUNCTION(netmulticast, reliable)
	MOVIESCENE_API void RPC_OnFinishPlaybackEvent(FFrameTime StoppedTime, int32 NewSerialNumber);

	/**
	 * Check whether this sequence player is an authority, as determined by its outer Actor
	 */
	MOVIESCENE_API bool HasAuthority() const;

	/**
	 * Update the replicated properties required for synchronizing to clients of this sequence player
	 */
	MOVIESCENE_API void UpdateNetworkSyncProperties();

	/**
	 * Analyse the set of samples we have estimating the server time if we have confidence over the data.
	 * Should only be called once per frame.
	 * @return An estimation of the server time, or the current local time if we cannot make a strong estimate
	 */
	MOVIESCENE_API FFrameTime UpdateServerTimeSamples();

	/**
	 * Check and correct network synchronization for the clients of this sequence player.
	 */
	MOVIESCENE_API void UpdateNetworkSync();

	/**
	 * Compute the latency for the client connection.
	 */
	MOVIESCENE_API float GetPing() const;

protected:

	/** Movie player status. */
	UPROPERTY()
	TEnumAsByte<EMovieScenePlayerStatus::Type> Status;

	/** Whether we're currently playing in reverse. */
	UPROPERTY(replicated)
	uint32 bReversePlayback : 1;

	/** Set to true to invoke OnStartedPlaying on first update tick for started playing */
	uint32 bPendingOnStartedPlaying : 1;

	enum class ETimeControllerState
	{
		ReadyToPlay,
		PreparingToPlay
	};
	ETimeControllerState TimeControllerState;

	/** Set to true when the player is currently in the main level update */
	uint32 bIsAsyncUpdate : 1;

	/** Flag that allows the player to tick its time controller without actually evaluating the sequence */
	uint32 bSkipNextUpdate : 1;

	/** Flag that notifies the player to check network synchronization on next update */
	uint32 bUpdateNetSync : 1;

	/** Flag that indicates whether to warn on zero duration playback */
	uint32 bWarnZeroDuration : 1;

	/** The sequence to play back */
	UPROPERTY(transient)
	TObjectPtr<UMovieSceneSequence> Sequence;

	/** Time (in playback frames) at which to start playing the sequence (defaults to the lower bound of the sequence's play range) */
	UPROPERTY(replicated)
	FFrameNumber StartTime;

	/** Time (in playback frames) at which to stop playing the sequence (defaults to the upper bound of the sequence's play range) */
	UPROPERTY(replicated)
	int32 DurationFrames;

	UPROPERTY(replicated)
	float DurationSubFrames;

	/** The number of times we have looped in the current playback */
	UPROPERTY(transient)
	int32 CurrentNumLoops;

	/**
	 * The serial number for the current update lifespan
	 * It is incremented every time we pass a "gate" such as an RPC call that stops/finishes the sequence.
	 */
	UPROPERTY(transient)
	int32 SerialNumber;

	/** Specific playback settings for the animation. */
	UPROPERTY(replicated)
	FMovieSceneSequencePlaybackSettings PlaybackSettings;

	/** Cached playback settings for restoring when playback stops. This is used for temporary playback overrides, e.g. PlayLooping **/
	TOptional<FMovieSceneSequencePlaybackSettings> CachedPlaybackSettings;

	/** The root template instance we're evaluating */
	UPROPERTY(transient)
	FMovieSceneRootEvaluationTemplateInstance RootTemplateInstance;

	/** Usually nullptr, but will be set when we are updating inside a TickFromSequenceTickManager call */
	FMovieSceneEntitySystemRunner* CurrentRunner;

	/** Play position helper */
	FMovieScenePlaybackPosition PlayPosition;

	/** Spawn register */
	TSharedPtr<FMovieSceneSpawnRegister> SpawnRegister;

	/** Sequence Weights */
	TUniquePtr<UE::MovieScene::FSequenceWeights> SequenceWeights;

	struct FServerTimeSample
	{
		/** The actual server sequence time in seconds, with client ping at the time of the sample baked in */
		double ServerTime;
		/** Wall-clock time that the sample was receieved */
		double ReceivedTime;
	};
	/**
	 * Array of server sequence times in seconds, with ping compensation baked in.
	 * Samples are sorted chronologically with the oldest samples first
	 */
	TArray<FServerTimeSample> ServerTimeSamples;

	/*
	* On UpdateServerTimeSamples, the last recorded time dilation. Used to update the server time samples each update to ensure we can smooth server time even on changing time dilation.
	*/
	float LastEffectiveTimeDilation = 1.0f;

	/** Replicated playback status and current time that are replicated to clients */
	UPROPERTY(replicated)
	FMovieSceneSequenceReplProperties NetSyncProps;

	/** External client pointer in charge of playing back this sequence */
	UPROPERTY(Transient)
	TScriptInterface<IMovieScenePlaybackClient> PlaybackClient;

	/** Global tick manager, held here to keep it alive while world sequences are in play */
	UPROPERTY(transient)
	TObjectPtr<UMovieSceneSequenceTickManager> TickManager;

	/** Local latent action manager for when we're running a blocking sequence */
	FMovieSceneLatentActionManager LatentActionManager;

	/** (Optional) Externally supplied time controller */
	TSharedPtr<FMovieSceneTimeController> TimeController;

	/** (Optional) Synchronous runner to use when no tick manager is in use */
	TSharedPtr<FMovieSceneEntitySystemRunner> SynchronousRunner;

	/** When true, ignore playback replication events. */
	bool bIgnorePlaybackReplication = false;

private:

	/** The event that will be broadcast every time the sequence is updated */
	mutable FOnMovieSceneSequencePlayerUpdated OnMovieSceneSequencePlayerUpdate;

	/** The tick interval we are currently registered with (if any) */
	TOptional<FMovieSceneSequenceTickInterval> RegisteredTickInterval;

	/** The maximum tick rate prior to playing (used for overriding delta time during playback). */
	TOptional<double> OldMaxTickRate;

	/** Whether dynamic resolution frame time budget is being overridden. */
	bool bOverridingDynResFrameTimeBudget = false;

	/**
	* The last world game time at which we were ticked. Game time used is dependent on bTickEvenWhenPaused
	* Valid only if we've been ticked at least once since having a tick interval
	*/
	TOptional<float> LastTickGameTimeSeconds;

	struct FPauseOnArgs
	{
		FFrameTime Time;
		bool bExclusive;
	};

	/** If set, pause playback on this frame */
	TOptional<FPauseOnArgs> PauseOnFrame;

	/** Pre and post evaluation callbacks, for async evaluations */
	DECLARE_DELEGATE(FOnEvaluationCallback);
	TArray<FOnEvaluationCallback> PreEvaluationCallbacks;
	TArray<FOnEvaluationCallback> PostEvaluationCallbacks;
};
