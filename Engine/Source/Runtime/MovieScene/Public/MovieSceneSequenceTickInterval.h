// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectMacros.h"

#include "MovieSceneSequenceTickInterval.generated.h"

class AActor;
class UActorComponent;
class UObject;

/**
 * Structure defining a concrete tick interval for a Sequencer based evaluation
 */
USTRUCT(BlueprintType)
struct FMovieSceneSequenceTickInterval
{
	GENERATED_BODY()

	FMovieSceneSequenceTickInterval() = default;

	/**
	 * Generate a tick interval from an actor's primary tick function
	 */
	MOVIESCENE_API FMovieSceneSequenceTickInterval(const AActor* InActor);

	/**
	 * Generate a tick interval from an component's primary tick function
	 */
	MOVIESCENE_API FMovieSceneSequenceTickInterval(const UActorComponent* InActorComponent);

public:

	/** Defines the rate at which the sequence should update, in seconds */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Playback", meta=(DisplayName="Tick Interval", Units=s))
	float TickIntervalSeconds = 0.f;

	/** Defines an approximate budget for evaluation of this sequence (and any other sequences with the same tick interval) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Playback", meta=(DisplayName="Evaluation Budget", ForceUnits=us))
	float EvaluationBudgetMicroseconds = 0.f;

	/** When true, the sequence will continue to tick and progress even when the world is paused */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Playback")
	bool bTickWhenPaused = false;

	/** When true, allow the sequence to be grouped with other sequences based on Sequencer.TickIntervalGroupingResolutionMs. Otherwise the interval will be used precisely. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Playback")
	bool bAllowRounding = true;

public:

	/**
	 * Round this interval to the nearest Sequencer.TickIntervalGroupingResolutionMs milliseconds
	 */
	MOVIESCENE_API int32 RoundTickIntervalMs() const;

	/**
	 * Resolve this tick interval within the specified context object (usually a movie scene player)
	 * inheriting properties from the first valid parent if possible
	 */
	static MOVIESCENE_API FMovieSceneSequenceTickInterval GetInheritedInterval(UObject* ContextObject);

	/**
	 * Equality comparison operator
	 */
	friend bool operator==(const FMovieSceneSequenceTickInterval& A, const FMovieSceneSequenceTickInterval& B)
	{
		return A.TickIntervalSeconds == B.TickIntervalSeconds
			&& A.EvaluationBudgetMicroseconds == B.EvaluationBudgetMicroseconds
			&& A.bTickWhenPaused == B.bTickWhenPaused
			&& A.bAllowRounding == B.bAllowRounding;
	}

	/**
	 * Inequality comparison operator
	 */
	friend bool operator!=(const FMovieSceneSequenceTickInterval& A, const FMovieSceneSequenceTickInterval& B)
	{
		return !(A == B);
	}
};
