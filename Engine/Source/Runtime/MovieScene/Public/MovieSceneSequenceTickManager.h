// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "MovieSceneSequenceTickInterval.h"
#include "MovieSceneLatentActionManager.h"

#include "MovieSceneSequenceTickManager.generated.h"


class FMovieSceneEntitySystemRunner;
class UMovieSceneEntitySystemLinker;
class IMovieSceneSequenceTickManagerClient;

/**
 * Global (one per-UWorld) manager object that manages ticking and updating any and all Sequencer-based
 * evaluations for the current frame, before any other actors are ticked.
 *
 * Ticking clients are registered based on their desired tick interval, and grouped together with other
 * clients that tick with the same interval (based on Sequencer.TickIntervalGroupingResolutionMs).
 * 
 * Sequencer data is shared between all instances within the same group, allowing them to blend together.
 * Clients ticking at different intervals do not support blending with each other.
 */
UCLASS(MinimalAPI)
class UMovieSceneSequenceTickManager : public UObject
{
public:
	GENERATED_BODY()

	MOVIESCENE_API UMovieSceneSequenceTickManager(const FObjectInitializer& Init);

	/**
	 * Retrieve the tick manager for the specified playback context
	 */
	static MOVIESCENE_API UMovieSceneSequenceTickManager* Get(UObject* PlaybackContext);

	/**
	 * Register a new client to receive a tick at the specified tick interval.
	 * This client will be grouped with all other clients that tick within Sequencer.TickIntervalGroupingResolutionMs of the specified interval.
	 *
	 * @param TickInterval    The interval at which to tick the client. 
	 * @param InTickInterface The client that will receieve a tick
	 */
	MOVIESCENE_API void RegisterTickClient(const FMovieSceneSequenceTickInterval& TickInterval, TScriptInterface<IMovieSceneSequenceTickManagerClient> InTickInterface);

	/**
	 * Unregister a previously registered client. The client must have been registered or an assertion will fail.
	 *
	 * @param InTickInterface The client to unregister
	 */
	MOVIESCENE_API void UnregisterTickClient(TScriptInterface<IMovieSceneSequenceTickManagerClient> InTickInterface);

	/**
	 * Retrieve the linker associated with the specified (resolved) tick interval.
	 * @return The linker that owns all the data for the specified tick interval, or nullptr if there is none
	 */
	MOVIESCENE_API UMovieSceneEntitySystemLinker* GetLinker(const FMovieSceneSequenceTickInterval& TickInterval);

	/**
	 * Retrieve the runner associated with the specified (resolved) tick interval.
	 * @return The runner that flushes evaluations for the specified tick interval, or nullptr if there is none
	 */
	MOVIESCENE_API TSharedPtr<FMovieSceneEntitySystemRunner> GetRunner(const FMovieSceneSequenceTickInterval& TickInterval);

	/**
	 * Add an action that will be executed once all the pending evaluations in this tick manager have been flushed.
	 * @note Latent actions are destroyed once they are called.
	 *
	 * @param Delegate A delegate to invoke when all evaluations have been flushed.
	 */
	MOVIESCENE_API void AddLatentAction(FMovieSceneSequenceLatentActionDelegate Delegate);

	/**
	 * Execute any pending latent actions
	 */
	MOVIESCENE_API void RunLatentActions();

	/**
	 * Clear latent actions pertaining to the specified object
	 */
	MOVIESCENE_API void ClearLatentActions(UObject* Object);

public:

	static MOVIESCENE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

private:

	/*~ UObject interface */
	MOVIESCENE_API virtual void BeginDestroy() override;

private:

	/** 24 bytes: Information pertaining to a tickable client */
	struct FTickableClientData
	{
		/** 8 bytes: The actual interface ptr. Should only be used after validating against WeakObject. */
		IMovieSceneSequenceTickManagerClient* Interface;
		/** 8 bytes: A weak object ptr for the UObject implementing the interface. Only required for lifetime validation. */
		TWeakObjectPtr<> WeakObject;
		/** 4 bytes: The index into LinkerGroups for this client's linker (derived from its tick interval) */
		int32 LinkerIndex;
		/** 1 byte: Whether this client should tick when it is paused or not. Note that evaluation data will still be evaluated every frame, but the sequence will not progress. */
		bool bTickWhenPaused;
	};

	/** Information pertaining to a group of tickable clients */
	struct FLinkerGroup
	{
		/** The linker that owns all the entity data, systems and instances */
		TObjectPtr<UMovieSceneEntitySystemLinker> Linker;
		/** Runner responsible for evaluating each phase of the pipeline */
		TSharedPtr<FMovieSceneEntitySystemRunner> Runner;

		/** The tick interval of this group, rounded to the nearest Sequencer.TickIntervalGroupingResolutionMs */
		int32 RoundedTickIntervalMs = 0;

		/** The frame budget for all linkers within this linker group in milliseconds */
		float FrameBudgetMs = 0.f;

		/** The value of UWorld::GetUnpausedTimeSeconds last time this group was evaluated */
		float LastUnpausedTimeSeconds = -1;
		/** The value of UWorld::GetTimeSeconds last time this group was evaluated */
		float LastTimeSeconds = -1;

		/** The number of times this group has been starved of frames for budgeted evaluation */
		uint16 BudgetedStarvationCount = 0;

		/** Total number of clients in this group */
		uint16 NumClients = 0;
	};

	/** A pending registration operation */
	struct FPendingOperation
	{
		enum class EType
		{
			Register, Unregister
		};
		/** The interface to (un)register */
		TScriptInterface<IMovieSceneSequenceTickManagerClient> Interface;
		/** The object that we're registering (only used to ensure the interface is still valid) */
		TWeakObjectPtr<UObject> WeakObject;
		/** Tick interval for registration (not used for UnregisterTickClient calls) */
		FMovieSceneSequenceTickInterval TickInterval;
		/** The type of operation (Register or Unregister) */
		EType Type;
	};

private:

	/**
	 * Main update loop called in response to a world tick
	 */
	void TickSequenceActors(float DeltaSeconds);

	/**
	 * Flush all runners maintained by this tick manager.
	 * Should only be called when running latent actions.
	 */
	void FlushRunners();

	/**
	 * Process any pending registration operations
	 */
	void ProcessPendingOperations(TArrayView<const FPendingOperation> InOperations);

	/**
	 * Unregister tick client without performing basic checks
	 */
	void UnregisterTickClientImpl(IMovieSceneSequenceTickManagerClient* InClientInterface);

private:

	/** Flat array of tickable client data with indices into LinkerGroups */
	TArray<FTickableClientData> TickableClients;

	/** Sparse array of group data - indices into this array are stable */
	TSparseArray<FLinkerGroup> LinkerGroups;

	/** Delegate handle for the world tick delegate */
	FDelegateHandle WorldTickDelegateHandle;

	/** Pointer to an array of pending operations - nullptr everywhere except where mutations to TickableClients or LinkerGroups is disallowed */
	TArray<FPendingOperation>* PendingActorOperations;

	/** Latent action manager for managing actions that need performing after evaluation */
	FMovieSceneLatentActionManager LatentActionManager;
};
