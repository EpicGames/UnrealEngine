// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "MovieSceneSequenceTickManagerClient.generated.h"


class FMovieSceneEntitySystemRunner;

/**
 * Interface for objects that are to be ticked by the tick manager.
 */
UINTERFACE(MinimalAPI)
class UMovieSceneSequenceTickManagerClient
	: public UInterface
{
public:
	GENERATED_BODY()
};

class IMovieSceneSequenceTickManagerClient
{
public:
	GENERATED_BODY()

	virtual void TickFromSequenceTickManager(float DeltaSeconds, FMovieSceneEntitySystemRunner* Runner) = 0;
};
