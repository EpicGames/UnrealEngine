// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Evaluation/MovieSceneSequenceTransform.h"
#include "HAL/Platform.h"
#include "Math/Range.h"
#include "Misc/FrameNumber.h"
#include "MovieSceneSequenceID.h"

class UMovieSceneSection;
class UMovieSceneSequence;
class UMovieSceneSubSection;
class UMovieSceneTrack;
struct FGuid;
struct FMovieSceneBinding;

/*
* Implements a visitor pattern to allow external code to easily iterate through a sequence hierarchy,
* running custom code for each track, object, sub-section, or section visited. This handles skipping 
* over non-evaluated sub-scenes as well as accumulation of clamp ranges and sequence transforms as
* it iterates through the hierarchy. This does not currently support looping (only visits once, though
* FSubSequenceSpace::LocalClampRange will be the range of all the loops).
*
* Example:
*
*	using FCameraCutInfo = TTuple<UMovieSceneCameraCutSection*, FMovieSceneSequenceID, int16>;
*	struct FCameraCutVisitor : UE::MovieScene::ISequenceVisitor
*	{
*		virtual void VisitTrack(UMovieSceneTrack* InTrack, const FGuid&, const UE::MovieScene::FSubSequenceSpace& LocalSpace)
*		{
*			if (UMovieSceneCameraCutTrack* CameraCutTrack = Cast<UMovieSceneCameraCutTrack>(InTrack))
*			{
*				for (UMovieSceneSection* Section : CameraCutTrack->GetAllSections())
*				{
*					if (!Section->IsActive())
*					{
*						continue;
*					}
*	
*					UMovieSceneCameraCutSection* CameraCutSection = CastChecked<UMovieSceneCameraCutSection>(Section);
*					TRange<FFrameNumber> RootCameraRange = TRange<FFrameNumber>::Intersection(LocalSpace.RootClampRange, CameraCutSection->GetRange() * LocalSpace.RootToSequenceTransform.InverseNoLooping());
*					if (!RootCameraRange.IsEmpty())
*					{
*						CameraCutTree.Add(RootCameraRange, MakeTuple(CameraCutSection, LocalSpace.SequenceID, LocalSpace.HierarchicalBias));
*					}
*				}
*			}
*		}
*		TMovieSceneEvaluationTree<FCameraCutInfo> CameraCutTree;
*	};
*	
*	UE::MovieScene::FSequenceVisitParams Params;
*	Params.bVisitRootTracks = true;
*	Params.bVisitSubSequences = true;
*	FCameraCutVisitor CameraCutVisitor;
*	
*	// Visit all camera cuts
*	VisitSequence(InSequence, Params, CameraCutVisitor);
*/


namespace UE
{
namespace MovieScene
{

struct FSubSequenceSpace
{
	MOVIESCENE_API FSubSequenceSpace();

	/** Transform from the root time-space to the current sequence's time-space */
	FMovieSceneSequenceTransform RootToSequenceTransform;
#if WITH_EDITORONLY_DATA
	/** Transform from the root time-space to the current sequence's time-space without time warp */
	FMovieSceneSequenceTransform RootToUnwarpedLocalTransform;
#endif
	FMovieSceneTransformBreadcrumbs StartBreadcrumbs;
	FMovieSceneTransformBreadcrumbs EndBreadcrumbs;
	/** The ID of the sequence being compiled */
	FMovieSceneSequenceID SequenceID;
	/** A range to clamp the visited sequence to in the root's time-space */
	TRange<FFrameNumber> RootClampRange;
	/** A range to clamp the visited sequence to in the current sequence's time-space */
	TRange<FFrameNumber> LocalClampRange;
	/** HBias */
	int16 HierarchicalBias;
};

struct ISequenceVisitor
{
	virtual ~ISequenceVisitor() = default;

	virtual void VisitObjectBinding(const FMovieSceneBinding&, const FSubSequenceSpace& LocalSpace) {}

	virtual void VisitTrack(UMovieSceneTrack*, const FGuid&, const FSubSequenceSpace& LocalSpace) {}

	virtual void VisitSection(UMovieSceneTrack*, UMovieSceneSection*, const FGuid&, const FSubSequenceSpace& LocalSpace) {}

	virtual void VisitSubSequence(UMovieSceneSequence*, const FGuid&, const FSubSequenceSpace& LocalSpace) {}

	virtual void VisitStartSubSequence(UMovieSceneSubSection*, const FGuid&, const FSubSequenceSpace& LocalSpace) {}
	virtual void VisitEndSubSequence(UMovieSceneSubSection*, const FGuid&, const FSubSequenceSpace& LocalSpace) {}
};

struct FSequenceVisitParams
{
	FSequenceVisitParams()
		: bVisitRootTracks(false)
		, bVisitObjectBindings(false)
		, bVisitTracks(false)
		, bVisitSections(false)
		, bVisitDisabledSections(false)
		, bVisitSubSequences(false)
		, bVisitDisabledSubSequences(false)
	{}

	bool CanVisitTracksOrSections() const
	{
		return bVisitTracks || bVisitSections;
	}

	bool bVisitRootTracks;
	bool bVisitObjectBindings;
	bool bVisitTracks;
	bool bVisitSections;
	bool bVisitDisabledSections;
	bool bVisitSubSequences;
	bool bVisitDisabledSubSequences;
};


MOVIESCENE_API void VisitSequence(UMovieSceneSequence* Sequence, const FSequenceVisitParams& InParams, ISequenceVisitor& Visitor);


} // namespace MovieScene
} // namespace UE



