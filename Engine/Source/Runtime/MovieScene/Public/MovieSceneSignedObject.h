// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "EventHandlers/ISignedObjectEventHandler.h"
#include "EventHandlers/MovieSceneDataEventContainer.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneSignedObject.generated.h"

class ITransactionObjectAnnotation;
class UMovieSceneSignedObject;

namespace UE
{
namespace MovieScene
{
class ISignedObjectEventHandler;

struct IDeferredSignedObjectFlushSignal
{
	virtual ~IDeferredSignedObjectFlushSignal()= default;

	virtual void OnDeferredModifyFlush() = 0;
};

/**
 * Application-wide utility interface that allows for deferral of UMovieSceneSignedObject::MarkAsChanged calls
 */
struct IDeferredSignedObjectChangeHandler
{
	virtual ~IDeferredSignedObjectChangeHandler()= default;

	virtual void Flush() = 0;
	virtual void DeferMarkAsChanged(UMovieSceneSignedObject* SignedObject) = 0;
	virtual bool CreateImplicitScopedModifyDefer() = 0;
	virtual void ResetImplicitScopedModifyDefer() = 0;
};

struct FScopedSignedObjectModifyDefer
{
	MOVIESCENE_API FScopedSignedObjectModifyDefer(bool bInForceFlush = false);
	MOVIESCENE_API ~FScopedSignedObjectModifyDefer();

private:
	bool bForceFlush;
};

struct FScopedSignedObjectModifySuppress
{
	MOVIESCENE_API FScopedSignedObjectModifySuppress(bool bInWarnModifyCompiledData = false);
	MOVIESCENE_API ~FScopedSignedObjectModifySuppress();

private:
	bool bWarnModifyCompiledData;
};

} // namespace MovieScene
} // namespace UE



UCLASS(MinimalAPI)
class UMovieSceneSignedObject : public UObject
{
public:
	GENERATED_BODY()

	MOVIESCENE_API UMovieSceneSignedObject(const FObjectInitializer& Init);

	static MOVIESCENE_API TWeakPtr<UE::MovieScene::IDeferredSignedObjectChangeHandler> GetDeferredHandler();
	static MOVIESCENE_API void SetDeferredHandler(TWeakPtr<UE::MovieScene::IDeferredSignedObjectChangeHandler>&& InHandler);
	static MOVIESCENE_API void AddFlushSignal(TWeakPtr<UE::MovieScene::IDeferredSignedObjectFlushSignal> Signal);
	static MOVIESCENE_API void ResetImplicitScopedModifyDefer();

	/**
	 * Mark this object as having been changed in any way. Will regenerate this object's Signature
	 * and schedule an event to be broadcast to notify subscribers of the change
	 */
	MOVIESCENE_API void MarkAsChanged();

	/**
	 * Retrieve this object's signature that uniquely identifies its current state.
	 * Any change to this object will result in a new signature. This is a GUID and
	 * not a hash - equivalent class state does not generate identical signatures.
	 */
	const FGuid& GetSignature() const
	{
		return Signature;
	}

	/**
	 * Immediately broadcast events for this object being changed
	 */
	MOVIESCENE_API void BroadcastChanged();

	/** Event that is triggered whenever this object's signature has changed */
	DECLARE_EVENT(UMovieSceneSignedObject, FOnSignatureChanged)
	FOnSignatureChanged& OnSignatureChanged() { return OnSignatureChangedEvent; }

	/** Event that is triggered whenever this object's signature has changed */
	UE::MovieScene::TDataEventContainer<UE::MovieScene::ISignedObjectEventHandler> EventHandlers;

public:

	MOVIESCENE_API virtual void PostInitProperties() override;
	MOVIESCENE_API virtual void PostLoad() override;

#if WITH_EDITOR
	MOVIESCENE_API virtual bool Modify(bool bAlwaysMarkDirty = true) override;
	MOVIESCENE_API virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent) override;
	MOVIESCENE_API virtual void PostEditUndo() override;
	MOVIESCENE_API virtual void PostEditUndo(TSharedPtr<ITransactionObjectAnnotation> TransactionAnnotation) override;
#endif

private:

	/** Unique generation signature */
	UPROPERTY()
	FGuid Signature;

	/** Event that is triggered whenever this object's signature has changed */
	FOnSignatureChanged OnSignatureChangedEvent;
};
