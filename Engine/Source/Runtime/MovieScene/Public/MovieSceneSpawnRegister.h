// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "Evaluation/IMovieScenePlaybackCapability.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IMovieSceneObjectSpawner.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Guid.h"
#include "Misc/OptionalFwd.h"
#include "MovieSceneFwd.h"
#include "MovieSceneSequenceID.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/ValueOrError.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UClass;
class UObject;
template <typename FuncType> class TFunctionRef;

enum class ESpawnOwnership : uint8;

class IMovieScenePlayer;
class ISequencer;
class UActorFactory;
class UMovieScene;
class USequencerSettings;
struct FMovieSceneEvaluationState;
struct FMovieSceneSpawnable;
struct FTransformData;
class UMovieSceneSpawnableBindingBase;

namespace UE::MovieScene
{
	struct FSharedPlaybackState;
}  // namespace UE::MovieScene

/**
 * Class responsible for managing spawnables in a movie scene
 */
class FMovieSceneSpawnRegister : public TSharedFromThis<FMovieSceneSpawnRegister>
{
public:

	using FSharedPlaybackState = UE::MovieScene::FSharedPlaybackState;

	UE_DECLARE_MOVIESCENE_PLAYBACK_CAPABILITY_API(MOVIESCENE_API, FMovieSceneSpawnRegister)

	MOVIESCENE_API FMovieSceneSpawnRegister();
	MOVIESCENE_API FMovieSceneSpawnRegister(const FMovieSceneSpawnRegister&);

	/**
	 * Virtual destructor
	 */
	MOVIESCENE_API virtual ~FMovieSceneSpawnRegister();

	/**
	 * Attempt to find a previously spawned object represented by the specified object and template IDs
	 * @param BindingId		ID of the object to find
	 * @param TemplateID	Unique ID of the template to look within
	 * @param BindingIndex 	For level sequences using custom spawnable bindings, the index of the binding reference.
	 * @return The spawned object if found; nullptr otherwise.
	 * The version without the binding index is to support old clients of the previous API
	 */
	MOVIESCENE_API TWeakObjectPtr<> FindSpawnedObject(const FGuid& BindingId, FMovieSceneSequenceIDRef TemplateID, int BindingIndex) const;

	/**
	 * Spawn an object for the specified GUID, from the specified sequence instance.
	 *
	 * @param BindingId 	ID of the object to spawn
	 * @param TemplateID 	Identifier for the current template we're evaluating
	 * @param Player 		Movie scene player that is ultimately responsible for spawning the object
	 * @param BindingIndex 	For level sequences using custom spawnable bindings, the index of the binding reference.
	 * @return the spawned object, or nullptr on failure
	 * The version without the binding index is to support old clients of the previous API
	 */
	MOVIESCENE_API virtual UObject* SpawnObject(const FGuid& BindingId, UMovieScene& MovieScene, FMovieSceneSequenceIDRef Template, TSharedRef<const FSharedPlaybackState> SharedPlaybackState, int32 BindingIndex);

	/**
	 * Destroy a specific previously spawned object
	 *
	 * @param BindingId		ID of the object to destroy
	 * @param TemplateID 	Identifier for the current template we're evaluating
	 * @param Player 		Movie scene player that is ultimately responsible for destroying the object
	 * @param BindingIndex 	For level sequences using custom spawnable bindings, the index of the binding reference.
	 *
	 * @return True if an object was destroyed, false otherwise
	 * The version without the binding index is to support old clients of the previous API
	 */

	MOVIESCENE_API bool DestroySpawnedObject(const FGuid& BindingId, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState, int32 BindingIndex);

	/**
	 * Destroy a specific previously spawned object, where its binding ID and sequence ID is not known.
	 * @note Should only be used for restoring pre-animated state, or where it is otherwise impossible to call DestroySpawnedObject
	 *
	 * @param InObject		the object to destroy
	 */
	void DestroyObjectDirectly(UObject& InObject) { DestroySpawnedObject(InObject, nullptr); }

	/**
	 * Destroy spawned objects using a custom predicate
	 *
	 * @param Player 		Movie scene player that is ultimately responsible for destroying the objects
	 * @param Predicate		Predicate used for testing whether an object should be destroyed. Returns true for destruction, false to skip.
	 */
	MOVIESCENE_API void DestroyObjectsByPredicate(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, const TFunctionRef<bool(const FGuid&, ESpawnOwnership, FMovieSceneSequenceIDRef, int32)>& Predicate);

	/**
	 * Purge any memory of any objects that are considered externally owned

	 * @param Player 		Movie scene player that is ultimately responsible for destroying the objects
	 */
	MOVIESCENE_API void ForgetExternallyOwnedSpawnedObjects(TSharedRef<const FSharedPlaybackState> SharedPlaybackState);

public:

	/**
	 * Called to indiscriminately clean up any spawned objects
	 */
	MOVIESCENE_API void CleanUp(TSharedRef<const FSharedPlaybackState> SharedPlaybackState);

	/**
	 * Called to clean up any non-externally owned spawnables that were spawned from the specified instance
	 */
	MOVIESCENE_API void CleanUpSequence(FMovieSceneSequenceIDRef TemplateID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState);

	/**
	 * Called when the current time has moved beyond the specified sequence's play range
	 */
	MOVIESCENE_API void OnSequenceExpired(FMovieSceneSequenceIDRef TemplateID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState);

public:

	// Backwards compatible API, to be deprecated later

	TWeakObjectPtr<> FindSpawnedObject(const FGuid& BindingId, FMovieSceneSequenceIDRef TemplateID) const { return FindSpawnedObject(BindingId, TemplateID, 0); }
	virtual UObject* SpawnObject(const FGuid& BindingId, UMovieScene& MovieScene, FMovieSceneSequenceIDRef Template, TSharedRef<const FSharedPlaybackState> SharedPlaybackState) { return SpawnObject(BindingId, MovieScene, Template, SharedPlaybackState, 0); }
	bool DestroySpawnedObject(const FGuid& BindingId, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState) { return DestroySpawnedObject(BindingId, TemplateID, SharedPlaybackState, 0); }
	MOVIESCENE_API UObject* SpawnObject(const FGuid& BindingId, UMovieScene& MovieScene, FMovieSceneSequenceIDRef Template, IMovieScenePlayer& Player);
	MOVIESCENE_API bool DestroySpawnedObject(const FGuid& BindingId, FMovieSceneSequenceIDRef TemplateID, IMovieScenePlayer& Player);
	MOVIESCENE_API void DestroyObjectsByPredicate(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, const TFunctionRef<bool(const FGuid&, ESpawnOwnership, FMovieSceneSequenceIDRef)>& Predicate);
	MOVIESCENE_API void DestroyObjectsByPredicate(IMovieScenePlayer& Player, const TFunctionRef<bool(const FGuid&, ESpawnOwnership, FMovieSceneSequenceIDRef)>& Predicate);
	MOVIESCENE_API void ForgetExternallyOwnedSpawnedObjects(FMovieSceneEvaluationState& State, IMovieScenePlayer& Player);
	MOVIESCENE_API void CleanUp(IMovieScenePlayer& Player);
	MOVIESCENE_API void CleanUpSequence(FMovieSceneSequenceIDRef TemplateID, IMovieScenePlayer& Player);
	MOVIESCENE_API void OnSequenceExpired(FMovieSceneSequenceIDRef TemplateID, IMovieScenePlayer& Player);

#if WITH_EDITOR
	
public:

	/**
	 * Whether this class is supported as a spawnable
	 * @param InClass The class
	 * @return Whether this class is supported as a spawnable
	 */
	virtual bool CanSpawnObject(UClass* InClass) const { return false; }

	/**
	 * Create a new spawnable type from the given source object
	 *
	 * @param SourceObject		The source object to create the spawnable from
	 * @param OwnerMovieScene	The owner movie scene that this spawnable type should reside in
	 * @param ActorFactory      Optional actor factory to use to create spawnable type
	 * @return the new spawnable type, or error
	 */
	virtual TValueOrError<FNewSpawnable, FText> CreateNewSpawnableType(UObject& SourceObject, UMovieScene& OwnerMovieScene, UActorFactory* ActorFactory = nullptr) { return MakeError(NSLOCTEXT("SpawnRegister", "NotSupported", "Not supported")); }

	/**
	 * Called to save the default state of the specified spawnable
	 */
	virtual void SaveDefaultSpawnableState(const FGuid& Guid, int32 BindingIndex, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState) {}

	/**
	 * Setup a new spawnable object with some default tracks and keys
	 *
	 * @param	SpawnedObject	The newly spawned object. This may be NULL in the case of a spawnable that has not yet bneen spawned.
	 * @param	Guid			The ID of the spawnable to setup defaults for
	 * @param	TransformData	The transform of the object to be spawned. This will usually be valid in the case of converting a possessable to a spawnable.
	 * @param	Sequencer		The sequencer this spawnable was just created by
	 * @param	Settings		The settings for this sequencer
	 */
	virtual void SetupDefaultsForSpawnable(UObject* SpawnedObject, const FGuid& Guid, const TOptional<FTransformData>& TransformData, TSharedRef<ISequencer> Sequencer, USequencerSettings* Settings) {}

	/**
	 * Called to handle cleanup of objects when we convert a possessable to a spawnable object
	 *
	 * @param	OldObject			The old possessable object
	 * @param	Player		The current player
	 * @param	OutTransformData	Transform data that can be used to recreate objects in the same location
	 */
	virtual void HandleConvertPossessableToSpawnable(UObject* OldObject, TSharedRef<const FSharedPlaybackState> SharedPlaybackState, TOptional<FTransformData>& OutTransformData) {}

	/**
	 * Check whether the specified binding can become a Possessable.
	 * @param BindingId		ID of the object to destroy
	 * @param TemplateID 	Identifier for the current template we're evaluating
	 * @return whether the conversion from Spawnable to Possessable can occur.
	 */
	MOVIESCENE_API virtual bool CanConvertToPossessable(const FGuid& Guid, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState, int32 BindingIndex=0) const;

public:

	// Backwards compatible API, to be deprecated later
#if WITH_EDITOR
	virtual void SaveDefaultSpawnableState(const FGuid& Guid, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState) { SaveDefaultSpawnableState(Guid, 0, TemplateID, SharedPlaybackState); }
	MOVIESCENE_API virtual void SaveDefaultSpawnableState(FMovieSceneSpawnable& Spawnable, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState);
	MOVIESCENE_API void SaveDefaultSpawnableState(FMovieSceneSpawnable& Spawnable, FMovieSceneSequenceIDRef TemplateID, IMovieScenePlayer& Player);
#endif
	MOVIESCENE_API void HandleConvertPossessableToSpawnable(UObject* OldObject, IMovieScenePlayer& Player, TOptional<FTransformData>& OutTransformData);
	MOVIESCENE_API UObject* SpawnObject(FMovieSceneSpawnable& Spawnable, FMovieSceneSequenceIDRef TemplateID, IMovieScenePlayer& Player);
	virtual bool CanConvertSpawnableToPossessable(FMovieSceneSpawnable& Spawnable) const { return true; }

#endif

protected:

	/**
	 * Spawn an object for the specified GUID, from the specified sequence instance.
	 *
	 * @param Object 		ID of the object to spawn
	 * @param TemplateID 	Identifier for the current template we're evaluating
	 * @param Player 		Movie scene player that is ultimately responsible for spawning the object
	 * @return the spawned object, or nullptr on failure
	 */
	virtual UObject* SpawnObject(FMovieSceneSpawnable& Spawnable, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState) = 0;

	/**
	 * Called right before a spawned object with the specified ID and template ID is destroyed
	 */
	UE_DEPRECATED(5.5, "Please use the version of this with BindingIndex")
	MOVIESCENE_API virtual void PreDestroyObject(UObject& Object, const FGuid& BindingId, FMovieSceneSequenceIDRef TemplateID);

	virtual void PreDestroyObject(UObject& Object, const FGuid& BindingId, int32 BindingIndex, FMovieSceneSequenceIDRef TemplateID) {}

	UE_DEPRECATED(5.5, "Please use the version of this with CustomSpawnableBinding")
	virtual void DestroySpawnedObject(UObject& Object) {}

	/**
	 * Destroy a specific previously spawned object
	 *
	 * @param Object 			The object to destroy
	 * @param CustomSpawnableBinding Optional custom spawnable binding to handle the destroy.
	 */
	virtual void DestroySpawnedObject(UObject& Object, UMovieSceneSpawnableBindingBase* CustomSpawnableBinding) = 0;

protected:

	/** Structure holding information pertaining to a spawned object */
	struct FSpawnedObject
	{
		FSpawnedObject(const FGuid& InGuid, UObject& InObject, ESpawnOwnership InOwnership)
			: Guid(InGuid)
			, Object(&InObject)
			, Ownership(InOwnership)
		{}

		/** The ID of the sequencer object binding that this object relates to */
		FGuid Guid;

		/** The object that has been spawned */
		TWeakObjectPtr<UObject> Object;

		/** What level of ownership this object was spawned with */
		ESpawnOwnership Ownership;
	};

	/**
	 *	Helper key type for mapping a guid and sequence instance to a specific value
	 */
	struct FMovieSceneSpawnRegisterKey
	{
		FMovieSceneSpawnRegisterKey(FMovieSceneSequenceIDRef InTemplateID, const FGuid& InBindingId, int32 InBindingIndex)
			: BindingId(InBindingId)
			, TemplateID(InTemplateID)
			, BindingIndex(InBindingIndex)
		{
		}

		bool operator==(const FMovieSceneSpawnRegisterKey& Other) const
		{
			return BindingId == Other.BindingId && TemplateID == Other.TemplateID && BindingIndex == Other.BindingIndex;
		}
		
		friend uint32 GetTypeHash(const FMovieSceneSpawnRegisterKey& Key)
		{
			return HashCombine(HashCombine(GetTypeHash(Key.BindingId), GetTypeHash(Key.TemplateID)), GetTypeHash(Key.BindingIndex));
		}

		/** BindingId of the object binding */
		FGuid BindingId;

		/** Movie Scene template identifier that spawned the object */
		FMovieSceneSequenceID TemplateID;

		/* For level sequences using custom spawnable bindings, the index of the binding reference. */
		int32 BindingIndex = 0;
	};

protected:

	/** Register of spawned objects */
	TMap<FMovieSceneSpawnRegisterKey, FSpawnedObject> Register;

	/** True when cleaning ourselves up. Used to bypass marking a sequence dirty when objects are modified since we're cleaning ourselves up */
	bool bCleaningUp = false;
};

class FNullMovieSceneSpawnRegister : public FMovieSceneSpawnRegister
{
public:
	virtual UObject* SpawnObject(FMovieSceneSpawnable&, FMovieSceneSequenceIDRef, TSharedRef<const FSharedPlaybackState>) override { check(false); return nullptr; }
	virtual void DestroySpawnedObject(UObject&, UMovieSceneSpawnableBindingBase* CustomSpawnableBinding) override {}
};
