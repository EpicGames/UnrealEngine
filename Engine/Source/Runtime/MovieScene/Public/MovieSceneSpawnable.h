// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "MovieSceneDynamicBinding.h"

#include "MovieSceneSpawnable.generated.h"

class IMovieScenePlayer;
class UMovieSceneSequence;
struct FMovieSceneSequenceID;

namespace UE::MovieScene
{
	struct FSharedPlaybackState;
}

UENUM()
enum class ESpawnOwnership : uint8
{
	/** The object's lifetime is managed by the sequence that spawned it */
	
	InnerSequence UMETA(DisplayName = "This Sequence"),

	/** The object's lifetime is managed by the outermost sequence */
	RootSequence,

	/** Once spawned, the object's lifetime is managed externally. */
	External,
};

/**
 * MovieSceneSpawnable describes an object that can be spawned for this MovieScene
 */
USTRUCT()
struct FMovieSceneSpawnable
{
	GENERATED_BODY()

	FMovieSceneSpawnable()
		: bContinuouslyRespawn(false)
		, bNetAddressableName(false)
		, ObjectTemplate(nullptr)
		, Ownership(ESpawnOwnership::InnerSequence)
#if WITH_EDITORONLY_DATA
		, GeneratedClass_DEPRECATED(nullptr)
#endif
	{
	}

	/** FMovieSceneSpawnable initialization constructor */
	FMovieSceneSpawnable(const FString& InitName, UObject& InObjectTemplate)
		: bContinuouslyRespawn(false)
		, bNetAddressableName(false)
		, Guid(FGuid::NewGuid())
		, Name(InitName)
		, ObjectTemplate(&InObjectTemplate)
		, Ownership(ESpawnOwnership::InnerSequence)
#if WITH_EDITORONLY_DATA
		, GeneratedClass_DEPRECATED(nullptr)
#endif
	{
		MarkSpawnableTemplate(InObjectTemplate);
	}

public:

	/**
	 * Check if the specified object is a spawnable template
	 * @param InObject 		The object to test
	 * @return true if the specified object is a spawnable template, false otherwise
	 */
	MOVIESCENE_API static bool IsSpawnableTemplate(const UObject& InObject);

	/**
	 * Indicate that the specified object is a spawnable template object
	 * @param InObject 		The object to mark
	 */
	MOVIESCENE_API static void MarkSpawnableTemplate(const UObject& InObject);

	/**
	 * Get the template object for this spawnable
	 *
	 * @return Object template
	 * @see GetGuid, GetName
	 */
	UObject* GetObjectTemplate()
	{
		return ObjectTemplate;
	}

	/**
	 * Get the template object for this spawnable
	 *
	 * @return Object template
	 * @see GetGuid, GetName
	 */
	const UObject* GetObjectTemplate() const
	{
		return ObjectTemplate;
	}

	/**
	 * Set the object template to the specified object directly.
	 * Used for Copy/Paste, typically you should use CopyObjectTemplate.
	 */
	void SetObjectTemplate(UObject* InObjectTemplate)
	{
		checkf(InObjectTemplate == nullptr || !InObjectTemplate->HasAnyFlags(RF_ClassDefaultObject), TEXT("Setting CDOs as object templates is not supported. Please use the class directly."));
		ObjectTemplate = InObjectTemplate;
	}

	/**
	 * Copy the specified object into this spawnable's template
	 *
	 * @param InSourceObject The source object to use. This object will be duplicated into the spawnable.
	 * @param MovieSceneSequence The movie scene sequence to which this spawnable belongs
	 */
	MOVIESCENE_API void CopyObjectTemplate(UObject& InSourceObject, UMovieSceneSequence& MovieSceneSequence);

	/**
	 * Get the unique identifier of the spawnable object.
	 *
	 * @return Object GUID.
	 * @see GetClass, GetName
	 */
	const FGuid& GetGuid() const
	{
		return Guid;
	}

	/**
	 * Set the unique identifier of this spawnable object
	 * @param InGuid The new GUID for this spawnable
	 * @note Be careful - this guid may be referenced by spawnable/possessable child-parent relationships.
	 * @see GetGuid
	 */
	void SetGuid(const FGuid& InGuid)
	{
		Guid = InGuid;
	}

	/**
	 * Get the name of the spawnable object.
	 *
	 * @return Object name.
	 * @see GetClass, GetGuid
	 */
	const FString& GetName() const
	{
		return Name;
	}

	/**
	 * Set the name of the spawnable object.
	 *
	 * @InName The desired spawnable name.
	 */
	void SetName(const FString& InName)
	{
		Name = InName;
	}

	/**
	 * Report the specified GUID as being an inner possessable dependency for this spawnable
	 *
	 * @param PossessableGuid The guid pertaining to the inner possessable
	 */
	void AddChildPossessable(const FGuid& PossessableGuid)
	{
		ChildPossessables.AddUnique(PossessableGuid);
	}

	/**
	 * Remove the specified GUID from this spawnables list of dependent possessables
	 *
	 * @param PossessableGuid The guid pertaining to the inner possessable
	 */
	void RemoveChildPossessable(const FGuid& PossessableGuid)
	{
		ChildPossessables.Remove(PossessableGuid);
	}

	/**
	 * @return const access to the child possessables set
	 */
	const TArray<FGuid>& GetChildPossessables() const
	{
		return ChildPossessables;
	}

	/**
	 * Get a value indicating what is responsible for this object once it's spawned
	 */
	ESpawnOwnership GetSpawnOwnership() const
	{
		return Ownership;
	}

	/**
	 * Set a value indicating what is responsible for this object once it's spawned
	 */
	void SetSpawnOwnership(ESpawnOwnership InOwnership)
	{
		Ownership = InOwnership;
	}

	/** Optional spawn transform */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Actor)
	FTransform SpawnTransform;

	/**
	 * Get the name of the level to spawn into.
	 *
	 * @return Level name.
	 */
	FName GetLevelName() const
	{
		return LevelName;
	}

	/**
	 * Set the name of the level to spawn into.
	 *
	 * @InLevelName The desired spawnable level name.
	 */
	void SetLevelName(FName InLevelName)
	{
		LevelName = InLevelName;
	}

	/**
	 * Get the name to use for spawning this object into a networked level
	 */
	MOVIESCENE_API FName GetNetAddressableName(TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, FMovieSceneSequenceID SequenceID) const;

	UE_DEPRECATED(5.4, "Please use the FSharedPlaybackState version of this method")
	MOVIESCENE_API FName GetNetAddressableName(IMovieScenePlayer& Player, FMovieSceneSequenceID SequenceID) const;

	/**
	 * Automatically determine a value for bNetAddressableName based on the spawnable type
	 */
	MOVIESCENE_API void AutoSetNetAddressableName();

	/* For sorts and BinarySearch so we can search quickly by Guid */
	inline bool operator<(const FMovieSceneSpawnable& RHS) const { return Guid < RHS.Guid; }
	inline friend bool operator<(const FGuid& InGuid, const FMovieSceneSpawnable& RHS) { return InGuid < RHS.GetGuid(); }
	inline bool operator<(const FGuid& InGuid) const { return Guid < InGuid; }

	/** Array of tags that can be used for grouping and categorizing. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category=Actor)
	TArray<FName> Tags;

	/** When enabled, this spawnable will always be respawned if it gets destroyed externally. When disabled, this object will only ever be spawned once for each spawn key even if it is destroyed externally. */
	UPROPERTY(EditAnywhere, Category=Actor)
	bool bContinuouslyRespawn;

	/** When enabled, the actor will be spawned with a unique name so that it can be addressable between clients and servers. */
	UPROPERTY(EditAnywhere, Category=Actor)
	bool bNetAddressableName;

	// Property has been deprecated- 'Dynamic Bindings' are now handled by UMovieSceneReplaceableDirectorBlueprintBinding and the custom binding system
	UPROPERTY()
	FMovieSceneDynamicBinding DynamicBinding_DEPRECATED;

private:

	/** Unique identifier of the spawnable object. */
	// @todo sequencer: Guids need to be handled carefully when the asset is duplicated (or loaded after being copied on disk).
	//					Sometimes we'll need to generate fresh Guids.
	UPROPERTY()
	FGuid Guid;

	/** Name label */
	// @todo sequencer: Should be editor-only probably
	UPROPERTY()
	FString Name;

	UPROPERTY()
	TObjectPtr<UObject> ObjectTemplate;

	/** Set of GUIDs to possessable object bindings that are bound to an object inside this spawnable */
	// @todo sequencer: This should be a TSet, but they don't duplicate correctly atm
	UPROPERTY()
	TArray<FGuid> ChildPossessables;

	/** Property indicating where ownership responsibility for this object lies */
	UPROPERTY()
	ESpawnOwnership Ownership;

#if WITH_EDITORONLY_DATA
public:
	/** Deprecated generated class */
	UPROPERTY()
	TObjectPtr<UClass> GeneratedClass_DEPRECATED;
#endif

	/** Name of level to spawn into */
	UPROPERTY()
	FName LevelName;
};
