// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "MovieSceneSequence.h"
#include "MovieSceneSequenceID.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UMovieSceneSequence;
class UObject;

/**
 * An annotation that's added to spawned objects from movie scene spawnables
 */
struct FMovieSceneSpawnableAnnotation
{
	FMovieSceneSpawnableAnnotation()
	{}

	/**
	 * Add the annotation to the specified spawned object, allowing a back-reference to the sequence and binding ID
	 */
	MOVIESCENE_API static void Add(UObject* SpawnedObject, const FGuid& ObjectBindingID, FMovieSceneSequenceID SequenceID, UMovieSceneSequence* InOriginatingSequence);

	/**
	 * Attempt to find an annotation for the specified object
	 */
	MOVIESCENE_API static TOptional<FMovieSceneSpawnableAnnotation> Find(UObject* SpawnedObject);

	bool IsDefault() const
	{
		return !ObjectBindingID.IsValid();
	}

	/** ID of the object binding that spawned the object */
	FGuid ObjectBindingID;

	/** Sequence that contains the object binding that spawned the object */
	TWeakObjectPtr<UMovieSceneSequence> OriginatingSequence;

	/** The sequence ID that spawned this object */
	FMovieSceneSequenceID SequenceID;
};
