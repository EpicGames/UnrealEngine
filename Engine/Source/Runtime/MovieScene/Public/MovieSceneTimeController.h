// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/QualifiedFrameTime.h"
#include "MovieSceneFwd.h"
#include "Misc/Attribute.h"
#include "UObject/ObjectMacros.h"

class IMovieScenePlayer;

struct FMovieSceneTimeController
{
public:

	virtual ~FMovieSceneTimeController() = default;

	/**
	 * Called whenever a sequence starts or resumes playback from a non-playing state
	 */
	MOVIESCENE_API void PrepareToPlay(const FQualifiedFrameTime& InStartTime);

	/**
	 * Called whenever a sequence starts or resumes playback from a non-playing state
	*/
	MOVIESCENE_API bool IsReadyToPlay();

	/**
	 * Called whenever a sequence starts or resumes playback from a non-playing state
	 */
	MOVIESCENE_API void StartPlaying(const FQualifiedFrameTime& InStartTime);

	/**
	 * Called whenever a sequence stops playback
	 */
	MOVIESCENE_API void StopPlaying(const FQualifiedFrameTime& InStopTime);

	/**
	 * Ticks this controller
	 *
	 * @param DeltaSeconds     The tick delta in seconds, dilated by the current world settings global dilation
	 * @param InPlayRate       The current play rate of the sequence
	 */
	MOVIESCENE_API void Tick(float DeltaSeconds, float InPlayRate);

	/**
	 * Request the current time based on the specified existing time and play rate.
	 * Times should be returned in the same play rate as that specified by InCurrentTime
	 *
	 * @param InCurrentTime    The current time of the sequence
	 * @param InPlayRate       The current play rate of the sequence, multiplied by any world actor settings global dilation
	 */
	MOVIESCENE_API FFrameTime RequestCurrentTime(const FQualifiedFrameTime& InCurrentTime, float InPlayRate, FFrameRate InDisplayRate);

	/**
	 * Called when the status of the owning IMovieScenePlayer has changed
	 */
	MOVIESCENE_API void PlayerStatusChanged(EMovieScenePlayerStatus::Type InStatus, const FQualifiedFrameTime& InCurrentTime);

	/**
	 * Called to stop and resume playback from the specified time
	 */
	MOVIESCENE_API void Reset(const FQualifiedFrameTime& InNewStartTime);

protected:

	virtual void OnTick(float DeltaSeconds, float InPlayRate){}
	virtual void OnPrepareToPlay(const FQualifiedFrameTime& InStartTime) {}
	virtual bool OnIsReadyToPlay() { return true; }
	virtual void OnStartPlaying(const FQualifiedFrameTime& InStartTime){}
	virtual void OnStopPlaying(const FQualifiedFrameTime& InStopTime){}
	virtual FFrameTime OnRequestCurrentTime(const FQualifiedFrameTime& InCurrentTime, float InPlayRate) = 0;

protected:

	TOptional<FQualifiedFrameTime> GetPlaybackStartTime() const
	{
		return PlaybackStartTime;
	}

	FFrameRate GetDisplayRate() const
	{
		return DisplayRate;
	}

private:

	TOptional<FQualifiedFrameTime> PlaybackStartTime;

	FFrameRate DisplayRate;
};

/**
 * A timing manager that retrieves its time from an external clock source
 */
struct FMovieSceneTimeController_ExternalClock : FMovieSceneTimeController
{
protected:

	FMovieSceneTimeController_ExternalClock()
		: AccumulatedDilation(0.0)
	{}

	virtual ~FMovieSceneTimeController_ExternalClock() = default;

	virtual double GetCurrentTime() const = 0;

protected:

	MOVIESCENE_API virtual void OnStartPlaying(const FQualifiedFrameTime& InStartTime) override;
	MOVIESCENE_API virtual void OnStopPlaying(const FQualifiedFrameTime& InStopTime) override;
	MOVIESCENE_API virtual FFrameTime OnRequestCurrentTime(const FQualifiedFrameTime& InCurrentTime, float InPlayRate) override;

private:

	double AccumulatedDilation;

	TOptional<double> ClockStartTime;
	TOptional<double> ClockLastUpdateTime;
};

/**
 * A timing manager that retrieves its time from the platform clock
 */
struct FMovieSceneTimeController_PlatformClock : FMovieSceneTimeController_ExternalClock
{
	MOVIESCENE_API virtual double GetCurrentTime() const override;
};

/**
 * A timing manager that retrieves its time from the audio clock
 */
struct FMovieSceneTimeController_AudioClock : FMovieSceneTimeController_ExternalClock
{
	MOVIESCENE_API virtual double GetCurrentTime() const override;
};


/**
* A timing manager that retrieves its time relative to the Timecode clock
*/
struct FMovieSceneTimeController_RelativeTimecodeClock : FMovieSceneTimeController_ExternalClock
{
	MOVIESCENE_API virtual FFrameTime OnRequestCurrentTime(const FQualifiedFrameTime& InCurrentTime, float InPlayRate) override;
	MOVIESCENE_API virtual double GetCurrentTime() const override;
	MOVIESCENE_API virtual void OnStopPlaying(const FQualifiedFrameTime& InStopTime) override;

protected:

	TOptional<FFrameTime> LastCurrentFrameTime;
	TOptional<double> TimeSinceCurrentFrameTime;

	double AccumulatedFrameTime = 0.0;
};


/**
* A timing manager that retrieves its time from the Timecode clock
*/
struct FMovieSceneTimeController_TimecodeClock : FMovieSceneTimeController
{
	MOVIESCENE_API virtual FFrameTime OnRequestCurrentTime(const FQualifiedFrameTime& InCurrentTime, float InPlayRate) override;
};


/**
 * A timing manager that accumulates delta times from a world
 */
struct FMovieSceneTimeController_Tick : FMovieSceneTimeController
{
	FMovieSceneTimeController_Tick()
		: CurrentOffsetSeconds(0.0)
	{}

protected:

	MOVIESCENE_API virtual void OnTick(float DeltaSeconds, float InPlayRate) override;
	MOVIESCENE_API virtual void OnStartPlaying(const FQualifiedFrameTime& InStartTime) override;
	MOVIESCENE_API virtual FFrameTime OnRequestCurrentTime(const FQualifiedFrameTime& InCurrentTime, float InPlayRate) override;

private:
	double CurrentOffsetSeconds;
};

/**
 * A timing manager that plays every display frame for a certain number of seconds
 */
struct FMovieSceneTimeController_PlayEveryFrame : FMovieSceneTimeController
{
	FMovieSceneTimeController_PlayEveryFrame()
		: PreviousPlatformTime(0.f)
	{}

protected:

	MOVIESCENE_API virtual void OnStartPlaying(const FQualifiedFrameTime& InStartTime) override;
	MOVIESCENE_API virtual FFrameTime OnRequestCurrentTime(const FQualifiedFrameTime& InCurrentTime, float InPlayRate) override;

private:
	double PreviousPlatformTime;
	FFrameTime CurrentTime;
};



