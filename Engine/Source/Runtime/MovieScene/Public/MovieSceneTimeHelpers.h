// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Math/NumericLimits.h"
#include "Math/Range.h"
#include "Math/RangeBound.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameTime.h"

class UMovieScene;
struct FFrameRate;

namespace UE
{
namespace MovieScene
{

struct IRetimingInterface;

class TimeHelpers
{
public:

	/**
	 * Migrate the frame times of the movie scene from the source frame rate to the destination frame rate
	 */

	static MOVIESCENE_API void MigrateFrameTimes(FFrameRate SourceRate, FFrameRate DestinationRate, UMovieScene* MovieScene, bool bApplyRecursively = false);

	static MOVIESCENE_API void MigrateFrameTimes(const IRetimingInterface& Retimer, UMovieScene* MovieScene, bool bApplyRecursively = false);

};

/**
 * Return the first frame number included by the specified closed lower bound. For example, a bound of (0 would return 1, and [0 would return 0
 */
inline FFrameNumber DiscreteInclusiveLower(const TRangeBound<FFrameNumber>& InLowerBound)
{
	check(!InLowerBound.IsOpen());

	// Add one for exclusive lower bounds since they start on the next subsequent frame
	static const int32 Offsets[]   = { 0, 1 };
	const int32        OffsetIndex = (int32)InLowerBound.IsExclusive();

	return InLowerBound.GetValue() + Offsets[OffsetIndex];
}


/**
 * Return the first frame number included by the specified range. Assumes a closed lower bound. For example, a range of (0:10) would return 1, and [0:10] would return 0
 */
inline FFrameNumber DiscreteInclusiveLower(const TRange<FFrameNumber>& InRange)
{
	return DiscreteInclusiveLower(InRange.GetLowerBound());
}


/**
 * Return the first frame number that is not contained by the specified closed upper bound. For example, a bound of 10) would return 10, and 10] would return 11
 */
inline FFrameNumber DiscreteExclusiveUpper(const TRangeBound<FFrameNumber>& InUpperBound)
{
	check(!InUpperBound.IsOpen());

	// Add one for inclusive upper bounds since they finish on the next subsequent frame
	static const int32 Offsets[]   = { 0, 1 };
	const int32        OffsetIndex = (int32)InUpperBound.IsInclusive();

	return InUpperBound.GetValue() + Offsets[OffsetIndex];
}


/**
 * Return the first frame number not contained by the specified range. Assumes a closed upper bound. For example, a range of (0:10) would return 10, and [0:10] would return 11
 */
inline FFrameNumber DiscreteExclusiveUpper(const TRange<FFrameNumber>& InRange)
{
	return DiscreteExclusiveUpper(InRange.GetUpperBound());
}


/**
 * Make a new range that includes the given minimum and excludes the given maximum.
 *
 * @param MinInclusive The minimum value for the inclusive lower bound
 * @param MaxExclusive The maximum value for the exclusive upper bound
 * @return A new range.
 */
inline TRange<FFrameNumber> MakeDiscreteRange(FFrameNumber MinInclusive, FFrameNumber MaxExclusive)
{
	return TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(MinInclusive), TRangeBound<FFrameNumber>::Exclusive(MaxExclusive));
}


/**
 * Make a new range that includes both the lower and upper bounds of the given range.
 */
inline TRange<FFrameNumber> MakeHullRange(const TRange<FFrameNumber>& InRange)
{
	return TRange<FFrameNumber>(
			InRange.HasLowerBound() ? TRangeBound<FFrameNumber>::Inclusive(InRange.GetLowerBoundValue()) : TRangeBound<FFrameNumber>::Open(),
			InRange.HasUpperBound() ? TRangeBound<FFrameNumber>::Inclusive(InRange.GetUpperBoundValue()) : TRangeBound<FFrameNumber>::Open());
}


/**
 * Make a new range that includes both the given minimum and maximum.
 *
 * @param MinInclusive The minimum value for the inclusive lower bound
 * @param MaxInclusive The maximum value for the inclusive lower bound
 * @return A new range.
 */
inline TRange<FFrameNumber> MakeHullRange(FFrameNumber MinInclusive, FFrameNumber MaxInclusive)
{
	return TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(MinInclusive), TRangeBound<FFrameNumber>::Inclusive(MaxInclusive));
}


/**
 * Return whether the given range is empty or zero frames wide
 */
inline bool IsEmptyOrZeroSize(const TRange<FFrameNumber>& InRange)
{
	return InRange.IsEmpty() || InRange.Size<FFrameNumber>() == 0;
}


/**
 * Make a new range using the specified lower bound, and a given size.
 */
inline TRange<FFrameNumber> MakeDiscreteRangeFromLower(const TRangeBound<FFrameNumber>& InLowerBound, int32 DiscreteSize)
{
	check(!InLowerBound.IsOpen());

	// Add one for exclusive lower bounds to ensure we end up with a range of the correct discrete size
	static const int32 Offsets[]   = { 0, 1 };
	const int32        OffsetIndex = (int32)InLowerBound.IsExclusive();

	const FFrameNumber ExclusiveUpperValue = InLowerBound.GetValue() + DiscreteSize + Offsets[OffsetIndex];
	return TRange<FFrameNumber>(InLowerBound, TRangeBound<FFrameNumber>::Exclusive(ExclusiveUpperValue));
}


/**
 * Make a new range using the specified upper bound, and a given size.
 */
inline TRange<FFrameNumber> MakeDiscreteRangeFromUpper(const TRangeBound<FFrameNumber>& InUpperBound, int32 DiscreteSize)
{
	check(!InUpperBound.IsOpen());

	// Add one for inclusve upper bounds to ensure we end up with a range of the correct discrete size
	static const int32 Offsets[]   = { 0, 1 };
	const int32        OffsetIndex = (int32)InUpperBound.IsInclusive();

	const FFrameNumber InclusiveLowerValue = InUpperBound.GetValue() - DiscreteSize + Offsets[OffsetIndex];
	return TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(InclusiveLowerValue), InUpperBound);
}

/**
 * Calculate the size of a discrete frame range, taking into account inclusive/exclusive boundaries.
 * 
 * @param InRange       The range to calculate for. Must be a frinite range.
 * @return The size of the range (considering inclusive and exclusive boundaries)
 */
inline int32 DiscreteSize(const TRange<FFrameNumber>& InRange)
{
	return (int64)DiscreteExclusiveUpper(InRange).Value - (int64)DiscreteInclusiveLower(InRange).Value;
}

/**
 * Check whether the specified range contains any integer frame numbers or not
 */
inline bool DiscreteRangeIsEmpty(const TRange<FFrameNumber>& InRange)
{
	if (InRange.GetLowerBound().IsOpen() || InRange.GetUpperBound().IsOpen())
	{
		return false;
	}

	// From here on we're handling ranges of the form [x,y], [x,y), (x,y] and (x,y)
	const bool bLowerInclusive = InRange.GetLowerBound().IsInclusive();
	const bool bUpperInclusive = InRange.GetUpperBound().IsInclusive();

	if (bLowerInclusive)
	{
		// Lower is inclusive
		return bUpperInclusive
			? InRange.GetLowerBoundValue() >  InRange.GetUpperBoundValue()		// [x, y] - empty if x >  y
			: InRange.GetLowerBoundValue() >= InRange.GetUpperBoundValue();		// [x, y) - empty if x >= y
	}
	else
	{
		// Lower is exclusive
		return bUpperInclusive
			? InRange.GetLowerBoundValue() >= InRange.GetUpperBoundValue()		// (x, y] - empty if x >= y
			: InRange.GetLowerBoundValue() >= InRange.GetUpperBoundValue()-1;	// (x, y) - empty if x >= y-1
	}
}

/**
 * Dilate the specified range by adding a specific size to the lower and upper bounds (if closed)
 */
template<typename T>
inline TRange<T> DilateRange(const TRange<T>& InRange, T LowerAmount, T UpperAmount)
{
	TRangeBound<T> LowerBound = InRange.GetLowerBound();
	TRangeBound<T> UpperBound = InRange.GetUpperBound();

	return TRange<T>(
		LowerBound.IsOpen()
			? TRangeBound<T>::Open()
			: LowerBound.IsInclusive()
				? TRangeBound<T>::Inclusive(LowerBound.GetValue() + LowerAmount)
				: TRangeBound<T>::Exclusive(LowerBound.GetValue() + LowerAmount),

		UpperBound.IsOpen()
			? TRangeBound<T>::Open()
			: UpperBound.IsInclusive()
				? TRangeBound<T>::Inclusive(UpperBound.GetValue() + UpperAmount)
				: TRangeBound<T>::Exclusive(UpperBound.GetValue() + UpperAmount)
		);
}


/**
 * Expand the specified range by subtracting the specified amount from the lower bound, and adding it to the upper bound
 */
template<typename T>
inline TRange<T> ExpandRange(const TRange<T>& InRange, T Amount)
{
	return DilateRange(InRange, -Amount, Amount);
}


/**
 * Translate the specified range by adding the specified amount to both bounds.
 */
template<typename T>
inline TRange<T> TranslateRange(const TRange<T>& InRange, T Amount)
{
	return DilateRange(InRange, Amount, Amount);
}


/**
 * Clamp the specified time to a range
 */
inline FFrameTime ClampToDiscreteRange(FFrameTime InTime, const TRange<FFrameNumber>& InRange)
{
	FFrameTime MinTime = InRange.GetLowerBound().IsClosed() ? DiscreteInclusiveLower(InRange) : FFrameTime(TNumericLimits<int32>::Lowest());
	FFrameTime MaxTime = FFrameTime(InRange.GetUpperBound().IsClosed() ? DiscreteExclusiveUpper(InRange)-1 : TNumericLimits<int32>::Max(), 0.99999994f);

	return FMath::Clamp(InTime, MinTime, MaxTime);
}


/** 
 * Convert a frame number range into a frame time range
 */
inline TRange<FFrameTime> ConvertToFrameTimeRange(const TRange<FFrameNumber>& InRange)
{
	TRange<FFrameTime> Result;
	if (InRange.HasLowerBound())
	{
		if (InRange.GetLowerBound().IsInclusive())
		{
			Result.SetLowerBound(TRangeBound<FFrameTime>::Inclusive(InRange.GetLowerBoundValue()));
		}
		else
		{
			Result.SetLowerBound(TRangeBound<FFrameTime>::Exclusive(InRange.GetLowerBoundValue()));
		}
	}
	if (InRange.HasUpperBound())
	{
		if (InRange.GetUpperBound().IsInclusive())
		{
			Result.SetUpperBound(TRangeBound<FFrameTime>::Inclusive(InRange.GetUpperBoundValue()));
		}
		else
		{
			Result.SetUpperBound(TRangeBound<FFrameTime>::Exclusive(InRange.GetUpperBoundValue()));
		}
	}
	return Result;
}

inline TRange<FFrameTime> ConvertToFrameTimeRange(const TRange<float>& InRange)
{
	TRange<FFrameTime> Result;
	if (InRange.HasLowerBound())
	{
		if (InRange.GetLowerBound().IsInclusive())
		{
			Result.SetLowerBound(TRangeBound<FFrameTime>::Inclusive(FFrameTime::FromDecimal(InRange.GetLowerBoundValue())));
		}
		else
		{
			Result.SetLowerBound(TRangeBound<FFrameTime>::Exclusive(FFrameTime::FromDecimal(InRange.GetLowerBoundValue())));
		}
	}
	if (InRange.HasUpperBound())
	{
		if (InRange.GetUpperBound().IsInclusive())
		{
			Result.SetUpperBound(TRangeBound<FFrameTime>::Inclusive(FFrameTime::FromDecimal(InRange.GetUpperBoundValue())));
		}
		else
		{
			Result.SetUpperBound(TRangeBound<FFrameTime>::Exclusive(FFrameTime::FromDecimal(InRange.GetUpperBoundValue())));
		}
	}
	return Result;
}

inline TRange<FFrameNumber> ConvertToDiscreteRange(const TRange<FFrameTime>& InRange)
{
	TRange<FFrameNumber> Result;
	if (InRange.HasLowerBound())
	{
		// A frame time of (10.5 does not include _all_ of frame 10, so we can't include that frame
		FFrameTime LowerBound = InRange.GetLowerBoundValue();
		if (InRange.GetLowerBound().IsInclusive() && LowerBound.GetSubFrame() == 0.f)
		{
			Result.SetLowerBound(TRangeBound<FFrameNumber>::Inclusive(LowerBound.GetFrame()));
		}
		else
		{
			Result.SetLowerBound(TRangeBound<FFrameNumber>::Inclusive(LowerBound.GetFrame()+1));
		}
	}
	if (InRange.HasUpperBound())
	{
		FFrameTime UpperBound = InRange.GetUpperBoundValue();
		Result.SetUpperBound(TRangeBound<FFrameNumber>::Exclusive(UpperBound.GetFrame()));
	}
	return Result;
}


/**
 * Converts a range from one type of bounds to another. The output bounds type must be implicitly
 * constructable from the input bounds type.
 */
template<typename InBoundType, typename OutBoundType>
inline TRange<OutBoundType> ConvertRange(const TRange<InBoundType>& Range)
{
	const TRangeBound<InBoundType> SourceLower = Range.GetLowerBound();
	TRangeBound<OutBoundType> DestLower = SourceLower.IsOpen() ?
		TRangeBound<OutBoundType>() :
		SourceLower.IsInclusive() ?
			TRangeBound<OutBoundType>::Inclusive(SourceLower.GetValue()) :
			TRangeBound<OutBoundType>::Exclusive(SourceLower.GetValue());

	const TRangeBound<InBoundType> SourceUpper = Range.GetUpperBound();
	TRangeBound<OutBoundType> DestUpper = SourceUpper.IsOpen() ?
		TRangeBound<OutBoundType>() :
		SourceUpper.IsInclusive() ?
			TRangeBound<OutBoundType>::Inclusive(SourceUpper.GetValue()) :
			TRangeBound<OutBoundType>::Exclusive(SourceUpper.GetValue());

	return TRange<OutBoundType>(DestLower, DestUpper);
}

// Specialization of ConvertRange for round down FFrameTime to FFrameNumber.
template<>
inline TRange<FFrameNumber> ConvertRange(const TRange<FFrameTime>& Range)
{
	const TRangeBound<FFrameTime> SourceLower = Range.GetLowerBound();
	TRangeBound<FFrameNumber> DestLower = SourceLower.IsOpen() ?
		TRangeBound<FFrameNumber>() :
		SourceLower.IsInclusive() ?
			TRangeBound<FFrameNumber>::Inclusive(SourceLower.GetValue().FloorToFrame()) :
			TRangeBound<FFrameNumber>::Exclusive(SourceLower.GetValue().FloorToFrame());

	const TRangeBound<FFrameTime> SourceUpper = Range.GetUpperBound();
	TRangeBound<FFrameNumber> DestUpper = SourceUpper.IsOpen() ?
		TRangeBound<FFrameNumber>() :
		SourceUpper.IsInclusive() ?
			TRangeBound<FFrameNumber>::Inclusive(SourceUpper.GetValue().FloorToFrame()) :
			TRangeBound<FFrameNumber>::Exclusive(SourceUpper.GetValue().FloorToFrame());

	return TRange<FFrameNumber>(DestLower, DestUpper);
}

} // namespace MovieScene
} // namespace UE

inline FString LexToString(const TRange<FFrameNumber>& InRange)
{
	TRangeBound<FFrameNumber> SourceLower = InRange.GetLowerBound();
	TRangeBound<FFrameNumber> SourceUpper = InRange.GetUpperBound();

	return *FString::Printf(TEXT("%s,%s"),
		SourceLower.IsOpen() ?
		TEXT("[-inf") :
		SourceLower.IsInclusive() ?
		*FString::Printf(TEXT("[%i"), SourceLower.GetValue().Value) :
		*FString::Printf(TEXT("(%i"), SourceLower.GetValue().Value),

		SourceUpper.IsOpen() ?
		TEXT("+inf]") :
		SourceUpper.IsInclusive() ?
		*FString::Printf(TEXT("%i]"), SourceUpper.GetValue().Value) :
		*FString::Printf(TEXT("%i)"), SourceUpper.GetValue().Value)
	);
}

inline FString LexToString(const TRange<FFrameTime>& InRange)
{
	TRangeBound<FFrameTime> SourceLower = InRange.GetLowerBound();
	TRangeBound<FFrameTime> SourceUpper = InRange.GetUpperBound();

	return *FString::Printf(TEXT("%s,%s"),
		SourceLower.IsOpen() ?
		TEXT("[-inf") :
		SourceLower.IsInclusive() ?
		*FString::Printf(TEXT("[%s"), *LexToShortString(SourceLower.GetValue())) :
		*FString::Printf(TEXT("(%s"), *LexToShortString(SourceLower.GetValue())),

		SourceUpper.IsOpen() ?
		TEXT("+inf]") :
		SourceUpper.IsInclusive() ?
		*FString::Printf(TEXT("%s]"), *LexToShortString(SourceUpper.GetValue())) :
		*FString::Printf(TEXT("%s)"), *LexToShortString(SourceUpper.GetValue()))
	);
}

