// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneTimeUnit.generated.h"

/**
* Specifies which frame of reference you want to set/get time values in. This allows users to work
* in reference space without having to manually convert back and forth all of the time.
*/
UENUM(BlueprintType)
enum class EMovieSceneTimeUnit : uint8
{
	/** Display Rate matches the values shown in the UI such as 30fps giving you 30 frames per second. Supports sub-frame values (precision limited to Tick Resolution) */
	DisplayRate,
	/** Tick Resolution is the internal resolution that data is actually stored in, such as 24000 giving you 24,000 frames per second. This is the smallest interval that data can be stored on. */
	TickResolution
};
