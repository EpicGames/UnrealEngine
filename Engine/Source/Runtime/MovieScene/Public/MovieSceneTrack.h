// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compilation/MovieSceneSegmentCompiler.h"
#include "IMovieSceneTrackVirtualAPI.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Evaluation/Blending/MovieSceneBlendType.h"
#include "Evaluation/MovieSceneEvaluationField.h"
#include "Decorations/MovieSceneDecorationContainer.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Guid.h"
#include "Misc/InlineValue.h"
#include "MovieSceneSignedObject.h"
#include "MovieSceneTrackEvaluationField.h"
#include "Templates/SubclassOf.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "Conditions/MovieSceneCondition.h"

#if WITH_EDITOR
#include "Styling/SlateColor.h"
#endif

#include "MovieSceneTrack.generated.h"

class UMovieSceneSection;
class UObject;
struct FMovieSceneEvaluationTrack;
struct FMovieSceneTrackRowSegmentBlender;
struct FMovieSceneTrackSegmentBlender;
struct IMovieSceneTemplateGenerator;
struct FMovieSceneConditionContainer;
template<typename> struct TMovieSceneEvaluationTree;

/** Flags used to perform cook-time optimization of movie scene data */
enum class ECookOptimizationFlags
{
	/** Perform no cook optimization */
	None 			= 0,
	/** Remove this track since its of no consequence to runtime */
	RemoveTrack		= 1 << 0,
	/** Remove this track's object since its of no consequence to runtime */
	RemoveObject	= 1 << 1,
	/** Remove this section's object since its of no consequence to runtime */
	RemoveSection	= 1 << 2,
};
ENUM_CLASS_FLAGS(ECookOptimizationFlags)

/** Generic evaluation options for any track */
USTRUCT()
struct FMovieSceneTrackEvalOptions
{
	GENERATED_BODY()
	
	FMovieSceneTrackEvalOptions()
		: bCanEvaluateNearestSection(false)
		, bEvalNearestSection(false)
		, bEvaluateInPreroll(false)
		, bEvaluateInPostroll(false)
		, bEvaluateNearestSection_DEPRECATED(false)
	{}

	/** true when the value of bEvalNearestSection is to be considered for the track */
	UPROPERTY()
	uint32 bCanEvaluateNearestSection : 1;

	/** When evaluating empty space on a track, will evaluate the last position of the previous section (if possible), or the first position of the next section, in that order of preference. */
	UPROPERTY(EditAnywhere, Category="Track", DisplayName="Evaluate Nearest Section", meta=(EditCondition=bCanEvaluateNearestSection, HideEditConditionToggle))
	uint32 bEvalNearestSection : 1;

	/** Evaluate this track as part of its parent sub-section's pre-roll, if applicable */
	UPROPERTY(EditAnywhere, Category="Track")
	uint32 bEvaluateInPreroll : 1;

	/** Evaluate this track as part of its parent sub-section's post-roll, if applicable */
	UPROPERTY(EditAnywhere, Category="Track")
	uint32 bEvaluateInPostroll : 1;

	UPROPERTY()
	uint32 bEvaluateNearestSection_DEPRECATED : 1;
};


/** Generic display options for any track */
USTRUCT()
struct FMovieSceneTrackDisplayOptions
{
	GENERATED_BODY()

	FMovieSceneTrackDisplayOptions()
		: bShowVerticalFrames(false)
	{}

	/** Show bounds as vertical frames */
	UPROPERTY(EditAnywhere, Category = "Track")
	uint32 bShowVerticalFrames : 1;
};

/** Returns what kind of section easing we support in the editor */
enum class EMovieSceneTrackEasingSupportFlags
{
	None = 0,
	AutomaticEaseIn = 1 << 0,
	AutomaticEaseOut = 1 << 1,
	ManualEaseIn = 1 << 2,
	ManualEaseOut = 1 << 3,
	AutomaticEasing = AutomaticEaseIn | AutomaticEaseOut,
	ManualEasing = ManualEaseIn | ManualEaseOut,
	All = AutomaticEasing | ManualEasing
};
ENUM_CLASS_FLAGS(EMovieSceneTrackEasingSupportFlags)

/** Parameters for the `SupportsEasing` method */
struct FMovieSceneSupportsEasingParams
{
	// Non-null if we are asking for a specific section.
	const UMovieSceneSection* ForSection;

	FMovieSceneSupportsEasingParams() : ForSection(nullptr) {}
	FMovieSceneSupportsEasingParams(const UMovieSceneSection* InSection) : ForSection(InSection) {}
};

/** Pre-compilation result */
struct FMovieSceneTrackPreCompileResult
{
	/**
	 * The default metadata that will be used for all entity provider sections found on the current track.
	 *
	 * A track can change this metadata by overriding PreCompile.
	 */
	FMovieSceneEvaluationFieldEntityMetaData DefaultMetaData;
};

#if WITH_EDITOR
/** Parameters for sections moving in the editor */
struct FMovieSceneSectionMovedParams
{
	EPropertyChangeType::Type MoveType;

	FMovieSceneSectionMovedParams(EPropertyChangeType::Type InMoveType) : MoveType(InMoveType) {}
};

/** Result of having moved sections in the editor */
enum class EMovieSceneSectionMovedResult
{
	/** Nothing significant has changed */
	None = 0,
	/** Sections have been added or removed, which requires a UI refresh */
	SectionsChanged = 1
};
ENUM_CLASS_FLAGS(EMovieSceneSectionMovedResult);

/** Parameters for helping to determine dynamic label color/tooltip*/
struct FMovieSceneLabelParams
{
	class IMovieScenePlayer* Player = nullptr;
	FGuid BindingID;
	FMovieSceneSequenceID SequenceID;
	bool bIsDimmed = false;
};
#endif

/* Metadata tied to a track row. */
USTRUCT()
struct FMovieSceneTrackRowMetadata
{
	GENERATED_BODY()
	
	/* Optional dynamic conditions tied to specific track rows. */
	UPROPERTY(EditAnywhere, Category="Track Row", meta=(ShowOnlyInnerProperties))
	FMovieSceneConditionContainer ConditionContainer;
};


// Disable some VS warnings that are emitted by Add/RemoveSection functions hiding the
//    private virtual methods in IMovieSceneTrackVirtualAPI
#if PLATFORM_COMPILER_CLANG
	#pragma clang diagnostic push
	#pragma clang diagnostic ignored "-Woverloaded-virtual"
#elif PLATFORM_MICROSOFT
	#pragma warning(push)
	#pragma warning(disable:4263)
	#pragma warning(disable:4264)
#else
	static_assert(false, "Unexpected compiler detected");
#endif

/**
 * Base class for a track in a Movie Scene
 */
UCLASS(abstract, DefaultToInstanced, MinimalAPI, BlueprintType)
class UMovieSceneTrack
	: public UMovieSceneDecorationContainerObject
	, public IMovieSceneTrackVirtualAPI
{
	GENERATED_BODY()

public:

	MOVIESCENE_API UMovieSceneTrack(const FObjectInitializer& InInitializer);
	~UMovieSceneTrack() = default;

public:

	/** General evaluation options for a given track */
	UPROPERTY(EditAnywhere, Category = "Track", meta = (ShowOnlyInnerProperties))
	FMovieSceneTrackEvalOptions EvalOptions;

#if WITH_EDITORONLY_DATA
	/** General display options for a given track */
	UPROPERTY(EditAnywhere, Category = "Track", meta = (ShowOnlyInnerProperties))
	FMovieSceneTrackDisplayOptions DisplayOptions;
#endif

	/* Parent track for this track. If a parent track is set, the track won't show up under the object binding. */
	UPROPERTY()
	TObjectPtr<UMovieSceneTrack> ParentTrack = nullptr;

	/** Optional dynamic condition for whether this track/any of the sections on this track evaluates at runtime. */
	UPROPERTY(EditAnywhere, Category = "Track")
	FMovieSceneConditionContainer ConditionContainer;

	UPROPERTY()
	bool bAutoExpandToKeyframes = true;

	/**
	 * Gets what kind of blending is supported by this section
	 */
	FMovieSceneBlendTypeField GetSupportedBlendTypes() const
	{
		return SupportedBlendTypes;
	}

	/** 
	 * Get compiler rules to use when compiling sections that overlap on the same row.
	 * These define how to deal with overlapping sections and empty space on a row
	 */
	MOVIESCENE_API virtual FMovieSceneTrackRowSegmentBlenderPtr GetRowSegmentBlender() const;

	/** 
	 * Get compiler rules to use when compiling sections that overlap on different rows.
	 * These define how to deal with overlapping sections and empty space at the track level
	 */
	MOVIESCENE_API virtual FMovieSceneTrackSegmentBlenderPtr GetTrackSegmentBlender() const;

	/**
	 * Update all auto-generated easing curves for all sections in this track
	 */
	MOVIESCENE_API virtual void UpdateEasing();


	void OnAddedToMovieScene(UMovieScene* MovieScene);
	void OnRemovedFromMovieScene();

protected:

	//~ UObject interface
	MOVIESCENE_API virtual void PostInitProperties() override;
	MOVIESCENE_API virtual void PostLoad() override;
	MOVIESCENE_API virtual bool IsPostLoadThreadSafe() const override;

	MOVIESCENE_API virtual void OnDecorationAdded(UObject* NewDecoration) override;
	MOVIESCENE_API virtual void OnDecorationRemoved(UObject* Decoration) override;

	/** Intentionally not a UPROPERTY so this isn't serialized */
	FMovieSceneBlendTypeField SupportedBlendTypes;

	/** Whether evaluation of this track has been disabled via mute/solo */
	UPROPERTY()
	bool bIsEvalDisabled;

	/** Which rows have been disabled via mute/solo */
	UPROPERTY()
	TArray<int32> RowsDisabled;

#if WITH_EDITORONLY_DATA

	/** Whether evaluation of this track has been disabled locally (not serialized with the asset) */
	UPROPERTY(Transient)
	bool bIsLocalEvalDisabled;

	/** Which rows have been disabled locally (not serialized with the asset) */
	UPROPERTY(Transient)
	TArray<int32> LocalRowsDisabled;

#endif

public:

	/**
	 * Run the pre-compilation step for this track.
	 * This method is called by the sequence compiler and is not meant to be called by 3rd party code.
	 */
	void PreCompile(FMovieSceneTrackPreCompileResult& OutPreCompileResult);

	/**
	 * Retrieve a fully up-to-date evaluation field for this track.
	 */
	MOVIESCENE_API const FMovieSceneTrackEvaluationField& GetEvaluationField();

	/**
	 * Retrieve a version number for the logic implemented in PopulateEvaluationTree.
	 *
	 * The evaluation field is cached with the data and is invalidated if the data signature changes, or
	 * if this version number changes. You should bump this version number if you start/stop overriding,
	 * or otherwise change the logic, of PopulateEvaluationTree.
	 */
	virtual int8 GetEvaluationFieldVersion() const { return 0; }

	MOVIESCENE_API FGuid FindObjectBindingGuid() const;

protected:

	enum class ETreePopulationMode : uint8
	{
		None,
		Blended,
		HighPass,
		HighPassPerRow,
	};

	ETreePopulationMode BuiltInTreePopulationMode;

protected:

	/** Forcibly update this evaluation tree without updating the signature. Does not invalidated any compiled data! */
	MOVIESCENE_API void ForceUpdateEvaluationTree();

private:

	/** Sub-classes can override this method to perforum custom evaluation tree population logic. */
	virtual bool PopulateEvaluationTree(TMovieSceneEvaluationTree<FMovieSceneTrackEvaluationData>& OutData) const
	{
		return false;
	}

	/** Sub-classes can override this method to perform custom pre-compilation logic. */
	virtual void PreCompileImpl(FMovieSceneTrackPreCompileResult& OutPreCompileResult)
	{
	}

	virtual void OnAddedToMovieSceneImpl(UMovieScene* InMovieScene)
	{
	}

	virtual void OnRemovedFromMovieSceneImpl()
	{
	}

private:

	void UpdateEvaluationTree();
	void AddSectionRangesToTree(TArrayView<UMovieSceneSection* const> Sections, TMovieSceneEvaluationTree<FMovieSceneTrackEvaluationData>& OutData);
	void AddSectionPrePostRollRangesToTree(TArrayView<UMovieSceneSection* const> Sections, TMovieSceneEvaluationTree<FMovieSceneTrackEvaluationData>& OutData);

	/** The guid of the object signature that the EvaluationField member relates to */
	UPROPERTY()
	FGuid EvaluationFieldGuid;

#if WITH_EDITORONLY_DATA
	/** The version of the logic in PopulateEvaluationTree when the EvaluationField was cached */
	UPROPERTY()
	int8 EvaluationFieldVersion;
#endif

	/** An array of entries that define when specific sections should be evaluated on this track */
	UPROPERTY()
	FMovieSceneTrackEvaluationField EvaluationField;

	/* Optional extra metadata tied to specific track rows. */
	UPROPERTY()
	TMap<int32, FMovieSceneTrackRowMetadata> TrackRowMetadata;

public:

	/**
	 * @return The name that makes this track unique from other track of the same class.
	 */
	virtual FName GetTrackName() const { return NAME_None; }

	/**
	 * @return Whether or not this track has any data in it.
	 */
	virtual bool IsEmpty() const PURE_VIRTUAL(UMovieSceneTrack::IsEmpty, return true;);

	/**
	 * Removes animation data.
	 */
	virtual void RemoveAllAnimationData() { }

	/**
	 * @return Whether or not this track supports multiple row indices.
	 */
	virtual bool SupportsMultipleRows() const
	{
		return SupportedBlendTypes.Num() != 0;
	}

	virtual EMovieSceneTrackEasingSupportFlags SupportsEasing(FMovieSceneSupportsEasingParams& Params) const
	{
		return SupportedBlendTypes.Num() > 0 ? EMovieSceneTrackEasingSupportFlags::All : EMovieSceneTrackEasingSupportFlags::None;
	}

	/** Set This Section as the one to key. If track doesn't support layered blends then don't implement
	 * @param InSection		Section that we want to key
	 */
	 virtual void SetSectionToKey(UMovieSceneSection* InSection) {};

	/** Get the section we want to key. If track doesn't support layered blends it will return nulltpr.
	 * @return The section to key if one exists
	 */
	 virtual UMovieSceneSection* GetSectionToKey() const { return nullptr; }

	/** Gets the greatest row index of all the sections owned by this track. */
	MOVIESCENE_API int32 GetMaxRowIndex() const;

	/**
	 * Updates the row indices of sections owned by this track so that all row indices which are used are consecutive with no gaps. 
	 * @return Whether or not fixes were made. 
	 */
	MOVIESCENE_API virtual bool FixRowIndices();

	/** Called when row indices have been fixed up */
	MOVIESCENE_API virtual void OnRowIndicesChanged(const TMap<int32, int32>& NewToOldRowIndices);

	/**
	 * @return Whether evaluation of this track should be disabled due to deactive setting
	 */
	MOVIESCENE_API bool IsEvalDisabled(const bool bInCheckLocal = true) const;
	MOVIESCENE_API bool IsRowEvalDisabled(const int32 InRowIndex, const bool bInCheckLocal = true) const;

	/**
	 * Called by Sequencer to set whether evaluation of this track should be disabled due to deactive setting
	 */
	void SetEvalDisabled(const bool bInEvalDisabled) { bIsEvalDisabled = bInEvalDisabled; }
	MOVIESCENE_API void SetRowEvalDisabled(const bool bInEvalDisabled, const int32 InRowIndex);

#if WITH_EDITOR

	/**
	 * @return Whether evaluation of this track should be disabled due to local muting/soloing (not serialized with the asset)
	 */
	bool IsLocalEvalDisabled() const { return bIsLocalEvalDisabled; }
	MOVIESCENE_API bool IsLocalRowEvalDisabled(const int32 InRowIndex) const;

	/**
	 * Called by Sequencer to set whether evaluation of this track should be disabled due to local muting/soloing (not serialized with the asset)
	 */
	void SetLocalEvalDisabled(const bool bInEvalDisabled) { bIsLocalEvalDisabled = bInEvalDisabled; }
	MOVIESCENE_API void SetLocalRowEvalDisabled(const bool bInEvalDisabled, int32 InRowIndex);

#endif

	/*
	 * Returns a pointer to optional track row metadata at the given RowIndex, or nullptr if none exists.
	 */
	MOVIESCENE_API const FMovieSceneTrackRowMetadata* FindTrackRowMetadata(int32 RowIndex) const;

	/*
	 * Returns a pointer to optional track row metadata at the given RowIndex, or nullptr if none exists;
	 */
	MOVIESCENE_API FMovieSceneTrackRowMetadata* FindTrackRowMetadata(int32 RowIndex);

	/*
	 * Returns a pointer to optional track row metadata at the given RowIndex, or nullptr if none exists;
	 */
	MOVIESCENE_API FMovieSceneTrackRowMetadata& FindOrAddTrackRowMetadata(int32 RowIndex);

public:

	/*
	 * Does this track support this section class type?

	 * @param ClassType The movie scene section class type
	 * @return Whether this track supports this section class type
	 */
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const PURE_VIRTUAL(UMovieSceneTrack::SupportsType, return false;);

	/** The decoration container associated with the given row, or null if this track has no per-row containers. */
	virtual UMovieSceneDecorationContainerObject* GetDecorationContainerForRow(int32 RowIndex) const { return nullptr; }

	// Parameter structs that only exist to differentiate overloads from IMovieSceneTrackVirtualAPI
	struct FSectionParameter
	{
		FSectionParameter(UMovieSceneSection& InSection) : Section(&InSection) {};
		UMovieSceneSection* Section;
	};
	struct FSectionIndexParameter
	{
		FSectionIndexParameter(int32 InSectionIndex) : SectionIndex(InSectionIndex) {};
		int32 SectionIndex;
	};

	/**
	 * Add a section to this track.
	 *
	 * @param Section The section to add.
	 */
	MOVIESCENE_API void AddSection(FSectionParameter Section);

	/**
	 * Removes a section from this track.
	 *
	 * @param Section The section to remove.
	 */
	MOVIESCENE_API void RemoveSection(FSectionParameter Section);

	/**
	 * Removes a section from this track at a particular index
	 *
	 * @param SectionIndex The section index to remove.
	 */
	MOVIESCENE_API void RemoveSectionAt(FSectionIndexParameter SectionIndex);

public:
	/**
	 * Generates a new section suitable for use with this track.
	 *
	 * @return a new section suitable for use with this track.
	 */
	virtual class UMovieSceneSection* CreateNewSection() PURE_VIRTUAL(UMovieSceneTrack::CreateNewSection, return nullptr;);
	
	/**
	 * Called when all the sections of the track need to be retrieved.
	 * 
	 * @return List of all the sections in the track.
	 */
	virtual const TArray<UMovieSceneSection*>& GetAllSections() const PURE_VIRTUAL(UMovieSceneTrack::GetAllSections, static TArray<UMovieSceneSection*> Empty; return Empty;);

	/**
	 * Checks to see if the section is in this track.
	 *
	 * @param Section The section to query for.
	 * @return True if the section is in this track.
	 */
	virtual bool HasSection(const UMovieSceneSection& Section) const PURE_VIRTUAL(UMovieSceneTrack::HasSection, return false;);

private:

	virtual void OnSectionAddedImpl(UMovieSceneSection* Section)
	{
	}

	virtual void OnSectionRemovedImpl(UMovieSceneSection* Section)
	{
	}

public:

#if WITH_EDITOR

	/**
	 * Called when this track's movie scene is being cooked to determine if/how this track should be cooked.
	 * @return ECookOptimizationFlags detailing how to optimize this track
	 */
	MOVIESCENE_API virtual ECookOptimizationFlags GetCookOptimizationFlags() const;

	/**
	 * Called when this track should be removed for cooking
	 */
	MOVIESCENE_API virtual void RemoveForCook();

	static bool RemoveMutedTracksOnCook();

#endif

#if WITH_EDITORONLY_DATA

	/**
	 * Get the track's display name.
	 *
	 * @return Display name text.
	 */
	virtual FText GetDisplayName() const PURE_VIRTUAL(UMovieSceneTrack::GetDisplayName, return FText::FromString(TEXT("Unnamed Track")););

	/**
	 * Get the track row's display name.
	 *
	 * @return Display name text.
	 */
	virtual FText GetTrackRowDisplayName(int32 RowIndex) const { return FText::FromString(TEXT("Unnamed Track")); }

	/**
	 * Get the track's display tooltip text to be shown on the track's name.
	 *
	 * @return Display tooltip text.
	 */
	virtual FText GetDisplayNameToolTipText(const FMovieSceneLabelParams& LabelParams) const { return FText::GetEmpty(); }

	/**
	 * Gets the track label's color.
	 * 
	 * @return Track label color.
	 */
	virtual FSlateColor GetLabelColor(const FMovieSceneLabelParams& LabelParams) const { return FSlateColor::UseForeground(); }

	/**
	 * Get this track's color tint.
	 *
	 * @return Color Tint.
	 */
	const FColor& GetColorTint() const
	{
		return TrackTint;
	}

	/**
	 * Set this track's color tint.
	 *
	 * @param InTrackTint The color to tint this track.
	 */
	void SetColorTint(const FColor& InTrackTint)
	{
		TrackTint = InTrackTint;
	}

	/**
	 * Get this folder's desired sorting order
	 */
	int32 GetSortingOrder() const
	{
		return SortingOrder;
	}

	/**
	 * Set this folder's desired sorting order.
	 *
	 * @param InSortingOrder The higher the value the further down the list the folder will be.
	 */
	void SetSortingOrder(const int32 InSortingOrder)
	{
		SortingOrder = InSortingOrder;
	}

	/**
	 * @return Whether or not this track supports the creation of default sections when the track is created.
	 */
	virtual bool SupportsDefaultSections() const
	{
		return bSupportsDefaultSections;
	}

	/**
	 * @return Whether or not this track supports conditions.
	 */
	virtual bool SupportsConditions() const
	{
		return bSupportsConditions;
	}

	/*
	* Returns an array of all conditions on track, track row, or section
	*/
	MOVIESCENE_API TArray<UMovieSceneCondition*> GetAllConditions();

protected:

	/** The object binding that this track resides within */
	UPROPERTY()
	FGuid ObjectBindingID;

	/** This track's tint color */
	UPROPERTY(EditAnywhere, Category="Track", DisplayName = Color)
	FColor TrackTint;

	/** This folder's desired sorting order */
	UPROPERTY()
	int32 SortingOrder;

	/** Does this track support the creation of a default section when created? */
	UPROPERTY()
	bool bSupportsDefaultSections;

	/** Does this track support conditions */
	UPROPERTY()
	bool bSupportsConditions;

public:
#endif

#if WITH_EDITOR
	/**
	 * Called if the section is moved in Sequencer.
	 *
	 * @param Section The section that moved.
	 */
	virtual EMovieSceneSectionMovedResult OnSectionMoved(UMovieSceneSection& Section, const FMovieSceneSectionMovedParams& Params)
	{
		return EMovieSceneSectionMovedResult::None;
	}

	#endif
};

#if PLATFORM_COMPILER_CLANG
	#pragma clang diagnostic pop
#elif PLATFORM_MICROSOFT
	#pragma warning(pop)
#endif
