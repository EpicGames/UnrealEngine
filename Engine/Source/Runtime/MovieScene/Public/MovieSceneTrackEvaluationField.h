// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "Evaluation/MovieSceneEvaluationTree.h"
#include "Evaluation/MovieSceneSegment.h"
#include "HAL/Platform.h"
#include "Math/NumericLimits.h"
#include "Math/Range.h"
#include "Misc/CoreDefines.h"
#include "Misc/FrameNumber.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "MovieSceneTrackEvaluationField.generated.h"

class UMovieSceneSection;
class UMovieSceneTrack;

USTRUCT()
struct FMovieSceneTrackEvaluationFieldEntry
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UMovieSceneSection> Section = nullptr;

	UPROPERTY()
	FFrameNumberRange Range;

	UPROPERTY()
	FFrameNumber ForcedTime;

	UPROPERTY()
	ESectionEvaluationFlags Flags = ESectionEvaluationFlags::None;

	UPROPERTY()
	int16 LegacySortOrder = 0;
};

USTRUCT()
struct FMovieSceneTrackEvaluationField
{
	GENERATED_BODY()

	void Reset(int32 NumExpected = 0)
	{
		Entries.Reset(NumExpected);
	}

	UPROPERTY()
	TArray<FMovieSceneTrackEvaluationFieldEntry>  Entries;
};

struct FMovieSceneTrackEvaluationData
{
	FMovieSceneTrackEvaluationData()
		: ForcedTime(TNumericLimits<int32>::Lowest())
		, SortOrder(0)
		, Flags(ESectionEvaluationFlags::None)
	{}

	MOVIESCENE_API static FMovieSceneTrackEvaluationData FromSection(UMovieSceneSection* InSection);

	MOVIESCENE_API static FMovieSceneTrackEvaluationData FromTrack(UMovieSceneTrack* InTrack);

	FMovieSceneTrackEvaluationData& SetFlags(ESectionEvaluationFlags InFlags)
	{
		Flags = InFlags;
		return *this;
	}

	FMovieSceneTrackEvaluationData& Sort(int16 InSortOrder)
	{
		SortOrder = InSortOrder;
		return *this;
	}

	TWeakObjectPtr<UMovieSceneTrack> Track;

	TWeakObjectPtr<UMovieSceneSection> Section;

	FFrameNumber ForcedTime;

	int16 SortOrder;

	ESectionEvaluationFlags Flags;
};
