// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/EnumClassFlags.h"

struct FMovieSceneTransformBreadcrumbs;

template<typename> struct TOptional;

namespace UE::MovieScene
{

/** Flags for controlling inverse evaluation */
enum class EInverseEvaluateFlags : uint8
{
	Forwards     = 1 << 0,
	Backwards    = 1 << 1,
	Equal        = 1 << 2,
	Cycle        = 1 << 3,
	IgnoreClamps = 1 << 4,

	AnyDirection = Forwards | Equal | Backwards,
};

ENUM_CLASS_FLAGS(EInverseEvaluateFlags)

/**
 * Parameter structure that controls time-transformation operations in MovieScene code
 */
struct FTransformTimeParams
{
	FTransformTimeParams()
		: Breadcrumbs(nullptr)
		, CycleCount(nullptr)
		, bIgnoreClamps(false)
	{}

	/**
	 * Instruct the algorithm to add breadcrumbs to the specified container as it performs the transformation
	 * @note: this function will empty the breadcrumbs
	 */
	MOVIESCENE_API FTransformTimeParams& HarvestBreadcrumbs(FMovieSceneTransformBreadcrumbs& OutBreadcrumbs);

	/**
	 * Instruct the algorithm to add breadcrumbs to the specified container as it performs the transformation
	 * @note: this function will not empty the breadcrumbs, allowing the callsite to append paths of breadcrumbs together
	 */
	MOVIESCENE_API FTransformTimeParams& AppendBreadcrumbs(FMovieSceneTransformBreadcrumbs& OutBreadcrumbs);

	/**
	 * Instruct the algorithm to track the cycle counts
	 */
	MOVIESCENE_API FTransformTimeParams& TrackCycleCounts(TOptional<int32>* OutCycleCounter);

	/**
	 * Ignore clamping operations while transforming the time.
	 */
	MOVIESCENE_API FTransformTimeParams& IgnoreClamps();

public:

	/** Breadcrumbs structure. When non-null, breadcrumbs will be added according to its FMovieSceneTransformBreadcrumbs::Mode */
	FMovieSceneTransformBreadcrumbs* Breadcrumbs;

	/** Pointer that receives the last encountered cycle count */
	TOptional<int32>* CycleCount;

	/** Whether to ignore clamp operations or not */
	uint8 bIgnoreClamps : 1;
};

/**
 * Parameter structure that controls inverse-time-transformation operations in MovieScene code
 */
struct FInverseTransformTimeParams
{
	FInverseTransformTimeParams()
		: Flags(EInverseEvaluateFlags::AnyDirection | EInverseEvaluateFlags::Cycle)
	{}

	FInverseTransformTimeParams(EInverseEvaluateFlags InFlags)
		: Flags(InFlags)
	{}

	EInverseEvaluateFlags Flags;
};


} // namespace UE::MovieScene