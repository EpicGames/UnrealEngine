// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneSection.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieSceneBindingLifetimeSection.generated.h"


/**
 * Binding lifetime section that will connect an object binding while active, and disconnect it afterwards.
 */
UCLASS(MinimalAPI)
class UMovieSceneBindingLifetimeSection
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
public:
	GENERATED_BODY()

	UMovieSceneBindingLifetimeSection(const FObjectInitializer& ObjInit);

	virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;
	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
	void ExternalPopulateEvaluationField(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder);
};


