// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneBoolChannel.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "MovieSceneSection.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneBoolSection.generated.h"

#define UE_API MOVIESCENE_API

class FArchive;
class UObject;

/**
 * A single bool section.
 */
UCLASS(MinimalAPI)
class UMovieSceneBoolSection 
	: public UMovieSceneSection
{
	GENERATED_UCLASS_BODY()

public:

	/** The default value to use when no keys are present - use GetCurve().SetDefaultValue() */
	UPROPERTY()
	bool DefaultValue_DEPRECATED;

public:

	FMovieSceneBoolChannel& GetChannel() { return BoolCurve; }
	const FMovieSceneBoolChannel& GetChannel() const { return BoolCurve; }

public:

	//~ UObject interface
	UE_API virtual void PostEditImport() override;
	UE_API virtual void PostLoad() override;

protected:

	/**
	 * Update the channel proxy to ensure it has the correct flags and pointers
	 */
	UE_API void ReconstructChannelProxy();

	/** Ordered curve data */
	UPROPERTY()
	FMovieSceneBoolChannel BoolCurve;

#if WITH_EDITORONLY_DATA

public:

	/**
	 * Set a flag indicating that the actual property that this bool represents is the opposite of the values stored in this section
	 */
	UE_API void SetIsExternallyInverted(bool bInIsExternallyInverted);

protected:

	/** Overloaded serializer to ensure that the channel proxy is updated correctly on load and duplicate */
	UE_API virtual void Serialize(FArchive& Ar) override;

	/** True if this section represents a property that is the inversion of the values stored on this channel */
	UPROPERTY()
	bool bIsExternallyInverted;

#endif
};

#undef UE_API
