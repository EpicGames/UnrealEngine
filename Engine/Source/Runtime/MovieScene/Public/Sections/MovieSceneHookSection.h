// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreTypes.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "Evaluation/IMovieSceneEvaluationHook.h"
#include "MovieSceneSection.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneHookSection.generated.h"

class UMovieSceneEntitySystemLinker;
class UObject;
struct FFrameNumber;
struct FMovieSceneEntityComponentFieldBuilder;
struct FMovieSceneEvaluationFieldEntityMetaData;
template <typename ElementType> class TRange;


/**
 * 
 */
UCLASS(MinimalAPI)
class UMovieSceneHookSection
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
	, public IMovieSceneEvaluationHook
{
public:

	GENERATED_BODY()

	MOVIESCENE_API UMovieSceneHookSection(const FObjectInitializer&);

	virtual TArrayView<const FFrameNumber> GetTriggerTimes() const { return TArrayView<const FFrameNumber>(); }

protected:

	/*~ Implemented in derived classes

	virtual void Begin(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const override;
	virtual void Update(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const override;
	virtual void End(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const override;

	virtual void Trigger(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const override;

	*/

	MOVIESCENE_API virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
	MOVIESCENE_API virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;

	MOVIESCENE_API void ImportRangedEntity(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity);
	MOVIESCENE_API void ImportTriggerEntity(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity);

protected:

	UPROPERTY()
	uint8 bRequiresRangedHook : 1;

	UPROPERTY()
	uint8 bRequiresTriggerHooks : 1;
};

