// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Variants/MovieSceneTimeWarpVariant.h"
#include "Misc/FrameNumber.h"
#include "MovieSceneSectionTimingParameters.generated.h"


struct FMovieSceneSequenceTransform;
class UMovieScene;

/**
 * Parameter utility that converts section timing parameters to a transform using Seconds values.
 * 
 * Transformation happens in the following order:
 * 
 * InputTime (relative to section start)
 *     >> Play Rate / Time Warp
 *     >> FrameRate conversion
 *     >> +StartTimeOffset
 *     >> Loop (% duration)
 *     >> Reverse
 **/
USTRUCT(BlueprintType)
struct FMovieSceneSectionTimingParametersSeconds
{
	GENERATED_BODY()

	/**
	 * Playrate optionally implemented as time-warp
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Timing")
	FMovieSceneTimeWarpVariant PlayRate;

	/**
	 * Start offset (in seconds) to apply to all loops
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Timing", meta=(Units=s))
	float InnerStartOffset = 0.f;

	/**
	 * End offset (in seconds) to apply to all loops ie, loop_range=[0 + InnerStartOffset, End- InnerEndOffset)
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Timing", meta=(Units=s))
	float InnerEndOffset = 0.f;

	/**
	 * Start offset to apply only to the first loop
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Timing", meta=(Units=s))
	float FirstLoopStartOffset = 0.f;

	/**
	 * When true, apply looping to the inner range. Mutually exclusive with bClampToInnerRange.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Timing")
	uint8 bLoop : 1 = 0;

	/**
	 * When true, apply clamping to the inner range. Mutually exclusive with bLoop.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Timing")
	uint8 bClampToInnerRange : 1 = 0;

	/**
	 * When true, reverses the play direction. Applied after all other transformations
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Timing")
	uint8 bReverse : 1 = 0;

	/**
	 * When true, apply clamping to the outer range before anything else.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Timing")
	uint8 bClampToOuterRange : 1 = 0;

	/**
	 * Make a transform out of these parameters
	 */
	MOVIESCENE_API FMovieSceneSequenceTransform MakeTransform(const FFrameRate& OuterFrameRate, const TRange<FFrameNumber>& OuterRange, double SourceDuration, double InnerPlayRate) const;

	/**
	 * Helpers
	 */
	MOVIESCENE_API void AddOffset(FMovieSceneSequenceTransform& Transform, const FFrameTime& Offset) const;

	MOVIESCENE_API void AddPositionInOuterAsOffset(FMovieSceneSequenceTransform& Transform, const TRange<FFrameNumber>& OuterRange) const;

	MOVIESCENE_API void AddPlayRate(FMovieSceneSequenceTransform& Transform) const;

	MOVIESCENE_API void AddFrameRateConversion(FMovieSceneSequenceTransform& Transform, const FFrameRate& OuterFrameRate, const FFrameRate& InnerFrameRate) const;

	MOVIESCENE_API void AddInnerStartOffset(FMovieSceneSequenceTransform& Transform, const FFrameTime& Offset) const;

	MOVIESCENE_API void AddLoopingOrClampingAndReverse(FMovieSceneSequenceTransform& Result, const double Duration, const double Offset) const;

	/**
	 * Deprecated
	 */
	UE_DEPRECATED(5.7, "Please use AddOffset, or the version of AddPositionInOuterAsOffset that takes an outer range")
	void AddPositionInOuterAsOffset(FMovieSceneSequenceTransform& Transform, const FFrameTime& Offset) const { AddOffset(Transform, Offset); }
};


/**
 * Parameter utility that converts section timing parameters to a transform using inner frame values.
 * 
 * Transformation happens in the following order:
 * 
 * InputTime (relative to section start)
 *     >> Play Rate / Time Warp
 *     >> FrameRate conversion
 *     >> +StartTimeOffset
 *     >> Loop (% duration)
 *     >> Reverse
 **/
USTRUCT(BlueprintType)
struct FMovieSceneSectionTimingParametersFrames
{
	GENERATED_BODY()

	/**
	 * Playrate optionally implemented as time-warp
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Timing")
	FMovieSceneTimeWarpVariant PlayRate;

	/**
	 * Start offset (in inner framerate frames) to apply to all loops
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Timing")
	FFrameNumber InnerStartOffset = 0;

	/**
	 * End offset (in inner framerate frames) to apply to all loops ie, loop_range=[0 + InnerStartOffset, End- InnerEndOffset)
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Timing")
	FFrameNumber InnerEndOffset = 0;

	/**
	 * Start offset to apply only to the first loop
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Timing")
	FFrameNumber FirstLoopStartOffset = 0;

	/**
	 * When true, apply looping to the inner range. Mutually exclusive with bClampToInnerRange.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Timing")
	uint8 bLoop : 1 = 0;

	/**
	 * When true, apply clamping to the inner range. Mutually exclusive with bLoop.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Timing")
	uint8 bClampToInnerRange : 1 = 0;

	/**
	 * When true, reverses the play direction. Applied after all other transformations
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Timing")
	uint8 bReverse : 1 = 0;

	/**
	 * When true, apply clamping to the outer range before anything else.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Timing")
	uint8 bClampToOuterRange : 1 = 0;

	/**
	 * Make a transform out of these parameters
	 */
	MOVIESCENE_API FMovieSceneSequenceTransform MakeTransform(const FFrameRate& OuterFrameRate, const TRange<FFrameNumber>& OuterRange, const FFrameRate& InnerFrameRate, const TRange<FFrameNumber>& InnerRange) const;

	/**
	 * Helpers
	 */
	MOVIESCENE_API void AddOffset(FMovieSceneSequenceTransform& Transform, const FFrameTime& Offset) const;

	MOVIESCENE_API void AddPositionInOuterAsOffset(FMovieSceneSequenceTransform& Transform, const TRange<FFrameNumber>& OuterRange) const;

	MOVIESCENE_API void AddPlayRate(FMovieSceneSequenceTransform& Transform) const;

	MOVIESCENE_API void AddFrameRateConversion(FMovieSceneSequenceTransform& Transform, const FFrameRate& OuterFrameRate, const FFrameRate& InnerFrameRate) const;

	MOVIESCENE_API void AddInnerStartOffset(FMovieSceneSequenceTransform& Transform, const FFrameTime& Offset) const;

	MOVIESCENE_API void AddLoopingOrClampingAndReverse(FMovieSceneSequenceTransform& Result, const FFrameNumber& Duration, const FFrameNumber& Offset) const;

	/**
	 * Deprecated
	 */
	UE_DEPRECATED(5.7, "Please use AddOffset, or the version of AddPositionInOuterAsOffset that takes an outer range")
	void AddPositionInOuterAsOffset(FMovieSceneSequenceTransform& Transform, const FFrameTime& Offset) const { AddOffset(Transform, Offset); }
};

namespace UE::MovieScene
{



} // namespace UE::MovieScene
