// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "Sections/MovieSceneBoolSection.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneSpawnSection.generated.h"

class UMovieSceneEntitySystemLinker;
class UObject;
struct FFrameNumber;
struct FMovieSceneEntityComponentFieldBuilder;
struct FMovieSceneEvaluationFieldEntityMetaData;
template <typename ElementType> class TRange;

/**
 * A spawn section.
 */
UCLASS(MinimalAPI)
class UMovieSceneSpawnSection 
	: public UMovieSceneBoolSection
	, public IMovieSceneEntityProvider
{
	GENERATED_BODY()

	UMovieSceneSpawnSection(const FObjectInitializer& Init);

	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
	virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;
};
