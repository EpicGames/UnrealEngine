// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Curves/KeyHandle.h"
#include "Delegates/Delegate.h"
#include "Engine/EngineTypes.h"
#include "Evaluation/MovieSceneEvaluationOperand.h"
#include "Evaluation/MovieSceneSectionParameters.h"
#include "Evaluation/MovieSceneSequenceHierarchy.h"
#include "Evaluation/MovieSceneSequenceTransform.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "GameFramework/Actor.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Math/Range.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameTime.h"
#include "Misc/Optional.h"
#include "MovieSceneFwd.h"
#include "MovieSceneSection.h"
#include "MovieSceneSequenceID.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Templates/SubclassOf.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneSubSection.generated.h"

class FProperty;
class UMovieScene;
class UMovieSceneEntitySystemLinker;
class UMovieSceneSequence;
class UObject;
struct FFrame;
struct FFrameRate;
struct FMovieSceneEvaluationTemplate;
struct FMovieSceneSectionParameters;
struct FMovieSceneTrackCompilerArgs;
struct FPropertyChangedEvent;
struct FQualifiedFrameTime;
struct FMovieSceneTransformMask;

namespace UE::MovieScene
{
	struct IRetimingInterface;
	struct FEntityImportParams;
	struct FImportedEntity;
}

DECLARE_DELEGATE_OneParam(FOnSequenceChanged, UMovieSceneSequence* /*Sequence*/);

struct FSubSequenceInstanceDataParams
{
	/** The ID of the sequence instance that is being generated */
	FMovieSceneSequenceID InstanceSequenceID;

	/** The object binding ID in which the section to be generated resides */
	FMovieSceneEvaluationOperand Operand;
};

USTRUCT()
struct FMovieSceneSubSectionOriginOverrideMask
{
	GENERATED_BODY()
	
	FMovieSceneSubSectionOriginOverrideMask()
		: Mask(0)
	{}

	FMovieSceneSubSectionOriginOverrideMask(EMovieSceneTransformChannel Channels)
		: Mask((__underlying_type(EMovieSceneTransformChannel))Channels)
	{}

	EMovieSceneTransformChannel GetChannels() const
	{
		return (EMovieSceneTransformChannel) Mask;
	}
	
private:
	UPROPERTY()
	uint32 Mask;
};

/**
 * Implements a section in sub-sequence tracks.
 */
UCLASS(BlueprintType, config = EditorPerProjectUserSettings, MinimalAPI)
class UMovieSceneSubSection
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
	GENERATED_BODY()

public:

	/** Object constructor. */
	MOVIESCENE_API UMovieSceneSubSection(const FObjectInitializer& ObjInitializer);

	/**
	 * Get the sequence that is assigned to this section.
	 *
	 * @return The sequence.
	 * @see SetSequence
	 */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	MOVIESCENE_API UMovieSceneSequence* GetSequence() const;

	/**
	 * Get the path name to this sub section from the outer moviescene
	 */
	MOVIESCENE_API FString GetPathNameInMovieScene() const;

	/**
	 * Get this sub section's sequence ID
	 */
	MOVIESCENE_API FMovieSceneSequenceID GetSequenceID() const;

	/** Generate subsequence data */
	MOVIESCENE_API virtual FMovieSceneSubSequenceData GenerateSubSequenceData(const FSubSequenceInstanceDataParams& Params) const;

public:

	/**
	 * Gets the transform that converts time from this section's time-base to its inner sequence's
	 */
	MOVIESCENE_API FMovieSceneSequenceTransform OuterToInnerTransform() const;

	/**
	 * Gets the transform that converts time from this section's time-base to its inner sequence's
	 */
	MOVIESCENE_API FMovieSceneSequenceTransform OuterToInnerTransform_NoInnerTimeWarp() const;

	/**
	 * Gets the transform that converts time from this section's time-base to its inner sequence's
	 */
	MOVIESCENE_API void AppendInnerTimeWarpTransform(FMovieSceneSequenceTransform& OutTransform) const;

	/**
	 * Gets the playrange of the inner sequence, in the inner sequence's time space, trimmed with any start/end offsets,
	 * and validated to make sure we get at least a 1-frame long playback range (e.g. in the case where excessive
	 * trimming results in an invalid range).
	 */
	MOVIESCENE_API bool GetValidatedInnerPlaybackRange(TRange<FFrameNumber>& OutInnerPlaybackRange) const;

	/**
	 * Helper function used by the above method, but accessible for other uses like track editors.
	 */
	static MOVIESCENE_API TRange<FFrameNumber> GetValidatedInnerPlaybackRange(const FMovieSceneSectionParameters& SubSectionParameters, const UMovieScene& InnerMovieScene);

	/**
	 * Sets the sequence played by this section.
	 *
	 * @param Sequence The sequence to play.
	 * @see GetSequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	MOVIESCENE_API void SetSequence(UMovieSceneSequence* Sequence);

	/**
	 * Gets the channel mask for the subsection origin overrides. 
	 * @return The mask with bit flags for each enabled channel.
	 */
	MOVIESCENE_API FMovieSceneSubSectionOriginOverrideMask GetMask() const;

	/**
	 * Sets the channel mask for the subsection origin overrides.
	 * @param MovieSceneTransformChannel the new mask.
	 */
	MOVIESCENE_API void SetMask(EMovieSceneTransformChannel MovieSceneTransformChannel);

#if WITH_EDITOR
	/**
	 * Sets the external Position for controlling the transform origin in the level editor through the edtior mode's gizmo
	 * @param InPosition An optional vector specifying the gizmo's current location
	 * @see FSubTrackEditorMode
	 */
	MOVIESCENE_API void SetKeyPreviewPosition(TOptional<FVector> InPosition);

	/**
	* Sets the external rotation for controlling the transform origin in the level editor through the edtior mode's gizmo
	* @param InRotation An optional vector specifying the gizmo's current location
	* @see FSubTrackEditorMode
	*/
	MOVIESCENE_API void SetKeyPreviewRotation(TOptional<FRotator> InRotation);

	/** Gets the optional value of the external position of the editor mode's gizmo */
	TOptional<FVector> GetKeyPreviewPosition() const { return KeyPreviewPosition; }
	
	/** Gets the optional value of the external Rotation of the editor mode's gizmo */
	TOptional<FRotator> GetKeyPreviewRotation() const { return KeyPreviewRotation; }

	/** Resets the optional values for the gizmo's position and rotation */
	MOVIESCENE_API void ResetKeyPreviewRotationAndLocation();
#endif

	MOVIESCENE_API virtual void PostLoad() override;

#if WITH_EDITOR
	MOVIESCENE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	MOVIESCENE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	/** Delegate to fire when our sequence is changed in the property editor */
	FOnSequenceChanged& OnSequenceChanged() { return OnSequenceChangedDelegate; }

	MOVIESCENE_API bool IsTransformOriginEditable() const;
#endif

	MOVIESCENE_API FFrameNumber MapTimeToSectionFrame(FFrameTime InPosition) const;
	MOVIESCENE_API bool HasAnyChannelData() const;

	EMovieSceneServerClientMask GetNetworkMask() const
	{
		return (EMovieSceneServerClientMask)NetworkMask;
	}

	void SetNetworkMask(EMovieSceneServerClientMask InNetworkMask)
	{
		NetworkMask = (uint8)InNetworkMask;
	}

	MOVIESCENE_API void DeleteChannels(TArrayView<const FName> ChannelNames);

	static const FName GetTranslationPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(UMovieSceneSubSection, Translation);
	}

	static const FName GetRotationPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(UMovieSceneSubSection, Rotation);
	}

public:

	//~ UMovieSceneSection interface
	MOVIESCENE_API virtual TOptional<TRange<FFrameNumber> > GetAutoSizeRange() const override;
	MOVIESCENE_API virtual void TrimSection( FQualifiedFrameTime TrimTime, bool bTrimLeft, bool bDeleteKeys) override;
	virtual TOptional<FFrameTime> GetOffsetTime() const override { return TOptional<FFrameTime>(FFrameTime(Parameters.StartFrameOffset)); }
	MOVIESCENE_API virtual void GetSnapTimes(TArray<FFrameNumber>& OutSnapTimes, bool bGetSectionBorders) const override;
	MOVIESCENE_API virtual void MigrateFrameTimes(FFrameRate SourceRate, FFrameRate DestinationRate) override;
	MOVIESCENE_API virtual void MigrateFrameTimes(const UE::MovieScene::IRetimingInterface& Retimer) override;
	MOVIESCENE_API virtual FMovieSceneTimeWarpVariant* GetTimeWarp() override;
	MOVIESCENE_API virtual UObject* GetSourceObject() const override;

protected:

	MOVIESCENE_API void BuildDefaultSubSectionComponents(UMovieSceneEntitySystemLinker* EntityLinker, const UE::MovieScene::FEntityImportParams& Params, UE::MovieScene::FImportedEntity* OutImportedEntity) const;

	MOVIESCENE_API virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;
	MOVIESCENE_API virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;

	MOVIESCENE_API virtual EMovieSceneChannelProxyType CacheChannelProxy() override;

public:

	UPROPERTY(config, BlueprintReadWrite, EditAnywhere, Category="General", meta=(ShowOnlyInnerProperties))
	FMovieSceneSectionParameters Parameters;

private:

	UPROPERTY()
	float StartOffset_DEPRECATED;

	UPROPERTY()
	float TimeScale_DEPRECATED;

	UPROPERTY()
	float PrerollTime_DEPRECATED;

	UPROPERTY(EditAnywhere, Category="Networking", meta=(Bitmask, BitmaskEnum="/Script/MovieScene.EMovieSceneServerClientMask"))
	uint8 NetworkMask;

	UPROPERTY()
	FMovieSceneSubSectionOriginOverrideMask OriginOverrideMask;

	UPROPERTY(meta = (LinearDeltaSensitivity = "1", Delta = "1.0"))
	FMovieSceneDoubleChannel Translation[3];
	
	UPROPERTY(meta = (LinearDeltaSensitivity = "1", Delta = "1.0"))
	FMovieSceneDoubleChannel Rotation[3];

#if WITH_EDITORONLY_DATA
	/** Preview value of position used for keying. This allows for transforms without needing to commit them to the channel */
	UPROPERTY(Transient)
	TOptional<FVector> KeyPreviewPosition;

	/** Preview value of rotation used for keying. This allows for transforms without needing to commit them to the channel */
	UPROPERTY(Transient)
	TOptional<FRotator> KeyPreviewRotation;
#endif	

protected:

	/** Movie scene being played by this section */
	UPROPERTY(EditAnywhere, Category="Sequence")
	TObjectPtr<UMovieSceneSequence> SubSequence;

#if WITH_EDITOR
	/** Delegate to fire when our sequence is changed in the property editor */
	FOnSequenceChanged OnSequenceChangedDelegate;

	/* Previous sub sequence, restored if changed sub sequence is invalid*/
	UMovieSceneSequence* PreviousSubSequence;
#endif
};
