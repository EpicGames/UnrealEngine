// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MovieSceneSection.h"
#include "Variants/MovieSceneTimeWarpVariant.h"
#include "MovieSceneTimeWarpSection.generated.h"

struct FMovieSceneNestedSequenceTransform;

/**
 * The section type contained within a UMovieSceneTimeWarpTrack.
 */
UCLASS(MinimalAPI)
class UMovieSceneTimeWarpSection
	: public UMovieSceneSection
{
public:
	GENERATED_BODY()

	MOVIESCENE_API UMovieSceneTimeWarpSection(const FObjectInitializer& ObjectInitializer);

	MOVIESCENE_API virtual void PostLoad() override;

	MOVIESCENE_API FMovieSceneNestedSequenceTransform GenerateTransform() const;

	/** Time-warp variant specifying the time-warp implementation (constant play rate by default) */
	UPROPERTY(EditAnywhere, Category="Time Warp")
	FMovieSceneTimeWarpVariant TimeWarp;

private:

#if WITH_EDITOR
	virtual bool Modify(bool bAlwaysMarkDirty = true) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	virtual EMovieSceneChannelProxyType CacheChannelProxy() override;
	virtual FMovieSceneTimeWarpVariant* GetTimeWarp() override;
};
