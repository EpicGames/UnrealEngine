// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/Interface.h"
#include "Components/SkinnedMeshComponent.h"
#include "Components/SkeletalMeshComponent.h"  // kept for WITH_EDITOR cast below


struct FSkeletalMeshEditorParams
{
	void SaveState(USkinnedMeshComponent* InSkinnedMeshComp)
	{
		if (InSkinnedMeshComp)
		{
			ChildSkelMesh = InSkinnedMeshComp;
			VisibilityBasedAnimTickOption = InSkinnedMeshComp->VisibilityBasedAnimTickOption;
			InSkinnedMeshComp->VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones;
			PredictedLODLevel = InSkinnedMeshComp->GetPredictedLODLevel();
#if WITH_EDITOR
			if (USkeletalMeshComponent* SkelComp = Cast<USkeletalMeshComponent>(InSkinnedMeshComp))
			{
				bUpdateAnimationInEditor = SkelComp->GetUpdateAnimationInEditor();
				bUpdateClothInEditor     = SkelComp->GetUpdateClothInEditor();
				SkelComp->SetUpdateAnimationInEditor(true);
				SkelComp->SetUpdateClothInEditor(true);
			}
#endif
		}
	}

	void RestoreState()
	{
		if (ChildSkelMesh.IsValid())
		{
			ChildSkelMesh->VisibilityBasedAnimTickOption = VisibilityBasedAnimTickOption;
			ChildSkelMesh->SetPredictedLODLevel(PredictedLODLevel);
#if WITH_EDITOR
			if (USkeletalMeshComponent* SkelComp = Cast<USkeletalMeshComponent>(ChildSkelMesh.Get()))
			{
				SkelComp->SetUpdateAnimationInEditor(bUpdateAnimationInEditor);
				SkelComp->SetUpdateClothInEditor(bUpdateClothInEditor);
			}
#endif
		}
	}
	void RestoreLOD()
	{
		if (ChildSkelMesh.IsValid())
		{
			ChildSkelMesh->SetPredictedLODLevel(PredictedLODLevel);
		}
	}
	TWeakObjectPtr<USkinnedMeshComponent> ChildSkelMesh;
	EVisibilityBasedAnimTickOption VisibilityBasedAnimTickOption;
	int32 PredictedLODLevel;
#if WITH_EDITOR
	bool bUpdateAnimationInEditor;
	bool bUpdateClothInEditor;
#endif
};

struct FSkeletalMeshRestoreState
{
	void SaveState(USkinnedMeshComponent* InComponent)
	{
		SkeletalMeshCompEditorParams.SetNum(0);
		FSkeletalMeshEditorParams Parent;
		Parent.SaveState(InComponent);
		SkeletalMeshCompEditorParams.Add(Parent);
		TArray<USceneComponent*> ChildComponents;
		InComponent->GetChildrenComponents(true, ChildComponents);
		for (USceneComponent* ChildComponent : ChildComponents)
		{
			USkinnedMeshComponent* SkinnedMeshComp = Cast<USkinnedMeshComponent>(ChildComponent);
			if (SkinnedMeshComp)
			{
				FSkeletalMeshEditorParams Params;
				Params.SaveState(SkinnedMeshComp);
				SkeletalMeshCompEditorParams.Add(Params);
			}
		}
	}

	UE_DEPRECATED(5.6, "InComponent is not required")
	void RestoreState(USkeletalMeshComponent* InComponent)
	{
		RestoreState();
	}
	UE_DEPRECATED(5.6, "InComponent is not required")
	void RestoreLOD(USkeletalMeshComponent* InComponent)
	{
		RestoreLOD();
	}

	void RestoreState()
	{
		for (FSkeletalMeshEditorParams& ChildParams : SkeletalMeshCompEditorParams)
		{
			ChildParams.RestoreState();
		}
	}
	void RestoreLOD()
	{
		for (FSkeletalMeshEditorParams& ChildParams : SkeletalMeshCompEditorParams)
		{
			ChildParams.RestoreLOD();
		}
	}
	TArray<FSkeletalMeshEditorParams> SkeletalMeshCompEditorParams;
	
};

