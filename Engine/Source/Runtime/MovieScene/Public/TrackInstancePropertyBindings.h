// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Map.h"
#include "Misc/TVariant.h"
#include "UObject/Class.h"
#include "UObject/ObjectKey.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakFieldPtr.h"

namespace UE::MovieScene
{

struct FSourcePropertyValue;

/**
 * A property path whose only tail propery information is cached, for faster (or at least less slow) access.
 */
struct FCachedProperty
{
	TWeakFieldPtr<FProperty> Property = nullptr;

	void* ContainerAddress = nullptr;

	int32 ArrayIndex = INDEX_NONE;

	FProperty* GetValidProperty() const
	{
		FProperty* PropertyPtr = Property.Get();
		if (ContainerAddress && PropertyPtr)
		{
			return PropertyPtr;
		}
		return nullptr;
	}

	template<typename ValueType>
	ValueType* GetPropertyAddress() const
	{
		if (FProperty* PropertyPtr = GetValidProperty())
		{
			if (ArrayIndex == INDEX_NONE)
			{
				return PropertyPtr->ContainerPtrToValuePtr<ValueType>(ContainerAddress);
			}
			else if (FArrayProperty* ArrayProp = CastFieldChecked<FArrayProperty>(PropertyPtr))
			{
				FScriptArrayHelper ArrayHelper(ArrayProp, ArrayProp->ContainerPtrToValuePtr<void>(ContainerAddress));
				check(ArrayHelper.IsValidIndex(ArrayIndex));
				return (ValueType*)ArrayHelper.GetRawPtr(ArrayIndex);
			}
		}
		return nullptr;
	}
};

/**
 * A step in the property path, for when that property path has volatile elements in it.
 */
struct FVolatilePropertyStep
{
private:

	using FData = TVariant<
		// ContainerOffset : A fixed offset to jump by.
		// Size = 4 bytes
		uint32,
		// CheckArrayIndex : A jump into an array's buffer, checking that the element index we'll 
		// offset to next is still valid.
		// Size = 4 bytes
		int32,
		// CheckStruct : A jump into an instanced struct's buffer, checking that the struct type
		// is what we expect.
		// Size = 8 bytes
		TWeakObjectPtr<UScriptStruct>
	>;
	FData Data;

public:

	void* ResolveAddress(void* ContainerAddress, bool& bNeedsRecaching) const;

	void SetContainerOffset(uint32 InOffset) { Data.Set<uint32>(InOffset); }
	void SetCheckArrayIndex(int32 InArrayIndex) { Data.Set<int32>(InArrayIndex); }
	void SetCheckStruct(TWeakObjectPtr<UScriptStruct> InScriptStruct) { Data.Set<TWeakObjectPtr<UScriptStruct>>(InScriptStruct); }
};
// The TVariant data type flag is 8 bytes so with the biggest union field being 8 bytes, it should 
// all fit inside of 16 bytes.
// TEMP: disable this for dSTM-Linux build targets where weak object pointers seem to be larger
//static_assert(sizeof(FVolatilePropertyStep) <= 16, "Try to fit FVolatilePropertyStep inside 16 bytes");

/**
 * A property path where no caching is done, and the address of the end value is re-resolved from
 * the root object every time. This is for property paths that go over re-allocatable memory such as
 * arrays, instanced structs, and other containers.
 */
struct FVolatileProperty
{
	const UObject* RootContainer = nullptr;
	FString PropertyPath;

	TArray<FVolatilePropertyStep> PropertySteps;
	TWeakFieldPtr<FProperty> LeafProperty;
	int32 LeafContainerStepIndex = INDEX_NONE;

	FProperty* GetValidProperty() const
	{
		FProperty* PropertyPtr = LeafProperty.Get();
		if (RootContainer && PropertyPtr)
		{
			return PropertyPtr;
		}
		return nullptr;
	}

	void* GetLeafContainerAddress() const
	{
		return ResolvePropertySteps(true);
	}

	template<typename ValueType>
	ValueType* GetPropertyAddress() const
	{
		return (ValueType*)ResolvePropertySteps(false);
	}

private:

	MOVIESCENE_API void* ResolvePropertySteps(bool bStopAtLeafStep) const;

	void* ResolvePropertyStepsImpl(bool bStopAtLeafStep, bool& bNeedsRecaching) const;
};

}  // namespace UE::MovieScene

/**
 * Manages bindings to keyed properties for a track instance. 
 * Calls UFunctions to set the value on runtime objects
 */
class FTrackInstancePropertyBindings
{
public:

	MOVIESCENE_API FTrackInstancePropertyBindings(FName InPropertyName, const FString& InPropertyPath);

	/**
	 * Calls the setter function for a specific runtime object or if the setter function does not exist, the property is set directly
	 *
	 * @param InRuntimeObject The runtime object whose function to call
	 * @param PropertyValue The new value to assign to the property
	 */
	template <typename ValueType>
	void CallFunction(UObject& InRuntimeObject, typename TCallTraits<ValueType>::ParamType PropertyValue)
	{
		const FResolvedPropertyAndFunction& PropAndFunction = FindOrAdd(InRuntimeObject);

		FProperty* Property = PropAndFunction.GetValidProperty();
		if (Property && Property->HasSetter())
		{
			Property->CallSetter(&InRuntimeObject, &PropertyValue);
		}
		else if (UFunction* SetterFunction = PropAndFunction.SetterFunction.Get())
		{
			InvokeSetterFunction(&InRuntimeObject, SetterFunction, PropertyValue);
		}
		else if (ValueType* Val = PropAndFunction.GetPropertyAddress<ValueType>())
		{
			*Val = MoveTempIfPossible(PropertyValue);
		}
	}

	/**
	 * Calls the setter function for a specific runtime object or if the setter function does not exist, the property is set directly
	 *
	 * @param InRuntimeObject The runtime object whose function to call
	 * @param PropertyValue The new value to assign to the property
	 */
	MOVIESCENE_API void CallFunctionForEnum( UObject& InRuntimeObject, int64 PropertyValue );

	/**
	 * Rebuilds the property and function mappings for a single runtime object, and adds them to the cache
	 *
	 * @param InRuntimeObject	The object to cache mappings for
	 */
	MOVIESCENE_API void CacheBinding(const UObject& InRuntimeObject);

	/**
	 * Gets the FProperty that is bound to the track instance.
	 *
	 * @param Object	The Object that owns the property
	 * @return			The property on the object if it exists
	 */
	MOVIESCENE_API FProperty* GetProperty(const UObject& Object);

	/**
	 * Returns whether this binding is valid for the given object.
	 *
	 * @param Object	The Object that owns the property
	 * @return          Whether the property binding is valid
	 */
	MOVIESCENE_API bool HasValidBinding(const UObject& Object);

	/**
	 * Gets the structure type of the bound property if it is a property of that type.
	 *
	 * @param Object	The Object that owns the property
	 * @return          The structure type
	 */
	MOVIESCENE_API const UStruct* GetPropertyStruct(const UObject& Object);

	/**
	 * Gets the current value of a property on an object
	 *
	 * @param Object	The object to get the property from
	 * @return ValueType	The current value
	 */
	template <typename ValueType>
	ValueType GetCurrentValue(const UObject& Object)
	{
		ValueType Value{};

		const FResolvedPropertyAndFunction& PropAndFunction = FindOrAdd(Object);
		TryGetPropertyValue<ValueType>(PropAndFunction, Value);

		return Value;
	}

	/**
	 * Optionally gets the current value of a property on an object
	 *
	 * @param Object	The object to get the property from
	 * @return (Optional) The current value of the property on the object
	 */
	template <typename ValueType>
	TOptional<ValueType> GetOptionalValue(const UObject& Object)
	{
		ValueType Value{};

		const FResolvedPropertyAndFunction& PropAndFunction = FindOrAdd(Object);
		if (TryGetPropertyValue<ValueType>(PropAndFunction, Value))
		{
			return Value;
		}

		return TOptional<ValueType>();
	}

	/**
	 * Gets the current value of a property on an object
	 *
	 * @param Object	The object to get the property from
	 * @return ValueType	The current value
	 */
	MOVIESCENE_API int64 GetCurrentValueForEnum(const UObject& Object);

	/**
	 * Sets the current value of a property on an object
	 *
	 * @param Object	The object to set the property on
	 * @param InValue   The value to set
	 */
	template <typename ValueType>
	void SetCurrentValue(UObject& Object, typename TCallTraits<ValueType>::ParamType InValue)
	{
		const FResolvedPropertyAndFunction& PropAndFunction = FindOrAdd(Object);

		if (ValueType* Val = PropAndFunction.GetPropertyAddress<ValueType>())
		{
			*Val = InValue;
		}
	}

	/** @return the property path that this binding was initialized from */
	const FString& GetPropertyPath() const
	{
		return PropertyPath;
	}

	/** @return the property name that this binding was initialized from */
	const FName& GetPropertyName() const
	{
		return PropertyName;
	}

public:

	/**
	 * Static function for accessing a property value on an object without caching its address
	 *
	 * @param Object			The object to get the property from
	 * @param InPropertyPath	The path to the property to retrieve
	 * @return (Optional) The current value of the property on the object
	 */
	MOVIESCENE_API static TOptional<UE::MovieScene::FSourcePropertyValue> StaticValue(const UObject* Object, FStringView InPropertyPath);
	MOVIESCENE_API static TOptional<TPair<const FProperty*, UE::MovieScene::FSourcePropertyValue>> StaticPropertyAndValue(const UObject* Object, FStringView InPropertyPath);

	/**
	 * Static function for accessing a property value on an object without caching its address
	 *
	 * @param Object			The object to get the property from
	 * @param InPropertyPath	The path to the property to retrieve
	 * @return (Optional) The current value of the property on the object
	 */
	template <typename ValueType>
	static TOptional<ValueType> StaticValue(const UObject* Object, FStringView InPropertyPath)
	{
		checkf(Object, TEXT("No object specified"));

		FResolvedPropertyAndFunction PropAndFunction = FindPropertyAndFunction(Object, InPropertyPath);

		ValueType Value;
		if (TryGetPropertyValue<ValueType>(PropAndFunction, Value))
		{
			return Value;
		}

		return TOptional<ValueType>();
	}

	/** Finds the property at the end of the given property path. */
	static MOVIESCENE_API FProperty* FindProperty(const UObject* Object, FStringView InPropertyPath);

private:

	/**
	 * Wrapper for UObject::ProcessEvent that attempts to pass the new property value directly to the function as a parameter,
	 * but handles cases where multiple parameters or a return value exists. The setter parameter must be the first in the list,
	 * any other parameters will be default constructed.
	 */
	template<typename T>
	static void InvokeSetterFunction(UObject* InRuntimeObject, UFunction* Setter, T&& InPropertyValue);

	using FCachedProperty = UE::MovieScene::FCachedProperty;
	using FVolatileProperty = UE::MovieScene::FVolatileProperty;

	struct FResolvedPropertyAndFunction
	{
		TVariant<FCachedProperty, FVolatileProperty> ResolvedProperty;
		TWeakObjectPtr<UFunction> SetterFunction;
		
		MOVIESCENE_API FProperty* GetValidProperty() const;

		MOVIESCENE_API void* GetContainerAddress() const;

		template<typename ValueType>
		ValueType* GetPropertyAddress() const
		{
			if (const FCachedProperty* CachedProperty = ResolvedProperty.TryGet<FCachedProperty>())
			{
				return CachedProperty->GetPropertyAddress<ValueType>();
			}
			else if (const FVolatileProperty* VolatileProperty = ResolvedProperty.TryGet<FVolatileProperty>())
			{
				return VolatileProperty->GetPropertyAddress<ValueType>();
			}
			else
			{
				return nullptr;
			}
		}

		FResolvedPropertyAndFunction()
			: ResolvedProperty()
			, SetterFunction( nullptr )
		{}
	};

	MOVIESCENE_API static FResolvedPropertyAndFunction FindPropertyAndFunction(const UObject* Object, FStringView InPropertyPath);

	template <typename ValueType>
	static bool TryGetPropertyValue(const FResolvedPropertyAndFunction& PropAndFunction, ValueType& OutValue)
	{
		if (const ValueType* Value = PropAndFunction.GetPropertyAddress<ValueType>())
		{
			OutValue = *Value;
			return true;
		}
		return false;
	}

	/** Find or add the FResolvedPropertyAndFunction for the specified object */
	MOVIESCENE_API const FResolvedPropertyAndFunction& FindOrAdd(const UObject& InObject);

private:

	/** Mapping of objects to bound functions that will be called to update data on the track */
	TMap< FObjectKey, FResolvedPropertyAndFunction > RuntimeObjectToFunctionMap;

	/** Path to the property we are bound to */
	FString PropertyPath;

	/** Actual name of the property we are bound to */
	FName PropertyName;

	friend class FTrackInstancePropertyBindingsTests;
};

/** Explicit specializations for bools */
template<> MOVIESCENE_API void FTrackInstancePropertyBindings::CallFunction<bool>(UObject& InRuntimeObject, TCallTraits<bool>::ParamType PropertyValue);
template<> MOVIESCENE_API bool FTrackInstancePropertyBindings::TryGetPropertyValue<bool>(const FResolvedPropertyAndFunction& PropAndFunction, bool& OutValue);
template<> MOVIESCENE_API void FTrackInstancePropertyBindings::SetCurrentValue<bool>(UObject& Object, TCallTraits<bool>::ParamType InValue);

/** Explicit specializations for object pointers */
template<> MOVIESCENE_API void FTrackInstancePropertyBindings::CallFunction<UObject*>(UObject& InRuntimeObject, UObject* PropertyValue);
template<> MOVIESCENE_API bool FTrackInstancePropertyBindings::TryGetPropertyValue<UObject*>(const FResolvedPropertyAndFunction& PropAndFunction, UObject*& OutValue);
template<> MOVIESCENE_API void FTrackInstancePropertyBindings::SetCurrentValue<UObject*>(UObject& Object, UObject* InValue);

template<typename T>
void FTrackInstancePropertyBindings::InvokeSetterFunction(UObject* InRuntimeObject, UFunction* Setter, T&& InPropertyValue)
{
	// CacheBinding already guarantees that the function has >= 1 parameters
	const int32 ParmsSize = Setter->ParmsSize;

	// This should all be const really, but ProcessEvent only takes a non-const void*
	void* InputParameter = const_cast<typename TDecay<T>::Type*>(&InPropertyValue);

	// By default we try and use the existing stack value
	uint8* Params = reinterpret_cast<uint8*>(InputParameter);

	check(InRuntimeObject && Setter);
	if (Setter->ReturnValueOffset != MAX_uint16 || Setter->NumParms > 0)
	{
		// Function has a return value or multiple parameters, we need to initialize memory for the entire parameter pack
		// We use alloca here (as in UObject::ProcessEvent) to avoid a heap allocation. Alloca memory survives the current function's stack frame.
		Params = reinterpret_cast<uint8*>(FMemory_Alloca(ParmsSize));

		bool bFirstProperty = true;
		for (FProperty* Property = Setter->PropertyLink; Property; Property = Property->PropertyLinkNext)
		{
			// Initialize the parameter pack with any param properties that reside in the container
			if (Property->IsInContainer(ParmsSize))
			{
				Property->InitializeValue_InContainer(Params);

				// The first encountered property is assumed to be the input value so initialize this with the user-specified value from InPropertyValue
				if (Property->HasAnyPropertyFlags(CPF_Parm) && !Property->HasAnyPropertyFlags(CPF_ReturnParm) && bFirstProperty)
				{
					const bool bIsValid = ensureMsgf(sizeof(T) == Property->GetElementSize(), TEXT("Property type does not match for Sequencer setter function %s::%s (%" SIZE_T_FMT "bytes != %ibytes"), *InRuntimeObject->GetName(), *Setter->GetName(), sizeof(T), Property->GetElementSize());
					if (bIsValid)
					{
						Property->CopyCompleteValue(Property->ContainerPtrToValuePtr<void>(Params), &InPropertyValue);
					}
					else
					{
						return;
					}
				}
				bFirstProperty = false;
			}
		}
	}

	// Now we have the parameters set up correctly, call the function
	InRuntimeObject->ProcessEvent(Setter, Params);
}

