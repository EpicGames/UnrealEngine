// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Curves/IntegralCurve.h"
#include "Internationalization/Text.h"
#include "Misc/Guid.h"
#include "MovieSceneSection.h"
#include "MovieSceneTrack.h"
#include "Templates/SubclassOf.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"

#include "MovieSceneBindingLifetimeTrack.generated.h"

/**
 * Handles when an object binding should be activated/deactivated
 */
UCLASS(MinimalAPI)
class UMovieSceneBindingLifetimeTrack
	: public UMovieSceneTrack
	, public IMovieSceneEntityProvider
{
public:

	MOVIESCENE_API UMovieSceneBindingLifetimeTrack(const FObjectInitializer& Obj);
	GENERATED_BODY()

public:

public:

	// UMovieSceneTrack interface
	MOVIESCENE_API virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	MOVIESCENE_API virtual UMovieSceneSection* CreateNewSection() override;
	MOVIESCENE_API virtual bool HasSection(const UMovieSceneSection& Section) const override;
	MOVIESCENE_API virtual void AddSection(UMovieSceneSection& Section) override;
	MOVIESCENE_API virtual void RemoveSection(UMovieSceneSection& Section) override;
	MOVIESCENE_API virtual void RemoveSectionAt(int32 SectionIndex) override;
	MOVIESCENE_API virtual void RemoveAllAnimationData() override;
	MOVIESCENE_API virtual bool IsEmpty() const override;
	MOVIESCENE_API virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;	
	
	/*~ IMovieSceneEntityProvider */
	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
	virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;

	MOVIESCENE_API static TArray<FFrameNumberRange> CalculateInverseLifetimeRange(const TArray<FFrameNumberRange>& Ranges);


#if WITH_EDITORONLY_DATA
	MOVIESCENE_API virtual FText GetDisplayName() const override;
#endif

protected:

	/** All the sections in this track */
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> Sections;
};


