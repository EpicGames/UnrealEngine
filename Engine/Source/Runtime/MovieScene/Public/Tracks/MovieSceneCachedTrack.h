// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "MovieSceneCachedTrack.generated.h"

UINTERFACE(MinimalAPI)
class UMovieSceneCachedTrack : public UInterface
{
	GENERATED_BODY()
};

/**
 * Can be implemented by tracks that hold cached data. Used by the take recorder plugin to regenerate the cache when recording. 
 */
class IMovieSceneCachedTrack
{
	GENERATED_BODY()

public:
	
	/**
	 * Deletes any existing cache data
	 */
	virtual void ResetCache() {};

	/**
	 * Used to enable or disable recording for this track
	 */
	virtual void SetCacheRecordingAllowed(bool bShouldRecord) {};

	virtual bool IsCacheRecordingAllowed() const { return true; };

	virtual int32 GetMinimumEngineScalabilitySetting() const { return -1; };
};
