// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Curves/IntegralCurve.h"
#include "Internationalization/Text.h"
#include "Misc/Guid.h"
#include "MovieSceneSection.h"
#include "MovieSceneTrack.h"
#include "Templates/SubclassOf.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneSpawnTrack.generated.h"

class UObject;
struct FFrameNumber;
struct FMovieSceneEvaluationTrack;
template <typename ElementType> class TRange;

/**
 * Handles when a spawnable should be spawned and destroyed
 */
UCLASS(MinimalAPI)
class UMovieSceneSpawnTrack
	: public UMovieSceneTrack
{
public:

	MOVIESCENE_API UMovieSceneSpawnTrack(const FObjectInitializer& Obj);
	GENERATED_BODY()

public:

	/** Get the object identifier that this spawn track controls */
	const FGuid& GetObjectId() const
	{
		return ObjectGuid;
	}

	/** Set the object identifier that this spawn track controls */
	void SetObjectId(const FGuid& InGuid)
	{
		ObjectGuid = InGuid;
	}

	static uint16 GetEvaluationPriority() { return uint16(0xFFF); }

	MOVIESCENE_API void PopulateSpawnedRangeMask(const TRange<FFrameNumber>& InOverlap, TArray<TRange<FFrameNumber>, TInlineAllocator<1>>& OutRanges) const;

public:

	// UMovieSceneTrack interface
	MOVIESCENE_API virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	MOVIESCENE_API virtual UMovieSceneSection* CreateNewSection() override;
	MOVIESCENE_API virtual bool HasSection(const UMovieSceneSection& Section) const override;
	MOVIESCENE_API virtual void AddSection(UMovieSceneSection& Section) override;
	MOVIESCENE_API virtual void RemoveSection(UMovieSceneSection& Section) override;
	MOVIESCENE_API virtual void RemoveSectionAt(int32 SectionIndex) override;
	MOVIESCENE_API virtual void RemoveAllAnimationData() override;
	MOVIESCENE_API virtual bool IsEmpty() const override;
	MOVIESCENE_API virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;

	//~ UObject interface
	MOVIESCENE_API virtual void PostLoad() override;

#if WITH_EDITOR
	MOVIESCENE_API virtual ECookOptimizationFlags GetCookOptimizationFlags() const override;
#endif

#if WITH_EDITORONLY_DATA
	MOVIESCENE_API virtual FText GetDisplayName() const override;
#endif

protected:

	/** All the sections in this track */
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> Sections;

	/** The guid relating to the object we are to spawn and destroy */
	UPROPERTY()
	FGuid ObjectGuid;
};
