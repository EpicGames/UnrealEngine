// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Internationalization/Text.h"
#include "Misc/FrameNumber.h"
#include "Misc/InlineValue.h"
#include "MovieSceneNameableTrack.h"
#include "MovieSceneSection.h"
#include "Templates/SubclassOf.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneSubTrack.generated.h"

class UMovieSceneSequence;
class UMovieSceneSubSection;
class UObject;
struct FMovieSceneSegmentCompilerRules;

/**
 * A track that holds sub-sequences within a larger sequence.
 */
UCLASS(MinimalAPI)
class UMovieSceneSubTrack
	: public UMovieSceneNameableTrack
{
	GENERATED_BODY()

public:

	MOVIESCENE_API UMovieSceneSubTrack( const FObjectInitializer& ObjectInitializer );

	/**
	 * Adds a movie scene section at the requested time.
	 *
	 * @param Sequence The sequence to add
	 * @param StartTime The time to add the section at
	 * @param Duration The duration of the section in frames
	 * @return The newly created sub section
	 */
	virtual UMovieSceneSubSection* AddSequence(UMovieSceneSequence* Sequence, FFrameNumber StartTime, int32 Duration) { return AddSequenceOnRow(Sequence, StartTime, Duration, INDEX_NONE); }

	/**
	 * Adds a movie scene section at the requested time.
	 *
	 * @param Sequence The sequence to add
	 * @param StartTime The time to add the section at
	 * @param Duration The duration of the section in frames
	 * @param bInsertSequence Whether or not to insert the sequence and push existing sequences out
	 * @return The newly created sub section
	 */
	MOVIESCENE_API virtual UMovieSceneSubSection* AddSequenceOnRow(UMovieSceneSequence* Sequence, FFrameNumber StartTime, int32 Duration, int32 RowIndex);

	/**
	 * Check whether this track contains the given sequence.
	 *
	 * @param Sequence The sequence to find.
	 * @param Recursively Whether to search for the sequence in sub-sequences.
	 * @param SectionToSkip Skip this section when searching the track (ie. the section is already set to this sequence). 
	 * @return true if the sequence is in this track, false otherwise.
	 */
	MOVIESCENE_API bool ContainsSequence(const UMovieSceneSequence& Sequence, bool Recursively = false, const UMovieSceneSection* SectionToSkip = nullptr) const;
	
	/**
	 * Finds all sections at the current time.
	 *
	 * @param Time The time relative to the owning movie scene where the sections should be
	 * @return The found sections.
	 */
	MOVIESCENE_API TArray<UMovieSceneSection*, TInlineAllocator<4>> FindAllSections(FFrameNumber Time) const;

	/**
	 * Finds a section at the current time.
	 *
	 * @param Time The time relative to the owning movie scene where the section should be
	 * @return The found section.
	 */	
	MOVIESCENE_API UMovieSceneSection* FindSection(FFrameNumber Time) const;
	
	/**
	 * Finds a section at the current time or extends an existing one
	 *
	 * @param Time The time relative to the owning movie scene where the section should be
	 * @param OutWeight The weight of the section if found
	 * @return The found section.
	 */
	MOVIESCENE_API UMovieSceneSection* FindOrExtendSection(FFrameNumber Time, float& OutWeight);

	/**
	 * Finds a section at the current time, or adds one if no section is found.
	 *
	 * @param Time The time relative to the owning movie scene where the section should be
	 * @param bSectionAdded Whether a section was added or not
	 * @return The found section, or the new section.
	 */
	MOVIESCENE_API UMovieSceneSection* FindOrAddSection(FFrameNumber Time, bool& bSectionAdded);

public:

	// UMovieSceneTrack interface

	MOVIESCENE_API virtual void AddSection(UMovieSceneSection& Section) override;
	MOVIESCENE_API virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	MOVIESCENE_API virtual UMovieSceneSection* CreateNewSection() override;
	MOVIESCENE_API virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	MOVIESCENE_API virtual bool HasSection(const UMovieSceneSection& Section) const override;
	MOVIESCENE_API virtual bool IsEmpty() const override;
	MOVIESCENE_API virtual void RemoveAllAnimationData() override;
	MOVIESCENE_API virtual void RemoveSection(UMovieSceneSection& Section) override;
	MOVIESCENE_API virtual void RemoveSectionAt(int32 SectionIndex) override;
	MOVIESCENE_API virtual bool SupportsMultipleRows() const override;

#if WITH_EDITORONLY_DATA
	MOVIESCENE_API virtual FText GetDefaultDisplayName() const override;
	virtual UMovieSceneSection* GetSectionToKey() const override { return SectionToKey; }
	MOVIESCENE_API virtual void SetSectionToKey(UMovieSceneSection* Section) override;
#endif

protected:

	/** All movie scene sections. */
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> Sections;

#if WITH_EDITORONLY_DATA

public:

	/**
	 * Get the height of this track's rows
	 */
	int32 GetRowHeight() const
	{
		return RowHeight;
	}

	/**
	 * Set the height of this track's rows
	 */
	void SetRowHeight(int32 NewRowHeight)
	{
		RowHeight = FMath::Max(16, NewRowHeight);
	}

private:

	/** The height for each row of this track */
	UPROPERTY()
	int32 RowHeight;

	/** Section we should Key */
	UPROPERTY()
	TObjectPtr<UMovieSceneSection> SectionToKey;

#endif
};
