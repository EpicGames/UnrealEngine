// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Decorations/MovieSceneTimeWarpDecoration.h"
#include "MovieSceneNameableTrack.h"
#include "MovieSceneTimeWarpTrack.generated.h"

struct FMovieSceneNestedSequenceTransform;

/**
 * A track that can be added to any sequence to affect the playback position and speed of that sequence and all its subsequences
 */
UCLASS(MinimalAPI)
class UMovieSceneTimeWarpTrack
	: public UMovieSceneTrack
	, public IMovieSceneTimeWarpSource
{
	GENERATED_BODY()

public:

	/**
	 * Create and initialize a new instance.
	 *
	 * @param ObjectInitializer The object initializer.
	 */
	UMovieSceneTimeWarpTrack(const FObjectInitializer& ObjectInitializer);

	/**
	 * Generate the a sequence transform that perform's this track's time-warp.
	 * @note: The resulting transform may be the identity transform
	 */
	MOVIESCENE_API virtual FMovieSceneNestedSequenceTransform GenerateTransform() const;

	/** UMovieSceneTrack interface */
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;


public:

	// UMovieSceneTrack interface

	virtual void RemoveAllAnimationData() override;
	virtual bool HasSection(const UMovieSceneSection& Section) const override;
	virtual void AddSection(UMovieSceneSection& Section) override;
	virtual void RemoveSection(UMovieSceneSection& Section) override;
	virtual void RemoveSectionAt(int32 SectionIndex) override;
	virtual bool IsEmpty() const override;
	virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	virtual void OnAddedToMovieSceneImpl(UMovieScene* InMovieScene) override;
	virtual void OnRemovedFromMovieSceneImpl() override;

	virtual FMovieSceneNestedSequenceTransform GenerateTimeWarpTransform() override;
	virtual bool IsTimeWarpActive() const override;
	virtual void SetIsTimeWarpActive(bool bInUse) override;
	virtual int32 GetTimeWarpSortOrder() const override;

#if WITH_EDITORONLY_DATA
	virtual FText GetDisplayName() const override;
	virtual FText GetDisplayNameToolTipText(const FMovieSceneLabelParams& LabelParams) const override;
	virtual FName GetTrackName() const override;
#endif

private:

	/** Array of sections contained within this track - should only ever be one. */
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> Sections;

	/** True if this track is the active timewarp in its sequence. Assigned as part of compilation */
	UPROPERTY()
	bool bIsActiveTimeWarp = true;
};
