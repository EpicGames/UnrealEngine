// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/SceneComponent.h"

/**
* Stores information about a transform for the purpose of adding keys to a transform section
*/
struct FTransformData
{
	/** Translation component */
	FVector Translation;
	/** Rotation component */
	FRotator Rotation;
	/** Scale component */
	FVector Scale;

	FTransformData()
		: Translation(ForceInitToZero)
		, Rotation(ForceInitToZero)
		, Scale(ForceInitToZero)
	{}

	/**
	* Constructor.  Builds the data from a scene component
	* Uses relative transform only
	*
	* @param InComponent	The component to build from
	*/
	FTransformData(const USceneComponent* InComponent)
		: Translation(InComponent->GetRelativeLocation())
		, Rotation(InComponent->GetRelativeRotation())
		, Scale(InComponent->GetRelativeScale3D())
	{}

	FTransformData(const FTransform& InTransform)
		: Translation(InTransform.GetTranslation())
		, Rotation(InTransform.GetRotation().Rotator())
		, Scale(InTransform.GetScale3D())
	{}

	FTransformData(const FVector& InTranslation, const FRotator& InRotation, const FVector& InScale)
		: Translation(InTranslation)
		, Rotation(InRotation)
		, Scale(InScale)
	{}
};
