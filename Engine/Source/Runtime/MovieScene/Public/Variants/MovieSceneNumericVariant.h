// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MovieSceneNumericVariant.generated.h"

class UMovieSceneNumericVariantGetter;

#ifndef UE_MOVIESCENE_WEAKNUMERICVARIANT_CHECKS
	#define UE_MOVIESCENE_WEAKNUMERICVARIANT_CHECKS DO_CHECK
#endif


/**
 * A variant type that masquerades as a numeric (double) value.
 * 
 * This type is 8 bytes (sizeof(double)) and uses a technique called NaN-boxing to encode variants into those 8-bytes,
 * while a literal double value maintains the exact same bits in-memory as a double. By default this variant can only
 * represent a double, or a UMovieSceneNumericVariantGetter*, but additional variant types can be encoded by deriving
 * from this type and associating type 'IDs' to typed-data (upto 48 bit in size), where the type bits are encoded into
 * the nan bits of the double.
 * 
 * Extensive reading around NaN-boxing techniques can be found elsewhere.
 * 
 * UMovieSceneNumericVariantGetter may be used to assign an external, dynamic value to this variant.
 * 
 * The benefit of using this technique is that this type can be used as a drop-in replacement for any double member
 * variable to provide it with dynamic getter functionality without inflating the size of the class, and with barely any
 * runtime overhead whatsoever. Automatic UPROPERTY upgrade exists for all numeric property types that make sense:
 * int64 and uint64 are not supported in this variant due to loss of precision (doubles only have 52 bits of mantissa)
 */
USTRUCT(BlueprintType)
struct FMovieSceneNumericVariant
{
	static_assert(PLATFORM_LITTLE_ENDIAN, "This class does not currently support big-endian platforms");

	GENERATED_BODY()

	/**
	 * No init constructor that leaves the underlying memory uninitialized
	 */
	FMovieSceneNumericVariant(ENoInit)
	{
	}


	/**
	 * Default constructor - initializes this variant to a value of 0.0
	 */
	MOVIESCENE_API FMovieSceneNumericVariant();


	/**
	 * Initialize this variant to an explicit literal value
	 */
	MOVIESCENE_API explicit FMovieSceneNumericVariant(double InValue);


	/**
	 * Initialize this variant to an object pointer that provides a value
	 */
	MOVIESCENE_API explicit FMovieSceneNumericVariant(UMovieSceneNumericVariantGetter* InGetter);


	/**
	 * Move-assign and constructible
	 */
	FMovieSceneNumericVariant(FMovieSceneNumericVariant&& In) = default;
	FMovieSceneNumericVariant& operator=(FMovieSceneNumericVariant&& In) = default;


	/**
	 * Equality comparison
	 */
	MOVIESCENE_API friend bool operator==(const FMovieSceneNumericVariant& A, const FMovieSceneNumericVariant& B);


	/**
	 * Inequality comparison
	 */
	MOVIESCENE_API friend bool operator!=(const FMovieSceneNumericVariant& A, const FMovieSceneNumericVariant& B);


public:

	/**
	 * Assign a new literal value to this variant, clearing any knowledge of a previously assigned value.
	 * 
	 * @param InLiteralValue      The new fixed value this variant should represent.
	 */
	MOVIESCENE_API void Set(double InLiteralValue);


	/**
	 * Assign a new dynamic value to this variant.
	 * 
	 * @param InDynamicValue      The new dynamic value this variant should represent.
	 */
	MOVIESCENE_API void Set(UMovieSceneNumericVariantGetter* InDynamicValue);


	/**
	 * Assign a new dynamic value to this variant as an unsafe weak ptr.
	 * @warning: This will result in a dangling pointer if it is not referenced strongly elsewhere. Use with caution.
	 * 
	 * @param InDynamicValue      The new dynamic value this variant should represent.
	 */
	MOVIESCENE_API void SetWeakUnsafe(UMovieSceneNumericVariantGetter* InDynamicValue);


	/**
	 * If this variant wraps a UMovieSceneTimeWarpGetter, turn it into an unsafe weak reference.
	 * @warning: This will result in a dangling pointer if it is not referenced strongly elsewhere. Use with caution.
	 */
	MOVIESCENE_API void MakeWeakUnsafe();


	/**
	 * Retrieve this variant's numeric value
	 * 
	 * @param InDynamicValue      The new dynamic value this variant should represent.
	 */
	MOVIESCENE_API double Get() const;


	/**
	 * Retrieves this variant as a UMovieSceneNumericVariantGetter pointer.
	 * @note: Only safe to call if IsCustomPtr() returns true
	 */
	MOVIESCENE_API UMovieSceneNumericVariantGetter* GetCustomPtr() const;


	/**
	 * Checks whether this variant is a literal double value or a different type
	 */
	bool IsLiteral() const;


	/**
	 * Retrieves this variant as a literal double representation.
	 */
	double GetLiteral() const;


	/**
	 * Retrieves this variant as a literal double representation, clamped to the range of a float.
	 */
	float GetLiteralAsFloat() const;


	/**
	 * Checks whether this variant is a UMovieSceneNumericVariantGetter pointer (ie, GetCustomPtr is valid to call).
	 */
	bool IsCustomPtr() const;


public:


	/**
	 * Make a shallow (bitwise) copy of this variant.
	 * Does not duplicate the underlying UMovieSceneNumericVariantGetter* if it is set: this function merely copies the ptr.
	 */
	MOVIESCENE_API FMovieSceneNumericVariant ShallowCopy() const;


	/**
	 * Make a deep copy of this variant by duplicating the underlying UMovieSceneNumericVariantGetter into a new outer if necessary.
	 */
	MOVIESCENE_API FMovieSceneNumericVariant DeepCopy(UObject* NewOuter) const;


public:


	/** For StructOpsTypeTraits */
	bool Serialize(FArchive& Ar);
	bool Identical(const FMovieSceneNumericVariant* Other, uint32 PortFlags) const;
	void AddStructReferencedObjects(FReferenceCollector& Collector);
	bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);
	bool ExportTextItem(FString& ValueStr, const FMovieSceneNumericVariant& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const;
	bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText, FArchive* InSerializingArchive = nullptr);
	void GetPreloadDependencies(TArray<UObject*>& OutDeps);

	bool SerializeCustom(FArchive& Ar, TFunctionRef<void(FArchive&, uint8&, void*)>);


public:


	/**
	 * Assign new, implementation defined typed data to this variant.
	 * 
	 * @note: If the instance of this class is to be serialized, the parent class should override Serialize
	 *        to provide robust serialization mechanisms besides the built-in bitwise serialization.
	 * 
	 * @param InValue      The new value to assign. Must be <= 48 bits.
	 * @param InType       A unique, compiletime fixed ID that identifies 'T'. Must be <= 7.
	 */
	template<typename T>
	void SetTypedData(const T& InValue, uint8 InType);


	/**
	 * Cast this variant to a user-defined type.
	 * @note: No type checking is performed other than checking !IsLiteral(). External protections must be made to ensure calling this function is safe.
	 */
	template<typename T>
	T UnsafePayloadCast() const;


	/**
	 * Set the type flags for this variant - should only be used where custom typed data is assigned
	 * @note: Only valid to be called where IsLiteral() == false.
	 * 
	 * @param InType       A unique, compiletime fixed ID that identifies the contained data. Must be <= 7.
	 */
	MOVIESCENE_API void SetTypeBits(uint8 InType);


	/**
	 * Retrieve the type flags for this variant.
	 * @note: Only valid to be called where IsLiteral() == false.
	 */
	MOVIESCENE_API uint8 GetTypeBits() const;


private:

	// Flags specifying different regions on an IEEE 754 double
	static constexpr uint64 HIGH_Bits      = 0xFFF0000000000000u; // All high bits (eg, Sign + exponent bits)
	static constexpr uint64 EXP_Bits       = 0x7FF0000000000000u; // Exponent bits 
	static constexpr uint64 SIGN_Bit       = 0x8000000000000000u; // Sign bit
	static constexpr uint64 QUIET_Bit      = 0x0008000000000000u; // Quiet NaN bit
	static constexpr uint64 TYPE_Bits      = 0x0007000000000000u; // Unused NaN bits that we repurpose for variant type information
	static constexpr uint64 TYPE_CustomPtr = 0x0000000000000000u; // INTENTIONALLY ZERO: Special value for TYPE_Bits when our data points to a UMovieSceneNumericVariantGetter*. 
	static constexpr uint64 PAYLOAD_Bits   = 0x0000FFFFFFFFFFFFu; // Bitmask specifying valid bits that can be used for custom payloads when any of TAGGED_Bits is set

	static constexpr uint64 CUSTOMPTR_FlagBits = 0x0000000000000003u; // Low bitmask that (ab)uses the alignment of UMovieSceneNumericVariantGetter to encode additional flags
	static constexpr uint64 CUSTOMPTR_Weak     = 0x0000000000000001u; // Low bit that signifies the wrapped custom pointer should not be reported to the reference graph.

	static constexpr uint64 TAGGED_Bits   = SIGN_Bit | EXP_Bits | QUIET_Bit;


	double& GetLiteralRef()
	{
		checkSlow(IsLiteral());
		return *reinterpret_cast<double*>(Data);
	}


	/**
	 * Checks whether this variant has the weak ptr flag.
	 * @note: Only valid to be called if IsCustomPtr() is true
	 */
	bool HasCustomWeakPtrFlag() const;

public:

	/** Functionally non-copyable except to the reflection layer - use Deep/Shallow copy instead */
	FMovieSceneNumericVariant(const FMovieSceneNumericVariant& Other) = default;
	FMovieSceneNumericVariant& operator=(const FMovieSceneNumericVariant& Other) = default;

private:

	alignas(8) uint8 Data[8];

#if UE_MOVIESCENE_WEAKNUMERICVARIANT_CHECKS
	TWeakObjectPtr<UMovieSceneNumericVariantGetter> WeakCustomGetter;
#endif
};

template<>
struct TStructOpsTypeTraits<FMovieSceneNumericVariant> : public TStructOpsTypeTraitsBase2<FMovieSceneNumericVariant>
{
	enum
	{
		WithCopy                       = true,
		WithIdenticalViaEquality       = true,
		WithSerializer                 = true,
		WithExportTextItem             = true,
		WithImportTextItem             = true,
		WithAddStructReferencedObjects = true,
		WithGetPreloadDependencies     = true,
		WithStructuredSerializeFromMismatchedTag = true,
	};
};


inline bool FMovieSceneNumericVariant::IsLiteral() const
{
	const uint64 Value = *reinterpret_cast<const uint64*>(Data);

	// Literal if the double is a quiet nan
	return (Value & TAGGED_Bits) != TAGGED_Bits;
}

inline double FMovieSceneNumericVariant::GetLiteral() const
{
	checkSlow(IsLiteral());
	return *reinterpret_cast<const double*>(Data);
}

inline float FMovieSceneNumericVariant::GetLiteralAsFloat() const
{
	return static_cast<float>(
		FMath::Clamp<double>(
			GetLiteral(),
			std::numeric_limits<float>::lowest(),
			std::numeric_limits<float>::max()
		)
	);
}

inline bool FMovieSceneNumericVariant::IsCustomPtr() const
{
	return !IsLiteral() && (GetTypeBits() == TYPE_CustomPtr);
}


template<typename T>
void FMovieSceneNumericVariant::SetTypedData(const T& InValue, uint8 InType)
{
	check(InType > TYPE_CustomPtr && (InType <= TYPE_Bits >> 48) );
	static_assert(sizeof(T) <= 6, "Type too big. Maximum supported size is 48 bits");

	uint64 NewValue = 0;

	void*  NewValueAddr  = &NewValue;
	uint8* NewValueBytes = static_cast<uint8*>(NewValueAddr);

	FMemory::Memcpy(NewValueBytes, &InValue, sizeof(T));

	*reinterpret_cast<uint64*>(Data) = TAGGED_Bits | NewValue;
	SetTypeBits(InType);
}

template<typename T>
T FMovieSceneNumericVariant::UnsafePayloadCast() const
{
	check(!IsLiteral());
	const uint64 PtrValue = *reinterpret_cast<const uint64*>(Data) & PAYLOAD_Bits;
	const void* ValuePtr = &PtrValue;
	return *reinterpret_cast<const T*>(ValuePtr);
}