// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MovieSceneSignedObject.h"
#include "MovieSceneNumericVariantGetter.generated.h"


/**
 * Base class for all dynamic getter types supplied to an FMovieSceneNumericVariant
 * 
 * Must be at least aligned to 8 bits since the low bits are used for referencing flags in FMovieSceneNumericVariant
 */
UCLASS(Abstract, MinimalAPI)
class alignas(8) UMovieSceneNumericVariantGetter : public UMovieSceneSignedObject
{
public:

	GENERATED_BODY()

	/**
	 * Retrieve the value for this getter
	 */
	virtual double GetValue() const
	{
		return 0.0;
	}

public:

	/**
	 * Reference to self used to report this object to the reference graph inside FMovieSceneNumericVariant::AddStructReferencedObjects
	 */
	UPROPERTY(transient)
	TObjectPtr<UMovieSceneNumericVariantGetter> ReferenceToSelf;
};

