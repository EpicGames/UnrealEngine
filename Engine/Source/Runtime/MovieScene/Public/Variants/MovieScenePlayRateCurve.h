// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneTimeWarpChannel.h"
#include "Channels/MovieScenePiecewiseCurve.h"
#include "Variants/MovieSceneTimeWarpGetter.h"
#include "MovieScenePlayRateCurve.generated.h"

class UMovieSceneSection;


UENUM()
enum class EMovieScenePlayRateCurveAnchorMode : uint8
{
	/** Anchor the integral at PlaybackStartFrame. For section-scoped curves this is conventionally 0 -
	 *  the enclosing section pre-shifts time so input frame 0 maps to the section start. */
	SectionRelative,
	/** Anchor the integral at MovieScene->GetPlaybackRange().GetLowerBoundValue().
	 *  Used for sequence-level curves whose input is absolute MovieScene time. */
	SequencePlaybackStart,
	/** Anchor the integral at the user-specified PlaybackStartFrame. */
	Manual,
};


/**
 * A time-warp defined as a play rate curve.
 * Time remapping is computed using the integral of the play rate curve.
 */
UCLASS(MinimalAPI, DisplayName="Play Rate Curve")
class UMovieScenePlayRateCurve : public UMovieSceneTimeWarpGetter
{
public:

	GENERATED_BODY()

	UMovieScenePlayRateCurve();

	/* Begin UMovieSceneTimeWarpGetter Implementation */
	FFrameTime RemapTime(FFrameTime In) const override;
	TOptional<FFrameTime> InverseRemapTimeCycled(FFrameTime InValue, FFrameTime InTimeHint, const UE::MovieScene::FInverseTransformTimeParams& Params) const override;
	TRange<FFrameTime> ComputeTraversedHull(const TRange<FFrameTime>& Range) const override;
	bool InverseRemapTimeWithinRange(FFrameTime InTime, FFrameTime RangeStart, FFrameTime RangeEnd, const TFunctionRef<bool(FFrameTime)>& VisitorCallback) const override;
	void InitializeDefaults(UMovieSceneSection* Section) override;
	EMovieSceneChannelProxyType PopulateChannelProxy(FMovieSceneChannelProxyData& OutProxyData, EAllowTopLevelChannels AllowTopLevel) override;
	bool DeleteChannel(FMovieSceneTimeWarpVariant& OutVariant, FName ChannelName) override;
	void ScaleBy(double UnwarpedScaleFactor) override;
	UE::MovieScene::ETimeWarpChannelDomain GetDomain() const override;
	/* End UMovieSceneTimeWarpGetter Implementation */

#if WITH_EDITOR
	virtual bool Modify(bool bAlwaysMarkDirty = true) override;
#endif

	MOVIESCENE_API virtual void PostLoad() override;

	MOVIESCENE_API const UE::MovieScene::FPiecewiseCurve& GetTimeWarpCurve() const;

protected:

	void InvalidateTimeWarp();

public:

	/** Curve defined in play-rate space. Does not support cycling. */
	UPROPERTY(EditAnywhere, Category="TimeWarp")
	FMovieSceneTimeWarpChannel PlayRate;

	UPROPERTY(EditAnywhere, Category="TimeWarp")
	FFrameNumber PlaybackStartFrame;

	UPROPERTY(EditAnywhere, Category="TimeWarp")
	EMovieScenePlayRateCurveAnchorMode AnchorMode = EMovieScenePlayRateCurveAnchorMode::SectionRelative;

	// Legacy. Migrated to AnchorMode in PostLoad. Do not set directly.
	UPROPERTY()
	bool bManualPlaybackStart = false;

private:

	/** Curve defined in time-warp space as an integral of the PlayRate curve */
	mutable UE::MovieScene::FPiecewiseCurve IntegratedTimeWarp;

public:

	/** false when IntegratedTimeWarp needs to be udpated */
	mutable bool bUpToDate = false;
};