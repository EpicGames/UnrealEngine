// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneTimeWarpChannel.h"
#include "Variants/MovieSceneTimeWarpGetter.h"
#include "MovieSceneTimeWarpCurve.generated.h"

class UMovieSceneSection;

/**
 * A custom time-warp curve providing a mapping from unwarped time to warped time
 */
UCLASS(MinimalAPI, DisplayName="Time Warp Curve")
class UMovieSceneTimeWarpCurve : public UMovieSceneTimeWarpGetter
{
public:

	GENERATED_BODY()

	UMovieSceneTimeWarpCurve();

	/* Begin UMovieSceneTimeWarpGetter Implementation */
	FFrameTime RemapTime(FFrameTime In) const override;
	TOptional<FFrameTime> InverseRemapTimeCycled(FFrameTime InValue, FFrameTime InTimeHint, const UE::MovieScene::FInverseTransformTimeParams& Params) const override;
	TRange<FFrameTime> ComputeTraversedHull(const TRange<FFrameTime>& Range) const override;
	bool InverseRemapTimeWithinRange(FFrameTime InTime, FFrameTime RangeStart, FFrameTime RangeEnd, const TFunctionRef<bool(FFrameTime)>& VisitorCallback) const override;
	void InitializeDefaults(UMovieSceneSection* Section) override;
	EMovieSceneChannelProxyType PopulateChannelProxy(FMovieSceneChannelProxyData& OutProxyData, EAllowTopLevelChannels AllowTopLevel) override;
	bool DeleteChannel(FMovieSceneTimeWarpVariant& OutVariant, FName ChannelName) override;
	void ScaleBy(double UnwarpedScaleFactor) override;
	UE::MovieScene::ETimeWarpChannelDomain GetDomain() const override;
	/* End UMovieSceneTimeWarpGetter Implementation */


	/** Curve defined as a 1:1 mapping from unwarped to warped time. Supports all cycle and extrap modes. */
	UPROPERTY(EditAnywhere, Category="TimeWarp")
	FMovieSceneTimeWarpChannel Channel;
};

