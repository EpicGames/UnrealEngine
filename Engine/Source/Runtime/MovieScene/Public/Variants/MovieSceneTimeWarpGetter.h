// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreTypes.h"
#include "Math/Range.h"
#include "Misc/FrameTime.h"
#include "Misc/Optional.h"
#include "Channels/IMovieSceneChannelOwner.h"
#include "Channels/MovieSceneTimeWarpChannel.h"
#include "Variants/MovieSceneNumericVariantGetter.h"
#include "MovieSceneTimeWarpGetter.generated.h"

class FName;
class UMovieSceneSection;

struct FMovieSceneTimeWarpVariant;
struct FMovieSceneChannelProxyData;

enum class EMovieSceneChannelProxyType : uint8;

template<typename>
class TFunctionRef;

namespace UE::MovieScene
{
	struct FInverseTransformTimeParams;
}

/**
 * Base class for all dynamic getter implementations of a FMovieSceneTimeWarpVariant
 */
UCLASS(Abstract, MinimalAPI)
class UMovieSceneTimeWarpGetter
	: public UMovieSceneNumericVariantGetter
	, public IMovieSceneChannelOwner
{
public:

	GENERATED_BODY()

	/** Enumeration specifying whether to allow top level channels or not when populating channel proxies */
	enum class EAllowTopLevelChannels
	{
		Yes, No
	};
	

	/** Default constructor */
	MOVIESCENE_API UMovieSceneTimeWarpGetter();


public:
	/*~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
	 *            Begin abstract API
	  ~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~*/

	/**
	 * Scale this time-warping in its time-domain based on the specified unwarped scale factor
	 */
	virtual void ScaleBy(double UnwarpedScaleFactor)
		PURE_VIRTUAL(UMovieSceneTimeWarpGetter::ScaleBy, )


	/**
	 * Remap the specified time using this time-warp
	 * 
	 * @param InTime     The time to remap
	 * @return The time-warped time
	 */
	virtual FFrameTime RemapTime(FFrameTime In) const
		PURE_VIRTUAL(UMovieSceneTimeWarpGetter::RemapTime, return In; )


	/**
	 * Given a continuous unwarped time range, compute the hull of warped times that are contained.
	 * eg: for looping time-warps and an input range with size > loop duration, this function will return the full loop range
	 * eg: for a time-warp that plays half speed, an input range of [0, 10) will yield [0, 5)
	 */
	virtual TRange<FFrameTime> ComputeTraversedHull(const TRange<FFrameTime>& Range) const
		PURE_VIRTUAL(UMovieSceneTimeWarpGetter::ComputeTraversedHull, return TRange<FFrameTime>(); )



	/**
	 * Attempt to transform a time-warped time into its corresponding non-time-warped time.
	 * @note: this operation can fail for warped times that have no representation in the non-time-warped space
	 * 
	 * @param WarpedTime         The time-warped time to remap
	 * @param UnwarpedTimeHint   A time 'hint' in the unwarped space to serve as a guide where there may be more than one solution.
	 *                           For example; consider a looping or cycling warp with duration 10, calling this function with InValue=3.
	 *                           There are an infinite number of unwarped times that can result in frame 3 (ie, every frame where %3==0). If we want to get the time that resulted in this
	 *                           from the 10th loop, we could pass the middle frame of that loop as the hint (ie, frame 95). The result would be the closest frame to frame 95 that warps
	 *                           to frame 3.
	 * @param Params             Parameter structure to control the inverse algorithm, eg limiting it to only searching one direction, ignoring clamping and cycles etc.
	 * @return The resulting unwarped frame time, or nothing if the time does not map to an unwarped time.
	 */
	virtual TOptional<FFrameTime> InverseRemapTimeCycled(FFrameTime InValue, FFrameTime InTimeHint, const UE::MovieScene::FInverseTransformTimeParams& Params) const
		PURE_VIRTUAL(UMovieSceneTimeWarpGetter::InverseRemapTimeCycled, return TOptional<FFrameTime>(); )



	/**
	 * Attempt to transform a time-warped time into its corresponding non-time-warped time within a specified unwarped range.
	 * @note: this operation can fail for warped times that have no representation in the non-time-warped space
	 * 
	 * @param InValue      The time-warped time to remap
	 * @param InTimeHint   A time 'hint' in the unwarped space to serve as a guide where there may be more than one solution.
	 *                     For example; consider a looping or cycling warp with duration 10, calling this function with InValue=3.
	 *                     There are an infinite number of unwarped times that can result in frame 3 (ie, every frame where %3==0). If we want to get the time that resulted in this
	 *                     from the 10th loop, we could pass the middle frame of that loop as the hint (ie, frame 95). The result would be the closest frame to frame 95 that warps
	 *                     to frame 3.
	 * @param Params       Parameter structure to control the inverse algorithm, eg limiting it to only searching one direction, ignoring clamping and cycles etc.
	 * @return The resulting unwarped frame time, or nothing if the time does not map to an unwarped time.
	 */
	virtual bool InverseRemapTimeWithinRange(FFrameTime InTime, FFrameTime RangeStart, FFrameTime RangeEnd, const TFunctionRef<bool(FFrameTime)>& VisitorCallback) const
		PURE_VIRTUAL(UMovieSceneTimeWarpGetter::InverseRemapTimeWithinRange, return true; )

	/**
	 * Retrieve the time domain that this time warp getter operates within
	 */
	virtual UE::MovieScene::ETimeWarpChannelDomain GetDomain() const
		PURE_VIRTUAL(UMovieSceneTimeWarpGetter::GetDomain, return UE::MovieScene::ETimeWarpChannelDomain::Time; )

	/*~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
	 *            End abstract API
	  ~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~*/

	MOVIESCENE_API virtual UE::MovieScene::FChannelOwnerCapabilities GetCapabilities(FName ChannelName) const override;
	MOVIESCENE_API virtual bool IsMuted(FName ChannelName) const override;
	MOVIESCENE_API virtual void SetIsMuted(FName ChannelName, bool bIsMuted) override;

	MOVIESCENE_API bool IsMuted() const;
	MOVIESCENE_API void SetIsMuted(bool bIsMuted);

public:

	/**
	 * Called to initialize the defaults for this time-warp based on the section
	 * Unimplemented by default
	 */
	virtual void InitializeDefaults(UMovieSceneSection* Section)
	{
	}


	/**
	 * Populate a channel proxy with this time-warp if possible
	 */
	MOVIESCENE_API virtual EMovieSceneChannelProxyType PopulateChannelProxy(FMovieSceneChannelProxyData& OutProxyData, EAllowTopLevelChannels AllowTopLevel);


	/**
	 * Attempt to delete this time-warp from a channel proxy if it matches the specified name
	 */
	MOVIESCENE_API virtual bool DeleteChannel(FMovieSceneTimeWarpVariant& OutVariant, FName ChannelName);

protected:

	/**
	 * Whether this getter is muted or not. Default: false.
	 */
	UPROPERTY()
	uint8 bMuted : 1;
};
