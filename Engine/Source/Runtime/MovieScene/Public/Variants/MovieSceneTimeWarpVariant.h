// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneNumericVariant.h"
#include "MovieSceneTimeWarpVariant.generated.h"


struct FFrameRate;
struct FMovieSceneTimeWarpLoop;
struct FMovieSceneTimeWarpClamp;
struct FMovieSceneTimeWarpLoopFloat;
struct FMovieSceneTimeWarpFrameRate;
struct FMovieSceneTimeWarpFixedFrame;
struct FMovieSceneTimeWarpClampFloat;

class UMovieSceneTimeWarpGetter;

namespace UE::MovieScene
{
	struct FInverseTransformTimeParams;
}


/** Enumeration defining the type stored within an FMovieSceneTimeWarpVariant */
UENUM()
enum class EMovieSceneTimeWarpType : uint8
{
	FixedPlayRate  = 0x0,   // FMovieSceneNumericVariant is a fixed double
	Custom         = 0x1,   // PAYLOAD_Bits is a UMovieSceneTimeWarpGetter* - matches FMovieSceneNumericVariant::TYPE_CustomPtr - 1
	FixedTime      = 0x2,   // PAYLOAD_Bits is a FMovieSceneTimeWarpFixedFrame (explicitly fixed time or zero timescale)
	FrameRate      = 0x3,   // PAYLOAD_Bits is a FMovieSceneTimeWarpFrameRate defining a frame rate from outer to inner space
	Loop           = 0x4,   // PAYLOAD_Bits is a FMovieSceneTimeWarpLoop
	Clamp          = 0x5,   // PAYLOAD_Bits is a FMovieSceneTimeWarpClamp

	LoopFloat      = 0x6,   // PAYLOAD_Bits is a FMovieSceneTimeWarpLoopFloat
	ClampFloat     = 0x7,   // PAYLOAD_Bits is a FMovieSceneTimeWarpClampFloat

	// Max of 8 types supported
};


/**
 * Numeric variant type that represents a 'time-warp' operation transforming a time into another time.
 * 
 * By default this variant is a literal value that represents a play rate of 1.0 (ie, a 1:1 mapping), but it can be customized
 * to provide a wide range of different transformations such as looping, clamping and custom curves
 */
USTRUCT(BlueprintType, meta=(HasNativeBreak="/Script/SequencerScripting.MovieSceneTimeWarpExtensions.BreakTimeWarp", HasNativeMake="/Script/SequencerScripting.MovieSceneTimeWarpExtensions.MakeTimeWarp"))
struct FMovieSceneTimeWarpVariant
{
	GENERATED_BODY()

	/**
	 *  Default construction: initializes this struct to a constant play rate of 1.0
	 */
	FMovieSceneTimeWarpVariant()
		: Variant(1.0)
	{
	}


	/**
	 *  Initialize this time-warp with a specific constant play rate
	 */
	explicit FMovieSceneTimeWarpVariant(double InLiteralPlayRate)
	{
		Set(InLiteralPlayRate);
	}


	/**
	 *  Initialize this time-warp with a specific fixed frame number
	 */
	explicit FMovieSceneTimeWarpVariant(const FMovieSceneTimeWarpFixedFrame& In)
		: Variant(NoInit)
	{
		Set(In);
	}


	/**
	 *  Initialize this time-warp with a looping time-warp
	 */
	explicit FMovieSceneTimeWarpVariant(const FMovieSceneTimeWarpLoop& In)
		: Variant(NoInit)
	{
		Set(In);
	}


	/**
	 *  Initialize this time-warp with a time-warp that clamps the time to a specific range
	 */
	explicit FMovieSceneTimeWarpVariant(const FMovieSceneTimeWarpClamp& In)
		: Variant(NoInit)
	{
		Set(In);
	}


	/**
	 *  Initialize this time-warp with custom time-warp getter
	 */
	explicit FMovieSceneTimeWarpVariant(UMovieSceneTimeWarpGetter* In)
		: Variant(NoInit)
	{
		Set(In);
	}


	/**
	 * Copy construction that performs a shallow copy
	 */
	FMovieSceneTimeWarpVariant(const FMovieSceneTimeWarpVariant& Other)
		: Variant(Other.Variant.ShallowCopy())
	{
	}


	/**
	 * Copy-assignment that performs a shallow copy
	 */
	FMovieSceneTimeWarpVariant& operator=(const FMovieSceneTimeWarpVariant& Other)
	{
		Variant = Other.Variant.ShallowCopy();
		return *this;
	}


	/**
	 * Move-construction
	 */
	FMovieSceneTimeWarpVariant(FMovieSceneTimeWarpVariant&&) = default;


	/**
	 * Move-assignment
	 */
	FMovieSceneTimeWarpVariant& operator=(FMovieSceneTimeWarpVariant&&) = default;


	/**
	 * Assignment from a literal play rate
	 */
	FMovieSceneTimeWarpVariant& operator=(double InLiteralPlayRate)
	{
		Set(InLiteralPlayRate);
		return *this;
	}


	/**
	 * Equality-comparison
	 */
	friend bool operator==(const FMovieSceneTimeWarpVariant& A, const FMovieSceneTimeWarpVariant& B)
	{
		return A.Variant == B.Variant;
	}


	/**
	 * Inequality-comparison
	 */
	friend bool operator!=(const FMovieSceneTimeWarpVariant& A, const FMovieSceneTimeWarpVariant& B)
	{
		return A.Variant != B.Variant;
	}


	/**
	 * Return a shallow copy of this variant via bit-wise copy.
	 * If the contained type is a custom object, the object ptr will be copied directly
	 */
	FMovieSceneTimeWarpVariant ShallowCopy() const
	{
		return FMovieSceneTimeWarpVariant(Variant.ShallowCopy());
	}


	/**
	 * Return a deep copy of this variant.
	 * If the contained type is a custom object, the object will be duplicated into the new outer, otherwise a bitwise copy is performed.
	 */
	FMovieSceneTimeWarpVariant DeepCopy(UObject* NewOuter) const
	{
		if (GetType() == EMovieSceneTimeWarpType::Custom)
		{
			return FMovieSceneTimeWarpVariant(Variant.DeepCopy(NewOuter));
		}

		return ShallowCopy();
	}


	/**
	 * If this variant wraps a UMovieSceneTimeWarpGetter, turn it into an unsafe weak reference.
	 * @warning: This will result in a dangling pointer if it is not referenced strongly elsewhere. Use with caution.
	 */
	void MakeWeakUnsafe()
	{
		Variant.MakeWeakUnsafe();
	}

public:


	/**
	 * Remap the specified time using this time-warp
	 * 
	 * @param InTime     The time to remap
	 * @return The time-warped time
	 */
	MOVIESCENE_API FFrameTime RemapTime(FFrameTime InTime) const;


public:


	/**
	 * Retrieve the type of this variant as an enumeration
	 */
	EMovieSceneTimeWarpType GetType() const;


	/*
	 * Retrieve this time-warp's constant play rate.
	 * @note: Only valid where GetType() == EMovieSceneTimeWarpType::FixedPlayRate
	 */
	MOVIESCENE_API double AsFixedPlayRate() const;


	/*
	 * Retrieve this time-warp's constant play rate clampoed to the range of a float
	 * @note: Only valid where GetType() == EMovieSceneTimeWarpType::FixedPlayRate
	 */
	MOVIESCENE_API float AsFixedPlayRateFloat() const;


	/*
	 * Cast this variant to a fixed time.
	 * @note: Only valid where GetType() == EMovieSceneTimeWarpType::FixedTime
	 */
	MOVIESCENE_API FMovieSceneTimeWarpFixedFrame AsFixedTime() const;


	/*
	 * Cast this variant to a frame-rate conversion
	 * @note: Only valid where GetType() == EMovieSceneTimeWarpType::FrameRate
	 */
	MOVIESCENE_API FMovieSceneTimeWarpFrameRate AsFrameRate() const;


	/*
	 * Cast this variant to a frame-based loop
	 * @note: Only valid where GetType() == EMovieSceneTimeWarpType::Loop
	 */
	MOVIESCENE_API FMovieSceneTimeWarpLoop AsLoop() const;


	/*
	 * Cast this variant to a clamped range
	 * @note: Only valid where GetType() == EMovieSceneTimeWarpType::Clamp
	 */
	MOVIESCENE_API FMovieSceneTimeWarpClamp AsClamp() const;


	/*
	 * Cast this variant to a floating-point loop
	 * @note: Only valid where GetType() == EMovieSceneTimeWarpType::LoopFloat
	 */
	MOVIESCENE_API FMovieSceneTimeWarpLoopFloat AsLoopFloat() const;


	/*
	 * Cast this variant to a floating-point clamped range
	 * @note: Only valid where GetType() == EMovieSceneTimeWarpType::ClampFloat
	 */
	MOVIESCENE_API FMovieSceneTimeWarpClampFloat AsClampFloat() const;


	/*
	 * Cast this variant to a custom time warp implementation
	 * @note: Only valid where GetType() == EMovieSceneTimeWarpType::Custom
	 */
	MOVIESCENE_API UMovieSceneTimeWarpGetter* AsCustom() const;


public:


	/**
	 * Make this time-warp play at a constant play-rate
	 */
	MOVIESCENE_API void Set(double InLiteralPlayRate);


	/**
	 * Make this time-warp always return a fixed frame number
	 */
	MOVIESCENE_API void Set(const FMovieSceneTimeWarpFixedFrame& InValue);


	/**
	 * Make this time-warp transform from one frame rate to another
	 */
	MOVIESCENE_API void Set(const FMovieSceneTimeWarpFrameRate& InValue);


	/**
	 * Make this time-warp loop within the specified bounds
	 */
	MOVIESCENE_API void Set(const FMovieSceneTimeWarpLoop& InValue);


	/**
	 * Make this time-warp clamp to the specified bounds
	 */
	MOVIESCENE_API void Set(const FMovieSceneTimeWarpClamp& InValue);


	/**
	 * Make this time-warp loop within the specified bounds
	 */
	MOVIESCENE_API void Set(const FMovieSceneTimeWarpLoopFloat& InValue);


	/**
	 * Make this time-warp clamp to the specified bounds
	 */
	MOVIESCENE_API void Set(const FMovieSceneTimeWarpClampFloat& InValue);


	/**
	 * Make this time-warp a custom dynamic value
	 */
	MOVIESCENE_API void Set(UMovieSceneTimeWarpGetter* InDynamicValue);


	/**
	 * Scale this time-warp by a factor
	 */
	MOVIESCENE_API void ScaleBy(double ScaleFactor);

public:

	/*~ For TStructOpsTypeTraits */
	bool Serialize(FArchive& Ar);
	bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);
	bool ExportTextItem(FString& ValueStr, const FMovieSceneTimeWarpVariant& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const;
	bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText);

	/**
	 * 
	 */
	FMovieSceneTimeWarpVariant(FMovieSceneNumericVariant&& Other)
		: Variant(MoveTemp(Other))
	{
	}

private:

	static_assert(PLATFORM_LITTLE_ENDIAN, "This class has not been written with big-endian support.");

	UPROPERTY()
	FMovieSceneNumericVariant Variant;
};

template<>
struct TStructOpsTypeTraits<FMovieSceneTimeWarpVariant> : public TStructOpsTypeTraitsBase2<FMovieSceneTimeWarpVariant>
{
	enum
	{
		WithCopy                                 = true,
		WithSerializer                           = true,
		WithExportTextItem                       = true,
		WithImportTextItem                       = true,
		WithIdenticalViaEquality                 = true,
		WithStructuredSerializeFromMismatchedTag = true,
	};
};


inline EMovieSceneTimeWarpType FMovieSceneTimeWarpVariant::GetType() const
{
	return Variant.IsLiteral() ? EMovieSceneTimeWarpType::FixedPlayRate : static_cast<EMovieSceneTimeWarpType>(Variant.GetTypeBits() + 1);
}