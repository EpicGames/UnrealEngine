// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "MovieSceneFwd.h"
#include "MovieSceneTimeWarpVariantPayloads.generated.h"


struct FFrameTime;

template<typename> class TRange;
template<typename> class TFunctionRef;

class UMovieSceneTimeWarpGetter;

namespace UE::MovieScene
{
	struct FInverseTransformTimeParams;
}

/**
 * Denotes a fixed time
 */
USTRUCT()
struct FMovieSceneTimeWarpFixedFrame
{
	GENERATED_BODY()

	UPROPERTY()
	FFrameNumber FrameNumber;
};


/**
 * Struct used only for text serialization of a time warp variant constant play rate
 */
USTRUCT()
struct FMovieSceneFixedPlayRateStruct
{
	GENERATED_BODY()

	/** The play rate */
	UPROPERTY()
	double PlayRate = 1.0;
};


/**
 * Struct used only for text serialization of a time warp getter struct
 */
USTRUCT()
struct FMovieSceneCustomTimeWarpGetterStruct
{
	GENERATED_BODY()

	/** The object implementation */
	UPROPERTY()
	TObjectPtr<UMovieSceneTimeWarpGetter> Object;
};


/**
 * Denotes Looping time range from [0:Duration)
 * @note: Specifically designed to fit into FMovieSceneNumericVariant::PAYLOAD_Bits
 */
USTRUCT()
struct FMovieSceneTimeWarpLoop
{
	GENERATED_BODY()

	UPROPERTY()
	FFrameNumber Duration;

	FFrameTime LoopTime(FFrameTime InTime) const;
	FFrameTime LoopTime(FFrameTime InTime, int32& OutLoop) const;
	TRange<FFrameTime> ComputeTraversedHull(const TRange<FFrameTime>& Range) const;
	TOptional<FFrameTime> InverseRemapTimeCycled(FFrameTime InValue, FFrameTime InTimeHint, const UE::MovieScene::FInverseTransformTimeParams& Params) const;
	bool InverseRemapTimeWithinRange(FFrameTime InTime, FFrameTime RangeStart, FFrameTime RangeEnd, const TFunctionRef<bool(FFrameTime)>& VisitorCallback) const;
	bool ExtractBoundariesWithinRange(const TRange<FFrameTime>& Range, const TFunctionRef<bool(FFrameTime)>& InVisitor) const;
};



/**
 * Denotes clamped time range from [0:Max]
 * @note: Specifically designed to fit into FMovieSceneNumericVariant::PAYLOAD_Bits
 */
USTRUCT()
struct FMovieSceneTimeWarpClamp
{
	GENERATED_BODY()

	UPROPERTY()
	FFrameNumber Max;

	FFrameTime Clamp(FFrameTime InTime) const;
	TRange<FFrameTime> ComputeTraversedHull(const TRange<FFrameTime>& Range) const;
};



/**
 * Denotes Looping time range from [0:Duration)
 * @note: Specifically designed to fit into FMovieSceneNumericVariant::PAYLOAD_Bits
 */
USTRUCT()
struct FMovieSceneTimeWarpLoopFloat
{
	GENERATED_BODY()

	UPROPERTY()
	float Duration = 1.f;

	FFrameTime LoopTime(FFrameTime InTime) const;
	FFrameTime LoopTime(FFrameTime InTime, int32& OutLoop) const;
	TRange<FFrameTime> ComputeTraversedHull(const TRange<FFrameTime>& Range) const;
	TOptional<FFrameTime> InverseRemapTimeCycled(FFrameTime InValue, FFrameTime InTimeHint, const UE::MovieScene::FInverseTransformTimeParams& Params) const;
	bool InverseRemapTimeWithinRange(FFrameTime InTime, FFrameTime RangeStart, FFrameTime RangeEnd, const TFunctionRef<bool(FFrameTime)>& VisitorCallback) const;
	bool ExtractBoundariesWithinRange(const TRange<FFrameTime>& Range, const TFunctionRef<bool(FFrameTime)>& InVisitor) const;
};


/**
 * Denotes clamped time range from [0:Max]
 * @note: Specifically designed to fit into FMovieSceneNumericVariant::PAYLOAD_Bits
 */
USTRUCT()
struct FMovieSceneTimeWarpClampFloat
{
	GENERATED_BODY()

	UPROPERTY()
	float Max = 1.f;

	FFrameTime Clamp(FFrameTime InTime) const;
	TRange<FFrameTime> ComputeTraversedHull(const TRange<FFrameTime>& Range) const;
};


/**
 * Denotes a framerate conversion
 * @note: Specifically designed to fit into FMovieSceneNumericVariant::PAYLOAD_Bits
 */
USTRUCT()
struct FMovieSceneTimeWarpFrameRate
{
	GENERATED_BODY()

	FMovieSceneTimeWarpFrameRate();
	FMovieSceneTimeWarpFrameRate(FFrameRate InRate);

	FFrameRate GetFrameRate() const;

private:

	// FFrameRate packed into 48 bits (24 each for numerator/denominator, max of 16777215 each)
	UPROPERTY()
	uint8 FrameRateNumerator[3];

	UPROPERTY()
	uint8 FrameRateDenominator[3];
};
